[CoreRedirects]
+ClassRedirects=(OldName="/Script/MetasoundMusic.MidiStepSequence",NewName="/Script/HarmonixMetasound.MidiStepSequence")
+PropertyRedirects=(OldName="MusicClockComponent.CurrentSongPos",NewName="CurrentSmoothedAudioRenderSongPos")

+StructRedirects=(OldName="/Script/HmxAudio.MidiNote",NewName="/Script/HarmonixMidi.MidiNote")
+ClassRedirects=(OldName="/Script/HmxAudio.MidiNoteFunctionLibrary",NewName="/Script/HarmonixMidi.MidiNoteFunctionLibrary")
+ClassRedirects=(OldName="/Script/HmxAudio.MusicalTickFunctionLibrary",NewName="/Script/HarmonixMidi.MusicalTickFunctionLibrary")
+FunctionRedirects=(OldName="MusicalTickFunctionLibrary.GetTicksPerBeat",NewName="GetTicksPerQuarterNote")
+FunctionRedirects=(OldName="MusicalTickFunctionLibrary.GetTicksPerBeatInt",NewName="GetTicksPerQuarterNoteInt")
+FunctionRedirects=(OldName="MusicalTickFunctionLibrary.GetBeatsPerTick",NewName="GetQuarterNotessPerTick")
+FunctionRedirects=(OldName="MusicalTickFunctionLibrary.TickToBeat",NewName="TickToQuarterNote")
+FunctionRedirects=(OldName="MusicalTickFunctionLibrary.BeatToTick",NewName="QuarterNoteToTick")

+FunctionRedirects=(OldName="HarmonixBlueprintUtil.SetMeasuredAudioLatencyMs",NewName="SetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs")
+FunctionRedirects=(OldName="HarmonixBlueprintUtil.GetMeasuredAudioLatencyMs",NewName="GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs")
+FunctionRedirects=(OldName="HarmonixBlueprintUtil.SetMeasuredVideoLatencyMs",NewName="SetMeasuredVideoToAudioRenderOffsetMs")
+FunctionRedirects=(OldName="HarmonixBlueprintUtil.GetMeasuredVideoLatencyMs",NewName="GetMeasuredVideoToAudioRenderOffsetMs")
+FunctionRedirects=(OldName="MusicClockComponent.GetSongRemainingTime", NewName="MusicClockComponent.GetSongRemainingMs")

+PropertyRedirects=(OldName="MidiFileData.LastTick",NewName="MidiFileData.LastEventTick")

+ClassRedirects=(OldName="/Script/HarmonixAudioReactivity.HarmonixPeakTamer",NewName="/Script/HarmonixMetasound.HarmonixPeakTamer")
+StructRedirects=(OldName="/Script/HarmonixAudioReactivity.HarmonixPeakTamerSettings",NewName="/Script/HarmonixMetasound.HarmonixPeakTamerSettings")

+PropertyRedirects=(OldName="SongLengthData.LengthBars", NewName="LengthFractionalBars")
