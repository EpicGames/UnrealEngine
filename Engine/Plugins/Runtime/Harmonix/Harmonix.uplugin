{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Harmonix",
	"Description": "A package of Harmonix music related audio functionality.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc. - Harmonix GenTech",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "Harmonix",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixDsp",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixDspTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixDspEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixMidi",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixMidiEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixMidiTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixMetasound",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixMetasoundEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HarmonixMetasoundTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Metasound",
			"Enabled": true
		},
		{
			"Name": "MusicEnvironment",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled":  true
		}
	]

}