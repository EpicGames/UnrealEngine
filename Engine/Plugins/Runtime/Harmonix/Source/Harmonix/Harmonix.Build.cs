// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;

public class Harmonix : ModuleRules
{
	public Harmonix(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AudioExtensions",
				"DeveloperSettings"
			});


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
			});
	}
}
