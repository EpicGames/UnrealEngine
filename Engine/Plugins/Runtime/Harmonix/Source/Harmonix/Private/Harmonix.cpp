// Copyright Epic Games, Inc. All Rights Reserved.

#include "Harmonix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Harmonix)

void FHarmonixModule::StartupModule()
{

}

void FHarmonixModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHarmonixModule, Harmonix)
