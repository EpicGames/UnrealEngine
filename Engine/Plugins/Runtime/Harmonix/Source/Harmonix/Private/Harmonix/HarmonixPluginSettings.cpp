// Copyright Epic Games, Inc. All Rights Reserved.


#include "Harmonix/HarmonixPluginSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixPluginSettings)


#define LOCTEXT_NAMESPACE "Harmonix"

UHarmonixPluginSettings::UHarmonixPluginSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("Harmonix");
}

#if WITH_EDITOR

FText UHarmonixPluginSettings::GetSectionText() const
{
	return LOCTEXT("PluginSettingsDisplayName", "Harmonix");
}


#endif

#undef LOCTEXT_NAMESPACE
