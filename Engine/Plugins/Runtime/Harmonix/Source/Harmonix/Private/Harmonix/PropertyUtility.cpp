// Copyright Epic Games, Inc. All Rights Reserved.

#include "Harmonix/PropertyUtility.h"

#if WITH_EDITORONLY_DATA

Harmonix::EPostEditType Harmonix::GetPropertyPostEditType(const FProperty* Property)
{
	if (const FString* PostEdit = Property->FindMetaData("PostEditType"))
	{
		if (PostEdit->Equals(TEXT("Trivial")))
		{
			return EPostEditType::Trivial;
		}
		else if (PostEdit->Equals(TEXT("NonTrivial")))
		{
			return EPostEditType::NonTrivial;
		}
	}

	return EPostEditType::None;
}

Harmonix::EPostEditAction Harmonix::GetPropertyPostEditAction(const FProperty* Property, EPropertyChangeType::Type ChangeType, EPostEditAction DefaultAction)
{
	EPostEditType PostEditType = GetPropertyPostEditType(Property);

	switch (ChangeType)
	{
	case EPropertyChangeType::Interactive:
	{
		if (PostEditType == EPostEditType::Trivial)
		{
			return EPostEditAction::UpdateTrivial;
		}

		// don't do anything if this property is Not Trivial
		// even if the default action is to Update NonTrivial
		// interactive property changes happen too often (ie. scrolling a slider)
		return EPostEditAction::DoNothing;
	}
	case EPropertyChangeType::ValueSet:
	{
		if (PostEditType == EPostEditType::Trivial)
		{
			return EPostEditAction::UpdateTrivial;
		}
		else if (PostEditType == EPostEditType::NonTrivial)
		{
			return EPostEditAction::UpdateNonTrivial;
		}

		// Use the default action for ValueSet changes without a meta data tag
		// (Defaults to NonTrivial, allowing all changes to go through)
		return DefaultAction;
	}
	case EPropertyChangeType::ArrayAdd:
	case EPropertyChangeType::ArrayClear:
	case EPropertyChangeType::ArrayRemove:
	case EPropertyChangeType::ArrayMove:
	{
		// assume all array changes are non trivial
		return EPostEditAction::UpdateNonTrivial;
	}
	}

	// if property change type is not accounted for, do nothing
	return EPostEditAction::DoNothing;
}

void Harmonix::LogPropertyValue(void* PropValuePtr, FProperty* Property)
{
	check(PropValuePtr);
	check(Property);

	FString ValueString;
	Property->ExportTextItem_Direct(ValueString, PropValuePtr, nullptr, nullptr, 0);
	UE_LOGF(LogPropertyUtility, Log, "\t%ls = %ls", *Property->GetName(), *ValueString);
}

bool Harmonix::CopyStructRecursive(void* InDest, void* InSrc, const UScriptStruct* ScriptStruct)
{
	check(InDest);
	check(InSrc);
	check(ScriptStruct);

	for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
	{
		FProperty* Property = *It;

		void* DestValuePtr = Property->ContainerPtrToValuePtr<void>(InDest);
		void* SrcValuePtr = Property->ContainerPtrToValuePtr<void>(InSrc);

		if (!ensureAlways(DestValuePtr && SrcValuePtr))
		{
			return false;
		}

		if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			if (!CopyStructRecursive(DestValuePtr, SrcValuePtr, StructProp->Struct))
			{
				return false;
			}
		}
		else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
		{
			FStructProperty* InnerStruct = CastField<FStructProperty>(ArrayProp->Inner);

			int idx = 0;
			void* DestElemPtr = ArrayProp->GetValueAddressAtIndex_Direct(ArrayProp->Inner, DestValuePtr, idx);
			void* SrcElemPtr = ArrayProp->GetValueAddressAtIndex_Direct(ArrayProp->Inner, SrcValuePtr, idx);

			if (!ensureAlways(DestElemPtr && SrcElemPtr))
			{
				return false;
			}

			while (DestElemPtr && SrcElemPtr)
			{
				if (InnerStruct)
				{
					if (!CopyStructRecursive(DestElemPtr, SrcElemPtr, InnerStruct->Struct))
					{
						return false;
					}
				}
				else
				{
					ArrayProp->Inner->CopySingleValue(DestElemPtr, SrcElemPtr);
				}

				++idx;

				DestElemPtr = ArrayProp->GetValueAddressAtIndex_Direct(InnerStruct, DestValuePtr, idx);
				SrcElemPtr = ArrayProp->GetValueAddressAtIndex_Direct(InnerStruct, SrcValuePtr, idx);
			}

		}
		else
		{
			Property->CopySingleValue(DestValuePtr, SrcValuePtr);
		}
	}

	return true;
}


void Harmonix::LogStructRecursive(void* PropPtr, const UScriptStruct* ScriptStruct)
{
	check(PropPtr);
	check(ScriptStruct);

	for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
	{
		FProperty* Property = *It;

		void* PropValuePtr = Property->ContainerPtrToValuePtr<void>(PropPtr);

		if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			UE_LOGF(LogPropertyUtility, Log, "%ls: {", *Property->GetName());
			LogStructRecursive(PropValuePtr, StructProp->Struct);
			UE_LOGF(LogPropertyUtility, Log, "} ~%ls", *Property->GetName());
		}
		else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
		{
			UE_LOGF(LogPropertyUtility, Log, "%ls: [", *Property->GetName());
			if (FStructProperty* InnerStruct = CastField<FStructProperty>(ArrayProp->Inner))
			{
				int idx = 0;
				while (void* ElemPtr = ArrayProp->GetValueAddressAtIndex_Direct(InnerStruct, PropValuePtr, idx))
				{
					UE_LOGF(LogPropertyUtility, Log, "Property: %ls Idx: %d", *Property->GetName(), idx);
					LogStructRecursive(ElemPtr, InnerStruct->Struct);
					++idx;
				}
			}
			else
			{
				int idx = 0;
				while (void* ElemPtr = ArrayProp->GetValueAddressAtIndex_Direct(InnerStruct, PropValuePtr, idx))
				{
					UE_LOGF(LogPropertyUtility, Log, "Property: %ls Idx: %d", *Property->GetName(), idx);
					LogPropertyValue(ElemPtr, ArrayProp->Inner);
				}
			}
			UE_LOGF(LogPropertyUtility, Log, "] ~%ls", *Property->GetName());
		}
		else
		{
			LogPropertyValue(PropValuePtr, Property);
		}
	}
}

#endif
