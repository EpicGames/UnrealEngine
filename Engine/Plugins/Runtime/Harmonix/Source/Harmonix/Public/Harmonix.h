// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Modules/ModuleManager.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "Harmonix.generated.h"

#define UE_API HARMONIX_API

class FHarmonixModule : public IModuleInterface
{
public:
	static inline FHarmonixModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FHarmonixModule>("Harmonix");
	}

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/* 
	 * User Experience Offset Ms is the number of milliseconds between the "smoothed 
	 * music render time" and the time the player experiences hearing the rendered audio, makes
	 * an input, and we detect it in the game code. This is time spent buffering audio, 
	 * marshaling it to the operating system, operating system buffering, DAC, home theater, 
	 * TV's audio system, transmission through the air, controller detecting input, sending
	 * that input back to the console, console OS sending it to game code, etc. This should be
	 * a positive number, otherwise the player is somehow hearing the audio BEFORE it is even
	 * rendered. So the player is experiencing...
	 * ExperiencedAudioTime = SmoothedMusicRenderMs - User Experience Offset. 
	 * (See more detail below)
	 */
	static void SetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs(float Milliseconds) { Get().AudioLatencyMs = Milliseconds; }
	static float GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs() { return Get().AudioLatencyMs; }

	/*
	 * Video Render Offset Ms is the number of milliseconds between the "smoothed music render time" and
	 * when graphics should be rendered that correspond to that time, so that the player experiences
	 * SEEING musically synchronized visual effects at the same moment they experience HEARING the
	 * associated musical moment.
	 * AudioTimeToRenderGraphicsFor = SmoothedMusicRenderMs - Video Render Offset MS
	 * (See more detail below)
	 */
	static void SetMeasuredVideoToAudioRenderOffsetMs(float Milliseconds) { Get().VideoLatencyMs = Milliseconds; }
	static float GetMeasuredVideoToAudioRenderOffsetMs() { return Get().VideoLatencyMs; }

	/*
	 * Typically the calibration flow in a game works using one of two methods...
	 ********************************************************************************************************
	 * METHOD 1 - THE OLD WAY (This is less optimal than METHOD 2, but provided for history)
	 *    
	 *     STEP 1 - User Experience Offset
	 *     
	 *     - Play a piece of music that is a simple metronome. Not too fast, or the beats will be 
	 *       too close together and it will be impossible to measure long latencies. Not too slow
	 *       or it will be very hard for the player to keep time.
	 *     
	 *     - Ask the player to press a button on their controller when they hear the beats. Each
	 *       time they press the button, measure the time between the 'smoothed audio render time'
	 *       at that moment and the time at which the nearest beat appears in the music being played. 
	 *       (Be sure to account for the fact that the player will sometimes be early, and sometimes
	 *       be late.)
	 *     
	 *     - Do some standard statistical stuff to determine the average amount of offset between 
	 *       rendering a musical beat, and the player experiencing that beat and pressing the button.
	 *       set that as the user experience offset.
	 *     
	 *     Now the Harmonix system can provide an "ExperiencedAudioTime" which is the smoothed current
	 *     audio render time MINUS (remember, the player will be hearing something rendered in the past)
	 *     the user experience offset.
	 *     
	 *     STEP 2 - Video Offset
	 *     
	 *     - Again play the music that is the simple metronome but DO NOT let the player hear it!
	 *       instead, use the SmoothedAudioRenderTime (just as in step 1) to determine when to render
	 *       some on-screen visual indication of a beat... like a standard swinging metronome bar, 
	 *       or flashing lights.
	 *     
	 *     - Ask the plater to press a button on their controller when they SEE the beats. Each
	 *       time they press the button, measure the time between the SmoothedAudioRenderTime at the
	 *       moment of the press, and the SmoothedAudioRenderTime of the last rendered, or about to 
	 *       be rendered (they could be "early swingers"), musically synchronized visual effect.
	 *    
	 *     - Do some standard statistical stuff to determine the average amount of offset between
	 *       SmoothedAudioRenderTime and the player experiencing the visual effect and pressing the
	 *       button. Set that as the video offset.
	 *     
	 *     Now the Harmonix system can provide an "VideoRenderTime" which is the "SmoothedAudioRenderTime" 
	 *     MINUS the video offset. 
	 * 
	 ********************************************************************************************************
	 * METHOD 2 - THE NEW WAY (This one is better)
	 *    
	 *     STEP 1 - Video Offset
	 *     
	 *     - Play a piece of music that is a simple metronome playing a series of 4 notes, one per beat.
	 *       Make the pitch of each note lower than the previous. This way the player can hear the rhythm
	 *       of the notes AND detect the long 4 beat cycle. This is important for the proper detection
	 *       of very long latencies. 
	 * 
	 *     - Set the measured video offset to 0.
	 * 
	 *     - Render something like a bouncing ball going down 4 steps. Use the "VideoRenderTime" to
	 *       to determine where the ball should be drawn so that it bounces down the steps on the beats. 
	 *       The visuals and the audio WILL NOT be in sync. To make it very obvious when the ball is 
	 *       hitting the ground show a particle effect or flash a light or something.
	 * 
	 *     - Give the user some way of adjusting a number that you set as the Video Offset Ms. As
	 *       they adjust this, and you send it to the API below, they will see the visuals shifting
	 *       relative to what they are hearing, and they should be able to find a number where the 
	 *       visuals and audio lines up. 
	 * 
	 *     This is your Video Offset Ms.
	 *     
	 *     - Now ask the player to press buttons (or key on their keyboard) to correspond to the bounces. 
	 *       Use a different input for each of the 4 bounces so that you can avoid the "off by one" error
	 *       that can happen on systems with very long latencies!
	 * 
	 *     - Now, each time they press a button, measure the time between the 'smoothed audio render time'
	 *       at that moment, and the actual musical time of the beat they were trying to hit. (Be sure 
	 *       to account for the fact that the player will sometimes be early, and sometimes be late.)
	 *     
	 *     - Do some standard statistical stuff to determine the average amount of offset between 
	 *       rendering a musical beat, and the player experiencing that beat and pressing the button.
	 *       set that as the user experience offset.
	 *     
	 *     Now the Harmonix system can provide an "ExperiencedAudioTime" which is the smoothed current
	 *     audio render time MINUS (remember, the player will be hearing something rendered in the past)
	 *     the user experience offset.
	 */

private:
	float AudioLatencyMs = 0.0f;
	float VideoLatencyMs = 0.0f;
};

UCLASS(MinimalAPI)
class UHarmonixBlueprintUtil : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "Harmonix|Calibration")
	static void SetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs(float Milliseconds)
	{
		FHarmonixModule::SetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs(Milliseconds);
	}
	UFUNCTION(BlueprintCallable, Category = "Harmonix|Calibration")
	static float GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs()
	{
		return FHarmonixModule::GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs();
	}
	UFUNCTION(BlueprintCallable, Category = "Harmonix|Calibration")
	static void SetMeasuredVideoToAudioRenderOffsetMs(float Milliseconds)
	{
		FHarmonixModule::SetMeasuredVideoToAudioRenderOffsetMs(Milliseconds); 
	}
	UFUNCTION(BlueprintCallable, Category = "Harmonix|Calibration")
	static float GetMeasuredVideoToAudioRenderOffsetMs()
	{
		return FHarmonixModule::GetMeasuredVideoToAudioRenderOffsetMs(); 
	}
};

#undef UE_API
