// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioProxyInitializer.h"

namespace Harmonix
{
	template<typename DATA_STRUCT, typename REFCOUNT_TYPE> class TSharedAudioRenderableDataPtr;
	template<typename DATA_STRUCT> class TGameThreadToAudioRenderThreadSettingQueue;

	// This macro is short hand for declaring a `TAudioRenderableProxy` that wraps
	// arround your "RootStructName" audio data type.
#define USING_AUDIORENDERABLE_PROXY(RootStructName, ProxyName) using ProxyName = Harmonix::TAudioRenderableProxy<RootStructName, Harmonix::TGameThreadToAudioRenderThreadSettingQueue<RootStructName>>;
	
	// This next macro needs to be added to the struct containing the settings 
	// (UPROPERTIES) you want to share between the UObject asset and the Metasound
	// Node. This so that later we can easily make the proxy with a template class. 
#define IMPL_AUDIORENDERABLE_PROXYABLE(StructName)   \
	static FName GetAudioProxyTypeName()             \
	{                                                \
		static FName MyProxyClassName = #StructName; \
		return MyProxyClassName;                     \
	}

	// This next class implements an intrusive ref counted wrapper around your audio renderable data.
	// It can be shared between the UObject thread(s) and the audio rendering thread(s).
	// In addition to the ref count, this version includes a TSharedAudioRenderableDataPtr to the
	// *next*, or *updated*, data. This allows for a lock-free queue of updated settings, where 
	// new settings can be generated on the UObject side and appended to this as new settings.
	// This way, consumers of these settings can look for updated data and move to that data at
	// their leisure. Because this is done through a ref counted pointer, eventually, when no
	// rendering instance is using an old instance of the data, the ref count will go to zero
	// and the unused settings will be freed.
	template<typename DATA_STRUCT>
	class TRefCountedAudioRenderableWithQueuedChanges
	{
	public:
		// ******************************************************************************************
		// FIRST: The Six...
		// Empty Construct
		TRefCountedAudioRenderableWithQueuedChanges()
			: Data()
			, RefCount(1)
			, QueuedDataUpdate()
		{}
		// Copy Construct
		TRefCountedAudioRenderableWithQueuedChanges(const TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>& Other)
			: Data(Other.Data)
			, RefCount(1)
			, QueuedDataUpdate(Other.QueuedDataUpdate)
		{}
		// Copy Assign
		TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>& operator=(const TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>& Other)
		{
			if (this != &Other)
			{
				Data = Other.Data;
				QueuedDataUpdate = Other.QueuedDataUpdate;
			}
			return *this;
		}
		// Move Construct
		TRefCountedAudioRenderableWithQueuedChanges(TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>&& Other) noexcept
			: Data(MoveTemp(Other.Data))
			, QueuedDataUpdate(MoveTemp(Other.QueuedDataUpdate))
		{
			RefCount.store(Other.RefCount.load());
			Other.RefCount = 0;
		}
		// Move Assign
		TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>& operator=(TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>&& Other) noexcept
		{
			Data = MoveTemp(Other.Data);
			RefCount.store(Other.RefCount.load());
			Other.RefCount = 0;
			QueuedDataUpdate = MoveTemp(Other.QueuedDataUpdate);
			return *this;
		}
		// Destruct
		~TRefCountedAudioRenderableWithQueuedChanges()
		{
			check(RefCount == 0);
		}

		// ******************************************************************************************
		// NOW: Some Specials...
		// New With Arguments For DATA_STRUCT
		template<typename ... TArgs>
		TRefCountedAudioRenderableWithQueuedChanges(TArgs&&... Args) noexcept
			: Data(Forward<TArgs>(Args)...)
			, RefCount(1)
		{}
		// Copy Construct From const DATA_STRUCT reference
		TRefCountedAudioRenderableWithQueuedChanges(const DATA_STRUCT& Other)
			: Data(Other)
			, RefCount(1)
			, QueuedDataUpdate()
		{}
		// Copy Construct From non-const DATA_STRUCT reference
		TRefCountedAudioRenderableWithQueuedChanges(DATA_STRUCT& Other)
			: Data(Other)
			, RefCount(1)
			, QueuedDataUpdate()
		{}
		// Copy Assign From DATA_STRUCT
		TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>& operator=(const DATA_STRUCT& Other)
		{
			Data = Other;
			QueuedDataUpdate = nullptr;
			return *this;
		}
		// Move Construct From DATA_STRUCT
		TRefCountedAudioRenderableWithQueuedChanges(DATA_STRUCT&& Other) noexcept
			: Data(MoveTemp(Other))
			, RefCount(1)
			, QueuedDataUpdate()
		{}
		// Move Assign From DATA_STRUCT
		TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>& operator=(DATA_STRUCT&& Other) noexcept
		{
			Data = MoveTemp(Other);
			QueuedDataUpdate = nullptr;
			return *this;
		}

		// ******************************************************************************************
		// The Wrapped Data
	private:
		DATA_STRUCT Data;
	public:
		// Conversion operators
		FORCEINLINE operator DATA_STRUCT*()
		{
			return &Data;
		}

		FORCEINLINE operator const DATA_STRUCT*() const
		{
			return &Data;
		}

		// Dereference operators
		FORCEINLINE DATA_STRUCT* operator->()
		{
			return &Data;
		}
		FORCEINLINE const DATA_STRUCT* operator->() const
		{
			return &Data;
		}

		// ******************************************************************************************
		// Comparisons
		FORCEINLINE bool operator==(const TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>& Other) const
		{
			return Data == Other.Data;
		}
		FORCEINLINE bool operator==(const DATA_STRUCT& Other) const
		{
			return Data == Other;
		}

		// ******************************************************************************************
		// Reference Counting
		// Note: RefCount is mutable and AddRef/RemoveRef are marked const as const accessors 
		//       should be able to "lock and unlock" the counted object when they only have a 
		//       const reference to it.
	private:
		mutable std::atomic<uint32> RefCount;
	public:
		FORCEINLINE void AddRef() const
		{
#if defined(_MSC_VER) && (defined(_M_X64) || defined(_M_IX86))
			// We do a regular SC increment here because it maps to an _InterlockedIncrement (lock inc).
			// The codegen for a relaxed fetch_add is actually much worse under MSVC (lock xadd).
			++RefCount;
#else
			RefCount.fetch_add(1, std::memory_order_relaxed);
#endif
		}
		FORCEINLINE void RemoveRef() const
		{
			uint32 OldCount = RefCount.fetch_sub(1, std::memory_order_release);
			checkSlow(OldCount > 0);
			if (OldCount == 1)
			{
				// Ensure that all other threads' accesses to the object are visible to this thread before we call the
				// destructor.
				std::atomic_thread_fence(std::memory_order_acquire);

				// Disable this if running clang's static analyzer. Passing shared pointers
				// and references to functions it cannot reason about, produces false
				// positives about use-after-free in the TSharedPtr/TSharedRef destructors.
#if !defined(__clang_analyzer__)
					// No more references to this reference count.  Destroy it!
				delete this;
#endif
			}
		}
		FORCEINLINE int32 GetRefCount()
		{
			return RefCount;
		}

	private:
		TSharedAudioRenderableDataPtr<DATA_STRUCT, TRefCountedAudioRenderableWithQueuedChanges> QueuedDataUpdate;
	public:
		void QueueUpdate(const TSharedAudioRenderableDataPtr<DATA_STRUCT, TRefCountedAudioRenderableWithQueuedChanges>& Update)
		{
			if (!QueuedDataUpdate)
			{
				QueuedDataUpdate = Update;
				return;
			}
			TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>* Walker = QueuedDataUpdate;
			
			while (Walker->HasUpdate())
			{
				Walker = Walker->QueuedDataUpdate;
			}
			Walker->QueuedDataUpdate = Update;
		}
		FORCEINLINE bool HasUpdate() const
		{
			return QueuedDataUpdate;
		}

		FORCEINLINE const TSharedAudioRenderableDataPtr<DATA_STRUCT, TRefCountedAudioRenderableWithQueuedChanges>& GetUpdate() const
		{
			return QueuedDataUpdate;
		}
	};

	// This next class is a shared pointer to an instance of the audio renderable data above.
	template<typename DATA_STRUCT, typename REFCOUNT_TYPE>
	class TSharedAudioRenderableDataPtr
	{
	public:
		// ******************************************************************************************
		// Static Create Functions
		static TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE> CreateSharedRenderable(const DATA_STRUCT& Source)
		{
			return TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>(new REFCOUNT_TYPE(Source));
		}
		static TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE> CreateSharedRenderable(DATA_STRUCT& Source)
		{
			return TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>(new REFCOUNT_TYPE(Source));
		}
		template<typename... TArgs>
		static TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE> CreateSharedRenderable(TArgs&& ... Args)
		{
			return TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>(new REFCOUNT_TYPE(Forward<TArgs>(Args)...));
		}

		// ******************************************************************************************
		// FIRST: The Six...
		// Empty Construct
		TSharedAudioRenderableDataPtr()
			: Renderable(nullptr)
		{}
		// Copy Construct
		TSharedAudioRenderableDataPtr(const TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>& Other)
			: Renderable(Other.Renderable)
		{
			if (Renderable)
			{
				Renderable->AddRef();
			}
		}
		// Copy Assign
		TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>& operator=(const TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>& Other)
		{
			if (this != &Other)
			{
				REFCOUNT_TYPE* OldRenderable = Renderable;
				Renderable = Other.Renderable;
				if (Renderable)
				{
					Renderable->AddRef();
				}
				if (OldRenderable)
				{
					OldRenderable->RemoveRef();
				}
			}
			return *this;
		}
		// Move Construct
		TSharedAudioRenderableDataPtr(TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>&& Other) noexcept
			: Renderable(MoveTemp(Other.Renderable))
		{}
		// Move Assign
		TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>& operator=(TSharedAudioRenderableDataPtr<DATA_STRUCT, REFCOUNT_TYPE>&& Other) noexcept
		{
			REFCOUNT_TYPE* OldRenderable = Renderable;
			Renderable = MoveTemp(Other.Renderable);
			Other.Renderable = nullptr;
			if (OldRenderable)
			{
				OldRenderable->RemoveRef();
			}
			return *this;
		}
		// Destruct
		~TSharedAudioRenderableDataPtr()
		{
			if (Renderable)
			{
				Renderable->RemoveRef();
				Renderable = nullptr;
			}
		}

		// From a new'd instance of AudioRenderableData or nullptr...
		TSharedAudioRenderableDataPtr(REFCOUNT_TYPE* SharedStruct)
			: Renderable(SharedStruct)
		{
			if (Renderable)
			{
				check (Renderable->GetRefCount() == 1);
			}
		}

		TSharedAudioRenderableDataPtr(const DATA_STRUCT& Source)
			: Renderable(new REFCOUNT_TYPE(Source))
		{}

		// ******************************************************************************************
		// The Wrapped Data
		// Conversion Operators...
		FORCEINLINE operator DATA_STRUCT* ()
		{
			return static_cast<DATA_STRUCT*>(*Renderable);
		}
		FORCEINLINE operator const DATA_STRUCT* () const
		{
			return static_cast<const DATA_STRUCT*>(*Renderable);
		}
		FORCEINLINE operator REFCOUNT_TYPE* ()
		{
			return Renderable;
		}
		FORCEINLINE operator const REFCOUNT_TYPE* () const
		{
			return Renderable;
		}
		// Dereference Operators...
		FORCEINLINE DATA_STRUCT* operator->()
		{
			return *Renderable;
		}
		FORCEINLINE const DATA_STRUCT* operator->() const
		{
			return *Renderable;
		}
		FORCEINLINE REFCOUNT_TYPE* operator*()
		{
			return Renderable;
		}
		FORCEINLINE const REFCOUNT_TYPE* operator*() const
		{
			return Renderable;
		}
		// Comparisons...
		FORCEINLINE operator bool()
		{
			return Renderable != nullptr;
		}
		FORCEINLINE operator bool() const
		{
			return Renderable != nullptr;
		}
		FORCEINLINE bool operator==(const TSharedAudioRenderableDataPtr& Other) const
		{
			return Renderable == *Other;
		}
		FORCEINLINE bool operator!=(const TSharedAudioRenderableDataPtr& Other) const
		{
			return Renderable != *Other;
		}

	private:
		REFCOUNT_TYPE* Renderable = nullptr;
	};

	// This next class is a "maintainer" of a queue of audio renderable data structs. This 
	// level of indirection is required because of the way Metasound's audio proxy system 
	// works. This small class just holds a shared pointer to the current tail of the 
	// change queue... so the current version of the settings. Any new rendering instance that
	// might come alive asks the asset's UObject for a unique ptr to a proxy. That proxy is 
	// actually a thin wrapper around one of these. So all rendering instamces see this same
	// proxy, and can get the current setting tail when they are instantiated.
	template<typename DATA_STRUCT>
	class TGameThreadToAudioRenderThreadSettingQueue
	{
	public:
		using NodeType = TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>;
		using SharedNodePtrType = TSharedAudioRenderableDataPtr<DATA_STRUCT, NodeType>;

		TGameThreadToAudioRenderThreadSettingQueue() = default;
		TGameThreadToAudioRenderThreadSettingQueue(const TGameThreadToAudioRenderThreadSettingQueue<DATA_STRUCT>& Other) = default;
		TGameThreadToAudioRenderThreadSettingQueue<DATA_STRUCT>& operator=(const TGameThreadToAudioRenderThreadSettingQueue<DATA_STRUCT>& Other) = default;
		TGameThreadToAudioRenderThreadSettingQueue(TGameThreadToAudioRenderThreadSettingQueue<DATA_STRUCT>&& Other) noexcept = default;
		TGameThreadToAudioRenderThreadSettingQueue<DATA_STRUCT>& operator=(TGameThreadToAudioRenderThreadSettingQueue<DATA_STRUCT>&& Other) = default;
		~TGameThreadToAudioRenderThreadSettingQueue() = default;

		TGameThreadToAudioRenderThreadSettingQueue(const DATA_STRUCT& InitialSettings)
			: Tail(InitialSettings)
		{}

		FORCEINLINE operator bool() const
		{
			return Tail;
		}

		FORCEINLINE DATA_STRUCT* operator->()
		{
			// CurrentSettings will cast to DATA_STRUCT*
			return Tail;
		}

		FORCEINLINE const DATA_STRUCT* operator->() const
		{
			// CurrentSettings will cast to DATA_STRUCT*
			return Tail;
		}

		FORCEINLINE SharedNodePtrType& operator*()
		{
			// CurrentSettings will cast to DATA_STRUCT*
			return Tail;
		}

		FORCEINLINE const SharedNodePtrType& operator*() const
		{
			// CurrentSettings will cast to DATA_STRUCT*
			return Tail;
		}

		FORCEINLINE operator SharedNodePtrType& ()
		{
			return Tail;
		}

		FORCEINLINE operator const SharedNodePtrType& () const
		{
			return Tail;
		}

		FORCEINLINE operator DATA_STRUCT* ()
		{
			return Tail;
		}

		FORCEINLINE operator const DATA_STRUCT* () const
		{
			return Tail;
		}

		FORCEINLINE void SetNewSettings(const SharedNodePtrType& NewSettings)
		{
			if (Tail)
			{
				TRefCountedAudioRenderableWithQueuedChanges<DATA_STRUCT>* Renderable = Tail;
				Renderable->QueueUpdate(NewSettings);
			}
			Tail = NewSettings;
		}

		FORCEINLINE void SetNewSettings(const DATA_STRUCT& NewSettings)
		{
			auto NewSharedRenderable = SharedNodePtrType::CreateSharedRenderable(NewSettings);
			SetNewSettings(NewSharedRenderable);
		}

	private:
		SharedNodePtrType Tail;
	};

	// And this next class simplifies that creation of the Audio::TProxyData that wrapps all of the above.
	template<typename DATA_STRUCT, typename QUEUE_TYPE>
	class TAudioRenderableProxy : public Audio::TProxyData<TAudioRenderableProxy<DATA_STRUCT, QUEUE_TYPE>>
	{
	public:
		using QueueType = QUEUE_TYPE;
		using NodePtr   = typename QUEUE_TYPE::SharedNodePtrType;

		//IMPL_AUDIOPROXY_CLASS(TAudioRenderableProxy);
		static FName GetAudioProxyTypeName()
		{
			return DATA_STRUCT::GetAudioProxyTypeName();
		}
		static FName GetInterfaceId() { return GetAudioProxyTypeName(); }
		virtual void* QueryInterface(const FName InterfaceId) override
		{
			if (InterfaceId == GetAudioProxyTypeName())
			{
				return this;
			}
			return Audio::IProxyData::QueryInterface(InterfaceId);
		}
		static constexpr bool bWasAudioProxyClassImplemented = true;
		friend class ::Audio::IProxyData;
		friend class ::Audio::TProxyData<TAudioRenderableProxy<DATA_STRUCT, QUEUE_TYPE>>;

		explicit TAudioRenderableProxy(const TSharedPtr<QUEUE_TYPE>& InSettingsQueue)
			: SettingsQueue(InSettingsQueue)
		{
		}

		TAudioRenderableProxy(const TAudioRenderableProxy<DATA_STRUCT, QUEUE_TYPE>& Other) = default;

		NodePtr GetRenderable()
		{
			if (!SettingsQueue.IsValid())
			{
				return NodePtr();
			}
			return *SettingsQueue;
		}

	private:
		TSharedPtr<QUEUE_TYPE> SettingsQueue;
	};
}
