// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDeveloperSettings.generated.h"

// base class for all settings you want to group with this plugin
// allows configuration to appear in the "Harmonix"
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UHarmonixDeveloperSettings : public UObject
{
	GENERATED_BODY()
};
