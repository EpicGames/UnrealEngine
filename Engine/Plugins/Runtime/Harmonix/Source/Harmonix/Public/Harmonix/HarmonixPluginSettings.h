// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "HarmonixPluginSettings.generated.h"

#define UE_API HARMONIX_API

UCLASS(MinimalAPI, config = Engine, defaultconfig, Meta = (DisplayName = "Harmonix"))
class UHarmonixPluginSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

public:

#if WITH_EDITOR
	//~ UDeveloperSettings interface
	UE_API virtual FText GetSectionText() const override;
#endif

	UE_API UHarmonixPluginSettings();
};

#undef UE_API
