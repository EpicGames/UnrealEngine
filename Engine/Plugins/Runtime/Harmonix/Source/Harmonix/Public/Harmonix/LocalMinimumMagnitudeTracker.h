// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/UnrealMathUtility.h"
#include <limits>

template <typename T, int32 SIZE>
class FLocalMinimumMagnitudeTracker
{
public:
	FLocalMinimumMagnitudeTracker() { Reset(); }

	void Reset()
	{
		Ring[0] = 0;
		NextWrite = 0;
		MinPosition = 0;
		AccumulatedError = 0;
		Wrapped = false;
	}

	void Push(T v)
	{
		if (Wrapped)
		{
			AccumulatedError -= Ring[NextWrite];
		}
		AccumulatedError += v;
		Ring[NextWrite] = v;
		if (NextWrite == MinPosition)
		{
			// search for new lowest...
			MinPosition = (NextWrite + 1) % SIZE;
			for (int32 i = 1; i < SIZE; ++i)
			{
				int32 testIdx = (NextWrite + 1 + i) % SIZE;
				if (FMath::Abs(Ring[testIdx]) <= FMath::Abs(Ring[MinPosition]))
					MinPosition = testIdx;
			}
		}
		else if (FMath::Abs(v) <= FMath::Abs(Ring[MinPosition]))
		{
			MinPosition = NextWrite;
		}
		Wrapped = Wrapped || (NextWrite + 1) >= SIZE;
		NextWrite = (NextWrite + 1) % SIZE;
	}

	T Min() const { return Ring[MinPosition]; }

	T Average() const 
	{
		int32 Count = Wrapped ? (T)SIZE : (T)NextWrite;
		if (Count == 0)
		{
			return 0;
		}
		return AccumulatedError / (T)Count;
	}

private:
	T Ring[SIZE];
	int32 NextWrite = 0;
	int32 MinPosition = 0;
	T AccumulatedError = 0;
	bool  Wrapped = false;
};
