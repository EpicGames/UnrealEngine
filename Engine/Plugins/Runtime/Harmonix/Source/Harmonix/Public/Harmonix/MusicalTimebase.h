// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "Misc/EnumRange.h"

#include "MusicalTimebase.generated.h"

UENUM(BlueprintType)
enum class ECalibratedMusicTimebase : uint8
{
	/*
	 * Tells you almost exactly where the audio renderer is right now (smoothed version of the jittery, 
	 * raw position of the audio rendering). Useful for queuing up musical events based on the current
	 * song time.
	 */
	AudioRenderTime,
	
	/*
	 * Tells you what the player is actually hearing & seeing this instant (when properly calibrated). 
	 * Useful for scoring player input.
	 */
	ExperiencedTime,

	/*
	 * Tells you what you should be drawing right now so it appears *in sync with the music.* (when properly calibrated).
	 * Useful for synchronizing animations, ui, and other visuals to the music.
	 */
	VideoRenderTime,

	/*
	 * Tells you exactly where the audio renderer is right now. NOTE: Un-smoothed, jittery,
	 * raw position of the audio rendering. Only useful for debugging!
	 */
	RawAudioRenderTime,

	Count UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(ECalibratedMusicTimebase, ECalibratedMusicTimebase::Count);
