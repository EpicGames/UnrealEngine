// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Logging/LogMacros.h"

#if WITH_EDITORONLY_DATA

#include "UObject/UnrealType.h"

DEFINE_LOG_CATEGORY_STATIC(LogPropertyUtility, Log, Log)


namespace Harmonix
{

enum EPostEditAction : uint8
{
	DoNothing,
	UpdateTrivial,
	UpdateNonTrivial
};

enum EPostEditType : uint8
{
	None,
	Trivial,
	NonTrivial
};

HARMONIX_API
EPostEditAction GetPropertyPostEditAction(const FProperty* Property, EPropertyChangeType::Type ChangeType, EPostEditAction DefaultAction = EPostEditAction::UpdateNonTrivial);

HARMONIX_API 
EPostEditType GetPropertyPostEditType(const FProperty* Property);

// Avoid using these methods directly. Use the template versions
void LogPropertyValue(void* PropValuePtr, FProperty* Property);
bool CopyStructRecursive(void* InDest, void* InSrc, const UScriptStruct* ScriptStruct);
void LogStructRecursive(void* PropPtr, const UScriptStruct* ScriptStruct);

/**
 * Given a pointer to a valid struct member,
 * walk the PropertyChain given the PropertyChangedChainEvent
 * constructing an FString representing the property chain. 
 * 
 * For example:
 * 
 *		FBaz
 *		{
 *			int Number = 4;
 *		}
 * 
 *		FBar
 *		{
 *			TArray<FBaz> BazArray;
 *		}
 * 
 * 
 *		UMyObject
 *		{
 *			FBar MyBar;
 * 
 *			virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override
 *			{
 *				FString PropertyChangedString = Harmonix::GetStructPropertyChainString(&MyBar, PropertyChangedChainEvent);
 *			}
 *		}
 * 
 *	if MyBar.BazArray[3].Number was changed from 4 to 6 for example, then the resulting string will be
 * 
 *	- "MyBar.BazArray[3].Number = 6"
 * 
 * ensures that the FProperty at the head of the PropertyChain is the same type as the struct passed in
 * 
 * @param StructPtr					- Src Data ptr of type TStructType
 * @param PropertyChangedChainEvent - Property chain that should correspond to the Struct type passed in. 
 *									  The Tail of the property chain should be a copiable value type.
 */
template<typename TStructType>
FString GetStructPropertyChainString(TStructType* StructPtr, const FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	if (!ensureAlways(StructPtr))
		return FString();

	auto Head = PropertyChangedChainEvent.PropertyChain.GetHead();

	if (FStructProperty* StructProp = CastField<FStructProperty>(Head->GetValue()))
	{
		// walk property values of struct that doesn't match the type of the property chain head
		if (!ensureAlways(StructProp->Struct->IsA(TStructType::StaticStruct()->StaticClass())))
		{
			return FString();
		}
	}
	else
	{
		ensureAlways(false);
		return FString();
	}

	FString PropertyPathString = Head->GetValue()->GetName();

	// start with the next node
	void* ValuePtr = (void*)StructPtr;
	auto Node = Head->GetNextNode();
	while (Node)
	{
		FProperty* Property = Node->GetValue();

		ValuePtr = Property->ContainerPtrToValuePtr<void>(ValuePtr);
		if (!ensureAlways(ValuePtr))
		{
			break;
		}

		int Index = PropertyChangedChainEvent.GetArrayIndex(Property->GetName());

		if (Index != INDEX_NONE)
		{
			PropertyPathString += FString::Printf(TEXT(".%s[%d]"), *Property->GetName(), Index);
		}
		else
		{
			PropertyPathString += FString::Printf(TEXT(".%s"), *Property->GetName());
		}


		if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
		{
			ValuePtr = ArrayProp->GetValueAddressAtIndex_Direct(ArrayProp->Inner, ValuePtr, Index);
			if (!ensureAlways(ValuePtr))
			{
				break;
			}
		}
		else if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			// we don't have to do anything here
		}
		else
		{
			// we've found the value that changed
			// (Hopefully)
			ensureAlways(Node == PropertyChangedChainEvent.PropertyChain.GetTail());
			FString ValueString;
			Property->ExportTextItem_Direct(ValueString, ValuePtr, nullptr, nullptr, 0);
			PropertyPathString += FString::Printf(TEXT(" = %s"), *ValueString);
		}

		Node = Node->GetNextNode();
	}

	return PropertyPathString;
}


/**
 * Given a pointer to a valid struct member,
 * will recursively iterate over all the properties in the struct
 * and all child structs and arrays
 * and log the values in each
 * 
 * logs each value on a new line
 *
 * For example:
 *
 *		FBaz
 *		{
 *			int Number = 4;
 *		}
 *
 *		FBar
 *		{
 *			TArray<FBaz> BazArray;
 *		}
 *
 *
 *		UMyObject
 *		{
 *			FBar MyBar;
 * 
 *			void LogProperty()
 *			{
 *				MyBar.BazArray.Add(FBaz());
 *				MyBar.BazArray.Add(FBaz());
 * 
 *				MyBar.BazArray[0].Number = 3;
 *				MyBar.BazArray[1].Number = 10;
 * 
 *				Harmonix::LogStructProperties(&MyBar);
 *			}
 *		}
 * 
 * Log: MyBar {
 * Log: BazArray [
 * Log: Baz {
 * Log:    Number = 3
 * Log: } ~Baz
 * Log: Baz {
 * Log:    Number = 10
 * Log: } ~Baz
 * Log: ] ~BazArray
 * Log: } ~MyBar
 *
 * ensures that the FProperty at the head of the PropertyChain is the same type as the struct passed in
 */
template<typename TStructType>
void LogStructProperties(TStructType* StructPtr)
{
	if (!ensure(StructPtr))
	{
		return;
	}
	LogStructRecursive(StructPtr, TStructType::StaticStruct());
}

/**
 * Given pointers to valid structs
 * walks the PropertyChain given the PropertyChangedChainEvent
 * copying the property that was changed from the Source Struct to the Dest Struct
 * 
 * Assumes that the number of elements in each struct and child struct arrays are identical
 *
 * For example:
 *
 *		FBaz
 *		{
 *			int Number = 4;
 *		}
 *
 *		FBar
 *		{
 *			TArray<FBaz> BazArray;
 *		}
 *
 *
 *		UMyObject
 *		{
 *			FBar MyBar;
 *			FBar MyBarProxy;
 *
 *			virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override
 *			{
 *				Harmonix::CopyStructProperty(&MyBarProxy, &MyBar, PropertyChangedChainEvent);
 *			}
 *		}
 *
 * if MyBar.BazArray[3].Number was changed from 4 to 6 for example, 
 * then the PropertyChangedChainEvent would point to that property that was changed.
 * 
 * So the value in MyBar.BazArray[3].Number will be copied into MyBarProxy.BazArray[3].Number;
 * 
 * ensures that the FProperty at the head of the PropertyChain is the same type as the struct passed in
 * 
 * returns whether the copy was successful
 * - Copy can fail if types don't match the PropertyChangedChainEvent
 * - Copy can fail if there is a size mismatch in any child arrays contained in any of the structs
 */
template<typename TStructType>
bool CopyStructProperty(TStructType* InDest, TStructType* InSrc, const FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	if (!ensure(InDest && InSrc))
	{
		UE_LOGF(LogPropertyUtility, Warning, "Can't copy structs. Dest and Src must both be valid pointers");
		return false;
	}

	auto Head = PropertyChangedChainEvent.PropertyChain.GetHead();

	if (FStructProperty* StructProp = CastField<FStructProperty>(Head->GetValue()))
	{
		// can't copy structs that don't match the head of the property chain
		if (!ensure(StructProp->Struct->IsA(TStructType::StaticStruct()->StaticClass())))
		{
			UE_LOGF(LogPropertyUtility, Warning, "Can't copy structs from property chain. Property Chain Head FStructType is of type: %ls, while the src struct is of type: %ls.",
				*StructProp->Struct->GetStructCPPName(), *TStructType::StaticStruct()->GetStructCPPName());
			return false;
		}
	}
	else
	{
		// property head chain isn't a FStructProperty
		ensure(false);
		UE_LOGF(LogPropertyUtility, Warning, "Can't copy structs from property chain. Property Chain head is not a FStructType.");
		return false;
	}

	// start with the next node
	void* SrcValuePtr = (void*)InSrc;
	void* DestValuePtr = (void*)InDest;
	auto Node = Head->GetNextNode();
	while (Node)
	{
		FProperty* Property = Node->GetValue();
		
		SrcValuePtr = Property->ContainerPtrToValuePtr<void>(SrcValuePtr);
		DestValuePtr = Property->ContainerPtrToValuePtr<void>(DestValuePtr);

		if (!SrcValuePtr || !DestValuePtr)
		{
			UE_LOGF(LogPropertyUtility, Warning,
				"Got a nullptr copying struct: \"%ls\"\n\tProperty: \"%ls\"",
				*Head->GetValue()->GetName(), *Property->GetName());
			return false;
		}

		int Index = PropertyChangedChainEvent.GetArrayIndex(Property->GetName());

		if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
		{
			check(Index != INDEX_NONE);

			SrcValuePtr = ArrayProp->GetValueAddressAtIndex_Direct(ArrayProp->Inner, SrcValuePtr, Index);
			DestValuePtr = ArrayProp->GetValueAddressAtIndex_Direct(ArrayProp->Inner, DestValuePtr, Index);

			if (!SrcValuePtr || !DestValuePtr)
			{
				UE_LOGF(LogPropertyUtility, Warning, 
					"Got a nullptr copying struct: \"%ls\"\n\tArrayProperty: \"%ls\", Index: %d", 
					*Head->GetValue()->GetName(), *Property->GetName(), Index);
				return false;
			}
		}
		else if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			// we don't have to do anything here
		}
		else
		{
			// we've found the value that changed
			// (Hopefully)
			if (ensureAlways(Node == PropertyChangedChainEvent.PropertyChain.GetTail()))
			{
				Property->CopySingleValue(DestValuePtr, SrcValuePtr);
				return true;
			}
			return false;
		}

		Node = Node->GetNextNode();
	}
	return false;
}


/**
 * Recursively copies the struct properties from InSrc to InDest
 * 
 * Asusmes
 */
template<typename TStructType>
bool CopyStructProperties(TStructType* InDest, TStructType* InSrc)
{ 
	if (!ensureAlways(InDest && InSrc))
	{
		return false;
	}

	return CopyStructRecursive(InDest, InSrc, TStructType::StaticStruct());
}


}

#endif