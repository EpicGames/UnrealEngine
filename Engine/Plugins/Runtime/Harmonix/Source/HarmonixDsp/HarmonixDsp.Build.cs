// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixDsp : ModuleRules
{
	public HarmonixDsp(ReadOnlyTargetRules Target) : base(Target)
	{
		//OptimizeCode = CodeOptimization.Never;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"HarmonixMidi",
				"AudioExtensions",
				"Harmonix"
			});
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"SignalProcessing"
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("AssetRegistry");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

		AddHmxDefines();
	}

	private void AddHmxDefines()
	{
		PublicDefinitions.Add("SIMD_IN_DEBUG=1");
		PublicDefinitions.Add("FUSION_VOICE_DEBUG_DUMP_ENABLED=0");
	}
}
