// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/AudioAnalysis/AnalysisUtilities.h"

#include "HAL/Platform.h"
#include "HarmonixDsp/AudioAnalysis/OutputSettings.h"

namespace Harmonix::Dsp::AudioAnalysis
{
	float SmoothEnergy(
		float TargetEnergyLinear,
		const float PreviousEnergyLinear,
		const float ElapsedMs,
		const FHarmonixAudioAnalyzerOutputSettings& Settings)
	{
		if (TargetEnergyLinear > PreviousEnergyLinear)
		{
			const int32 RiseBlocks = static_cast<int32>(Settings.RiseMs / ElapsedMs);
			const float MaxRise = RiseBlocks > 1 ? 1.0f / RiseBlocks : 1.0f;
			if (PreviousEnergyLinear + MaxRise < TargetEnergyLinear)
			{
				TargetEnergyLinear = PreviousEnergyLinear + MaxRise;
			}
		}
		else
		{
			const int32 FallBlocks = static_cast<int32>(Settings.FallMs / ElapsedMs);
			const float MaxFall = FallBlocks > 1 ? 1.0f / FallBlocks : 1.0f;
			if (PreviousEnergyLinear - MaxFall > TargetEnergyLinear)
			{
				TargetEnergyLinear = PreviousEnergyLinear - MaxFall;
			}
		}
		return TargetEnergyLinear;
	}

}
