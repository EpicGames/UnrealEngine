// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/AudioAnalysis/FFTAnalyzer.h"

#include "DSP/FFTAlgorithm.h"
#include "DSP/FloatArrayMath.h"
#include "HarmonixDsp/AudioAnalysis/AnalysisUtilities.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FFTAnalyzer)

namespace Harmonix::Dsp::AudioAnalysis
{
	FFFTAnalyzer::FFFTAnalyzer(float InSampleRate) : SampleRate(InSampleRate)
	{
		Reset();
	}
	
	bool SettingsRequireRecalculate(const FHarmonixFFTAnalyzerSettings& Old, const FHarmonixFFTAnalyzerSettings& New)
	{
		return Old.MinFrequencyHz != New.MinFrequencyHz
		|| Old.MaxFrequencyHz != New.MaxFrequencyHz
		|| Old.MelScaleBinning != New.MelScaleBinning
		|| Old.FFTSize != New.FFTSize
		|| Old.NumResultBins != New.NumResultBins;
	}

	// Clean the settings before applying so we get reasonable values and don't crash
	void CleanSettings(FHarmonixFFTAnalyzerSettings& InOutSettings)
	{
		InOutSettings.FFTSize = FMath::Clamp(InOutSettings.FFTSize, FFFTAnalyzer::MinFFTSize, FFFTAnalyzer::MaxFFTSize);
		InOutSettings.MinFrequencyHz = FMath::Clamp(InOutSettings.MinFrequencyHz, FFFTAnalyzer::MinFrequency, FFFTAnalyzer::MaxFrequency);
		InOutSettings.MaxFrequencyHz = FMath::Clamp(InOutSettings.MaxFrequencyHz, FFFTAnalyzer::MinFrequency, FFFTAnalyzer::MaxFrequency);
		InOutSettings.NumResultBins = InOutSettings.NumResultBins > 0
		? FMath::Min(InOutSettings.NumResultBins, InOutSettings.FFTSize / 2)
		: InOutSettings.FFTSize / 2;
		InOutSettings.OutputSettings.RiseMs = FMath::Clamp(InOutSettings.OutputSettings.RiseMs, FFFTAnalyzer::MinSmoothingTime, FFFTAnalyzer::MaxSmoothingTime);
		InOutSettings.OutputSettings.FallMs = FMath::Clamp(InOutSettings.OutputSettings.FallMs, FFFTAnalyzer::MinSmoothingTime, FFFTAnalyzer::MaxSmoothingTime);
	}

	void FFFTAnalyzer::SetSettings(const FHarmonixFFTAnalyzerSettings& InSettings)
	{
		FHarmonixFFTAnalyzerSettings SettingsCopy = InSettings;
		CleanSettings(SettingsCopy);
		
		FScopeLock Lock{ &SettingsGuard };
		NeedsRecalculate = NeedsRecalculate || SettingsRequireRecalculate(Settings, SettingsCopy);
		Settings = SettingsCopy;
	}

	void FFFTAnalyzer::Reset()
	{
		SetSettings(FHarmonixFFTAnalyzerSettings{});
		State = FState{};
	}

	void FFFTAnalyzer::Process(const Audio::FAlignedFloatBuffer& InBuffer, FHarmonixFFTAnalyzerResults& InOutResults)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Harmonix::Dsp::AudioAnalysis::FFFTAnalyzer::Process);
		
		// Make a copy of the settings so we don't hold up another thread
		FHarmonixFFTAnalyzerSettings SettingsCopy;
		{
			bool ShouldRecalculate;
			{
				FScopeLock Lock{ &SettingsGuard };
				SettingsCopy = Settings;
				ShouldRecalculate = NeedsRecalculate;
				NeedsRecalculate = false;
			}

			if (SampleRate <= 0.0f)
			{
				return;
			}

			if (ShouldRecalculate)
			{
				Recalculate(SampleRate, SettingsCopy, State);
			}
		}

		if (!State.FFT.IsValid())
		{
			return;
		}

		// Process however many windows we need to process
		{
			// make the sliding window
			Audio::TAutoSlidingWindow<float> SlidingWindow(*State.SlidingBuffer, InBuffer, State.WindowedBuffer);

			// process each window
			for (TArray<float>& Window : SlidingWindow)
			{
				// Apply the window
				State.Window->ApplyToBuffer(Window.GetData());

				// Perform the FFT and scale the result
				State.FFT->ForwardRealToComplex(Window.GetData(), State.FFTOutput.GetData());
				Audio::ArrayComplexToPower(State.FFTOutput, State.RawSpectrumOutput);
				Audio::ArrayMultiplyByConstantInPlace(State.RawSpectrumOutput, State.FFTScaling);
				Audio::ArraySqrtInPlace(State.RawSpectrumOutput);

				// Arrange the raw output into bins and smooth the result
				const float ElapsedMs = 1000.0f * SettingsCopy.FFTSize / SampleRate;
				if (InOutResults.Spectrum.Num() != SettingsCopy.NumResultBins)
				{
					InOutResults.Spectrum.SetNumZeroed(SettingsCopy.NumResultBins);
				}
				for (int32 i = 0; i < SettingsCopy.NumResultBins; ++i)
				{
					// find the strongest value for the output bin
					float Value = 0.0f;
					int32 StartBin = static_cast<int32>(State.BinRanges[i]);
					const int32 EndBin = static_cast<int32>(State.BinRanges[i + 1]);
					do
					{
						if (State.RawSpectrumOutput[StartBin] > Value)
						{
							Value = State.RawSpectrumOutput[StartBin];
						}
						++StartBin;
					}
					while (StartBin < EndBin);

					// smooth the result
					const float PreviousValue = InOutResults.Spectrum[i];
					const bool IsRising = Value > PreviousValue;
					const float Target = IsRising ? FMath::Max(State.RiseTargets[i], Value) : Value;
					InOutResults.Spectrum[i] = SmoothEnergy(Target, PreviousValue, ElapsedMs, SettingsCopy.OutputSettings);
					// if we didn't make it to the target, remember for next time
					if (IsRising)
					{
						State.RiseTargets[i] = Target > InOutResults.Spectrum[i] ? Target : -1;
					}
					else
					{
						State.RiseTargets[i] = -1;
					}
				}
			}
		}
	}

	float BinToFrequency(const int32 BinIndex, const int32 FFTSize, const float SampleRate)
	{
		return static_cast<float>(BinIndex) * SampleRate / FFTSize;
	}
	
	// http://en.wikipedia.org/wiki/Mel_scale
	float FrequencyToMel(const float Frequency)
	{
		return 1127.01048f * FMath::Loge(1.0f + Frequency / 700.0f);
	}
	
	float MelToFrequency(const float Mel)
	{
		return 700.0f * (FMath::Exp(Mel / 1127.01048f) - 1.0f);
	}

	void ComputeBinRanges(const float SampleRate, const FHarmonixFFTAnalyzerSettings& InSettings, TArray<float>& BinRanges)
	{
		// compute the frequencies corresponding to the underlying FFT bins
		TArray<float> Frequencies;
		Frequencies.SetNumUninitialized(InSettings.FFTSize / 2 + 1);
		for (int32 i = 0; i < Frequencies.Num(); ++i)
		{
			Frequencies[i] = BinToFrequency(i, InSettings.FFTSize, SampleRate);
		}

		// compute the limits in mels, if we're using mel-scale binning
		float MinFreqMels;
		float MaxFreqMels;
		if (InSettings.MelScaleBinning)
		{
			MinFreqMels = FrequencyToMel(InSettings.MinFrequencyHz);
			MaxFreqMels = FrequencyToMel(InSettings.MaxFrequencyHz);
		}

		const int32 NumResultBins = InSettings.NumResultBins < 0 ? InSettings.FFTSize / 2 : InSettings.NumResultBins;
		const int32 NumSrcBins = Frequencies.Num() - 3; // two at the bottom, one at the top
		int32 SrcBinIdx = 2;
		for (int32 DstBinIdx = 0; DstBinIdx <= NumResultBins; DstBinIdx++)
		{
			// compute the desired frequency at this bin boundary
			float DstFrac = static_cast<float>(DstBinIdx) / NumResultBins; //-V609
			float Freq;
			if (InSettings.MelScaleBinning)
			{
				const float FreqMels = FMath::Lerp(MinFreqMels, MaxFreqMels, DstFrac);
				Freq = MelToFrequency(FreqMels);
			}
			else
			{
				// linear distribution of frequencies into bins
				Freq = FMath::Lerp(InSettings.MinFrequencyHz, InSettings.MaxFrequencyHz, DstFrac);
			}

			// scan forward for the next integer bin index which yields a 
			// frequency just after the desired frequency for this bin
			for (; SrcBinIdx < (NumSrcBins - 1); SrcBinIdx++)
			{
				const float NextFreq = Frequencies[SrcBinIdx + 1];
				if (NextFreq > Freq)
				{
					break;
				}
			}

			// determine *fractional* src bin indices
			const float PrevFreq = Frequencies[SrcBinIdx];
			const float NextFreq = Frequencies[SrcBinIdx + 1];
			const float FracIntoBin = (Freq - PrevFreq) / (NextFreq - PrevFreq);
			const float FractionalSrcBinIdx = static_cast<float>(SrcBinIdx) + FracIntoBin;
			BinRanges[DstBinIdx] = FractionalSrcBinIdx;
		}
	}

	void FFFTAnalyzer::Recalculate(const float SampleRate, FHarmonixFFTAnalyzerSettings& InOutSettings, FState& InOutState)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Harmonix::Dsp::AudioAnalysis::FFFTAnalyzer::Recalculate);
		
		// Make the FFT
		{
			Audio::FFFTSettings FFTSettings;
			FFTSettings.bArrays128BitAligned = true;
			FFTSettings.bEnableHardwareAcceleration = true;

			// Figure out the FFT size for the algorithm
			InOutSettings.FFTSize = FMath::RoundUpToPowerOfTwo(InOutSettings.FFTSize);
			InOutSettings.FFTSize = FMath::Clamp(InOutSettings.FFTSize, MinFFTSize, MaxFFTSize);
			FFTSettings.Log2Size = 0;
			while (InOutSettings.FFTSize > (1 << FFTSettings.Log2Size))
			{
				++FFTSettings.Log2Size;
			}

			// Make the FFT
			InOutState.FFT = Audio::FFFTFactory::NewFFTAlgorithm(FFTSettings);

			if (!InOutState.FFT.IsValid())
			{
				return;
			}

			// Calculate forward scaling factor to apply to output
			switch (InOutState.FFT->ForwardScaling())
			{
			case Audio::EFFTScaling::MultipliedByFFTSize:
				InOutState.FFTScaling = 1.0f / (InOutSettings.FFTSize * InOutSettings.FFTSize);
				break;

			case Audio::EFFTScaling::MultipliedBySqrtFFTSize:
				InOutState.FFTScaling = 1.0f / InOutSettings.FFTSize;
				break;

			case Audio::EFFTScaling::DividedByFFTSize:
				InOutState.FFTScaling = InOutSettings.FFTSize * InOutSettings.FFTSize;
				break;

			case Audio::EFFTScaling::DividedBySqrtFFTSize:
				InOutState.FFTScaling = InOutSettings.FFTSize;
				break;

			case Audio::EFFTScaling::None:
			default:
				InOutState.FFTScaling = 1.f;
				break;
			}
		}

		// Resize the buffers
		const int32 NumWindowFrames = InOutSettings.FFTSize;
		InOutState.SlidingBuffer = MakeUnique<Audio::TSlidingBuffer<float>>(NumWindowFrames, NumWindowFrames);
		InOutState.Window = MakeUnique<Audio::FWindow>(Audio::EWindowType::Hamming, NumWindowFrames, 1, false);
		InOutState.WindowedBuffer.SetNumUninitialized(NumWindowFrames);
		InOutState.FFTOutput.SetNumUninitialized(InOutState.FFT->NumOutputFloats());
		InOutState.RawSpectrumOutput.SetNumUninitialized(InOutState.FFTOutput.Num() / 2);
		InOutSettings.NumResultBins = InOutSettings.NumResultBins > 0
		? FMath::Min(InOutSettings.NumResultBins, InOutSettings.FFTSize / 2)
		: InOutSettings.FFTSize / 2;
		InOutState.RiseTargets.SetNumUninitialized(InOutSettings.NumResultBins);
		for (float& RiseTarget : InOutState.RiseTargets)
		{
			RiseTarget = -1;
		}
		InOutState.BinRanges.SetNumUninitialized(InOutSettings.NumResultBins + 1);
		ComputeBinRanges(SampleRate, InOutSettings, InOutState.BinRanges);
	}
}
