// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/AudioAnalysis/WaveFileComparison.h"

#include "DSP/EnvelopeFollower.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/PlatformFileManager.h"
#include "HarmonixDsp/AudioAnalysis/AnalysisUtilities.h"

DEFINE_LOG_CATEGORY_STATIC(LogWaveFileComparison, Log, Log);

namespace Harmonix::Dsp::AudioAnalysis
{
	bool FWaveFileComparison::LoadForCompare(const FString& Wave1FilePath, const FString& Wave2FilePath)
	{
		IPlatformFile& PlatformFileApi = FPlatformFileManager::Get().GetPlatformFile();
		IFileHandle* FileHandle = PlatformFileApi.OpenRead(*Wave1FilePath);
		if (!FileHandle)
		{
			return false;
		}
		TSharedRef<FArchive> Wave1Archive = MakeShared<FArchiveFileReaderGeneric>(FileHandle, *Wave1FilePath, FileHandle->Size());

		FileHandle = PlatformFileApi.OpenRead(*Wave2FilePath);
		if (!FileHandle)
		{
			return false;
		}
		TSharedRef<FArchive> Wave2Archive = MakeShared<FArchiveFileReaderGeneric>(FileHandle, *Wave2FilePath, FileHandle->Size());
		
		return LoadForCompare(*Wave1Archive, *Wave2Archive);
	}

	bool FWaveFileComparison::LoadForCompare(FArchive& Wave1Archive, FArchive& Wave2Archive)
	{
		bOk = false;
		
		if (!Wave1.Load(Wave1Archive))
		{
			UE_LOGF(LogWaveFileComparison, Log, "Failed to load Wave 1.");
			return false;
		}
		
		if (!Wave2.Load(Wave2Archive))
		{
			UE_LOGF(LogWaveFileComparison, Log, "Failed to load Wave 2.");
			return false;
		}

		if (*Wave1.Info.pChannels != *Wave2.Info.pChannels)
		{
			UE_LOGF(LogWaveFileComparison, Log, "Can't compare... different channel counts.");
			return false;
		}

		if (*Wave1.Info.pFormatTag != *Wave2.Info.pFormatTag)
		{
			UE_LOGF(LogWaveFileComparison, Log, "Can't compare... different sample formats.");
			return false;
		}

		if (*Wave1.Info.pBitsPerSample!= *Wave2.Info.pBitsPerSample)
		{
			UE_LOGF(LogWaveFileComparison, Log, "Can't compare... different bits per sample.");
			return false;
		}

		if (*Wave1.Info.pFormatTag != FWaveModInfo::WAVE_INFO_FORMAT_PCM && *Wave1.Info.pFormatTag != FWaveModInfo::WAVE_INFO_FORMAT_IEEE_FLOAT)
		{
			UE_LOGF(LogWaveFileComparison, Log, "Can't compare... samples are not shorts or floats.");
			return false;
		}

		if (*Wave1.Info.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_PCM && *Wave1.Info.pBitsPerSample != 16)
		{
			UE_LOGF(LogWaveFileComparison, Log, "Can't compare... pcm samples are not shorts.");
			return false;
		}

		bOk = true;
		return true;
	}

	float FWaveFileComparison::GetPSNR(bool bCommonSizeOnly) const
	{
		if (!bOk)
		{
			return std::numeric_limits<float>::max();
		}

		if (*Wave1.Info.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_PCM)
		{
			return Harmonix::Dsp::AudioAnalysis::CalculatePSNR<int16>(reinterpret_cast<const int16*>(Wave1.Info.SampleDataStart),
				reinterpret_cast<const int16*>(Wave2.Info.SampleDataStart),
				*Wave1.Info.pChannels,
				FMath::Min(Wave1.Info.GetNumSamples(), Wave2.Info.GetNumSamples()) / *Wave1.Info.pChannels);
		}
		return Harmonix::Dsp::AudioAnalysis::CalculatePSNR<float>(reinterpret_cast<const float*>(Wave1.Info.SampleDataStart),
			reinterpret_cast<const float*>(Wave2.Info.SampleDataStart),
			*Wave1.Info.pChannels,
			FMath::Min(Wave1.Info.GetNumSamples(), Wave2.Info.GetNumSamples()) / *Wave1.Info.pChannels);
	}

	bool FWaveFileComparison::FOneWaveFile::Load(FArchive& Archive)
	{
		if (ensureAlways(Archive.IsLoading()))
		{
			BulkData.SetNumUninitialized(Archive.TotalSize());
			Archive.Serialize(BulkData.GetData(), BulkData.Num());
			if (!Info.ReadWaveInfo(BulkData.GetData(), BulkData.Num()))
			{
				return false;
			}
			return true;
		}
		return false;
	}

}