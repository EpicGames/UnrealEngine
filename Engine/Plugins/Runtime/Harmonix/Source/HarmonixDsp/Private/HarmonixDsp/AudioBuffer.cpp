// Copyright Epic Games, Inc. All Rights Reserved.

// Harmonix
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/Conversions.h"

// UE 
#include "Containers/UnrealString.h"

LLM_DEFINE_TAG(Harmonix_AudioBuffer);

DEFINE_LOG_CATEGORY(LogAudioBuffer);

const FString GetAsciiArtLevel(float X)
{
	float Inc = 1.0 / 8.0;
	// ten levels + two clipping levels
	if (X < -1)
	{
		return "#----<    |";
	}
	if (X < -7.0 * Inc)
	{
		return "|o---<    |";
	}
	if (X < -5.0 * Inc)
	{
		return "| o--<    |";
	}
	if (X < -3.0 * Inc)
	{
		return "|  o-<    |";
	}
	if (X < -1.0 * Inc)
	{
		return "|   o<    |";
	}
	if (X < 0)
	{
		return "|    <    |";
	}
	if (X == 0.0f)
	{
		return "|    .    |";
	}
	if (X < Inc)
	{
		return "|    >    |";
	}
	if (X < 3.0 * Inc)
	{
		return "|    >o   |";
	}
	if (X < 5.0 * Inc)
	{
		return "|    >-o  |";
	}
	if (X < 7.0 * Inc)
	{
		return "|    >--o |";
	}
	if (X <= 1)
	{
		return "|    >---o|";
	}
	return    "|    >----#";

}

void HarmonixDsp::FAudioBuffer::DebugLog(const int16* InData, uint64 InNumSamples)
{
	FString DebugString = FString("smp#: int16 value\n");
	DebugString += FString("------------------------------------\n");
	for (uint64 SampleIdx = 0; SampleIdx < InNumSamples; ++SampleIdx)
	{
		float Sample = HarmonixDsp::k1Dot15FixedToFloating * (float)InData[SampleIdx];
		float Db = HarmonixDsp::dBFS(FMath::Abs(Sample));
		if (Db < -99.9f)
		{
			Db = -99.9f;
		}
		constexpr int32 ColumnWidth = 6;
		DebugString += FString::Printf(TEXT("%" UINT64_FMT ": %d %d %s (%f db)\n"), SampleIdx, ColumnWidth, InData[SampleIdx], *GetAsciiArtLevel(Sample), Db);
	}
	DebugString += FString("------------------------------------\n");

	UE_LOGF(LogAudioBuffer, Verbose, "%ls", *DebugString);
}

void HarmonixDsp::FAudioBuffer::DebugLog(const float* InData, uint64 InNumSamples)
{
	FString DebugString = FString("smp#: float value\n");
	DebugString += FString("------------------------------------\n");
	for (uint64 SampleIdx = 0; SampleIdx < InNumSamples; ++SampleIdx)
	{
		float Db = HarmonixDsp::dBFS(FMath::Abs(InData[SampleIdx]));
		if (Db < -99.9f)
		{
			Db = -99.9f;
		}
		DebugString += FString::Printf(TEXT("%" UINT64_FMT ":%f  %s (%f dB)\n"), SampleIdx, InData[SampleIdx], *GetAsciiArtLevel(InData[SampleIdx]), Db);
	}
	DebugString += FString("------------------------------------\n");
	UE_LOGF(LogAudioBuffer, Verbose, "%ls", *DebugString);
}

void HarmonixDsp::FAudioBuffer::Convert(const TAudioBuffer<float>& Source, TAudioBuffer<int16>& Destination)
{
	check(Destination.GetMaxConfig() == Source.GetMaxConfig());
	int32 NumFrames = Destination.GetMaxNumFrames();
	for (int32 Ch = 0; Ch < Destination.GetMaxNumChannels(); ++Ch)
	{
		int16* DstData = Destination.GetValidChannelData(Ch);
		const float* SrcData = Source.GetValidChannelData(Ch);
		for (int32 SampleIdx = 0; SampleIdx < NumFrames; ++SampleIdx)
		{
			float Sample = SrcData[SampleIdx] * HarmonixDsp::kFloatingTo1Dot15Fixed;
			DstData[SampleIdx] = (int16)Sample;
		}
	}
}

void HarmonixDsp::FAudioBuffer::Convert(const TAudioBuffer<int16>& Source, TAudioBuffer<float>& Destination)
{
	check(Source.GetMaxConfig() == Destination.GetMaxConfig());
	int32 FrameNum = Source.GetMaxNumFrames();
	for (int32 Ch = 0; Ch < Source.GetMaxNumChannels(); ++Ch)
	{
		const int16* SrcData = Source.GetValidChannelData(Ch);
		float* DstData = Destination.GetValidChannelData(Ch);
		for (int32 SampleIdx = 0; SampleIdx < FrameNum; ++SampleIdx)
		{
			float Sample = (float)SrcData[SampleIdx] * HarmonixDsp::k1Dot15FixedToFloating;
			DstData[SampleIdx] = Sample;
		}
	}
}
