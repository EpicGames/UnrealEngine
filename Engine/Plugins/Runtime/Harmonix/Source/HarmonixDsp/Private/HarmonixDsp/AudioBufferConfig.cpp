// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/AudioBufferConfig.h"

DEFINE_LOG_CATEGORY(LogAudioBufferConfig);

using namespace HarmonixDsp;

FAudioBufferConfig::FAudioBufferConfig()
	: SampleRate(0.0f)
	, NumChannels(0)
	, NumFrames(0)
	, Interleaved(false)
{
	SetChannelLayout(EAudioBufferChannelLayout::UnsupportedFormat);
}

FAudioBufferConfig::FAudioBufferConfig(int32 InNumChannels, int32 InNumFrames, float InSampleRate, bool InInterleaved)
	: SampleRate(InSampleRate)
	, NumChannels(InNumChannels)
	, NumFrames(InNumFrames)
	, Interleaved(InInterleaved)
{
	SetChannelLayout(EAudioBufferChannelLayout::Raw);
}

FAudioBufferConfig::FAudioBufferConfig(EAudioBufferChannelLayout InChannelLayout, int32 InNumFrames, float InSampleRate, bool InInterleaved)
	: SampleRate(InSampleRate)
	, NumChannels(FAudioBuffer::GetNumChannelsInChannelLayout(InChannelLayout))
	, NumFrames(InNumFrames)
	, Interleaved(InInterleaved)
{
	SetChannelLayout(InChannelLayout);
}

FAudioBufferConfig::FAudioBufferConfig(EAudioBufferChannelLayout InChannelLayout, int32 InNumChannels, int32 InNumFrames, float InSampleRate, bool InInterleaved)
	: SampleRate(InSampleRate)
	, NumChannels(InNumChannels)
	, NumFrames(InNumFrames)
	, Interleaved(InInterleaved)
{
	check(InChannelLayout == EAudioBufferChannelLayout::Raw || InNumChannels == FAudioBuffer::GetNumChannelsInChannelLayout(InChannelLayout));
	SetChannelLayout(InChannelLayout);
}

int32 FAudioBufferConfig::GetNumTotalSamples() const
{
	return NumChannels * NumFrames;
}

float FAudioBufferConfig::GetSampleRate() const
{
	return SampleRate;
}

void FAudioBufferConfig::SetSampleRate(float InSamplesPerSecond)
{
	check(0 <= InSamplesPerSecond && InSamplesPerSecond <= 192000.0f);
	SampleRate = InSamplesPerSecond;
}

void FAudioBufferConfig::SetNumChannels(int32 InNumChannels)
{
	NumChannels = InNumChannels;
	if (NumChannels < FAudioBuffer::GetNumChannelsInChannelLayout(ChannelLayout) && ChannelLayout != EAudioBufferChannelLayout::Raw)
	{
		UE_LOGF(LogAudioBufferConfig, Warning, "NumChannels Set below number of channels required by channel layout!");
	}
}

void FAudioBufferConfig::SetNumFrames(int32 InNumFrames)
{
	NumFrames = InNumFrames;
}

FAudioBufferConfig FAudioBufferConfig::GetInterleavedConfig() const
{
	return FAudioBufferConfig(ChannelLayout, NumFrames, SampleRate, true);
}

FAudioBufferConfig FAudioBufferConfig::GetDeinterleavedConfig() const
{
	return FAudioBufferConfig(ChannelLayout, NumFrames, SampleRate, false);
}

void FAudioBufferConfig::SetChannelLayout(EAudioBufferChannelLayout InChannelLayout)
{
	ChannelLayout = InChannelLayout;
	if (NumChannels < FAudioBuffer::GetNumChannelsInChannelLayout(ChannelLayout) && ChannelLayout != EAudioBufferChannelLayout::Raw)
	{
		UE_LOGF(LogAudioBufferConfig, Warning, "New Channel Layout requires more channels!");
	}
}

void FAudioBufferConfig::SetChannelMask(uint32 InChannelMask)
{
	ChannelMask = InChannelMask;
}

bool FAudioBufferConfig::operator==(const FAudioBufferConfig& Other) const
{
	if (GetNumChannels() != Other.GetNumChannels())
	{
		return false;
	}
	if (GetNumFrames() != Other.GetNumFrames())
	{
		return false;
	}
	if (GetSampleRate() != Other.GetSampleRate())
	{
		return false;
	}
	if (Interleaved != Other.Interleaved)
	{
		return false;
	}
	return true;
}