// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/AudioData/StreamingAudioRendererV2.h"
#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/FusionSampler/FusionSampler.h"
#include "HarmonixDsp/GainMatrix.h"
#include "HarmonixDsp/PannerDetails.h"
#include "Math/UnrealMathUtility.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProxyPlayer.h"

DEFINE_LOG_CATEGORY(LogHarmonixStreamingAudioRendererV2);

FStreamingAudioRendererV2::FStreamingAudioRendererV2()
{
}

FStreamingAudioRendererV2::~FStreamingAudioRendererV2()
{
}

void FStreamingAudioRendererV2::Reset()
{
	WavePlayer.Reset();
	InterleavedCircularBuffer.SetNum(0);
	Shifter = nullptr;
}

void FStreamingAudioRendererV2::SetSoundWaveData(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const FSettings& InSettings)
{
	SoundWaveData = InSoundWaveData.ToSharedPtr();

	WavePlayer.Reset();
	WavePlayer = CreateWavePlayer(InSoundWaveData);

	check(WavePlayer->IsPlayerValid())

	NumDeinterleaveChannels = SoundWaveData->GetNumChannels();
	int32 DecodeBufferSize = NumDeinterleaveChannels * DeinterleaveBlockSizeInFrames;
	DecodeBuffer.Reset(DecodeBufferSize);
	DecodeBuffer.AddUninitialized(DecodeBufferSize);


	LastLoopFrameCache.SetNumUninitialized(NumDeinterleaveChannels);
	bLastLoopFrameCached = false;

	// interleaved circular buffer for pre-decoded source audio
	// 2x the decode block size to absorb I/O latency on streaming sources
	int32 FrameCapacity = DecodeBufferSize * 2;
	InterleavedCircularBuffer.SetCapacity(FrameCapacity);

	TrackChannelInfo = InSettings.TrackChannelInfo;
	Shifter = InSettings.Shifter;
	MySampler = InSettings.Sampler;

	if (Shifter)
	{
		Shifter->SetSampleSourceReset(InSoundWaveData, AsShared(), InSettings.ShifterSettings);
	}
}

TSharedPtr<const FSoundWaveData> FStreamingAudioRendererV2::GetSoundWaveData() const
{
	return SoundWaveData;
}

void FStreamingAudioRendererV2::MigrateToSampler(const FFusionSampler* InSampler)
{
	MySampler = InSampler;
}

void FStreamingAudioRendererV2::SetFrame(uint32 InFrameNum)
{
	if (WavePlayer->IsPlayerValid())
	{
		SeekSourceAudioToFrame(InFrameNum);
	}
}

double FStreamingAudioRendererV2::Render(TAudioBuffer<float>& OutBuffer, double InPos, int32 InMaxFrame, double InResampleInc, double InPitchShift, double InSpeed, bool MaintainPitchWhenSpeedChanges, const FGainMatrix& InGain)
{
	if (!SoundWaveData || SoundWaveData->GetNumFrames() == 0)
	{
		OutBuffer.ZeroValidFrames();
		return InPos;
	}

	if (!Shifter)
	{
		double Increment = InResampleInc * InPitchShift;
		if (!MaintainPitchWhenSpeedChanges)
		{
			Increment *= InSpeed;
		}
		return RenderInternal(OutBuffer, InPos, InMaxFrame, Increment, InGain);
	}

	return Shifter->Render(OutBuffer, InPos, InMaxFrame, InResampleInc, InPitchShift, InSpeed, MaintainPitchWhenSpeedChanges, InGain);
}

double FStreamingAudioRendererV2::RenderInternal(TAudioBuffer<float>& OutBuffer, double InPos, int32 InMaxFrame, double InInc, const FGainMatrix& InGain)
{
	check(OutBuffer.GetNumValidFrames() <= AudioRendering::kMicroSliceSize);

	if (!SoundWaveData || SoundWaveData->GetNumFrames() < FMath::FloorToInt32(InPos))
	{
		OutBuffer.ZeroValidFrames();
		return InPos;
	}
	
	uint32 NumOutFrames = OutBuffer.GetNumValidFrames();

	FLerpData LerpArray[AudioRendering::kMicroSliceSize];
	InPos = CalculateLerpData(LerpArray, AudioRendering::kMicroSliceSize, NumOutFrames, InPos, InMaxFrame, InInc);

	if (TrackChannelInfo && TrackChannelInfo->Num() > 0)
	{
		RenderMultiChannelRoutedUnshifted(OutBuffer, LerpArray, NumOutFrames, InGain, InInc);
	}
	else if (SoundWaveData->GetNumChannels() <= 2)
	{
		RenderSimpleUnshifted(OutBuffer, LerpArray, NumOutFrames, InGain, InInc);
	}
	else
	{
		RenderMultiChannelUnshifted(OutBuffer, LerpArray, NumOutFrames, InGain, InInc);
	}

	return InPos;
}

double FStreamingAudioRendererV2::RenderUnshifted(TAudioBuffer<float>& OutBuffer, double InPos, int32 InMaxFrame, double InInc, const FGainMatrix& InGain)
{
	if (!SoundWaveData)
	{
		OutBuffer.ZeroValidFrames();
		return InPos;
	}

	return RenderInternal(OutBuffer, InPos, InMaxFrame, InInc, InGain);
}

int32 FStreamingAudioRendererV2::CalculateNumFramesNeeded(const FLerpData* LerpData, int32 NumPoints)
{
	if (NumPoints == 0)
	{
		return 0;
	}

	if (LerpData[NumPoints - 1].PosB >= LerpData[0].PosA)
	{
		// no loop in lerp!
		return 1 + LerpData[NumPoints - 1].PosB - LerpData[0].PosA;
	}

	// Fail-safe for if we got what appears to be a loop, but we don't have a loop in the audio data
	if (SoundWaveData->GetLoopRegions().IsEmpty())
	{
		return SoundWaveData->GetNumFrames() - LerpData[0].PosA;
	}

	const FSoundWaveCuePoint& LoopRegion = SoundWaveData->GetLoopRegions()[0];

	// loop in lerp. 
	// we need all the samples from LerpData[0].PosA to the end of the loop, 
	// and then all the samples from the start of the loop to LerpData[NumPoints-1]PosB.
	// REMEMBER: Loop start and end frames are INCLUSIVE!
	uint32 FirstFrameInLoop = LoopRegion.FramePosition;
	uint32 LastFrameInLoop = LoopRegion.FramePosition + LoopRegion.FrameLength;
	int32 NumNeeded = 1 + LastFrameInLoop - LerpData[0].PosA;
	NumNeeded += 1 + LerpData[NumPoints - 1].PosB - FirstFrameInLoop;
	return NumNeeded;
}

void FStreamingAudioRendererV2::RenderSimpleUnshifted(TAudioBuffer<float>& OutBuffer, const FLerpData* LerpArray, uint32 InNumFrames, const FGainMatrix& InGain, double InInc)
{
	check(InNumFrames <= AudioRendering::kMicroSliceSize);

	// need to zero out here because we are going to do an accumulate below
	OutBuffer.ZeroValidFrames();

	int32 NumOutChannels = OutBuffer.GetNumValidChannels();
	int32 NumInputChannels = SoundWaveData->GetNumChannels();

	uint32 StartFrameIndex = LerpArray[0].PosA;
	int32 NumSourceFramesNeeded = CalculateNumFramesNeeded(LerpArray, InNumFrames);

	// Interleaved
	int32 NumSamplesNeeded = NumInputChannels * NumSourceFramesNeeded;

	if (NumSamplesNeeded == 0)
	{
		return;
	}
	
	WorkBuffer.SetNum(NumSamplesNeeded);
	GenerateSourceAudio(StartFrameIndex, WorkBuffer);

	int32 PosA;
	int32 PosB;
	float SampleA;
	float SampleB;

	for (int32 ich = 0; ich < NumInputChannels; ++ich)
	{
		for (uint32 FrameNum = 0; FrameNum < InNumFrames; ++FrameNum)
		{
			const FLerpData& Lerp = LerpArray[FrameNum];

			PosA = Lerp.PosARelative * NumInputChannels + ich;
			PosB = Lerp.PosBRelative * NumInputChannels + ich;

			SampleA = PosA < WorkBuffer.Num() ? WorkBuffer[PosA] : 0.0f;
			SampleB = PosB < WorkBuffer.Num() ? WorkBuffer[PosB] : 0.0f;
			const float Sample = SampleA * Lerp.WeightA + SampleB * Lerp.WeightB;

			for (int32 och = 0; och < NumOutChannels; ++och)
			{
				if (FMath::Abs(InGain[ich].f[och]) < UE_KINDA_SMALL_NUMBER)
				{
					continue;
				}

				float* OutData = OutBuffer.GetValidChannelData(och);
				OutData[FrameNum] += Sample* InGain[ich].f[och];
			}
		}
	}
}

void FStreamingAudioRendererV2::RenderMultiChannelUnshifted(TAudioBuffer<float>& OutBuffer, const FLerpData* LerpArray, uint32 InNumFrames, const FGainMatrix& InGain, double InInc)
{
	check(InNumFrames <= AudioRendering::kMicroSliceSize);

	// need to zero out here because we are going to do an accumulate below
	OutBuffer.ZeroValidFrames();

	int32 NumOutChannels = OutBuffer.GetNumValidChannels();
	int32 NumInputChannels = SoundWaveData->GetNumChannels();

	// interleaved
	Audio::FAlignedFloatBuffer ResampleBuffer;
	constexpr int32 kNumResampleChannels = 2;
	ResampleBuffer.SetNum(kNumResampleChannels * InNumFrames);

	uint32 StartFrameIndex = LerpArray[0].PosA;
	int32 NumSourceFramesNeeded = CalculateNumFramesNeeded(LerpArray, InNumFrames);

	// Interleaved
	int32 NumSamplesNeeded = NumInputChannels * NumSourceFramesNeeded;

	if (NumSamplesNeeded == 0)
	{
		return;
	}
	
	WorkBuffer.SetNum(NumSamplesNeeded);
	GenerateSourceAudio(StartFrameIndex, WorkBuffer);

	int32 PosA;
	int32 PosB;
	float SampleA;
	float SampleB;

	for (int32 ich = 0; ich < NumInputChannels; ++ich)
	{
		for (uint32 FrameNum = 0; FrameNum < InNumFrames; ++FrameNum)
		{
			const FLerpData& Lerp = LerpArray[FrameNum];

			PosA = Lerp.PosARelative * NumInputChannels + ich;
			PosB = Lerp.PosBRelative * NumInputChannels + ich;

			SampleA = PosA < WorkBuffer.Num() ? WorkBuffer[PosA] : 0.0f;
			SampleB = PosB < WorkBuffer.Num() ? WorkBuffer[PosB] : 0.0f;
			const float Sample = SampleA * Lerp.WeightA + SampleB * Lerp.WeightB;
			
			uint32 ipos = FrameNum * kNumResampleChannels + ich;
			ResampleBuffer[ipos] += Sample;
		}
	}

	// map the rendered stereo into the output buffer
	for (int32 ich = 0; ich < kNumResampleChannels; ++ich)
	{
		for (int32 och = 0; och < NumOutChannels; ++och)
		{
			if (FMath::Abs(InGain[ich].f[och]) < UE_KINDA_SMALL_NUMBER)
			{
				continue;
			}

			float* OutChannelData = OutBuffer.GetValidChannelData(och);

			for (uint32 FrameIdx = 0; FrameIdx < InNumFrames; ++FrameIdx)
			{
				uint32 ipos = FrameIdx * kNumResampleChannels + ich;
				OutChannelData[FrameIdx] += ResampleBuffer[ipos] * InGain[ich].f[och];
			}
		}
	}
}

void FStreamingAudioRendererV2::RenderMultiChannelRoutedUnshifted(TAudioBuffer<float>& OutBuffer, const FLerpData* LerpArray, uint32 InNumFrames, const FGainMatrix& InGain, double InInc)
{
	check(InNumFrames <= AudioRendering::kMicroSliceSize);

	// need to zero out here because we are going to do an accumulate below
	OutBuffer.ZeroValidFrames();

	int32 NumOutChannels = OutBuffer.GetNumValidChannels();
	int32 NumInputChannels = SoundWaveData->GetNumChannels();

	// interleaved
	Audio::FAlignedFloatBuffer ResampleBuffer;
	constexpr int32 kNumResampleChannels = 2;
	ResampleBuffer.SetNum(kNumResampleChannels * InNumFrames);

	uint32 StartFrameIndex = LerpArray[0].PosA;
	int32 NumSourceFramesNeeded = CalculateNumFramesNeeded(LerpArray, InNumFrames);

	// Interleaved
	int32 NumSamplesNeeded = NumInputChannels * NumSourceFramesNeeded;

	if (NumSamplesNeeded == 0)
	{
		return;
	}
	
	WorkBuffer.SetNum(NumSamplesNeeded);
	GenerateSourceAudio(StartFrameIndex, WorkBuffer);

	int32 PosA;
	int32 PosB;
	float SampleA;
	float SampleB;
	
	for (int32 ich = 0; ich < NumInputChannels; ++ich)
	{
		// get gain for this channel...
		float chGain = 1.0f;
		float ssGain = 1.0f;
		FGainMatrix chGainMatrix(kNumResampleChannels, OutBuffer.GetNumValidChannels(), OutBuffer.GetChannelLayout());
		bool FoundTrack = false;
		for (uint32 Idx = 0; Idx < (uint32)TrackChannelInfo->Num(); Idx++)
		{
			if ((*TrackChannelInfo)[Idx].GetStreamIndexesGain(ich, chGain))
			{
				FPannerDetails chPan;
				(*TrackChannelInfo)[Idx].GetStreamIndexesPan(ich, chPan);
				ssGain = MySampler ? MySampler->GetSubstreamGain(Idx) : 1.0f;
				chGainMatrix.Set(ssGain, chPan);
				FoundTrack = true;
				break;
			}
		}

		UE_LOGF(LogHarmonixStreamingAudioRendererV2, Warning, "TODO: Apply chGain and ssGain to gain matrix!");

		if (!FoundTrack)
		{
			continue;
		}

		float chPanMixLeft = 0.0f;
		float chPanMixRight = 0.0f;
		UE_LOGF(LogHarmonixStreamingAudioRendererV2, Warning, "TODO: Don't use chPanMix... vars but use gain matrix!");
		//PanToGainsConstantPower(chPan, chPanMixLeft, chPanMixRight);

		for (uint32 FrameNum = 0; FrameNum < InNumFrames; ++FrameNum)
		{
			const FLerpData& Lerp = LerpArray[FrameNum];

			PosA = Lerp.PosARelative * NumInputChannels + ich;
			PosB = Lerp.PosBRelative * NumInputChannels + ich;

			SampleA = PosA < WorkBuffer.Num() ? WorkBuffer[PosA] : 0.0f;
			SampleB = PosB < WorkBuffer.Num() ? WorkBuffer[PosB] : 0.0f;
			const float Sample = SampleA * Lerp.WeightA + SampleB * Lerp.WeightB;

			uint32 ipos0 = FrameNum * kNumResampleChannels;
			uint32 ipos1 = FrameNum * kNumResampleChannels + 1;
			ResampleBuffer[ipos0] += Sample * chGain * chPanMixLeft * ssGain;
			ResampleBuffer[ipos1] += Sample * chGain * chPanMixRight * ssGain;
		}
	}

	// map the rendered stereo into the output buffer
	for (int32 ich = 0; ich < kNumResampleChannels; ++ich)
	{
		for (int32 och = 0; och < NumOutChannels; ++och)
		{
			if (FMath::Abs(InGain[ich].f[och]) < UE_KINDA_SMALL_NUMBER)
			{
				continue;
			}

			float* OutChannelData = OutBuffer.GetValidChannelData(och);

			for (uint32 FrameIdx = 0; FrameIdx < InNumFrames; ++FrameIdx)
			{
				uint32 ipos = FrameIdx * kNumResampleChannels + ich;
				OutChannelData[FrameIdx] += ResampleBuffer[ipos] * InGain[ich].f[och];
			}
		}
	}
}

void FStreamingAudioRendererV2::SeekSourceAudioToFrame(uint32 FrameIdx)
{
	check(WavePlayer->IsPlayerValid());

	// no need to seek
	uint32 SourceFrameIndex = GetSourceAudioFrameIndex();

	if (SourceFrameIndex == FrameIdx)
	{
		return;
	}
	uint32 FramesInWave = (uint32)WavePlayer->GetSourceNumFrames();
	if (FrameIdx >= FramesInWave)
	{
		// passed the end
		return;
	}

	// check if we can advance to the desired frame by popping from the buffer
	if (FrameIdx > SourceFrameIndex)
	{
		if (bLastLoopFrameCached && HasLoopSection() && FrameIdx == GetLoopEndFrame())
		{
			return;
		}

		int32 NumFramesAhead = FrameIdx - SourceFrameIndex;
		int32 NumSourceFramesAvailable = InterleavedCircularBuffer.Num() / NumDeinterleaveChannels;

		if (NumFramesAhead <= NumSourceFramesAvailable)
		{
			InterleavedCircularBuffer.Pop(NumFramesAhead * NumDeinterleaveChannels);
			check(GetSourceAudioFrameIndex() == FrameIdx);
			return;
		}
	}

	// do the actual seek
	if (WavePlayer->SeekToFrame(FrameIdx))
	{
		// at this point, we should be synced up!!
		ensure(WavePlayer->GetCurrentPlaybackFrame() == FrameIdx);
		InterleavedCircularBuffer.SetNum(0);
	}

	checkf(GetSourceAudioFrameIndex() == FrameIdx, TEXT("Failed to seek. SourceFrameIdx = %d, ExpectedFrameIdx = %d"), GetSourceAudioFrameIndex(), FrameIdx);
}

void FStreamingAudioRendererV2::DecodeSourceAudio(Audio::TCircularAudioBuffer<float>& OutBuffer)
{
	check(WavePlayer->IsPlayerValid());

	const int32 NumSamplesToGenerate = DeinterleaveBlockSizeInFrames * NumDeinterleaveChannels;
	check(NumSamplesToGenerate == DecodeBuffer.Num());

	if (OutBuffer.Remainder() == 0)
	{
		return;
	}

	int32 NumSamplesToRead = FMath::Min(static_cast<int32>(OutBuffer.Remainder()), NumSamplesToGenerate);
	int32 NumFramesRead = WavePlayer->GenerateSourceAudio(MakeArrayView(DecodeBuffer.GetData(), NumSamplesToRead));
	int32 NumSamplesRead = NumFramesRead * NumDeinterleaveChannels;
	if (NumSamplesRead > 0)
	{
		OutBuffer.Push(DecodeBuffer.GetData(), NumSamplesRead);
	}
}

uint32 FStreamingAudioRendererV2::GetSourceAudioFrameIndex()
{
	check(WavePlayer->IsPlayerValid());
	int32 PlaybackFrame = WavePlayer->GetCurrentPlaybackFrame();
	int32 ReaderFramesInWave = WavePlayer->GetSourceNumFrames();
	if (ReaderFramesInWave < PlaybackFrame)
	{
		PlaybackFrame = ReaderFramesInWave;
	}

	int32 NumBufferedFrames = InterleavedCircularBuffer.Num() / NumDeinterleaveChannels;
	return PlaybackFrame - NumBufferedFrames;
}

TUniquePtr<FSoundWaveProxyPlayer> FStreamingAudioRendererV2::CreateWavePlayer(const TSharedRef<const FSoundWaveData>& WaveData)
{
	float SampleRate = WaveData->GetSampleRate();
	int32 NumChannels = WaveData->GetNumChannels();
	FSoundWaveProxyPlayer::FSettings PlayerSettings(SampleRate, NumChannels);
	PlayerSettings.MaxDecodeSizeFrames = MaxDecodeSizeInFrames;
	PlayerSettings.bMaintainAudioSync = true;
	TUniquePtr<FSoundWaveProxyPlayer> NewWavePlayer = FSoundWaveProxyPlayer::Create(PlayerSettings);
	NewWavePlayer->SetSoundWave(WaveData);
	return MoveTemp(NewWavePlayer);
}

bool FStreamingAudioRendererV2::HasLoopSection() const
{
	check(SoundWaveData);
	return SoundWaveData->GetLoopRegions().Num() > 0;
}

uint32 FStreamingAudioRendererV2::GetLoopStartFrame() const
{
	check(HasLoopSection());
	return SoundWaveData->GetLoopRegions()[0].FramePosition;
}

uint32 FStreamingAudioRendererV2::GetLoopEndFrame() const
{
	check(HasLoopSection());
	const FSoundWaveCuePoint& LoopRegion = SoundWaveData->GetLoopRegions()[0];
	return LoopRegion.FramePosition + LoopRegion.FrameLength;
}

void FStreamingAudioRendererV2::GenerateSourceAudio(uint32 StartFrameIndex, Audio::FAlignedFloatBuffer& OutAudio)
{
	check(OutAudio.Num() % NumDeinterleaveChannels == 0);

	if (HasLoopSection())
	{
		uint32 NumFramesRequested = OutAudio.Num() / NumDeinterleaveChannels;
		uint32 FirstFrameInLoop = GetLoopStartFrame();
		uint32 LastFrameInLoop = GetLoopEndFrame();
		uint32 LoopLengthFrames = 1 + LastFrameInLoop - FirstFrameInLoop;
		bool bLoopingThisChunk = (StartFrameIndex + NumFramesRequested) > (LastFrameInLoop + 1);

		if (!ensure(StartFrameIndex <= LastFrameInLoop))
		{
			GenerateSourceAudioInternal(StartFrameIndex, OutAudio);
			return;
		}

		if (bLoopingThisChunk)
		{
			check(NumFramesRequested < LoopLengthFrames);
			// include the end frame index
			int32 NumFrames = 1 + LastFrameInLoop - StartFrameIndex;
			int32 NumSamples = NumFrames * NumDeinterleaveChannels;

			if (NumFrames == 1)
			{
				if (bLastLoopFrameCached)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < NumDeinterleaveChannels; ++ChannelIndex)
					{
						OutAudio[ChannelIndex] = LastLoopFrameCache[ChannelIndex];
					}
				}
				else
				{
					// NOTE: What if StartFrameIndex is just one or two samples ahead of 
					// the current SourceFrameIndex? In that case we will use 0 here!
					// Maybe that is better than ACTUALLY seeking forward one or two samples!
					// Hmmmm.
					if(ensure(GetSourceAudioFrameIndex() == StartFrameIndex))
					{
						GenerateSourceAudioInternal(StartFrameIndex, OutAudio);
					}
					else
					{
						for (int32 ChannelIndex = 0; ChannelIndex < NumDeinterleaveChannels; ++ChannelIndex)
						{
							OutAudio[ChannelIndex] = 0.0f;
						}
					}
				}
			}
			else
			{
				GenerateSourceAudioInternal(StartFrameIndex, OutAudio);
			}

			/// cache loop end frame...
			if (!bLastLoopFrameCached)
			{
				for (int32 ChannelIndex = 0; ChannelIndex < NumDeinterleaveChannels; ++ChannelIndex)
				{
					LastLoopFrameCache[ChannelIndex] = OutAudio[(NumFrames - 1) * NumDeinterleaveChannels + ChannelIndex];
				}
				bLastLoopFrameCached = true;
			}

			int32 SampleOffset = NumSamples;
			NumFrames = NumFramesRequested - NumFrames;
			NumSamples = NumFrames * NumDeinterleaveChannels;
			GenerateSourceAudioInternal(FirstFrameInLoop, MakeArrayView(OutAudio.GetData() + SampleOffset, NumSamples));
		}
		else
		{
			GenerateSourceAudioInternal(StartFrameIndex, OutAudio);
		}
	}
	else
	{
		GenerateSourceAudioInternal(StartFrameIndex, OutAudio);
	}
}

void FStreamingAudioRendererV2::GenerateSourceAudioInternal(uint32 StartFrameIndex, TArrayView<float> OutAudio)
{
	int32 NumSamplesRequested = OutAudio.Num();
	int32 BufferIdx = 0;

	while (NumSamplesRequested > 0 && WavePlayer->IsPlayerValid())
	{
		if (GetSourceAudioFrameIndex() != StartFrameIndex)
		{
			SeekSourceAudioToFrame(StartFrameIndex);
		}

		check(InterleavedCircularBuffer.Num() % NumDeinterleaveChannels == 0);

		int32 NumSamplesAvailable = InterleavedCircularBuffer.Num();

		if (NumSamplesRequested > NumSamplesAvailable)
		{
			DecodeSourceAudio(InterleavedCircularBuffer);
		}

		int32 NumSamplesToRead = FMath::Min(static_cast<int32>(InterleavedCircularBuffer.Num()), NumSamplesRequested);

		if (NumSamplesToRead == 0)
		{
			break;
		}

		InterleavedCircularBuffer.Peek(OutAudio.GetData() + BufferIdx, NumSamplesToRead);
		NumSamplesRequested -= NumSamplesToRead;
		BufferIdx += NumSamplesToRead;
		StartFrameIndex += (uint32)(NumSamplesToRead / NumDeinterleaveChannels);
	}

	if (NumSamplesRequested > 0)
	{
		FMemory::Memzero(OutAudio.GetData() + BufferIdx, sizeof(float) * NumSamplesRequested);

		UE_LOGF(LogHarmonixStreamingAudioRendererV2, Verbose, "%ls: Failed to generated samples: StartFrameIndex: %d, NumSamplesRequested: %d",
			*SoundWaveData->GetFName().ToString(), StartFrameIndex, NumSamplesRequested);
	}
}
