// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/Streaming/TrackChannelInfo.h"
#include "HarmonixDsp/AudioDataRenderer.h"

#include "Templates/SharedPointer.h"

#include "DSP/BufferVectorOperations.h"
#include "DSP/ConvertDeinterleave.h"
#include "DSP/Dsp.h"
#include "DSP/MultichannelBuffer.h"
#include "DSP/MultichannelLinearResampler.h"

#define UE_API HARMONIXDSP_API

class FSoundWaveData;
class FSoundWaveProxy;
class FSoundWaveProxyPlayer;
class FFusionSampler;
class IStretcherAndPitchShifter;

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixStreamingAudioRendererV2, Log, All);

class FStreamingAudioRendererV2 : public IAudioDataRenderer
{
public:

	using FAlignedInt16Buffer = TArray<int16, Audio::FAudioBufferAlignedAllocator>;

	UE_API FStreamingAudioRendererV2();
	UE_API ~FStreamingAudioRendererV2();

	UE_API virtual void Reset() override;
	UE_API virtual void SetSoundWaveData(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const FSettings& InSettings) override;
	UE_API virtual TSharedPtr<const FSoundWaveData> GetSoundWaveData() const override;
	
	UE_API virtual void MigrateToSampler(const FFusionSampler* InSampler) override;

	UE_API virtual void SetFrame(uint32 InFrameNum) override;

	UE_API virtual double Render(TAudioBuffer<float>& OutBuffer, double InPos, int32 InMaxFrame, double InResampleInc, double InPitchShift, double InSpeed,
		bool MaintainPitchWhenSpeedChanges, const FGainMatrix& InGain) override;

	UE_API double RenderInternal(TAudioBuffer<float>& OutBuffer, double Pos, int32 MaxFrame, double Inc, const FGainMatrix& Gain);

	UE_API virtual double RenderUnshifted(TAudioBuffer<float>& OutBuffer, double InPos, int32 InMaxFrame, double InInc,
		const FGainMatrix& InGain) override;

	UE_API void GenerateSourceAudio(uint32 StartFrame, Audio::FAlignedFloatBuffer& OutAudio);

private:

	void RenderSimpleUnshifted(TAudioBuffer<float>& OutBuffer, const FLerpData* LerpArray, uint32 InNumFrames, const FGainMatrix& InGain, double InInc);
	void RenderMultiChannelUnshifted(TAudioBuffer<float>& OutBuffer, const FLerpData* LerpArray, uint32 InNumFrames, const FGainMatrix& InGain, double InInc);
	void RenderMultiChannelRoutedUnshifted(TAudioBuffer<float>& OutBuffer, const FLerpData* LerpArray, uint32 InNumFrames, const FGainMatrix& InGain, double InInc);

	void SeekSourceAudioToFrame(uint32 FrameIdx);
	void DecodeSourceAudio(Audio::TCircularAudioBuffer<float>& OutBuffer);
	void GenerateSourceAudioInternal(uint32 StartFrameIndex, TArrayView<float> OutAudio);

	uint32 GetSourceAudioFrameIndex();

	static TUniquePtr<FSoundWaveProxyPlayer> CreateWavePlayer(const TSharedRef<const FSoundWaveData>& WaveData);

	bool HasLoopSection() const;
	uint32 GetLoopStartFrame() const;
	uint32 GetLoopEndFrame() const;

	const FFusionSampler* MySampler = nullptr;
	const TArray<FTrackChannelInfo>* TrackChannelInfo = nullptr;
	TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> Shifter;

	// Ref to the actual streaming audio data
	// this data is a shared instance of audio data
	// it will get loaded on construction
	TSharedPtr<const FSoundWaveData> SoundWaveData;

	TUniquePtr<FSoundWaveProxyPlayer> WavePlayer;

	Audio::FAlignedFloatBuffer DecodeBuffer;
	Audio::TCircularAudioBuffer<float> InterleavedCircularBuffer;

	Audio::FAlignedFloatBuffer WorkBuffer;

	int32 NumDeinterleaveChannels;
	Audio::FAlignedFloatBuffer LastLoopFrameCache;
	bool bLastLoopFrameCached = false;

	// needs to be small enough to avoid audio artifacts and syncing issues
	// but also large enough that we're decoding multiple times per block
	static constexpr int32 DeinterleaveBlockSizeInFrames = 256;
	static constexpr uint32 MaxDecodeSizeInFrames = 1024;

	int32 CalculateNumFramesNeeded(const FLerpData* LerpData, int32 NumPoints);
};

#undef UE_API
