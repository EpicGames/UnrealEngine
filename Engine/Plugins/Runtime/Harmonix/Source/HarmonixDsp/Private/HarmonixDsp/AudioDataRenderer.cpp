// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/AudioDataRenderer.h"
#include "Sound/SoundWave.h"

void IAudioDataRenderer::SetSoundWaveData(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const FSettings& InSettings)
{
}

TSharedPtr<const FSoundWaveData> IAudioDataRenderer::GetSoundWaveData() const
{
	return {};
}

double IAudioDataRenderer::CalculateLerpData(FLerpData* LerpArray, int32 InNumPoints, uint32 InNumOutFrames, double InPos, int32 InMaxFrame, double InInc) const
{
	check(InNumPoints >= (int32)InNumOutFrames);
	FMemory::Memset(LerpArray, 0, sizeof(FLerpData) * InNumPoints);

	const TSharedPtr<const FSoundWaveData> SoundWaveData = GetSoundWaveData();
	if (!SoundWaveData)
	{
		return 0.0;
	}

	if (InMaxFrame < 0 || InMaxFrame > (int32)SoundWaveData->GetNumFrames())
	{
		InMaxFrame = (int32)SoundWaveData->GetNumFrames();
	}

	double NumSourceFrames = (double)InMaxFrame;

	uint32 LoopStartFrameIndex = 0;
	uint32 LoopEndFrameIndex = 0;
	bool IsLooping = false;
	if (SoundWaveData->GetLoopRegions().Num() > 0)
	{
		IsLooping = true;
		const FSoundWaveCuePoint& LoopRegion = SoundWaveData->GetLoopRegions()[0];
		LoopStartFrameIndex = LoopRegion.FramePosition;
		LoopEndFrameIndex = LoopRegion.FramePosition + LoopRegion.FrameLength;
	}


	double LoopEnd = (double)LoopEndFrameIndex + 1.0;
	double LoopLength = LoopEnd - (double)LoopStartFrameIndex;

	// keep track of our "Relative" position values. Start with index 0
	// make sure to take the fractional part of the InPos
	double PosRelative = InPos - FMath::Floor(InPos);

	// for each frame in a channel
	// compute the interpolation array
	for (uint32 FrameNum = 0; FrameNum < InNumOutFrames; ++FrameNum)
	{
		FLerpData& LerpData = LerpArray[FrameNum];
		if (InPos < NumSourceFrames)
		{
			// posA is the earlier position.
			LerpData.PosA = (uint32)FMath::Floor(InPos);

			if (IsLooping && (LerpData.PosA == LoopEndFrameIndex))
			{
				LerpData.PosB = LoopStartFrameIndex;
			}
			else
			{
				LerpData.PosB = LerpData.PosA + 1;
			}

			// compute the weighting between A and B (A weight is 1.0 minus B weight)
			LerpData.WeightB = (float)InPos - (float)LerpData.PosA;
			checkSlow(0.0 <= LerpData.WeightB && LerpData.WeightB <= 1.0);
			LerpData.WeightA = 1.0f - LerpData.WeightB;

			// if B will index past the sample buffer,
			// then zero it out (weight = 0, pos is simply in bounds)
			if (LerpData.PosB >= NumSourceFrames)
			{
				LerpData.PosB = LerpData.PosA;
				LerpData.WeightB = 0.0f;
			}
		}

		InPos += InInc;
		if (IsLooping && (InPos >= LoopEnd))
		{
			InPos -= LoopLength;
		}

		// Calculate Relative Positions
		LerpData.PosARelative = (uint32)FMath::Floor(PosRelative);
		LerpData.PosBRelative = LerpData.PosARelative + 1;
		PosRelative += InInc;
	}

	// We return the updated `InPos` value, which will have looped if looping is enabled!
	return InPos;
}
