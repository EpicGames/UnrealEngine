// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/AudioUtility.h"

namespace HarmonixDsp
{
	float MapLinearToDecibelRange(float ValueLinear, float MaxDecibels, float RangeDecibels)
	{
		float ValueDb = -1000.0f;
		if (ValueLinear > 1.0e-9f)
		{
			ValueDb = 10 * log10f(ValueLinear);
		}

		float Output;
		
		if (ValueDb >= MaxDecibels)
		{
			Output = 1;
		}
		else if (ValueDb <= MaxDecibels - RangeDecibels)
		{
			Output = 0;
		}
		else
		{
			Output = 1 - ((MaxDecibels - ValueDb) / RangeDecibels);
		}
		
		return Output;
	}
}
