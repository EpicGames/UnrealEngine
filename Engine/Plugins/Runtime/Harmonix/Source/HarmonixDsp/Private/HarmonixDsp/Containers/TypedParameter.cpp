// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Containers/TypedParameter.h"
#include "HAL/Platform.h"

#include "Serialization/Archive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedParameter)

bool FTypedParameter::Serialize(FArchive& Ar)
{
	uint8 Version = kVersion;
	Ar << Version;
	Ar << Value;
	return true;
}

EParameterType FTypedParameter::GetType() const
{
	return FromVariantType(Value.GetType());
}

void FTypedParameter::SetType(EParameterType InType)
{
	if (GetType() == InType)
		return;

	switch (InType)
	{
	case EParameterType::Bool:
		Value = FVariant(false);
		return;
	case EParameterType::Double:
		Value = FVariant(double(0));
		return;
	case EParameterType::Float:
		Value = FVariant(float(0.0f));
		return;
	case EParameterType::Int8:
		Value = FVariant(int8(0));
		return;
	case EParameterType::Int16:
		Value = FVariant(int16(0));
		return;
	case EParameterType::Int32:
		Value = FVariant(int32(0));
		return;
	case EParameterType::Int64:
		Value = FVariant(int64(0));
		return;
	case EParameterType::Name:
		Value = FVariant(FName());
		return;
	case EParameterType::String:
		Value = FVariant(FString());
		return;
	case EParameterType::UInt8:
		Value = FVariant(uint8(0));
		return;
	case EParameterType::UInt16:
		Value = FVariant(uint16(0));
		return;
	case EParameterType::UInt32:
		Value = FVariant(uint32(0));
		return;
	case EParameterType::UInt64:
		Value = FVariant(uint64(0));
		return;
	default:
		checkNoEntry();
		Value = FVariant(int32(0));
		return;
	}
}
