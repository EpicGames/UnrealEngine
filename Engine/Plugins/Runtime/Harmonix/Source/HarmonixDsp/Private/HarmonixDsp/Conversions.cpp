// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Conversions.h"

#include "Math/UnrealMathUtility.h"

#include "HAL/Platform.h"

namespace HarmonixDsp

{
	// anonymous namespace to make this inaccessible from outside this file
	namespace
	{
		template<bool bAccumulate>
		FORCEINLINE double SampleRateConvert(const float* Input, uint32 InNumInputSamples,
			double InStartPos, double InIncrement,
			float* Output, uint32 InNumOutputSamples, float InGain)
		{
			if (InNumInputSamples < 2)
			{
				// not enough samples
				return InStartPos;
			}

			// make sure we have enough Input samples
			// end Pos is the last Position we will generate a sample for
			double EndPos = InStartPos + InIncrement * (InNumOutputSamples - 1);

			// for now, we are doing linear interpolation.
			// this tells us how many samples we need before the start pos
			// so that we con compute the interpolation
			static const uint32 numLeadSamples = 0;
			static const uint32 numTrailSamples = 1;

			double FirstInputSampleRequired = FMath::Floor(InStartPos);
			double LastInputSampleRequired = FMath::Floor(EndPos) + 1.0;
			check(0 <= FirstInputSampleRequired);

			//check(LastInputSampleRequired < InNumInputSamples);
			uint32 NumSamplesToProduce = InNumOutputSamples;
			if (LastInputSampleRequired > InNumInputSamples)
			{
				NumSamplesToProduce = (uint32)((double)(InNumInputSamples - InStartPos) / InIncrement);
			}

			double Pos = InStartPos;
			float Posf;
			uint32 PosA;
			uint32 PosB;
			float WeightB;


			// for each frame in a channel
			uint32 FrameNum;
			for (FrameNum = 0; FrameNum < NumSamplesToProduce; ++FrameNum)
			{
				float Sample = 0.0f;
				Posf = (float)Pos;

				// PosA is the earlier Position.
				PosA = (uint32)FMath::Floor(Posf);
				// PosB is the later Position.
				PosB = PosA + 1;
				WeightB = Posf - (float)PosA;
				check(0.0f <= WeightB && WeightB < 1.0f);

				float SampleA = Input[PosA];
				float SampleB = Input[PosB];

				Sample = (1.0f - WeightB) * SampleA + (WeightB) * SampleB;

				Pos += InIncrement;

				// write to the Output
				if (bAccumulate)
				{
					Output[FrameNum] += Sample * InGain;
				}
				else
				{
					Output[FrameNum] = Sample * InGain;
				}
			}

			if (!bAccumulate)
			{
				for (; FrameNum < InNumOutputSamples; ++FrameNum)
				{
					Output[FrameNum] = 0.0f;
				}
			}


			return Pos; // this is the next start Position (assuming the same InIncrement)
		}
	}


	double SampleRateConvert(
		const float* Input, 
		uint32 InNumInputSamples,
		double InStartPos, 
		double InIncrement,
		float* Output, 
		uint32 InNumOutputSamples, 
		float InGain, 
		bool Accumulate)
	{
		if (Accumulate)
		{
			return SampleRateConvert<true>(Input, InNumInputSamples, InStartPos, InIncrement, Output, InNumOutputSamples, InGain);
		}
		else
		{
			return SampleRateConvert<false>(Input, InNumInputSamples, InStartPos, InIncrement, Output, InNumOutputSamples, InGain);
		}
	}

	uint32 GetNumInputSamplesRequiredForSRC(uint32 InNumOutputSamplesRequired,
		double InStartPos, double InIncrement)
	{
		check(InNumOutputSamplesRequired > 0);

		// calculate the end Position
		double EndPos = InStartPos + InIncrement * (InNumOutputSamplesRequired - 1);

		// last index we will need to access
		uint32 LastInputSampleRequired = (uint32)FMath::Floor(EndPos) + 1;

		// number of Input Samples required to access that index
		return LastInputSampleRequired + 1;
	}

};