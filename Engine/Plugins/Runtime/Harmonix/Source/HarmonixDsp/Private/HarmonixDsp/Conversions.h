// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// UE
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "Math/NumericLimits.h"

namespace HarmonixDsp
{
	// this is an asymmetrical mapping from [-32768, 32767] to [-1, 0.999969482421875]
	// in this scenario, the floating point number 1.0f is out of the range of numbers representable by 1.15 fixed,
	// although -1.0 is representable
	// so if you are synthesizing floating point audio data to be converted to 1.15 fixed point,
	// be careful to deal with numbers less than -1.0f and greater OR EQUAL to 1.0f.
	const float kFloatingTo1Dot15Fixed = (float)((uint64)(1 << 15));
	const float k1Dot15FixedToFloating = 1.0f / (float)((uint64)(1 << 15));

	const float kMax1Dot15AsFloat = (float)MAX_int16 * k1Dot15FixedToFloating;

	// also asymmetrical
	const float kFloatingTo1Dot31Fixed = (float)((uint64)(1 << 31));

	// the -1 makes sure that numbers on the range [-1,1] map symmetrically 
	const float kFloatingTo8Dot24Fixed = (float)((uint64)(1 << 24) - 1);

	uint32 GetNumInputSamplesRequiredForSRC(
		uint32 NumOutputSamplesRequired,
		double StartPos, 
		double Increment);

	double /*nextStartPos*/ SampleRateConvert(
		const float* Input, 
		uint32 NumInputSamples,
		double StartPos, 
		double Increment,
		float* Output, 
		uint32 NumOutputSamples, 
		float gain = 1.0f, 
		bool Accumulate = false);

}
