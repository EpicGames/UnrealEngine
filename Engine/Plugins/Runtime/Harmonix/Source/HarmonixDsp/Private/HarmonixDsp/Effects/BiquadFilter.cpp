// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Effects/BiquadFilter.h"
#include "HarmonixDsp/Math/Complex.h"
#include "HarmonixDsp/Math/Polynomial.h"

namespace Harmonix::Dsp::Effects
{

	void FBiquadFilterCoefs::GetMagnitudeResponse(float const* InFrequencies, int32 InNumFreq, float* OutResponse)
	{
		// this algorithm comes from the result of taking
		// |B[z]|^2 = B[z]B[z]* = (B0 + B1z^-1 + B2z^-2)(B0 + B1z^1 + B2z^2)

		double B0Temp = B0 * B0 + B1 * B1 + B2 * B2;
		double B1Temp = (B0 * B1 + B1 * B2) * 2.0f;
		double B2Temp = (B0 * B2) * 2.0f;

		double A0Temp = 1.0f + A1 * A1 + A2 * A2;
		double A1Temp = (1.0f * A1 + A1 * A2) * 2.0f;
		double A2Temp = (1.0f * A2) * 2.0f;

		double ToRadians = UE_TWO_PI / SR;

		double Omega;
		// z   + z^1  = 2*(cos( Omega ))    
		// the twos get computed ahead of time in B1, B2, A1, A2
		double Half_Z_Z;
		// z^2 + z^-2 = 2*(cos(2*Omega))
		double Half_Z2_Z2;
		double B;
		double A;

		for (int32 n = 0; n < InNumFreq; ++n)
		{
			Omega = InFrequencies[n] * ToRadians;
			Half_Z_Z = FMath::Cos(Omega);
			Half_Z2_Z2 = FMath::Cos(2 * Omega);
			B = B0Temp + B1Temp * Half_Z_Z + B2Temp * Half_Z2_Z2;
			A = A0Temp + A1Temp * Half_Z_Z + A2Temp * Half_Z2_Z2;
			OutResponse[n] = (float)FMath::Sqrt(B / A);
		}
	}

	void FBiquadFilterCoefs::MakeFromSettings(const FBiquadFilterSettings& InSettings)
	{
		B0 = 1.0f;
		B1 = 0.0f;
		B2 = 0.0f;
		A1 = 0.0f;
		A2 = 0.0f;
		if (InSettings.IsEnabled)
		{
			switch (InSettings.Type)
			{
			case EBiquadFilterType::LowPass:
				MakeLowPassCoefs(InSettings.Freq, InSettings.Q);
				break;

			case EBiquadFilterType::HighPass:
				MakeHighPassCoefs(InSettings.Freq, InSettings.Q);
				break;

			case EBiquadFilterType::BandPass:
				MakeBandPassCoefs(InSettings.Freq, InSettings.Q);
				break;

			case EBiquadFilterType::Peaking:
				MakePeakingCoefs(InSettings.Freq, InSettings.DesignedDBGain, InSettings.Q);
				break;

			case EBiquadFilterType::LowShelf:
				MakeLowShelfCoefs(InSettings.Freq, InSettings.DesignedDBGain, InSettings.Q);
				break;

			case EBiquadFilterType::HighShelf:
				MakeHighShelfCoefs(InSettings.Freq, InSettings.DesignedDBGain, InSettings.Q);
				break;
			}
		}
	}

	bool FBiquadFilterCoefs::IsStable() const
	{
		TComplex<double> Pole1;
		TComplex<double> Pole2;
		GetPoles(Pole1, Pole2);

		// the filter is stable if (and only if) all of the poles
		// are inside the unit circle in the complex plane

		if (Pole1.Magnitude() > 1.0)
			return false;

		if (Pole2.Magnitude() > 1.0)
			return false;

		return true;
	}


	void FBiquadFilterCoefs::GetPoles(TComplex<double>& OutP1, TComplex<double>& OutP2) const
	{
		double A0 = 1.0;
		uint32 numRoots = FPolynomial::ComputeRootsOfQuadratic(A0, A1, A2, OutP1, OutP2);

		//since A0 is 1, we should always have two roots
		check(numRoots == 2);
	}

	void FBiquadFilterCoefs::MakeLowPassCoefs(float InF0, float InQ)
	{
		if (InF0 > ((float)SR / 2.0f))
		{
			InF0 = (float)SR / 2.0f;
		}
		double W0 = UE_TWO_PI * InF0 / SR;
		double CosW0 = FMath::Cos(W0);
		double SinW0 = FMath::Sin(W0);

		double alpha = SinW0 / (2 * InQ);

		double OneMinusCosW0 = 1 - CosW0;

		double OneOverA0 = 1.0f / (1.0f + alpha);

		B0 = 0.5 * OneMinusCosW0 * OneOverA0;
		B1 = OneMinusCosW0 * OneOverA0;
		B2 = 0.5 * OneMinusCosW0 * OneOverA0;
		A1 = -2.0 * CosW0 * OneOverA0;
		A2 = (1.0 - alpha) * OneOverA0;
	}

	void FBiquadFilterCoefs::MakeHighPassCoefs(float InF0, float InQ)
	{
		if (InF0 > ((float)SR / 2.0f))
		{
			InF0 = (float)SR / 2.0f;
		}
		double W0 = UE_TWO_PI * InF0 / SR;
		double CosW0 = FMath::Cos(W0);
		double SinW0 = FMath::Sin(W0);

		double alpha = SinW0 / (2 * InQ);

		double OnePlusCosW0 = 1 + CosW0;

		double OneOverA0 = 1.0f / (1.0f + alpha);

		B0 = 0.5 * OnePlusCosW0 * OneOverA0;
		B1 = -1.0 * OnePlusCosW0 * OneOverA0;
		B2 = 0.5 * OnePlusCosW0 * OneOverA0;
		A1 = -2.0 * CosW0 * OneOverA0;
		A2 = (1.0 - alpha) * OneOverA0;
	}

	void FBiquadFilterCoefs::MakeBandPassCoefs(float InF0, float InQ)
	{
		if (InF0 > ((float)SR / 2.0f))
		{
			InF0 = (float)SR / 2.0f;
		}
		// constant 0 dB peak gain
		double W0 = UE_TWO_PI * InF0 / SR;
		double CosW0 = FMath::Cos(W0);
		double SinW0 = FMath::Sin(W0);

		double alpha = SinW0 / (2 * InQ);

		double OnePlusCosW0 = 1 + CosW0;

		double OneOverA0 = 1.0f / (1.0f + alpha);

		B0 = alpha * OneOverA0;
		B1 = 0.0;
		B2 = -alpha * OneOverA0;
		A1 = -2.0 * CosW0 * OneOverA0;
		A2 = (1.0 - alpha) * OneOverA0;
	}

	void FBiquadFilterCoefs::MakePeakingCoefs(float InF0, float InGainDb, float InQ)
	{
		if (InF0 > ((float)SR / 2.0f))
		{
			InF0 = (float)SR / 2.0f;
		}

		double V = FMath::Pow(10, FMath::Abs(InGainDb) / 20.0f);
		double K = FMath::Tan(UE_PI * InF0 / SR);
		if (InGainDb >= 0)
		{
			double norm = 1 / (1 + 1 / InQ * K + K * K);
			B0 = (1 + V / InQ * K + K * K) * norm;
			B1 = 2 * (K * K - 1) * norm;
			B2 = (1 - V / InQ * K + K * K) * norm;
			A1 = B1;
			A2 = (1 - 1 / InQ * K + K * K) * norm;
		}
		else // cut
		{
			double norm = 1 / (1 + V / InQ * K + K * K);
			B0 = (1 + 1 / InQ * K + K * K) * norm;
			B1 = 2 * (K * K - 1) * norm;
			B2 = (1 - 1 / InQ * K + K * K) * norm;
			A1 = B1;
			A2 = (1 - V / InQ * K + K * K) * norm;
		}
	}

	void FBiquadFilterCoefs::MakeLowShelfCoefs(float InF0, float InGainDb, float InQ)
	{
		if (InF0 > ((float)SR / 2.0f))
		{
			InF0 = (float)SR / 2.0f;
		}
		double W0 = UE_TWO_PI * InF0 / SR;
		double CosW0 = FMath::Cos(W0);
		double SinW0 = FMath::Sin(W0);
		double A = FMath::Pow(10, InGainDb / 40.0f);

		double z = (A + 1 / A) * (1 / InQ - 1) + 2;
		if (z < 0.001)
		{
			z = 0.001;
		}

		double alpha = SinW0 / 2 * FMath::Sqrt(z);

		double A0 = (A + 1) + ((A - 1) * CosW0) + (2 * FMath::Sqrt(A) * alpha);
		double OneOverA0 = 1.0 / A0;

		B0 = (A * ((A + 1) - ((A - 1) * CosW0) + (2 * FMath::Sqrt(A) * alpha))) * OneOverA0;
		B1 = (2 * A * ((A - 1) - ((A + 1) * CosW0))) * OneOverA0;
		B2 = (A * ((A + 1) - ((A - 1) * CosW0) - (2 * FMath::Sqrt(A) * alpha))) * OneOverA0;
		A1 = (-2 * ((A - 1) + ((A + 1) * CosW0))) * OneOverA0;
		A2 = ((A + 1) + ((A - 1) * CosW0) - (2 * FMath::Sqrt(A) * alpha)) * OneOverA0;
	}

	void FBiquadFilterCoefs::MakeHighShelfCoefs(float InF0, float InGainDb, float InQ)
	{
		if (InF0 > ((float)SR / 2.0f))
		{
			InF0 = (float)SR / 2.0f;
		}
		double W0 = UE_TWO_PI * InF0 / SR;
		double CosW0 = FMath::Cos(W0);
		double SinW0 = FMath::Sin(W0);
		double A = FMath::Pow(10, InGainDb / 40.0f);

		double z = (A + 1 / A) * (1 / InQ - 1) + 2;
		if (z < 0.001) z = 0.001;
		double alpha = SinW0 / 2 * FMath::Sqrt(z);

		double A0 = (A + 1) - ((A - 1) * CosW0) + (2 * FMath::Sqrt(A) * alpha);
		double OneOverA0 = 1.0 / A0;

		B0 = (A * ((A + 1) + ((A - 1) * CosW0) + (2 * FMath::Sqrt(A) * alpha))) * OneOverA0;
		B1 = (-2 * A * ((A - 1) + ((A + 1) * CosW0))) * OneOverA0;
		B2 = (A * ((A + 1) + ((A - 1) * CosW0) - (2 * FMath::Sqrt(A) * alpha))) * OneOverA0;
		A1 = (2 * ((A - 1) - ((A + 1) * CosW0))) * OneOverA0;
		A2 = ((A + 1) - ((A - 1) * CosW0) - (2 * FMath::Sqrt(A) * alpha)) * OneOverA0;
	}
};