// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Effects/Delay.h"

#include "DSP/FloatArrayMath.h"

#define LOG10OFONEBIT -4.5154367f

namespace Harmonix::Dsp::Effects
{
	void FDelay::FreeUpMemory()
	{
		WetChannelInterleaved.Reset();
	};

	FDelay::FDelay()
	{
		SetWetGain(0.5f);
		SetDryGain(0.5f);
		SetFeedbackGain(0.0f);

		float NumRampPerSecond = (float)SampleRate / (float)HopNum;

		SetDelaySeconds(DelayTimeSeconds);
		// delay time should be updated more frequently
		DelayRamper.SetRampTimeMs((float)SampleRate / 2.0f, 500.0f);

		WetRamper.SetTarget(WetGain);
		WetRamper.SetRampTimeMs(NumRampPerSecond, 20.0f);

		DryRamper.SetTarget(DryGain);
		DryRamper.SetRampTimeMs(NumRampPerSecond, 20.0f);

		FeedbackRamper.SetTarget(FeedbackGain);
		FeedbackRamper.SetRampTimeMs(NumRampPerSecond, 20.0f);

		DelaySpreadLeft.SetTarget(0.0f);
		DelaySpreadLeft.SetRampTimeMs(NumRampPerSecond, 20.0f);

		DelaySpreadRight.SetTarget(100.0f);
		DelaySpreadRight.SetRampTimeMs(NumRampPerSecond, 20.0f);

		LfoSettings.Shape = EWaveShape::Sine;
		LfoSettings.IsEnabled = false;
		LfoSettings.ShouldRetrigger = true;
		LfoSettings.BeatSync = LfoSyncOption == ETimeSyncOption::TempoSync;
		LfoSettings.Shape = EWaveShape::Triangle;
		LfoSettings.Freq = 1.0f;
		LfoSettings.InitialPhase = 0.0f;
		LfoSettings.TempoBPM = 120.0f;
		LfoSettings.Depth = 1.0f; // always 1.0, we use the range instead
		SetLfoDepth(0);
		Lfo.UseSettings(&LfoSettings);

		WetFilters.Prepare(float(SampleRate));
		WetFilters.SetTargetGain(1.0f, true);
		// we want to process this filter sample-by-sample, so we have to ramp params every sample
		FeedbackFilters.Prepare(SampleRate, -1, SampleRate);
		FeedbackFilters.SetTargetGain(1.0f, true);

		SetParamsToTargets();
	}

	FDelay::~FDelay()
	{
		FreeUpMemory();
	}

	void FDelay::Prepare(float InSampleRate, uint32 InMaxChannels, float InMaxDelayTimeMs)
	{
		FreeUpMemory();

		SampleRate = (uint32)InSampleRate;
		MaxChannels = InMaxChannels;

		MaxBlockSize = 1;

		float numRampPerSecond = (float)SampleRate / (float)HopNum;

		SetDelaySeconds(DelayTimeSeconds);
		DelayRamper.SetRampTimeMs((float)SampleRate / 2.0f, 500.0f);     // delay time should be updated more frequently

		WetRamper.SetTarget(WetGain);
		WetRamper.SetRampTimeMs(numRampPerSecond, 20.0f);

		DryRamper.SetTarget(DryGain);
		DryRamper.SetRampTimeMs(numRampPerSecond, 20.0f);

		FeedbackRamper.SetTarget(FeedbackGain);
		FeedbackRamper.SetRampTimeMs(numRampPerSecond, 20.0f);

		DelaySpreadLeft.SetTarget(0.0f);
		DelaySpreadLeft.SetRampTimeMs(numRampPerSecond, 20.0f);

		DelaySpreadRight.SetTarget(100.0f);
		DelaySpreadRight.SetRampTimeMs(numRampPerSecond, 20.0f);

		Lfo.Prepare(float(SampleRate));

		WetFilters.Prepare(float(SampleRate));
		WetFilters.SetTargetGain(1.0f, true);

		SetParamsToTargets();

		MaxDelayInSamples = FMath::CeilToFloat(InSampleRate * InMaxDelayTimeMs / 1000.0f);
		MaxDelayInMs = InMaxDelayTimeMs;

		// minimum length for our delay line
		// we need to add in the input block size if we are not going to
		// impose a minimum delay. if the minimum delay is greater than the block
		// size, then we could compute the output before updating the buffer
		// with new input
		uint32 minLength = (uint32)MaxDelayInSamples + MaxBlockSize;

		// we need the actual length to be a power of 2
		// and we need a corresponding mask to index quickly
		// into the array
		Length = 1;
		while (Length < minLength)
		{
			Length *= 2;
		}

		PosMask = Length - 1;

		// allocate and clear the delay line array;
		DelayLineInterleaved.Reset();
		DelayLineInterleaved.SetNumZeroed(MaxChannels * Length);
		// allocate the wet channel buffer
		WetChannelInterleaved.Reset();
		WetChannelInterleaved.SetNumZeroed(MaxChannels * Length);

		Speed = 1.0f;
	}

	void FDelay::Unprepare()
	{
		DelayLineInterleaved.Reset();
		WetFilters.ResetState();
		FeedbackFilters.ResetState();
		FreeUpMemory();
	}

	void FDelay::ApplyNewParams()
	{
		if (DelayRamper.GetTarget() != DelayInSamples && !LfoSettings.IsEnabled)
		{
			DelayRamper.SetTarget(DelayInSamples);
		}

		if (WetRamper.GetTarget() != WetGain)
		{
			WetRamper.SetTarget(WetGain);
		}

		if (DryRamper.GetTarget() != DryGain)
		{
			DryRamper.SetTarget(DryGain);
		}

		if (FeedbackRamper.GetTarget() != FeedbackGain)
		{
			FeedbackRamper.SetTarget(FeedbackGain);
		}

		if (CanSlamParams)
		{
			SetParamsToTargets();
		}

		Lfo.UseSettings(&LfoSettings);
	}

	void FDelay::Process(Audio::FMultichannelBufferView& InOutBuffer)
	{
		const int32 NumFrames = Audio::GetMultichannelBufferNumFrames(InOutBuffer);
		ActiveChannels = InOutBuffer.Num();
		check(MaxChannels >= ActiveChannels);

		ApplyNewParams();

		CanSlamParams = false; // Since we are now processing, we have to ramp params...

		float PanParams[2] = { 1.0f, 1.0f };

		check(ActiveChannels <= AbsoluteMaxChannels);
		TDynamicStridePtr<float> InOutDataPointers[AbsoluteMaxChannels];
		TDynamicStridePtr<float> DelayLinePointers[AbsoluteMaxChannels];
		TDynamicStridePtr<float> WetChannelPointers[AbsoluteMaxChannels];
		for (int32 i = 0; i < ActiveChannels; ++i)
		{
			InOutDataPointers[i] = TDynamicStridePtr(&InOutBuffer[i][0], 1);
			DelayLinePointers[i] = TDynamicStridePtr(&DelayLineInterleaved[i], ActiveChannels);
			WetChannelPointers[i] = TDynamicStridePtr(&WetChannelInterleaved[i], ActiveChannels);
		}
		
		for (int32 s = 0; s < NumFrames; ++s)
		{
			if (DelayType == EDelayStereoType::CustomSpread)
			{
				PanParams[1] = (DelaySpreadLeft.GetCurrent() + DelaySpreadRight.GetCurrent()) / 200.0f;
				PanParams[0] = (200.0f - DelaySpreadLeft.GetCurrent() - DelaySpreadRight.GetCurrent()) / 200.0f;
			}

			float ChSum = 0.0f;
			for (int32 c = 0; c < ActiveChannels; ++c)
			{
				if (LfoSettings.IsEnabled)
				{
					const float DelayWithLfo = FMath::CeilToFloat(DelayInSamples + Lfo.GetValue());
					DelayRamper.SetTarget(FMath::Clamp(DelayWithLfo, 1, MaxDelayInSamples));
				}

				const float Input = InOutDataPointers[c][s];
				float ProcessInput = Input;
				if (DelayType == EDelayStereoType::PingPongSum)
				{
					ChSum += Input;
					if (c == ActiveChannels - 1)
					{
						ProcessInput = ChSum / static_cast<float>(ActiveChannels);
					}
					else
					{
						ProcessInput = 0.0f;
					}
				}
				else if (DelayType == EDelayStereoType::PingPongForceLR)
				{
					ChSum += Input;
					if ((c & 1) == 1)
					{
						ProcessInput = ChSum / 2.0f;
						ChSum = 0.0f;
					}
					else
					{
						ProcessInput = 0.0f;
					}
				}

				FDelayOutput Output;
				ProcessInternal(ProcessInput, DelayLinePointers[c], Output);

				// Set the dry channel
				InOutDataPointers[c][s] = DryRamper.GetCurrent() * Input;

				// Set the wet channel
				WetChannelPointers[c][s] = Output.Delay * WetRamper.GetCurrent() * PanParams[c % 2];

				// Set the feedback
				const int32 PongCh = GetPongCh(c, ActiveChannels);
				DelayLinePointers[PongCh][DelayPos] = Output.Feedback;
			}

			// Filter the feedback
			if (FeedbackFilters.GetSettings().IsEnabled)
			{
				float* DelayLineFramePointer = DelayLineInterleaved.GetData() + DelayPos * MaxChannels;
				FeedbackFilters.ProcessInterleavedInPlace(DelayLineFramePointer, 1, MaxChannels, ActiveChannels);
			}
			
			RampValues(s);
		}

		// Filter the wet signal
		if (WetFilters.GetSettings().IsEnabled)
		{
			WetFilters.ProcessInterleavedInPlace(WetChannelInterleaved.GetData(), NumFrames, MaxChannels, ActiveChannels);
		}

		// Add the wet signal to the output
		for (int32 s = 0; s < NumFrames; ++s)
		{
			for (int32 c = 0; c < ActiveChannels; ++c)
			{
				InOutDataPointers[c][s] += WetChannelPointers[c][s];
			}
		}

		// Scale the output
		if (OutputGain != 1.0f)
		{
			Algo::ForEach(InOutBuffer, [Gain = OutputGain](TArrayView<float>& Channel)
			{
				Audio::ArrayMultiplyByConstantInPlace(Channel, Gain);
			});
		}
	}

	int32 FDelay::GetPongCh(int32 ChannelIndex, int32 NumChannels) const
	{
		int32 Rv = ChannelIndex;
		if ((DelayType == EDelayStereoType::PingPongSum)
			|| (DelayType == EDelayStereoType::PingPongIndividual)
			|| (DelayType == EDelayStereoType::PingPongForceLR && NumChannels == 2))
		{
			if (NumChannels == 2)
			{
				Rv = (ChannelIndex + 1) % NumChannels;
			}
			else if (NumChannels == 6 && FiveOneSurroundRotation[ChannelIndex] >= 0)
			{
				Rv = FiveOneSurroundRotation[ChannelIndex];
			}
			else if (NumChannels == 8 && SevenOneSurroundRotation[ChannelIndex] >= 0)
			{
				Rv = SevenOneSurroundRotation[ChannelIndex];
			}
			else if (NumChannels == 12 && SevenOneFourSurroundRotation[ChannelIndex] >= 0)
			{
				Rv = SevenOneFourSurroundRotation[ChannelIndex];
			}
		}
		else if (DelayType == EDelayStereoType::PingPongForceLR)
		{
			if (NumChannels == 6 && FiveOneSurroundLRForce[ChannelIndex] >= 0)
			{
				Rv = FiveOneSurroundLRForce[ChannelIndex];
			}
			else if (NumChannels == 8 && SevenOneSurroundLRForce[ChannelIndex] >= 0)
			{
				Rv = SevenOneSurroundLRForce[ChannelIndex];
			}
			else if (NumChannels == 12 && SevenOneFourSurroundLRForce[ChannelIndex] >= 0)
			{
				Rv = SevenOneFourSurroundLRForce[ChannelIndex];
			}
		}
		return Rv;
	}

	void FDelay::ProcessInternal(float const X, const TDynamicStridePtr<float>& DelayLinePtr, FDelayOutput& OutDelay) const
	{
		// current position in delay line (where next sample goes)
		check(DelayPos < Length);
		check(DelayLinePtr != nullptr);

		// find delay line output by linearly interpolating between
		// two (logically) adjacent samples. The "A position" is the integer
		// delay that is furthest from the input, and the "B position" is
		// the position just closer than A.
		const float CurrentDelay = FMath::Max(DelayRamper.GetCurrent(), 1);

		// this describes the max integer delay (our so-called A position)
		const uint32 DelayA = FMath::CeilToInt(CurrentDelay);

		// and this tells us how much B needs to be blended in
		const float WeightB = static_cast<float>(DelayA) - CurrentDelay;
		check(0.0f <= WeightB && WeightB < 1.0f);

		const uint32 DelayPosA = (Length + DelayPos - DelayA) & PosMask;
		const uint32 DelayPosB = (DelayPosA + 1) & PosMask;

		const float XDelayA = DelayLinePtr[DelayPosA];
		const float XDelayB = DelayLinePtr[DelayPosB];

		OutDelay.Delay = XDelayA * (1.0f - WeightB) + XDelayB * (WeightB);

		OutDelay.Feedback = X + FeedbackRamper.GetCurrent() * OutDelay.Delay;
	}

	void FDelay::RampValues(int32 FrameNum)
	{
		DelayPos = (DelayPos + 1) & PosMask;

		if ((FrameNum % HopNum) == 0)
		{
			WetRamper.Ramp();
			DryRamper.Ramp();
			FeedbackRamper.Ramp();
			DelaySpreadLeft.Ramp();
			DelaySpreadRight.Ramp();
		}

		if ((FrameNum % 2) == 0)
		{
			DelayRamper.Ramp();
			Lfo.Advance(1);
		}
	}

	void FDelay::SetDelaySeconds(float InSeconds)
	{
		InSeconds = FMath::Max(InSeconds, UE_SMALL_NUMBER);
		DelayTimeSeconds = InSeconds;

		switch (TimeSyncOption)
		{
		case ETimeSyncOption::TempoSync:
			InSeconds *= 60 / TempoBpm;
			break;
		case ETimeSyncOption::SpeedScale:
			InSeconds /= Speed;
			break;
		case ETimeSyncOption::None:
			break;
		default:
			break;
		}

		float numSamples = InSeconds * SampleRate;
		DelayInSamples = FMath::Clamp(numSamples, 1, MaxDelayInSamples);
	}

	void FDelay::SetTempo(float Bpm)
	{
		Bpm = FMath::Max(Bpm, 1.0f);
		if (TempoBpm != Bpm)
		{
			TempoBpm = Bpm;
			if (TimeSyncOption != ETimeSyncOption::None)
			{
				SetDelaySeconds(DelayTimeSeconds);
			}

			LfoSettings.TempoBPM = Bpm;
			switch (LfoSyncOption)
			{
			case ETimeSyncOption::TempoSync:
				Lfo.UseSettings(&LfoSettings);
				break;
			case ETimeSyncOption::SpeedScale:
			case ETimeSyncOption::None:
			default:
				break;
			}
		}
	}

	void FDelay::SetSpeed(float InSpeed)
	{
		InSpeed = FMath::Max(InSpeed, UE_SMALL_NUMBER);
		if (Speed != InSpeed)
		{
			Speed = InSpeed;
			if (TimeSyncOption != ETimeSyncOption::None)
			{
				SetDelaySeconds(DelayTimeSeconds);
			}

			switch (LfoSyncOption)
			{
			case ETimeSyncOption::SpeedScale:
				LfoSettings.Freq = LfoBaseFrequency / Speed;
				Lfo.UseSettings(&LfoSettings);
				break;
			case ETimeSyncOption::TempoSync:
			case ETimeSyncOption::None:
			default:
				break;
			}
		}
	}

	void FDelay::SetTimeSyncOption(ETimeSyncOption InOption)
	{
		if (TimeSyncOption != InOption)
		{
			TimeSyncOption = InOption;
			SetDelaySeconds(DelayTimeSeconds);
		}
	}

	void FDelay::SetFeedbackGain(float InFeedbackGain)
	{
		InFeedbackGain = FMath::Clamp(InFeedbackGain, 0.0f, 1.0f);
		if (HarmonixDsp::LinearToDB(InFeedbackGain) < -90.0f)
		{
			InFeedbackGain = 0.0f;
		}
		FeedbackGain = InFeedbackGain;
	}

	float FDelay::GetFeedbackGain() const
	{
		return FeedbackGain;
	}

	void FDelay::SetWetGain(float InWetGain)
	{
		InWetGain = FMath::Clamp(InWetGain, 0.0f, 1.0f);
		if (HarmonixDsp::LinearToDB(InWetGain) < -90.0f)
		{
			InWetGain = 0.0f;
		}
		WetGain = InWetGain;
	}

	float FDelay::GetWetGain() const
	{
		return WetGain;
	}

	void FDelay::SetDryGain(float InDryGain)
	{
		InDryGain = FMath::Clamp(InDryGain, 0.0f, 1.0f);
		if (HarmonixDsp::LinearToDB(InDryGain) < -90.0f)
		{
			InDryGain = 0.0f;
		}
		DryGain = InDryGain;
	}

	float FDelay::GetDryGain() const
	{
		return DryGain;
	}

	void FDelay::SetWetFilterEnabled(bool Enabled)
	{
		if (Enabled != WetFilters.GetIsEnabled())
		{
			WetFilters.SetEnabled(Enabled);
		}
	}

	void FDelay::SetFeedbackFilterEnabled(bool Enabled)
	{
		if (Enabled != FeedbackFilters.GetIsEnabled())
		{
			FeedbackFilters.SetEnabled(Enabled);
		}
	}

	void FDelay::SetFilterFreq(float Freq)
	{
		Freq = FMath::Clamp(Freq, 20.0f, 20000.0f);
		if (Freq != WetFilters.GetFreqTarget())
		{
			WetFilters.SetFreqTarget(Freq);
			FeedbackFilters.SetFreqTarget(Freq);
		}
	}

	void FDelay::SetFilterQ(float InQ)
	{
		InQ = FMath::Clamp(InQ, 0.01f, 10.0f);
		if (InQ != WetFilters.GetQTarget())
		{
			WetFilters.SetQTarget(InQ);
			FeedbackFilters.SetQTarget(InQ);
		}
	}

	void FDelay::SetFilterType(EDelayFilterType DelayFilter)
	{
		SetFilterType((EBiquadFilterType)((uint8)(DelayFilter)));
	}

	void FDelay::SetFilterType(EBiquadFilterType Type)
	{
		if (Type != WetFilters.GetType())
		{
			WetFilters.SetType(Type);
			FeedbackFilters.SetType(Type);
		}
	}

	void FDelay::SetLfoEnabled(bool Enabled)
	{
		LfoSettings.IsEnabled = Enabled;
	}

	void FDelay::SetLfoTimeSyncOption(ETimeSyncOption InOption)
	{
		if (LfoSyncOption != InOption)
		{
			LfoSyncOption = InOption;
			switch (LfoSyncOption)
			{
			case ETimeSyncOption::TempoSync:
				LfoSettings.BeatSync = true;
				LfoSettings.Freq = LfoBaseFrequency;
				break;
			case ETimeSyncOption::SpeedScale:
				LfoSettings.BeatSync = false;
				LfoSettings.Freq = LfoBaseFrequency / Speed;
				break;
			case ETimeSyncOption::None:
			default:
				LfoSettings.BeatSync = false;
				LfoSettings.Freq = LfoBaseFrequency;
				break;
			}
			Lfo.UseSettings(&LfoSettings);
		}
	}

	void FDelay::SetLfoFreq(float InFreq)
	{
		InFreq = FMath::Clamp(InFreq, UE_SMALL_NUMBER, 40.0f);
		if (InFreq != LfoBaseFrequency)
		{
			LfoBaseFrequency = InFreq;
			LfoSettings.Freq = LfoBaseFrequency;
			if (LfoSyncOption == ETimeSyncOption::SpeedScale)
			{
				LfoSettings.Freq /= Speed;
			}
			Lfo.UseSettings(&LfoSettings);
		}
	}

	void FDelay::SetLfoDepth(float InDepth)
	{
		// the input param is either seconds or quarter notes, depending on the delay time setting
		// convert here so we can do fewer calculations
		float DepthSeconds = InDepth;
		
		switch (TimeSyncOption)
		{
		case ETimeSyncOption::TempoSync:
			DepthSeconds = InDepth * 60 / TempoBpm;
			break;
		default:
			break;
		}

		const float DepthSamples = FMath::Clamp(SampleRate * DepthSeconds, 0.0f, MaxDelayInSamples);
		Lfo.SetRangeAndMode(TInterval<float>(0, DepthSamples), Modulators::ELfoMode::MinUp);
	}

	float FDelay::GetLfoDepth() const
	{
		return Lfo.GetRange().Size();
	}

	void FDelay::SetStereoSpreadLeft(float InSpread)
	{
		DelaySpreadLeft.SetTarget(FMath::Clamp(InSpread, 0.0f, 100.0f));
	}

	void FDelay::SetStereoSpreadRight(float InSpread)
	{
		DelaySpreadRight.SetTarget(FMath::Clamp(InSpread, 0.0f, 100.0f));
	}


	void FDelay::SetStereoType(EDelayStereoType InType)
	{
		DelayType = EDelayStereoType(InType);
	}
	
	float FDelay::CalculateSecsToIdle()
	{
		float AbsGain = FMath::Abs(FeedbackGain);

		if (AbsGain > 0.99999f)
		{
			// more than 30 days
			return 2592000.0f;
		}

		if (AbsGain < 0.0001f)
		{
			AbsGain = 0.0001f;
		}

		return DelayTimeSeconds * (LOG10OFONEBIT / HarmonixDsp::Log10(AbsGain));
	}

};
