// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Effects/DistortionV2.h"

#include "Misc/ScopeLock.h"

namespace Harmonix::Dsp::Effects
{
	/*
	FIR filter designed with
	http://t-filter.appspot.com

	sampling frequency: 192000 Hz

	* 0 Hz - 16000 Hz
	  gain = 1
	  desired ripple = 1 dB
	  actual ripple = 1.5641490908926312 dB

	* 23500 Hz - 96000 Hz
	  gain = 0
	  desired attenuation = -40 dB
	  actual attenuation = -33.65716453863208 dB

	*/

	const float FDistortionV2::kOversamplingFilterTaps[FDistortionV2::kNumFilterTaps] = {
	   -0.0017550795184103213f,  0.016839220734025377f,  0.01899531546857971f,  0.021968081098656493f,
	   0.018965410381779454f,  0.008391946110521412f,  -0.00811498323834502f,  -0.025738425781131266f,
	   -0.03742631465538428f,  -0.035990831360696585f,  -0.016605054794654413f,  0.02101117947222433f,
	   0.07165682689298712f,  0.12554544829043665f,  0.17070038439045346f,  0.19645543880380015f,
	   0.19645543880380015f,  0.17070038439045346f,  0.12554544829043665f,  0.07165682689298712f,
	   0.02101117947222433f,  -0.016605054794654413f,  -0.035990831360696585f,  -0.03742631465538428f,
	   -0.025738425781131266f,  -0.00811498323834502f,  0.008391946110521412f,  0.018965410381779454f,
	   0.021968081098656493f,  0.01899531546857971f,  0.016839220734025377f,  -0.0017550795184103213f
	};

	FDistortionV2::FDistortionV2(uint32 InSampleRate, uint32 InMaxRenderBufferSize)
		: Type(EDistortionTypeV2::Clean)
		, DoOversampling(false)
	{
		Setup(InSampleRate, InMaxRenderBufferSize);
	}

	void FDistortionV2::Setup(uint32 InSampleRate, uint32 InMaxRenderBufferSize)
	{
		SampleRate = InSampleRate;
		float NumRampPerSecond = SampleRate / (float)kRampHops;

		for (int32 i = 0; i < FDistortionSettingsV2::kNumFilters; i++)
		{
			FilterCoefs[i].SetRampTimeMs(NumRampPerSecond, 20.0f);
			FilterCoefs[i].SnapTo(FBiquadFilterCoefs());
			FilterPreClip[i] = false;
			FilterGain[i].SetRampTimeMs(NumRampPerSecond, 20.0f);
			FilterGain->SnapTo(1.0f);
		}

		for (int32 i = 0; i < kMaxChannels; ++i)
		{
			OversampleFilterUp[i].Init(kOversamplingFilterTaps, kNumFilterTaps, false);
			OversampleFilterDown[i].Init(kOversamplingFilterTaps, kNumFilterTaps, false);
		}

		InputGain.SetRampTimeMs(NumRampPerSecond, 20.0f);
		InputGain.SnapTo(1.0f);
		OutputGain.SetRampTimeMs(NumRampPerSecond, 20.0f);
		OutputGain.SnapTo(1.0f);
		DCAdjust.SetRampTimeMs(NumRampPerSecond, 20.0f);
		DCAdjust.SnapTo(0.0f);
		DryGain.SetRampTimeMs(NumRampPerSecond, 20.0f);
		DryGain.SnapTo(0.0f);
		WetGain.SetRampTimeMs(NumRampPerSecond, 20.0f);
		WetGain.SnapTo(1.0f);

		// allow for 4x oversampling
		int32 NumFrames = InMaxRenderBufferSize * 4;
		UpsampleBuffer.SetNumUninitialized(NumFrames);
	}

	void FDistortionV2::SetInputGainDb(float InGainDb, bool Snap)
	{
		SetInputGain(HarmonixDsp::DBToLinear(InGainDb), Snap);
	}

	float FDistortionV2::GetInputGainDb() const
	{
		return HarmonixDsp::LinearToDB(InputGain);
	}

	void FDistortionV2::SetInputGain(float InGain, bool Snap /*= false*/)
	{
		InGain = FMath::Clamp(InGain, 0.0f, 11.0f);
		if (InGain == InputGain.GetTarget())
		{
			return;
		}

		FScopeLock Lock(&SettingsLock);

		InputGain = InGain;
		if (Snap)
		{
			InputGain.SnapToTarget();
		}
	}

	void FDistortionV2::SetOutputGainDb(float InGainDb, bool Snap /*= false*/)
	{
		SetOutputGain(HarmonixDsp::DBToLinear(InGainDb), Snap);
	}

	float FDistortionV2::GetOutputGainDb() const
	{
		return HarmonixDsp::LinearToDB(OutputGain);
	}

	void FDistortionV2::SetOutputGain(float InGain, bool Snap /*= false*/)
	{
		InGain = FMath::Clamp(InGain, 0.0f, 11.0f);
		if (InGain == OutputGain.GetTarget())
			return;

		FScopeLock Lock(&SettingsLock);
		OutputGain = InGain;
		if (Snap)
		{
			OutputGain.SnapToTarget();
		}
	}

	void FDistortionV2::SetDryGainDb(float InGainDb, bool InSnap)
	{
		SetDryGain(HarmonixDsp::DBToLinear(InGainDb), InSnap);
	}

	float FDistortionV2::GetDryGainDb() const
	{
		return HarmonixDsp::LinearToDB(DryGain);
	}

	void FDistortionV2::SetDryGain(float InGain, bool Snap /*= false*/)
	{
		InGain = FMath::Clamp(InGain, 0.0f, 1.0f);
		if (InGain == DryGain.GetTarget())
		{
			return;
		}

		FScopeLock Lock(&SettingsLock);

		DryGain = InGain;
		if (Snap)
		{
			DryGain.SnapToTarget();
		}
	}

	void FDistortionV2::SetWetGainDb(float InGainDb, bool InSnap)
	{
		SetWetGain(HarmonixDsp::DBToLinear(InGainDb), InSnap);

	}

	float FDistortionV2::GetWetGainDb() const
	{
		return HarmonixDsp::LinearToDB(WetGain);
	}

	void FDistortionV2::SetWetGain(float InGain, bool Snap /*= false*/)
	{
		InGain = FMath::Clamp(InGain, 0.0f, 1.0f);
		if (InGain == WetGain.GetTarget())
		{
			return;
		}


		FScopeLock Lock(&SettingsLock);

		WetGain = InGain;
		if (Snap)
		{
			WetGain.SnapToTarget();
		}
	}

	void FDistortionV2::SetMix(float Mix, bool InSnap)
	{
		float NewDry;
		float NewWet;
		HarmonixDsp::PanToGainsConstantPower(Mix * 2.f - 1.f, NewDry, NewWet);
		NewDry = FMath::Clamp(NewDry, 0.0f, 1.0f);
		NewWet = FMath::Clamp(NewWet, 0.0f, 1.0f);

		FScopeLock Lock(&SettingsLock);

		if (NewDry == DryGain.GetTarget() && NewWet == WetGain.GetTarget())
		{
			return;
		}

		DryGain = NewDry;
		WetGain = NewWet;
		
		if (InSnap)
		{
			DryGain.SnapToTarget();
			WetGain.SnapToTarget();
		}
	}

	void FDistortionV2::SetDCOffset(float Offset, bool InSnap)
	{
		Offset = FMath::Clamp(Offset, -1.0f, 1.0f);
		if (Offset == DCAdjust.GetTarget())
		{
			return;
		}

		FScopeLock Lock(&SettingsLock);

		DCAdjust = Offset;

		if (InSnap)
		{
			DCAdjust.SnapToTarget();
		}
	}

	void FDistortionV2::SetType(EDistortionTypeV2 InType)
	{
		if (Type == InType)
		{
			return;
		}

		FScopeLock Lock(&SettingsLock);

		check((uint8)Type >= 0 && (uint8)Type <= (uint8)EDistortionTypeV2::Num);
		Type = InType;
	}

	void FDistortionV2::SetType(uint8 InType)
	{
		SetType((EDistortionTypeV2)InType);
	}

	void FDistortionV2::SetupFilter(int32 Index, const FDistortionFilterSettings& InSettings)
	{
		check(Index < FDistortionSettingsV2::kNumFilters);
		if (FilterSettings[Index] == InSettings.Filter
			&& FilterPreClip[Index] == InSettings.FilterPreClip
			&& FilterPasses[Index] == InSettings.NumPasses)
		{
			return;
		}

		FScopeLock Lock(&SettingsLock);

		if (InSettings.Filter.IsEnabled)
		{
			switch (InSettings.Filter.Type)
			{
			case EBiquadFilterType::Peaking:
			case EBiquadFilterType::LowShelf:
			case EBiquadFilterType::HighShelf:
			{
				FilterGain[Index] = 1.0f;
				break;
			}
			default:
			{
				FilterGain[Index] = HarmonixDsp::DBToLinear(InSettings.Filter.DesignedDBGain);
				break;
			}
			}

			FilterCoefs[Index].SetTarget(FBiquadFilterCoefs(InSettings.Filter, (float)SampleRate));
			FilterPreClip[Index] = InSettings.FilterPreClip;
		}
		else
		{
			FilterGain[Index] = 1.0f;
		}

		FilterSettings[Index] = InSettings.Filter;
		FilterPasses[Index] = InSettings.NumPasses;
	}

	void FDistortionV2::SetOversample(bool InDoOversample, int32 InRenderBufferSizeFrames)
	{
		if (InDoOversample == DoOversampling)
		{
			return;
		}

		FScopeLock Lock(&SettingsLock);
		if (InDoOversample && !DoOversampling)
		{
			for (int32 i = 0; i < kMaxChannels; ++i)
			{
				OversampleFilterUp[i].Reset();
				OversampleFilterDown[i].Reset();
			}

			// make sure up-sample buffer can fit enough samples for 4x oversampling...
			UpsampleBuffer.SetNumUninitialized(InRenderBufferSizeFrames * 4);
		}
		DoOversampling = InDoOversample;
	}

	void FDistortionV2::SetSampleRate(uint32 InSampleRate)
	{
		if (InSampleRate == SampleRate)
		{
			return;
		}

		FScopeLock Lock(&SettingsLock);
		SampleRate = InSampleRate;
		for (int32 i = 0; i < FDistortionSettingsV2::kNumFilters; ++i)
		{
			FilterCoefs[i].SetTarget(FBiquadFilterCoefs(FilterSettings[i], (float)SampleRate));
		}
	}

	void FDistortionV2::Setup(const FDistortionSettingsV2& InSettings, uint32 InSampleRate, uint32 InRenderBufferSizeFrames, bool InSnap)
	{
		SetInputGainDb(InSettings.InputGainDb, InSnap);
		SetOutputGainDb(InSettings.OutputGainDb, InSnap);
		SetDryGain(InSettings.DryGain, InSnap);
		SetWetGain(InSettings.WetGain, InSnap);
		SetDCOffset(InSettings.DCAdjust, InSnap);
		SetType(InSettings.Type);
		SetupFilter(0, InSettings.Filters[0]);
		SetupFilter(1, InSettings.Filters[1]);
		SetupFilter(2, InSettings.Filters[2]);
		SetOversample(InSettings.Oversample, InRenderBufferSizeFrames);
		SetSampleRate(InSampleRate);
	}

	void FDistortionV2::Reset()
	{
		FScopeLock Lock(&SettingsLock);
		for (int32 j = 0; j < FDistortionSettingsV2::kNumFilters; ++j)
		{
			for (int32 i = 0; i < kMaxChannels; ++i)
			{
				Filter[j][i].ResetState();
			}
		}
		for (int32 i = 0; i < kMaxChannels; ++i)
		{
			OversampleFilterUp[i].Reset();
			OversampleFilterDown[i].Reset();
		}
	}

	void FDistortionV2::Process(const TArray<TArrayView<const float>>& InBuffer, const TArray<TArrayView<float>>& OutBuffer)
	{
		FScopeLock Lock(&SettingsLock);
		TLinearRamper<float>  InputGainTemp = InputGain;
		TLinearRamper<float>  OutputGainTemp = OutputGain;
		TLinearRamper<float>  DCAdjustTemp = DCAdjust;
		TLinearRamper<float>  DryGainTemp = DryGain;
		TLinearRamper<float>  WetGainTemp = WetGain;
		TLinearRamper<FBiquadFilterCoefs> FilterCoefsTemp[FDistortionSettingsV2::kNumFilters];
		TLinearRamper<float>  FilterGainTemp[FDistortionSettingsV2::kNumFilters];

		const int32 NumChannels = InBuffer.Num();
		check(NumChannels > 0);
		check(OutBuffer.Num() == NumChannels);

		const int32 NumFrames = InBuffer[0].Num();
		check(OutBuffer[0].Num() == NumFrames);

		for (int32 ch = 0; ch < NumChannels; ++ch)
		{
			// process "pre-clip" filters...
			bool DidFilter = false;
			for (int32 i = 0; i < FDistortionSettingsV2::kNumFilters; ++i)
			{
				FilterCoefsTemp[i] = FilterCoefs[i];
				FilterGainTemp[i] = FilterGain[i];
				if (FilterSettings[i].IsEnabled && FilterPreClip[i])
				{
					const float* InPtr = InBuffer[ch].GetData();
					float* OutPtr = OutBuffer[ch].GetData();
					
					for (int32 f = 0; f < NumFrames; f += kRampHops)
					{
						FilterCoefsTemp[i].Ramp();
						FilterGainTemp[i].Ramp();
						Filter[i][ch].Process(InPtr, OutPtr, kRampHops, FilterCoefsTemp[i], (double)FilterGainTemp[i], FilterPasses[i]);
						InPtr += kRampHops;
						OutPtr += kRampHops;
					}
					DidFilter = true;
				}
			}

			const float* ProcInPtr = (DidFilter) ? OutBuffer[ch].GetData() : InBuffer[ch].GetData();
			float* ProcOutPtr = OutBuffer[ch].GetData();
			int32 ProcFrames = NumFrames;


			if (DoOversampling)
			{
				// up sample...
				check(NumFrames * 4 <= UpsampleBuffer.Num());
				float* FilterOut = UpsampleBuffer.GetData();
				
				for (int32 i = 0; i < NumFrames; ++i)
				{
					OversampleFilterUp[ch].Upsample4x(ProcInPtr[i] * 4.0f, FilterOut);
					FilterOut += 4;
				}
				
				ProcInPtr = UpsampleBuffer.GetData();
				ProcOutPtr = UpsampleBuffer.GetData();
				ProcFrames = NumFrames * 4;
			}

			InputGainTemp = InputGain;
			OutputGainTemp = OutputGain;
			DCAdjustTemp = DCAdjust;
			DryGainTemp = DryGain;
			WetGainTemp = WetGain;

			switch (Type)
			{
			case EDistortionTypeV2::Clip:
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					ProcOutPtr[i] = FMath::Clamp((ProcInPtr[i] + DCAdjustTemp) * InputGainTemp, -1.0f, 1.0f) * OutputGainTemp;
				}
			}
			break;
			case EDistortionTypeV2::Clean:
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					ProcOutPtr[i] = tanhf((ProcInPtr[i] + DCAdjustTemp) * UE_HALF_PI * InputGainTemp) * OutputGainTemp;
				}
			}
			break;
			case EDistortionTypeV2::Warm:
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					ProcOutPtr[i] = FMath::Sin((ProcInPtr[i] + DCAdjustTemp) * UE_HALF_PI * InputGainTemp) * OutputGainTemp;
				}
			}
			break;
			case EDistortionTypeV2::Soft:   //soft and asymmetric from http://www.music.mcgill.ca/~gary/courses/projects/618_2009/NickDonaldson/#Distortion
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					float OutSample = (ProcInPtr[i] + DCAdjustTemp) * InputGainTemp;
					if (OutSample > 1)
					{
						OutSample = 0.66666f;
					}
					else if (OutSample < -1)
					{
						OutSample = -0.66666f;
					}
					else
					{
						OutSample = OutSample - (OutSample * OutSample * OutSample) / 3.0f;
					}
					ProcOutPtr[i] = OutSample * OutputGainTemp;
				}
			}
			break;
			case EDistortionTypeV2::Asymmetric:
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					float OutSample = (ProcInPtr[i] + DCAdjustTemp) * InputGainTemp;
					if (OutSample >= .320018f)
					{
						OutSample = .630035f;
					}
					else if (OutSample >= -.08905f)
					{
						OutSample = -6.153f * OutSample * OutSample + 3.9375f * OutSample;
					}
					else if (OutSample >= -1)
					{
						OutSample = -.75f * (1 - FMath::Pow(1 - (FMath::Abs(OutSample) - .032847f), 12) + .333f * (FMath::Abs(OutSample) - .032847f)) + .01f;
					}
					else
					{
						OutSample = -.9818f;
					}
					ProcOutPtr[i] = OutSample * OutputGainTemp;
				}
			}
			break;
			case EDistortionTypeV2::Cruncher:
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					float OutSample = (ProcInPtr[i] + DCAdjustTemp) * InputGainTemp;
					float sign = (OutSample < 0.0f) ? -1.0f : 1.0f;
					OutSample = (FMath::Abs(2.0f * OutSample) - (OutSample * OutSample)) * sign;
					ProcOutPtr[i] = FMath::Clamp(OutSample * OutputGainTemp, -1.0f, 1.0f);
				}
			}
			break;
			case EDistortionTypeV2::CaptCrunch:
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					float OutSample = (ProcInPtr[i] + DCAdjustTemp) * InputGainTemp;
					OutSample = (3.f * OutSample / 2.f) * (1 - (OutSample * OutSample / 3));
					ProcOutPtr[i] = FMath::Clamp(OutSample * OutputGainTemp, -1.0f, 1.0f);
				}
			}
			break;
			case EDistortionTypeV2::Rectifier:
			{
				for (int32 i = 0; i < ProcFrames; ++i)
				{
					if (i % kRampHops == 0)
					{
						DCAdjustTemp.Ramp();
						InputGainTemp.Ramp();
						OutputGainTemp.Ramp();
					}
					float OutSample = (ProcInPtr[i] + DCAdjustTemp) * InputGainTemp;
					if (OutSample < 0.0f)
					{
						OutSample *= -1.0f;
					}
					OutSample = (OutSample * 2.0f) - 1.0f;
					ProcOutPtr[i] = FMath::Clamp(OutSample * OutputGainTemp, -1.0f, 1.0f);
				}
			}
			break;
			default:
				checkNoEntry();
			}
			if (DoOversampling)
			{
				// down sample...
				float* FilterIn = UpsampleBuffer.GetData();
				for (int32 i = 0; i < NumFrames; ++i)
				{
					OversampleFilterDown[ch].AddData(FilterIn, 4);
					FilterIn += 4;
					OutBuffer[ch][i] = OversampleFilterDown[ch].GetSample();
				}
			}

			// process "post-clip" filters...
			for (int32 i = 0; i < FDistortionSettingsV2::kNumFilters; ++i)
			{
				if (FilterSettings[i].IsEnabled && !FilterPreClip[i])
				{
					const float* InPtr = OutBuffer[ch].GetData();
					float* OutPtr = OutBuffer[ch].GetData();
					for (int32 f = 0; f < NumFrames; f += kRampHops)
					{
						FilterCoefsTemp[i].Ramp();
						FilterGainTemp[i].Ramp();
						Filter[i][ch].Process(InPtr, OutPtr, kRampHops, FilterCoefsTemp[i], FilterGainTemp[i], FilterPasses[i]);
						InPtr += kRampHops;
						OutPtr += kRampHops;
					}
				}
			}

			// now wet/dry mix
			for (int32 i = 0; i < NumFrames; ++i)
			{
				if (i % kRampHops == 0)
				{
					WetGainTemp.Ramp();
					DryGainTemp.Ramp();
				}
				OutBuffer[ch][i] = FMath::Clamp(OutBuffer[ch][i] * WetGainTemp + InBuffer[ch][i] * DryGainTemp, -1.0f, 1.0f);
			}
		}
		InputGain = InputGainTemp;
		OutputGain = OutputGainTemp;
		DCAdjust = DCAdjustTemp;
		DryGain = DryGainTemp;
		WetGain = WetGainTemp;
		for (int32 i = 0; i < FDistortionSettingsV2::kNumFilters; ++i)
		{
			FilterGain[i] = FilterGainTemp[i];
			FilterCoefs[i] = FilterCoefsTemp[i];
		}
	}

};