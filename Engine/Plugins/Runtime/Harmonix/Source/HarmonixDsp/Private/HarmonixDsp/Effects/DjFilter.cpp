// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Effects/DjFilter.h"

#include "DSP/FloatArrayMath.h"

namespace Harmonix::Dsp::Effects
{
	FDjFilter::FDjFilter(const float InSampleRate)
	{
		Reset(InSampleRate);
	}

	void FDjFilter::Reset(const float InSampleRate)
	{
		Filter.Init(InSampleRate, 1 /* NumChannels */);
		Filter.Reset();
		Filter.SetBandStopControl(0);

		Amount = Amount.Default;
		Resonance = Resonance.Default;
		LowPassMinFrequency = LowPassMinFrequency.Default;
		LowPassMaxFrequency = LowPassMaxFrequency.Default;
		HighPassMinFrequency = HighPassMinFrequency.Default;
		HighPassMaxFrequency = HighPassMaxFrequency.Default;
		DeadZoneSize = DeadZoneSize.Default;

		LastFrequency = -1;
		LastQ = -1;
		LastDryGain = 1;
		LastWetGain = 0;
	}

	float AmountToFrequency(float Amount, float MinFrequency, float MaxFrequency, float DeadZoneSize)
	{
		// shift the amount
		if (Amount < 0)
		{
			Amount += 1;
		}

		return Audio::GetLogFrequencyClamped(Amount, { DeadZoneSize, 1.0f }, { MinFrequency, MaxFrequency });
	}

	void CalculateCrossFade(float Amount, const float DeadZoneSize, float* DryGain, float* WetGain)
	{
		// The behavior of the cross-fade is mirrored about zero
		Amount = FMath::Abs(Amount);

		// If we're outside the dead zone, it's just wet
		if (Amount > DeadZoneSize)
		{
			*DryGain = 0;
			*WetGain = 1;
			return;
		}

		// Get the normalized linear amount
		Amount /= DeadZoneSize;
		FMath::SinCos(WetGain, DryGain, Amount * HALF_PI);
	}

	void FDjFilter::Process(const Audio::FAlignedFloatBuffer& InBuffer, Audio::FAlignedFloatBuffer& OutBuffer)
	{
		// Cache the amount and dead zone in case it changes mid-block
		const float CurrentAmount = Amount.Get();
		const float CurrentDeadZoneSize = DeadZoneSize.Get();

		// Update the filter frequency and resonance if necessary
		bool NeedsUpdate = false;
		
		const float CurrentFrequency = AmountToFrequency(
			CurrentAmount,
			CurrentAmount >= 0 ? HighPassMinFrequency : LowPassMinFrequency,
			CurrentAmount >= 0 ? HighPassMaxFrequency : LowPassMaxFrequency,
			CurrentDeadZoneSize);

		if (CurrentFrequency != LastFrequency)
		{
			Filter.SetFrequency(CurrentFrequency);
			NeedsUpdate = true;
			LastFrequency = CurrentFrequency;
		}

		const float CurrentQ = Resonance.Get();

		if (CurrentQ != LastQ)
		{
			Filter.SetQ(CurrentQ);
			NeedsUpdate = true;
			LastQ = CurrentQ;
		}

		if (NeedsUpdate)
		{
			Filter.Update();
		}

		// Set the filter type
		Filter.SetFilterType(Amount > 0 ? Audio::EFilter::HighPass : Audio::EFilter::LowPass);
		
		// Process the filter
		check(InBuffer.Num() == OutBuffer.Num());
		const int32 NumFrames = InBuffer.Num();
		Filter.ProcessAudio(InBuffer.GetData(), OutBuffer.Num(), OutBuffer.GetData());

		// Calculate the cross-fade between dry and wet if we're in the dead zone
		float DryGain, WetGain;
		CalculateCrossFade(CurrentAmount, CurrentDeadZoneSize, &DryGain, &WetGain);
		
		// If we're inside the dead zone, mix in the dry signal
		if (DryGain > 0.0f)
		{
			// This will internally skip to the cheaper constant multiply versions if the gains didn't change
			Audio::ArrayFade(OutBuffer, LastWetGain, WetGain);
			Audio::ArrayMixIn(InBuffer, OutBuffer, LastDryGain, DryGain);
		}

		// Keep track of the last gain values so we can fade from them
		LastWetGain = WetGain;
		LastDryGain = DryGain;
	}
}
