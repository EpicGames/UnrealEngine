// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Effects/FirFilter.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"

namespace Harmonix::Dsp::Effects
{
	FFirFilter::FFirFilter()
		: Taps(nullptr)
		, TapCount(0)
		, OwnTaps(false)
		, History(nullptr)
		, HistoryIndex(0)
	{}

	FFirFilter::~FFirFilter()
	{
		if (OwnTaps)
		{
			FMemory::Free((void*)Taps);
			Taps = nullptr;
		}
		FMemory::Free((void*)History);
		History = nullptr;
	}


	void FFirFilter::Init(const float* InTaps, int32 InTapCount, bool InCopyTaps)
	{
		if (OwnTaps && Taps)
		{
			FMemory::Free((void*)Taps);
		}

		if (InCopyTaps)
		{
			Taps = (float*)FMemory::Malloc(sizeof(float) * InTapCount, 0);
			FMemory::Memcpy((void*)Taps, InTaps, sizeof(float) * InTapCount);
			OwnTaps = true;
		}
		else
		{
			Taps = InTaps;
			OwnTaps = false;
		}
		TapCount = InTapCount;
		if (History)
		{
			FMemory::Free(History);
		}

		History = (float*)FMemory::Malloc(sizeof(float) * InTapCount, 0);
		FMemory::Memset((void*)History, 0, sizeof(float) * InTapCount);
		HistoryIndex = 0;
	}

	void FFirFilter::AddData(float* Input, int32 InCount)
	{
		for (int32 Idx = 0; Idx < InCount; ++Idx)
		{
			History[HistoryIndex++] = Input[Idx];
			if (HistoryIndex == (uint32)TapCount)
			{
				HistoryIndex = 0;
			}
		}
	}

	void FFirFilter::AddData(float Input)
	{
		History[HistoryIndex++] = Input;
		if (HistoryIndex == (uint32)TapCount)
		{
			HistoryIndex = 0;
		}
	}

	float FFirFilter::GetSample()
	{
		float Accumulator = 0.0f;
		int32 Index = HistoryIndex;
		for (int32 TapIdx = 0; TapIdx < TapCount; ++TapIdx)
		{
			Index = (Index != 0) ? Index - 1 : TapCount - 1;
			Accumulator += History[Index] * Taps[TapIdx];
		}
		return Accumulator;
	}

	void FFirFilter::Reset()
	{
		FMemory::Memset(History, 0, sizeof(float) * TapCount);
		HistoryIndex = 0;
	}

	void FFirFilter32::Init(const float* InTaps, int32 InTapCount, bool InCopyTaps)
	{
		check(InTapCount == 32);
		FFirFilter::Init(InTaps, InTapCount, InCopyTaps);
	}

	void FFirFilter32::AddData(float* Input, int32 InCount)
	{
		for (int32 Idx = 0; Idx < InCount; ++Idx)
		{
			History[(HistoryIndex++) & 31] = Input[Idx];
		}
	}

	void FFirFilter32::AddData(float Input)
	{
		History[(HistoryIndex++) & 31] = Input;
	}

	float FFirFilter32::GetSample()
	{
		float Accumulator = 0.0f;
		uint32 Index = HistoryIndex;
		Accumulator += History[(Index--) & 31] * Taps[0];
		Accumulator += History[(Index--) & 31] * Taps[1];
		Accumulator += History[(Index--) & 31] * Taps[2];
		Accumulator += History[(Index--) & 31] * Taps[3];
		Accumulator += History[(Index--) & 31] * Taps[4];
		Accumulator += History[(Index--) & 31] * Taps[5];
		Accumulator += History[(Index--) & 31] * Taps[6];
		Accumulator += History[(Index--) & 31] * Taps[7];
		Accumulator += History[(Index--) & 31] * Taps[8];
		Accumulator += History[(Index--) & 31] * Taps[9];
		Accumulator += History[(Index--) & 31] * Taps[10];
		Accumulator += History[(Index--) & 31] * Taps[11];
		Accumulator += History[(Index--) & 31] * Taps[12];
		Accumulator += History[(Index--) & 31] * Taps[13];
		Accumulator += History[(Index--) & 31] * Taps[14];
		Accumulator += History[(Index--) & 31] * Taps[15];
		Accumulator += History[(Index--) & 31] * Taps[16];
		Accumulator += History[(Index--) & 31] * Taps[17];
		Accumulator += History[(Index--) & 31] * Taps[18];
		Accumulator += History[(Index--) & 31] * Taps[19];
		Accumulator += History[(Index--) & 31] * Taps[20];
		Accumulator += History[(Index--) & 31] * Taps[21];
		Accumulator += History[(Index--) & 31] * Taps[22];
		Accumulator += History[(Index--) & 31] * Taps[23];
		Accumulator += History[(Index--) & 31] * Taps[24];
		Accumulator += History[(Index--) & 31] * Taps[25];
		Accumulator += History[(Index--) & 31] * Taps[26];
		Accumulator += History[(Index--) & 31] * Taps[27];
		Accumulator += History[(Index--) & 31] * Taps[28];
		Accumulator += History[(Index--) & 31] * Taps[29];
		Accumulator += History[(Index--) & 31] * Taps[30];
		Accumulator += History[(Index--) & 31] * Taps[31];
		return Accumulator;
	}

	void FFirFilter32::Upsample4x(float InSample, float* OutSamples)
	{
		History[HistoryIndex & 31] = InSample;
		HistoryIndex += 4;

		uint32 Index = HistoryIndex;
		OutSamples[0] = 0.0f;
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[0];
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[4];
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[8];
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[12];
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[16];
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[20];
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[24];
		OutSamples[0] += History[(Index -= 4) & 31] * Taps[28];

		Index = HistoryIndex;
		OutSamples[1] = 0.0f;
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[1];
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[5];
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[9];
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[13];
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[17];
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[21];
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[25];
		OutSamples[1] += History[(Index -= 4) & 31] * Taps[29];

		Index = HistoryIndex;
		OutSamples[2] = 0.0f;
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[2];
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[6];
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[10];
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[14];
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[18];
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[22];
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[26];
		OutSamples[2] += History[(Index -= 4) & 31] * Taps[30];

		Index = HistoryIndex;
		OutSamples[3] = 0.0f;
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[3];
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[7];
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[11];
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[15];
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[19];
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[23];
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[27];
		OutSamples[3] += History[(Index -= 4) & 31] * Taps[31];
	}
}