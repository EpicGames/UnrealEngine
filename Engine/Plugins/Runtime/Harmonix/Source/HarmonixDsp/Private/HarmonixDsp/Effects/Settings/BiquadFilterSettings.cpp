// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Effects/Settings/BiquadFilterSettings.h"
#include "HarmonixDsp/Effects/BiquadFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BiquadFilterSettings)

void FBiquadFilterSettings::GetMagnitudeResponse(
	float const* FrequenciesOfInterest, 
	int32 NumFrequencies,
	float* MagnitudeResponse, 
	float Fs)
{
	Harmonix::Dsp::Effects::FBiquadFilterCoefs Coefs(*this, Fs);
	Coefs.GetMagnitudeResponse(FrequenciesOfInterest, NumFrequencies, MagnitudeResponse);
}
