// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Effects/Settings/VocoderSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VocoderSettings)

FVocoderSettings::FVocoderSettings()
{
	uint8 Idx = (uint8)EVocoderBandConfig::Num - 1;
	int32 MaxBandCount = FVocoderBandConfig::sBandConfigs[Idx].BandCount;
	Bands.Reserve(MaxBandCount);
	for (int32 i = 0; i < MaxBandCount; ++i)
	{
		Bands.Add(FVocoderBand());
	}
}

// bandCount = ln(128 * ratio)/ln(ratio), such that
// 80*(ratio)^(bandCount - 1) = 10240 (this means when we generate frequencies,
// We should get a nice logarithmic spread between 80Hz and 10.24kHz)
const FVocoderBandConfig FVocoderBandConfig::sBandConfigs[(uint8)EVocoderBandConfig::Num] = {
   FVocoderBandConfig(4, 5.03968f, FName("4")),
   FVocoderBandConfig(8, 2.00000f, FName("8")),
   FVocoderBandConfig(16, 1.38191f, FName("16")),
   FVocoderBandConfig(32, 1.16943f, FName("32")),
   FVocoderBandConfig(64, 1.08006f, FName("64")),
   FVocoderBandConfig(128, 1.03894f, FName("128")),
   FVocoderBandConfig(256, 1.01921f, FName("256"))
};
