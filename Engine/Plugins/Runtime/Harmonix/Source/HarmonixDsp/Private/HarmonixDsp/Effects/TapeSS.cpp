// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Effects/TapeSS.h"
#include "HarmonixDsp/AudioBuffer.h"

namespace Harmonix::Dsp::Effects
{

	FTapeStopStart::FTapeStopStart()
		: MaxNumChannels(0)
		, ActiveNumChannels(0)
		, SampleRate(44100.0f)
		, BufferWritePos(0)
		, BufferReadPos(0.0f)
		, NumSamplesInFullRamp(0)
		, RampProgressInSamples(0)
		, CurrentState(FTapeStopStart::EState::Play)
		, DesiredState(FTapeStopStart::EState::Play)
		, RampTimeSeconds(1.0f)
	{
		SRRamper.SetTarget(1.0f);
		SRRamper.SnapToTarget();
		SRRamper.SetRampTimeMs(SampleRate, 1000.0f);
	}


	void FTapeStopStart::Prepare(float SampleRateIn, int32 MaxNumChannelsIn, float MaxDecelerationSeconds)
	{
		SampleRate = SampleRateIn;
		MaxNumChannels = MaxNumChannelsIn;
		ActiveNumChannels = MaxNumChannels;

		SRRamper.SetTarget(1.0f);
		SRRamper.SnapToTarget();
		SRRamper.SetRampTimeMs(SampleRate / float(kHopNum), RampTimeSeconds * 1000.0f);

		NumSamplesInFullRamp = FMath::CeilToInt32(RampTimeSeconds * SampleRate);
		RampProgressInSamples = 0;

		// We only really need a buffer about half the size of the max slow down time.
		int32 MaxCaptureFrames = FMath::CeilToInt32(MaxDecelerationSeconds / 1.8f * SampleRate);
		Buffer.Configure(MaxNumChannels, MaxCaptureFrames, EAudioBufferCleanupMode::Delete);
		Buffer.ZeroData();

		BufferWritePos = 0;
		DesiredState = CurrentState = EState::Play;
		BufferReadPos = 0.0f;

		CrossfadeFloor.SetNum(MaxNumChannels);
	}

	FTapeStopStart::EPhase FTapeStopStart::Process(const TArray<const float*>& InChannelData, const TArray<float*>& OutChannelData, const int32 NumFrames)
	{
		check(InChannelData.Num() <= MaxNumChannels);
		if (InChannelData.Num() != ActiveNumChannels)
		{
			Clear();
			ActiveNumChannels = InChannelData.Num();
		}

		CheckForStateUpdates();

		int32 FrameIndex = 0;

		EPhase CurrentPhase = EPhase::Playing;

		// First process frames in the "speeding up" or "slowing down" segment...
		while (!SRRamper.IsAtTarget() && FrameIndex != NumFrames)
		{
			if (SRRamper.GetTarget() > 0.5f)
			{
				SpeedUp(FrameIndex, InChannelData, OutChannelData, NumFrames);
				CurrentPhase = EPhase::SpeedingUp;
			}
			else
			{
				SlowDown(FrameIndex, InChannelData, OutChannelData, NumFrames);
				CurrentPhase = EPhase::SlowingDown;
			}
		}

		// now any portion that is not in the "speeding up" or "slowing down" segment...
		if (SRRamper.IsAtTarget() && FrameIndex != NumFrames)
		{
			if (SRRamper.GetCurrent() < 1.0f)
			{
				// silence!
				for (int32 ChannelIndex = 0; ChannelIndex < ActiveNumChannels; ++ChannelIndex)
				{
					FMemory::Memset(OutChannelData[ChannelIndex] + FrameIndex, 0, sizeof(float) * (NumFrames - FrameIndex));
				}
				CurrentPhase = EPhase::Stopped;
			}
			else
			{
				// input!
				for (int32 ChannelIndex = 0; ChannelIndex < ActiveNumChannels; ++ChannelIndex)
				{
					FMemory::Memcpy(OutChannelData[ChannelIndex] + FrameIndex, InChannelData[ChannelIndex] + FrameIndex, sizeof(float) * (NumFrames - FrameIndex));
				}
				CurrentPhase = EPhase::Playing;
			}
		}
		
		return CurrentPhase;
	}

	void FTapeStopStart::CheckForStateUpdates()
	{
		if (CurrentState == DesiredState)
			return;

		CurrentState = DesiredState;

		BufferWritePos = 0;
		BufferReadPos = 0.0f;

		float AmountDone = (1.0f - FMath::Abs(SRRamper.GetCurrent() - SRRamper.GetTarget()));
		SRRamper.SetRampTimeMs(SampleRate / float(kHopNum), AmountDone * RampTimeSeconds * 1000.0f);
		NumSamplesInFullRamp = FMath::CeilToInt32(AmountDone * RampTimeSeconds * SampleRate);
		RampProgressInSamples = 0;

		SRRamper.SetRampTimeMs(SampleRate / float(kHopNum), AmountDone * RampTimeSeconds * 1000.0f);
		if (CurrentState == EState::Play)
			SRRamper.SetTarget(1.0f);
		else
			SRRamper.SetTarget(0.0f);
	}

	void FTapeStopStart::SnapState(FTapeStopStart::EState TransportState)
	{
		DesiredState = CurrentState = TransportState;
		SRRamper.SetTarget(CurrentState == EState::Play ? 1.0f : 0.0f);
		SRRamper.SnapToTarget();
	}

	void FTapeStopStart::SetState(FTapeStopStart::EState TransportState, float DurationSeconds)
	{
		DesiredState = TransportState;
		RampTimeSeconds = DurationSeconds;
	}

	void FTapeStopStart::SpeedUp(int32& FrameIndex, const TArray<const float*>& InChannelData, const TArray<float*>& OutChannelData, const int32 NumFrames)
	{
		int32 PosB = FMath::CeilToInt32(BufferReadPos);
		int32 PosA = FMath::Max(0, PosB - 1);
		float WeightA = (float)PosB - BufferReadPos;
		if (RampProgressInSamples < kCrossfadeSamps)
		{
			float Crossfade = float(RampProgressInSamples) / float(kCrossfadeSamps);
			for (int32 ChannelIndex = 0; ChannelIndex < ActiveNumChannels; ++ChannelIndex)
			{
				Buffer[ChannelIndex][BufferWritePos] = *(InChannelData[ChannelIndex] + FrameIndex);
				float NewSample = (Buffer[ChannelIndex][PosA] * WeightA + Buffer[ChannelIndex][PosB] * (1.0f - WeightA)) * Crossfade + CrossfadeFloor[ChannelIndex] * (1.0f - Crossfade);
				*(OutChannelData[ChannelIndex] + FrameIndex) = NewSample;
			}
		}
		else if (RampProgressInSamples + kCrossfadeSamps > NumSamplesInFullRamp)
		{
			float Crossfade = float(NumSamplesInFullRamp - RampProgressInSamples) / float(kCrossfadeSamps);
			for (int32 ChannelIndex = 0; ChannelIndex < ActiveNumChannels; ++ChannelIndex)
			{
				Buffer[ChannelIndex][BufferWritePos] = *(InChannelData[ChannelIndex] + FrameIndex);
				float NewSample = (Buffer[ChannelIndex][PosA] * WeightA + Buffer[ChannelIndex][PosB] * (1.0f - WeightA)) * Crossfade + Buffer[ChannelIndex][BufferWritePos] * (1.0f - Crossfade);
				*(OutChannelData[ChannelIndex] + FrameIndex) = NewSample;
			}
		}
		else
		{
			for (int32 ChannelIndex = 0; ChannelIndex < ActiveNumChannels; ++ChannelIndex)
			{
				Buffer[ChannelIndex][BufferWritePos] = *(InChannelData[ChannelIndex] + FrameIndex);
				CrossfadeFloor[ChannelIndex] = Buffer[ChannelIndex][PosA] * WeightA + Buffer[ChannelIndex][PosB] * (1.0f - WeightA);
				*(OutChannelData[ChannelIndex] + FrameIndex) = CrossfadeFloor[ChannelIndex];
			}
		}
		if (FrameIndex % kHopNum == 0)
		{
			SRRamper.Ramp();
		}
		BufferWritePos = (BufferWritePos + 1) % Buffer.GetNumValidFrames();
		BufferReadPos += FMath::Abs(SRRamper.GetCurrent());
		if (BufferReadPos >= Buffer.GetNumValidFrames())
		{
			BufferReadPos -= Buffer.GetNumValidFrames();
		}
		++FrameIndex;
		++RampProgressInSamples;
	}

	void FTapeStopStart::SlowDown(int32& FrameIndex, const TArray<const float*>& InChannelData, const TArray<float*>& OutChannelData, const int32 NumFrames)
	{
		int32 PosB = FMath::CeilToInt32(BufferReadPos);
		int32 PosA = FMath::Max(0, PosB - 1);
		float WeightA = (float)PosB - BufferReadPos;
		float Crossfade = FMath::Min(float(NumSamplesInFullRamp - RampProgressInSamples) / float(kCrossfadeSamps), 1.0f);
		for (int32 ChannelIndex = 0; ChannelIndex < ActiveNumChannels; ++ChannelIndex)
		{
			Buffer[ChannelIndex][BufferWritePos] = *(InChannelData[ChannelIndex] + FrameIndex);
			CrossfadeFloor[ChannelIndex] = Crossfade * (Buffer[ChannelIndex][PosA] * WeightA + Buffer[ChannelIndex][PosB] * (1.0f - WeightA));
			*(OutChannelData[ChannelIndex] + FrameIndex) = CrossfadeFloor[ChannelIndex];
		}
		if (FrameIndex % kHopNum == 0)
		{
			SRRamper.Ramp();
		}
		BufferWritePos = (BufferWritePos + 1) % Buffer.GetNumValidFrames();
		BufferReadPos += FMath::Abs(SRRamper.GetCurrent());
		if (BufferReadPos >= Buffer.GetNumValidFrames())
		{
			BufferReadPos -= Buffer.GetNumValidFrames();
		}
		++FrameIndex;
		++RampProgressInSamples;
	}
}
