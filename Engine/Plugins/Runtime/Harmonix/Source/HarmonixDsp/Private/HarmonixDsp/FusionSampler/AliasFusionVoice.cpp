// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/AliasFusionVoice.h"
#include "HarmonixDsp/FusionSampler/SingletonFusionVoicePool.h"

#include "HAL/CriticalSection.h"

FAliasFusionVoice::FAliasFusionVoice(FSingletonFusionVoicePool* InPool)
	: VoicePool(InPool)
{
}

FAliasFusionVoice::FAliasFusionVoice(const FAliasFusionVoice& Other)
{
	State = Other.State;
	VoicePool = Other.VoicePool;
}

TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> FAliasFusionVoice::GetPitchShifter() const
{
	// how do lock here?
	//CONST_LOCK_MY_POOL("GetShifter");
	FScopeLock Lock(&VoicePool->GetLock());
	return MyVoice ? VoicePool->GetPitchShifter() : nullptr;
}

Harmonix::Dsp::Modulators::EAdsrStage FAliasFusionVoice::GetAdsrStage() const
{
	//CONST_LOCK_MY_POOL("GetADSRStage");
	FScopeLock Lock(&VoicePool->GetLock());
	return MyVoice ? VoicePool->GetAdsrStage(this) : Harmonix::Dsp::Modulators::EAdsrStage::Idle;
}

void FAliasFusionVoice::SetPitchOffset(double InNumCents)
{
	//LOCK_MY_POOL("SetPitchOffset");
	FScopeLock Lock(&VoicePool->GetLock());
	if (!MyVoice)
	{
		return;
	}

	VoicePool->SetPitchOffset(this, InNumCents);
}

#if FUSION_VOICE_DEBUG_DUMP_ENABLED
void FAliasFusionVoice::AttackWithTargetNote(
	uint8 MidiNoteNumber, 
	float InGain, 
	int32 InEventTick,
	int32 InTriggerTick, 
	double InStartPointMs, 
	const char* InFilePath)
#else
void FAliasFusionVoice::AttackWithTargetNote(
	uint8 MidiNoteNumber, 
	float InGain, 
	int32 InEventTick,
	int32 InTriggerTick, 
	double InStartPointMs)
#endif
{
	//LOCK_MY_POOL("AttackWithTargetNote");
	if (!VoicePool)
	{
		return;
	}

	TargetMidiNote = MidiNoteNumber;

#if FUSION_VOICE_DEBUG_DUMP_ENABLED
	VoicePool->AttackWithTargetNote(this, 
		MidiNoteNumber, 
		InGain, 
		InEventTick, 
		InTriggerTick, 
		InStartPointMs, 
		InFilePath);
#else
	VoicePool->AttackWithTargetNote(this, 
		MidiNoteNumber, 
		InGain, 
		InEventTick, 
		InTriggerTick, 
		InStartPointMs);
#endif
}

uint32 FAliasFusionVoice::Process(
	uint32 SliceIndex, 
	uint32 SubsliceIndex, 
	float** Output,
	uint32 InNumChannels, 
	uint32 InMaxNumSamples, 
	float InSpeed, 
	float InTempoBPM,
	bool  MaintainPitchWhenSpeedChanges)
{
	//LOCK_MY_POOL("Process");
	FScopeLock Lock(&VoicePool->GetLock());
	if (!MyVoice)
	{
		Kill();
		return 0;
	}

	uint32 NumSamples = VoicePool->Process(this, 
		SliceIndex, 
		SubsliceIndex, 
		Output, 
		InNumChannels,
		InMaxNumSamples, 
		InSpeed, 
		InTempoBPM,
		MaintainPitchWhenSpeedChanges);

	// Should we fade in?
	if (State == EState::Starting)
	{
		// only fade in voices that are NOT the driver...
		if (!VoicePool->IsDriver(this))
		{
			int32 NumFadeSamples = FMath::Min((int32)NumSamples, 64);
			for (int32 Ch = 0; Ch < (int32)InNumChannels; ++Ch)
			{
				for (int32 f = 0; f < NumFadeSamples; ++f)
				{
					Output[Ch][f] = Output[Ch][f] * (float(f) / float(NumFadeSamples));
				}
			}
		}
		State = EState::Running;
	}
	if (State == EState::Released && !VoicePool->IsDriver(this))
	{
		Kill();
		// fade out this block...
		int32 NumFadeSamples = FMath::Min((int32)NumSamples, 64);
		for (int32 Ch = 0; Ch < (int32)InNumChannels; ++Ch)
		{
			int32 StartFrame = NumSamples - NumFadeSamples;
			for (int32 FrameNum = 0; FrameNum < NumFadeSamples; ++FrameNum)
			{
				Output[Ch][FrameNum + StartFrame] = Output[Ch][FrameNum + StartFrame] * (float(FrameNum) / float(NumFadeSamples));
			}
		}
	}
	return NumSamples;
}

void FAliasFusionVoice::Release()
{
	//LOCK_MY_POOL("Release");
	FScopeLock Lock(&VoicePool->GetLock());
	if (!MyVoice)
	{
		return;
	}
	check(IsStartingOrRunning());
	State = EState::Released;
	VoicePool->Release(this);
}

void FAliasFusionVoice::FastRelease()
{
	//LOCK_MY_POOL("FastRelease");
	FScopeLock Lock(&VoicePool->GetLock());
	if (!MyVoice)
	{
		return;
	}

	check(IsStartingOrRunning());
	State = EState::Released;
	VoicePool->FastRelease(this);
}

void FAliasFusionVoice::Kill()
{
	//LOCK_MY_POOL("Kill");
	FScopeLock Lock(&VoicePool->GetLock());
	check(IsInUse());
	State = EState::Idle;
	if (RelinquishHandler)
	{
		RelinquishHandler(this);
		RelinquishHandler = nullptr;
	}
	VoicePool->Kill(this);
}

bool FAliasFusionVoice::MatchesIDs(
	const FFusionSampler* InSampler, 
	FMidiVoiceId InVoiceID,
	const FKeyzoneSettings* InKeyzone)
{
	//LOCK_MY_POOL("MatchesIDs");
	FScopeLock Lock(&VoicePool->GetLock());
	if (!MyVoice)
	{
		return false;
	}

	if ((VoiceID == InVoiceID || InVoiceID == FMidiVoiceId::Any())
		&& GetSampler() == InSampler
		&& (InKeyzone == nullptr || &VoicePool->GetKeyzone() == InKeyzone))
	{
		return true;
	}

	return false;
}

bool FAliasFusionVoice::UsesKeyzone(const FKeyzoneSettings* InKeyzone) const
{
	//CONST_LOCK_MY_POOL("UsesKeyzone");
	FScopeLock Lock(&VoicePool->GetLock());
	if (!MyVoice) 
		return false;

	return VoicePool->UsesKeyzone(InKeyzone);
}

void FAliasFusionVoice::SetupLfo(uint8 Index, const FLfoSettings& InSettings)
{
	//LOCK_MY_POOL("SetupLFO");
	FScopeLock Lock(&VoicePool->GetLock());
	if (!MyVoice)
	{
		return;
	}

	VoicePool->SetupLfo(this, Index, InSettings);
}

void FAliasFusionVoice::Activate(
	FFusionSampler* InSampler, 
	FMidiVoiceId InVoiceID, 
	FFusionVoice* RealVoice,
	FRelinquishHandler Handler)
{
	//LOCK_MY_POOL("Activate");
	FScopeLock Lock(&VoicePool->GetLock());
	VoiceID = InVoiceID;
	MyVoice = RealVoice;
	RelinquishHandler = Handler;
	MySampler = InSampler;
	State = EState::Starting;
}
