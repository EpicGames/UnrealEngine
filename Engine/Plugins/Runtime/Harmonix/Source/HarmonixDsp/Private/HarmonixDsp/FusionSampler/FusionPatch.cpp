// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "HarmonixDsp/FusionSampler/FusionPatchCustomVersion.h"
#include "HarmonixDsp/FusionSampler/SingletonFusionVoicePool.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactoryConfig.h"
#include "EditorFramework/AssetImportData.h"

#include "Harmonix/PropertyUtility.h"

#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UnrealType.h"

#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FusionPatch)

DEFINE_LOG_CATEGORY_STATIC(LogFusionPatch, Log, All)


TSharedPtr<Audio::IProxyData> UFusionPatch::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	if (!RenderableFusionPatchData)
	{
		RenderableFusionPatchData = MakeShared<FFusionPatchDataProxy::QueueType>(FusionPatchData);

		(*RenderableFusionPatchData)->InitProxyData(InitParams);
	}
	return MakeShared<FFusionPatchDataProxy>(RenderableFusionPatchData);
}

UFusionPatch::UFusionPatch() : Super()
{
	FFusionPatchSettings& DefaultSettings = FusionPatchData.Settings;
	FAdsrSettings& VolumeAdsr = DefaultSettings.Adsr[0];
	VolumeAdsr.Target = EAdsrTarget::Volume;
	VolumeAdsr.Depth = 1.0f;
	VolumeAdsr.IsEnabled = true;
}

void UFusionPatch::UpdateRenderableForNonTrivialChange()
{
	if (!RenderableFusionPatchData)
	{
		return;
	}

	Audio::FProxyDataInitParams InitParams;

	auto NewSettings = FFusionPatchDataProxy::QueueType::SharedNodePtrType::CreateSharedRenderable(FusionPatchData);
	NewSettings->InitProxyData(InitParams);
	RenderableFusionPatchData->SetNewSettings(NewSettings);
}

void UFusionPatch::UpdateSettings(const FFusionPatchSettings& InSettings)
{
	FusionPatchData.Settings = InSettings;
	UpdateRenderableForNonTrivialChange();
}

void UFusionPatch::UpdateKeyzones(const TArray<FKeyzoneSettings>& NewKeyzones)
{
	FusionPatchData.Keyzones = NewKeyzones;
	UpdateRenderableForNonTrivialChange();
}


void UFusionPatch::SetUserData(TSharedRef<FFusionPatchUserData> UserData)
{
	FusionPatchData.UserData = UserData;
	UpdateRenderableForNonTrivialChange();
}

void UFusionPatch::UpdatePatch(const FFusionPatchData& InPatchData)
{
	FusionPatchData = InPatchData;
	UpdateRenderableForNonTrivialChange();
}

#if WITH_EDITORONLY_DATA

void UFusionPatch::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UFusionPatch::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	// we only care about changes to FusionPatchData
	const FProperty* MemberChanged = PropertyChangedChainEvent.PropertyChain.GetHead()->GetValue();
	if (MemberChanged->GetFName() != GET_MEMBER_NAME_CHECKED(UFusionPatch, FusionPatchData))
	{
		return;
	}

	bool bSettingsChanged = false;
	const FProperty* ChildChanged = PropertyChangedChainEvent.PropertyChain.GetHead()->GetNextNode()->GetValue();
	if (ChildChanged && ChildChanged->GetFName() == GET_MEMBER_NAME_CHECKED(FFusionPatchData, Settings))
	{
		UE_LOGF(LogFusionPatch, Log, "Fusion patch settings changed");
		bSettingsChanged = true;
	}

	const FFusionPatchSettings& DefaultSettings = FusionPatchData.Settings;
	for (FKeyzoneSettings& Keyzone : FusionPatchData.Keyzones)
	{
		int8 MinNote = Keyzone.MinNote;
		int8 MaxNote = Keyzone.MaxNote;
		int8 MinVelocity = Keyzone.MinVelocity;
		int8 MaxVelocity = Keyzone.MaxVelocity;
		
		//clamp note numbers:
		//MinNote <= MaxNote
		if (MinNote > MaxNote)
		{
			Keyzone.MinNote = MaxNote;
		}

		//clamp velocities
		//MinVelocity < MaxVelocity
 		if (MinVelocity > MaxVelocity)
		{
			Keyzone.MinVelocity = MaxVelocity;
		}

		// update keyzone ADSR default settings for non-overridden keyzones
		// maintains continuity when deciding to override ADSR settings later.
		// only update the keyzones if the default settings have changed
		if (bSettingsChanged && !Keyzone.bOverrideAdsr)
		{
			Keyzone.Adsr = DefaultSettings.Adsr[0];
		}
	}

	// don't need to do anything if there isn't a RenderableFusionPatchData to copy data to
	if (!RenderableFusionPatchData)
	{
		return;
	}

	const FProperty* PropertyChanged = PropertyChangedChainEvent.Property;
	const EPropertyChangeType::Type PropertyChangeType = PropertyChangedChainEvent.ChangeType;
	Harmonix::EPostEditAction PostEditAction = Harmonix::GetPropertyPostEditAction(PropertyChanged, PropertyChangeType);

	if (PostEditAction == Harmonix::EPostEditAction::UpdateTrivial)
	{
		FFusionPatchData* CopyToStruct = *RenderableFusionPatchData;
		Harmonix::CopyStructProperty(CopyToStruct, &FusionPatchData, PropertyChangedChainEvent);
	}
	else if (PostEditAction == Harmonix::EPostEditAction::UpdateNonTrivial)
	{
		UpdateRenderableForNonTrivialChange();
	}

	// otherwise, do nothing
}

void UFusionPatch::PostInitProperties()
{
	if (!HasAnyFlags(RF_ClassDefaultObject) && !AssetImportData)
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}

	Super::PostInitProperties();
}

void UFusionPatch::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
}

#endif // WITH_EDITOR_ONLY_DATA

#if WITH_EDITOR

EDataValidationResult UFusionPatch::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);
	const int32 Version = GetLinkerCustomVersion(FFusionPatchCustomVersion::GUID);

	if (Version < FFusionPatchCustomVersion::PitchShifterNameRedirects)
	{
		Context.AddWarning(FText::Format(INVTEXT("Asset saved with a version that is no longer supported: {0}. Resave asset to latest version: {1}."),
			Version, FFusionPatchCustomVersion::LatestVersion));

		Result = EDataValidationResult::Invalid;
	}

	return Result;
}

#endif // WITH_EDITOR

void UFusionPatch::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFusionPatchCustomVersion::GUID);
	const int32 Version = Ar.CustomVer(FFusionPatchCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Version < FFusionPatchCustomVersion::PitchShifterNameRedirects)
		{
			UE_LOGF(LogFusionPatch, Warning, "Fusion patch was loaded with an outdated version. Asset requires reimport: %ls", *GetPathName());
		}

		if (Version < FFusionPatchCustomVersion::DeprecatedPresets)
		{
			if (FusionPatchData.Presets_DEPRECATED.IsValidIndex(0))
			{
				FusionPatchData.Settings = FusionPatchData.Presets_DEPRECATED[0];
			}

			if (FusionPatchData.Presets_DEPRECATED.Num() > 1)
			{
				UE_LOGF(LogFusionPatch, Warning, "Fusion patch (%ls) has more than one preset, but presets have been deprecated. Only the first (Default) preset will be loaded", *GetPathName());
			}
		}

		if (Version < FFusionPatchCustomVersion::DeprecateTypedSettingsArray)
		{
			for (int32 Idx = 0; Idx < FFusionPatchSettings::kNumAdsrs; ++Idx)
			{
				FusionPatchData.Settings.Adsr[Idx] = FusionPatchData.Settings.Adsrs_DEPRECATED.Array[Idx];
			}

			for (int32 Idx = 0; Idx < FFusionPatchSettings::kNumLfos; ++Idx)
			{
				FusionPatchData.Settings.Lfo[Idx] = FusionPatchData.Settings.Lfos_DEPRECATED.Array[Idx];
			}

			for (int32 Idx = 0; Idx < FFusionPatchSettings::kNumRandomizers; ++Idx)
			{
				FusionPatchData.Settings.Randomizer[Idx] = FusionPatchData.Settings.Randomizers_DEPRECATED.Array[Idx];
			}

			
			for (int32 Idx = 0; Idx < FFusionPatchSettings::kNumModulators; ++Idx)
			{
				FusionPatchData.Settings.Randomizer[Idx] = FusionPatchData.Settings.Randomizers_DEPRECATED.Array[Idx];
			}
			
		}
	}

	const UStretcherAndPitchShifterFactoryConfig* FactoryConfig = GetDefault<UStretcherAndPitchShifterFactoryConfig>();
	for (FKeyzoneSettings& Keyzone : FusionPatchData.Keyzones)
	{
		if (const FPitchShifterNameRedirect* Redirect = FactoryConfig->FindFactoryNameRedirect(Keyzone.TimeStretchConfig.PitchShifter))
		{
			Keyzone.TimeStretchConfig.PitchShifter = Redirect->NewName;
		}
	}
}

void UFusionPatch::PostLoad()
{
	Super::PostLoad();

	// apply the default settings to each keyzone ADSR on post load
	// only do this for keyzones that are _NOT_ currently overridden
	// that way, when overriding a keyzone, it'll initialize with the current patch
	const FFusionPatchSettings& DefaultSettings = FusionPatchData.Settings;
	for (FKeyzoneSettings& Keyzone : FusionPatchData.Keyzones)
	{
		if (!Keyzone.bOverrideAdsr)
		{
			Keyzone.Adsr = DefaultSettings.Adsr[0];
		}
	}
}

void FFusionPatchData::InitProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	for (auto& Keyzone : Keyzones)
	{
		Keyzone.InitProxyData(InitParams);
	}
}

void FFusionPatchData::ConnectSampler(const FFusionSampler* Sampler)
{
	if (UserData)
	{
		UserData->OnSamplerConnected(Sampler);
	}
}

void FFusionPatchData::DisconnectSampler(const FFusionSampler* Sampler)
{
	for (FKeyzoneSettings& Keyzone : Keyzones)
	{
		if (Keyzone.SingletonFusionVoicePool.IsValid())
		{
			Keyzone.SingletonFusionVoicePool->SamplerDisconnecting(Sampler);
		}
	}

	if (UserData)
	{
		UserData->OnSamplerDisconnected(Sampler);
	}
}
