// Copyright Epic Games, Inc. All Rights Reserved.
#include "FusionPatchCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FFusionPatchCustomVersion::GUID(TEXT("4588AE1F-FC5B-4510-8703-A79221038375"));

FCustomVersionRegistration GRegisterFusionPatchCustomVersion(FFusionPatchCustomVersion::GUID, FFusionPatchCustomVersion::LatestVersion, TEXT("FusionPatchVer"));