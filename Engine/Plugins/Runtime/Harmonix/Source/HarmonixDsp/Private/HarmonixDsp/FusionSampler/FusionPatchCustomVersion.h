// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

#define UE_API HARMONIXDSP_API

struct FFusionPatchCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,

		KeyzonesUseMappedParameters = 1,

		PitchShifterNameRedirects = 2,

		PanImportingFromDTAFixed = 3,

		DeprecatedPresets = 4,

		DeprecatedUnusedEffectsSettings = 5,

		DeprecateTypedSettingsArray = 6,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static UE_API const FGuid GUID;

private:

	FFusionPatchCustomVersion() {}
};

#undef UE_API
