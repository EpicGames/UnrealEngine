// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/FusionSampler.h"
#include "HarmonixDsp/FusionSampler/FusionVoice.h"
#include "HarmonixDsp/FusionSampler/FusionVoicePool.h"
#include "HarmonixDsp/TimeSyncOption.h"

#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/Math/Interp.h"

DEFINE_LOG_CATEGORY(LogFusionSampler);

namespace Defaults
{
	static const float kVolumeDb = 12.0f;
	static const float kPortamentoTimeMs = 300.0f;
	
	static const float kMinPitchBend = -200.0f;
	static const float kMaxPitchBend = 200.0f;
	
	constexpr float kMinRenderSpeed = 0.001f;
	constexpr float kMaxRenderSpeed = 10.0f;

	constexpr float kMinPitchShift = 0.0001f;
	constexpr float kMaxPitchShift = 10.0f;

	static const int32 kMaxNumVoices = 100;
}

// interpolates for mapping [0, 1] to reasonable ranges for each parameter type
static FLerp gStartPointTimeInterp(0.0f, 1000.0f);
static FLerp gPortamentoTimeInterp(0.0f, 10000.0f);
static FInterpEaseIn gFreqInterp(20.0f, 20000.0f, 4.335f);
static FInterpEaseIn gFilterQInterp(0.1f, 10.0f, 4.03f);
static FInterpEaseIn gLfoFreqInterp(0.1f, 30.0f, 2.74f);
static FInterpEaseIn gDelayTimeInterp(0.0001f, 4.0f, 2.0f);

FFusionSampler::FFusionSampler(float InSampleRateHz, bool InUsePitchShifters)
	: RampCallRateHz(1)
	, Speed(1.0f)
	, MidiChannelVolume(Defaults::kVolumeDb)
	, MidiChannelMuted(false)
	, TrimVolume(Defaults::kVolumeDb)
	, ExpressionGain(1.0f)
	, IsPortamentoEnabled(false)
	, PortamentoMode(EPortamentoMode::Legato)
	, PortamentoTimeMs(Defaults::kPortamentoTimeMs)
	, IsPortamentoActive(false)
	, ExtraPitchBend(0)
	, PitchBendFactor(1.0f)
	, FineTuneCents(0.0f)
	, StartPointMs(0.0f)
	, MaxNumVoices(Defaults::kMaxNumVoices)
	, MinPitchBendCents(Defaults::kMinPitchBend)
	, MaxPitchBendCents(Defaults::kMaxPitchBend)
	, VoicePool(nullptr)
	, bUsePitchShifters(InUsePitchShifters)
	, CurrentTempoBPM(120.0f)
	, RawPitchMultiplier(1.0f)
	, Transposition(0)
	, KeyzoneSelectMode(EKeyzoneSelectMode::Layers)
{
	// Can't use 'alignas' because we may have been allocated in an array of FusionSamplers.
	// In that case alignas can't be honored :-(
	float* BufferStart = VoiceWorkBuffer;
	if ((size_t)VoiceWorkBuffer & 0xF)
	{
		int32 OffsetFloats = (16 - ((size_t)VoiceWorkBuffer % 16)) / sizeof(float);
		check (OffsetFloats > 0 && OffsetFloats < 4);
		BufferStart += OffsetFloats;
	}

	for (int32 Idx = 0; Idx < FAudioBufferConfig::kMaxAudioBufferChannels; ++Idx)
	{
		VoiceWorkBufferChannels[Idx] = BufferStart + (kScratchBufferFrames * Idx);
	}

	PendingNoteActions.Reserve(16);
	ResetNoteActions(true);
	ResetNoteStatus();

	SetSampleRate(InSampleRateHz);
	ResetInstrumentState();
	ResetPatchRelatedState();

	if (bUsePitchShifters)
	{
		IStretcherAndPitchShifterFactory::AddClient(this, InSampleRateHz);
	}
}

FFusionSampler::~FFusionSampler()
{
	if (FusionPatchData)
	{
		FusionPatchData->DisconnectSampler(this);
	}
	SetVoicePool(nullptr, true);
	IStretcherAndPitchShifterFactory::RemoveClient(this);
}

void FFusionSampler::SetVoicePool(FSharedFusionVoicePoolPtr InPool, bool NoCallbacks)
{
	if (VoicePool == InPool)
	{
		return;
	}

	FScopeLock Lock(&GetBusLock());

	if (InPool)
	{
		InPool->AddClient(this);
	}

	if (VoicePool)
	{
		VoicePool->KillVoices(this, NoCallbacks);
		VoicePool->RemoveClient(this);
	}

	VoicePool = InPool;

	if (VoicePool)
	{
		SetSampleRate(VoicePool->GetSampleRate());
	}
}

void FFusionSampler::VoicePoolWillDestruct(const FFusionVoicePool* InPool)
{
	if (VoicePool.IsValid() && VoicePool.Get() == InPool)
	{
		FScopeLock Lock(&GetBusLock());
		SetVoicePool(nullptr, false);
	}
}

void FFusionSampler::NoteOn(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InVelocity, int8 InMidiChannel, int32 InEventTick, int32 InCurrentTick, float InOffsetMs)
{
	// we ignore InMidiChannel because this is a single channel instrument!
	if (InMidiNoteNumber > Harmonix::Midi::Constants::GMaxNote)
	{
		return;
	}

	FScopeLock Lock(&sNoteActionCritSec);
	
	FPendingNoteAction* ExitingAction = PendingNoteActions.FindByPredicate([=](const FPendingNoteAction& Action){ return Action.VoiceId == InVoiceId;});
	int8 PendingVelocity = ExitingAction ? ExitingAction->Velocity : 0;

	if ( InVelocity > PendingVelocity || InVelocity == kNoteOff)
	{
		PendingNoteActions.Add(
			{ 
				/* .MidiNote    = */ InMidiNoteNumber,
				/* .Velocity    = */ InVelocity,
				/* .EventTick   = */ InEventTick,
				/* .TriggerTick = */ InCurrentTick,
				/* .OffsetMs    = */ InOffsetMs,
				/* .FrameOffset = */ 0,
				/* .VoiceId     = */ InVoiceId
			});
	}
}

void FFusionSampler::NoteOnWithFrameOffset(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InVelocity, int8 InMidiChannel, int32 InNumFrames)
{
	// we ignore InMidiChannel because this is a single channel instrument!
	if (InMidiNoteNumber > Harmonix::Midi::Constants::GMaxNote)
	{
		return;
	}

	FScopeLock Lock(&sNoteActionCritSec);

	FPendingNoteAction* ExitingAction = PendingNoteActions.FindByPredicate([=](const FPendingNoteAction& Action) { return Action.VoiceId == InVoiceId; });
	int8 PendingVelocity = ExitingAction ? ExitingAction->Velocity : 0;

	if (InVelocity > PendingVelocity || InVelocity == kNoteOff)
	{
		PendingNoteActions.Add(
			{
				/* .MidiNote    = */ InMidiNoteNumber,
				/* .Velocity    = */ InVelocity,
				/* .EventTick   = */ 0,
				/* .TriggerTick = */ 0,
				/* .OffsetMs    = */ 0.0f,
				/* .FrameOffset = */ InNumFrames,
				/* .VoiceId     = */ InVoiceId
			});
	}
}

bool FFusionSampler::NoteIsOn(int8 InMidiNoteNumber, int8 InMidiChannel /*= 0*/)
{
	if (!VoicePool)
	{
		return false;
	}

	{
		FScopeLock Lock(&sNoteActionCritSec);
		if (PendingNoteActions.FindByPredicate([=](const FPendingNoteAction& Action){ return Action.MidiNote == InMidiNoteNumber; }))
		{
			return true;
		}
	}

	{
		FScopeLock Lock(&sNoteStatusCritSec);
		if (NoteStatus[InMidiNoteNumber].KeyedOn)
		{
			return true;
		}
		else if (NoteStatus[InMidiNoteNumber].NumActiveVoices > 0)
		{
			return true;
		}
	}

	return false;
}

void FFusionSampler::NoteOff(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InMidiChannel /*= 0*/)
{
	// we ignore InMidiChannel because this is a single channel instrument!
	NoteOn(InVoiceId, InMidiNoteNumber, kNoteOff);
}

void FFusionSampler::NoteOffWithFrameOffset(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InMidiChannel /*= 0*/, int32 InNumFrames /*= 0*/)
{
	// we ignore InMidiChannel because this is a single channel instrument!
	NoteOnWithFrameOffset(InVoiceId, InMidiNoteNumber, kNoteOff, InMidiChannel, InNumFrames);
}

void FFusionSampler::SetSpeed(float InSpeed, bool InMaintainPitch)
{
	FScopeLock Lock(&GetBusLock());
	if (InSpeed <= Defaults::kMinRenderSpeed)
	{
		UE_LOGF(LogFusionSampler, Warning, "Speed value cannot be less than %.8f. Clamping", Defaults::kMinRenderSpeed);
		Speed = Defaults::kMinRenderSpeed;	
	}
	else if (InSpeed > Defaults::kMaxRenderSpeed)
	{
		UE_LOGF(LogFusionSampler, Warning, "Speed value cannot greater than %.2f. Clamping", Defaults::kMaxRenderSpeed);
		Speed = Defaults::kMaxRenderSpeed;
	}
	else
	{
		Speed = InSpeed;
	}

	// this line applies Speed to portamento time
	SetPortamentoTime(GetPortamentoTime());   

	MaintainPitchWhenSpeedChanges = InMaintainPitch;
}

void FFusionSampler::SetTempo(float InBPM)
{
	if (CurrentTempoBPM == InBPM)
	{
		return;
	}

	CurrentTempoBPM = InBPM;

	for (int32 LfoIdx = 0; LfoIdx < kNumLfos; ++LfoIdx)
	{
		LfoSettings[LfoIdx].TempoBPM = InBPM * Speed;
		Lfos[LfoIdx].UseSettings(&LfoSettings[LfoIdx]);
	}
	
	UpdateVoiceLfos();
}

void FFusionSampler::SetQuarterNote(float QuarterNote)
{
	FScopeLock Lock(&GetBusLock());
	CurrentQuarterNote = QuarterNote;
}

void FFusionSampler::ResetNoteActions(bool ClearNotes)
{
	FScopeLock Lock(&GetBusLock());
	
	if (ClearNotes)
	{
		PendingNoteActions.Empty(16);
		return;
	}

	PendingNoteActions.RemoveAll([](const FPendingNoteAction& Action){ return Action.FrameOffset <= 0; });
}

void FFusionSampler::ResetNoteStatus()
{
	FScopeLock Lock(&GetBusLock());
	for (int32 NoteIdx = 0; NoteIdx < Harmonix::Midi::Constants::GMaxNumNotes; ++NoteIdx)
	{
		NoteStatus[NoteIdx].KeyedOn = false;
		NoteStatus[NoteIdx].NumActiveVoices = 0;
	}
}

void FFusionSampler::ProcessNoteActions(int32 InNumFrames)
{
	// Do a quick test here without locking...
	if (!PendingNoteActions.Num())
	{
		return;
	}

	// finalize the list of note actions to take
	TArray<FPendingNoteAction> NoteActionsToDo;
	{
		FScopeLock Lock(&sNoteActionCritSec);
		for (FPendingNoteAction& Action : PendingNoteActions)
		{
			if (Action.FrameOffset > 0)
			{
				Action.FrameOffset -= InNumFrames;
				if (Action.FrameOffset < 0)
				{
					Action.FrameOffset = 0;
				}
			}

			if (Action.FrameOffset == 0)
			{
				NoteActionsToDo.Add(Action);
			}
		}
		ResetNoteActions(false);
	}

	// process each note action individually
	int32 AvailableNoteOns = MaxNumVoices;
	for (FPendingNoteAction& Action : NoteActionsToDo)
	{
		if (Action.Velocity != kNoteIgnore && Action.FrameOffset == 0)
		{
			if (Action.Velocity == kNoteOff)
			{
				TArray<uint8> NotesNeedingStopKeyzoneCheck = StopNote(Action.VoiceId, LastVelocity[Action.MidiNote]);
				// if Action.VoiceId == kMidiVoiceIdAny then NewStopNote may have stopped many notes, and we mayhave many 
				// notes to check for "on note-off keyzones"...
				if (Action.VoiceId == FMidiVoiceId::Any() && NotesNeedingStopKeyzoneCheck.Num())
				{
					for (uint8 MidiNote : NotesNeedingStopKeyzoneCheck)
					{
						if (LastVelocity[MidiNote] > 0)
						{
							AvailableNoteOns -= StartNote(Action.VoiceId, MidiNote, LastVelocity[MidiNote], false, 0, 0, 0.0f);
						}
					}
				}
				// if Action.VoiceId != kMidiVoiceIdAny then are can do our normal check for "on note-off keyzones"...
				if (Action.VoiceId != FMidiVoiceId::Any() && LastVelocity[Action.MidiNote] > 0)
				{
					AvailableNoteOns -= StartNote(Action.VoiceId, Action.MidiNote, LastVelocity[Action.MidiNote], false, 0, 0, 0.0f);
				}

				LastVelocity[Action.MidiNote] = 0;
				FScopeLock Lock(&sNoteStatusCritSec);
				NoteStatus[Action.MidiNote].KeyedOn = false;
			}
			else if (AvailableNoteOns > 0)
			{
				AvailableNoteOns -= StartNote(Action.VoiceId, Action.MidiNote, Action.Velocity, true, Action.EventTick, Action.TriggerTick, Action.OffsetMs);
				LastVelocity[Action.MidiNote] = Action.Velocity;
				FScopeLock Lock(&sNoteStatusCritSec);
				NoteStatus[Action.MidiNote].KeyedOn = true;
			}
		}
	}
}

int32 FFusionSampler::GatherMatchingKeyzones(uint8 TransposedNote, uint8 InVelocity, bool IsNoteOn, const TArray<FKeyzoneSettings>& InKeyzones, uint16* OutMathingZones)
{
	int32 OutNumMatchingZones = 0;
	for (int32 KeyzoneIdx = 0; KeyzoneIdx < InKeyzones.Num(); ++KeyzoneIdx)
	{
		const FKeyzoneSettings& Keyzone = InKeyzones[KeyzoneIdx];
		if (!Keyzone.SoundWaveData)
		{
			continue;
		}

		// When we look for a matching Keyzone, we use the transposed note...
		if (!Keyzone.ContainsNoteAndVelocity(TransposedNote, InVelocity))
		{
			continue;
		}

		if (Keyzone.IsNoteOnZone() != IsNoteOn)
		{
			continue;
		}

		// completely exclude any Keyzones with a random weight of 0.
		if ((KeyzoneSelectMode == EKeyzoneSelectMode::Random ||
			KeyzoneSelectMode == EKeyzoneSelectMode::RandomWithRepetition) &&
			Keyzone.RandomWeight == 0)
		{
			continue;
		}
		OutMathingZones[OutNumMatchingZones++] = (uint16)KeyzoneIdx;
	}
	return OutNumMatchingZones;
}



int32 FFusionSampler::StartNote(FMidiVoiceId InVoiceId, uint8 InMidiNoteNumber, uint8 InVelocity, bool IsNoteOn, int32 InEventTick, int32 InTriggerTick, float InOffsetMs)
{
	check(InMidiNoteNumber <= Harmonix::Midi::Constants::GMaxNote);

	if (VoicePool == nullptr)
	{
		UE_LOGF(LogFusionSampler, Warning, "Cannot \"start note \" on a sampler with no voice pool!");
		return 0;
	}

	int32 NumVoicesStarted = 0;

	uint8 TransposedNote = InMidiNoteNumber + Transposition;

	// more than one voice may trigger with a note on.
	// we need to test all the PatchKeyzones to determine which get triggered.
	if (FusionPatchData)
	{
		int8* LayerSelector = (IsNoteOn) ? LastStartLayerSelect : LastStopLayerSelect;

		// maybe too many hops
  		const TArray<FKeyzoneSettings>& PatchKeyzones = FusionPatchData->GetKeyzones();
		uint16 MatchingZones[kMaxLayersPerNote];
		int32  NumMatchingZones = GatherMatchingKeyzones(TransposedNote, InVelocity, IsNoteOn, PatchKeyzones, MatchingZones);

		if (NumMatchingZones != 0)
		{
			if (NumMatchingZones == 1 || KeyzoneSelectMode == EKeyzoneSelectMode::Layers)
			{
				for (int32 Idx = 0; Idx < NumMatchingZones; Idx++)
				{
					if (TryKeyOnZone(InVoiceId, InMidiNoteNumber, TransposedNote, InVelocity, InEventTick, InTriggerTick, InOffsetMs, &PatchKeyzones[MatchingZones[Idx]]))
					{
						++NumVoicesStarted;
					}
				}
			}
			else
			{
				switch (KeyzoneSelectMode)
				{
				case EKeyzoneSelectMode::Random:
				case EKeyzoneSelectMode::RandomWithRepetition:
				{
					check(NumMatchingZones > 1);

					uint32 TriggerKeyzone = 0;
					int32 ExcludedKeyzone = -1;
					if (KeyzoneSelectMode == EKeyzoneSelectMode::Random)
					{
						ExcludedKeyzone = LayerSelector[InMidiNoteNumber];
					}

					// gather total weight of all PatchKeyzones except the excluded one
					float TotalWeight = 0;
					for (int32 Idx = 0; Idx < NumMatchingZones; ++Idx)
					{
						if (Idx != ExcludedKeyzone)
						{
							TotalWeight += PatchKeyzones[MatchingZones[Idx]].RandomWeight;
						}
					}

					// pick randomly based on total weight
					float Rnd = FMath::FRand() * TotalWeight;
					for (int32 Idx = 0; Idx < NumMatchingZones; ++Idx)
					{
						if (Idx != ExcludedKeyzone)
						{
							// ensure we always give a result
							TriggerKeyzone = Idx; 
							Rnd -= PatchKeyzones[MatchingZones[Idx]].RandomWeight;
							if (Rnd < 0)
							{
								break;
							}
						}
					}

					LayerSelector[InMidiNoteNumber] = (int8)TriggerKeyzone;
					if (TryKeyOnZone(InVoiceId, InMidiNoteNumber, TransposedNote, InVelocity, InEventTick, InTriggerTick, InOffsetMs, &PatchKeyzones[MatchingZones[TriggerKeyzone]]))
					{
						++NumVoicesStarted;
					}
				}
				break;
				case EKeyzoneSelectMode::Cycle:
				{
					uint32 TriggerKeyzone = ((uint32)LayerSelector[InMidiNoteNumber] + 1) % NumMatchingZones;
					LayerSelector[InMidiNoteNumber] = (int8)TriggerKeyzone;
					if (TryKeyOnZone(InVoiceId, InMidiNoteNumber, TransposedNote, InVelocity, InEventTick, InTriggerTick, InOffsetMs, &PatchKeyzones[MatchingZones[TriggerKeyzone]]))
					{
						++NumVoicesStarted;
					}
				}
				break;
				}
			}
		}
	}

	VoicePool->FastReleaseExcessVoices();
	VoicePool->FastReleaseExcessVoices(this);
	VoiceUsage.Merge((double)ActiveVoices.Num());

	return NumVoicesStarted;
}

bool FFusionSampler::TryKeyOnZone(FMidiVoiceId InVoiceId, uint8 InTriggeredNote, uint8 InTransposedNote, uint8 InVelocity, int32 InEventTick, int32 InTriggerTick, float InOffsetMs, const FKeyzoneSettings* InKeyzone)
{
	if (!ensure(VoicePool))
		return false;

	check(InKeyzone);

	static float sVelocityNormalizer = 1.0f / 127.0f;

	IStretcherAndPitchShifterFactory* PitchShifterFactory = nullptr;
	if (bUsePitchShifters)
	{
		if (InKeyzone->TimeStretchConfig.bMaintainTime)
		{
			PitchShifterFactory = IStretcherAndPitchShifterFactory::FindBestFactory(InKeyzone->TimeStretchConfig);
		}
	}
	
	// We send in the original, triggered note so that the allocator
	// properly stops other sounding voices that started as a result of
	// a trigger on the same note...
	FFusionVoice* Voice = VoicePool->GetFreeVoice(this, InVoiceId, InKeyzone, [this](FFusionVoice* Voice) { return RelinquishVoice(Voice); }, PitchShifterFactory);
	if (Voice == nullptr)
	{
		UE_LOGF(LogFusionSampler, Warning, "Voice pool failed to create voice, can't play note!");
		return false;
	}

	{
		FScopeLock Lock(&sNoteStatusCritSec);
		NoteStatus[InTriggeredNote].NumActiveVoices += 1;
	}

	ActiveVoices.AddTail(Voice);

	// From here on out we use the transposed note to that the correct pitch is played...
	
	float VelocityNorm = (float)InVelocity * sVelocityNormalizer;

	float AttackGain = VelocityNorm; // should this really just be linear?
	if (!InKeyzone->bVelocityToGain)
	{
		AttackGain = 1.0f;
	}

	if (GetIsPortamentoEnabled())
	{
		PortamentoPitchRamper.SetTarget(InTransposedNote);
		if (!IsPortamentoActive)
		{
			PortamentoPitchRamper.SnapToTarget();
		}

		IsPortamentoActive = true;
	}

	double AttackStartPointMs = StartPointMs;
	AttackStartPointMs += (double)InOffsetMs;
	if (AttackStartPointMs < 0.0)
	{
		AttackStartPointMs = 0.0;
	}

	Voice->SetPitchOffset(FineTuneCents);

// TODO:
//#if FUSION_VOICE_DEBUG_DUMP_ENABLED
//	Voice->AttackWithTargetNote(InTransposedNote, AttackGain, InEventTick, InTriggerTick, AttackStartPointMs, InKeyzone->AudioSample->GetFile().Str());
//#else
	Voice->AttackWithTargetNote(InTransposedNote, AttackGain, InEventTick, InTriggerTick, AttackStartPointMs);
//#endif

	return true;
}

TArray<uint8> FFusionSampler::StopNote(FMidiVoiceId InVoiceId, uint8 LastTriggeredVelocity)
{
	TArray<uint8> MidiNotesToCheckForStopKeyzones;

	FFusionVoice* Voice = nullptr;
	if (VoicePool == nullptr)
	{
		return MidiNotesToCheckForStopKeyzones;
	}

	// find Voice with the ID (there can be more than one since keyzones can overlap)
	for (auto Node = ActiveVoices.GetHead(); Node; Node = Node->GetNextNode())
	{
		Voice = Node->GetValue();

		// ignore notes already being released or idle
		if (Voice->GetAdsrStage() == Harmonix::Dsp::Modulators::EAdsrStage::Release
		 || Voice->GetAdsrStage() == Harmonix::Dsp::Modulators::EAdsrStage::Idle)
		{
			continue;
		}

		if (Voice->MatchesIDs(this, InVoiceId))
		{
			MidiNotesToCheckForStopKeyzones.Add(Voice->GetTriggeredMidiNote());

			Voice->Release();

			// possibly disable portamento pitch bends
			if (PortamentoMode == EPortamentoMode::Legato)
			{
				IsPortamentoActive = false;
			}
		}
	}
	return MidiNotesToCheckForStopKeyzones;
}

void FFusionSampler::KillAllVoices()
{
	FScopeLock Lock(&GetBusLock());

	// isolated scope locks for each of the reset methods
	{
		FScopeLock NoteActionLock(&sNoteActionCritSec);
		ResetNoteActions(true);
	}
	{
		FScopeLock NoteStatusLock(&sNoteStatusCritSec);
		ResetNoteStatus();
	}
	
	auto Node = ActiveVoices.GetHead();
	while (Node)
	{
		// kill might delete the one we are sitting on~
		auto NextNode = Node->GetNextNode();
		FFusionVoice* Voice = Node->GetValue();
		Voice->Kill();
		Node = NextNode;
	}

	ActiveVoices.Empty();
}


void FFusionSampler::AllNotesOff()
{
	FScopeLock Lock(&GetBusLock());
	NoteOff(FMidiVoiceId::Any(), 0);
}

void FFusionSampler::AllNotesOffWithFrameOffset(int32 InNumFrames)
{
	FScopeLock Lock(&GetBusLock());
	NoteOffWithFrameOffset(FMidiVoiceId::Any(), 0, 0, InNumFrames);
}

void FFusionSampler::SetPatch(FFusionPatchDataProxy::NodePtr PatchData)
{
	if (PatchData == FusionPatchData)
	{
		return;
	}

	FScopeLock Lock(&GetBusLock());

	if (FusionPatchData)
	{
		FusionPatchData->DisconnectSampler(this);
	}
		
	FusionPatchData = PatchData;

	if (FusionPatchData)
	{
		FusionPatchData->ConnectSampler(this);
		ApplyPatchSettings();
	}
}

void FFusionSampler::ApplyPatchSettings()
{
	check(FusionPatchData);
	const FFusionPatchSettings& Settings = FusionPatchData->GetSettings();
	SetTrimVolume(Settings.VolumeDb);
	SetMinPitchBendCents(Settings.DownPitchBendCents);
	SetMaxPitchBendCents(Settings.UpPitchBendCents);
	SetMaxNumVoices(Settings.MaxVoices);
	SetStartPointMs(Settings.StartPointOffsetMs);
	SetFineTuneCents(Settings.FineTuneCents);
	SetPan(Settings.PannerDetails);
	KeyzoneSelectMode = Settings.KeyzoneSelectMode;
	
	AdsrVolumeSettings.CopySettings(Settings.Adsr[0]);
	AdsrVolumeSettings.CopyCurveTables(Settings.Adsr[0]);
	
	AdsrAssignableSettings.CopySettings(Settings.Adsr[1]);
	AdsrAssignableSettings.CopyCurveTables(Settings.Adsr[1]);
	
	//-------------------------------------
	// get the filter info
	//------------------------------------
	FilterSettings = Settings.Filter;

	//------------------------------------
	// Read in Lfo settings
	//------------------------------------

	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		LfoSettings[Idx].CopySettings(Settings.Lfo[Idx]);
	}

	for (int32 Idx = 0; Idx < kNumModulators; ++Idx)
	{
		RandomizerSettings[Idx] = Settings.Randomizer[Idx];
	}

	for (int32 Idx = 0; Idx < kNumModulators; ++Idx)
	{
		VelocityModulatorSettings[Idx] = Settings.VelocityModulator[Idx];
	}

	//------------------------------------
	// Read in portamento settings
	//------------------------------------
	{
		const FPortamentoSettings& Portamento = Settings.Portamento;
		SetIsPortamentoEnabled(Portamento.IsEnabled);
		SetPortamentoMode(Portamento.Mode);
		SetPortamentoTime(Portamento.Seconds);
	}

}

void FFusionSampler::Prepare(float InSampleRateHz, EAudioBufferChannelLayout InChannelLayout, uint32 InMaxSamples, bool bInAllocateBuffer /*= true*/)
{
	FMusicalAudioBus::Prepare(1.0f, InChannelLayout, InMaxSamples, bInAllocateBuffer);
	SetSpeed(1.0f);
	SetSampleRate(InSampleRateHz);
}

void FFusionSampler::SetGainTable(const FGainTable* InGainTable)
{
	GainTable = InGainTable;
}

void FFusionSampler::SetSampleRate(float InSampleRateHz)
{
	if ((float)GetSamplesPerSecond() == InSampleRateHz)
		return;

	// call super class
	FMusicalAudioBus::SetSampleRate(InSampleRateHz);

	RampCallRateHz = InSampleRateHz / (float)FMusicalAudioBus::GetMaxFramesPerProcessCall();

	ExpressionGainRamper.SetRampTimeMs(RampCallRateHz, AudioRendering::kMicroFadeMs);
	ExpressionGainRamper.SnapToTarget();

	MidiChannelGainRamper.SetRampTimeMs(RampCallRateHz, AudioRendering::kMicroFadeMs);
	MidiChannelGainRamper.SnapToTarget();

	MidiChannelMuteGainRamper.SetRampTimeMs(RampCallRateHz, AudioRendering::kMicroFadeMs);
	MidiChannelMuteGainRamper.SetTarget(MidiChannelMuted ? 0.0f : 1.0f);
	MidiChannelMuteGainRamper.SnapToTarget();

	PitchBendRamper.SetRampTimeMs(RampCallRateHz, 5.0f);
	PitchBendRamper.SetTarget(0.0f);
	PitchBendRamper.SnapToTarget();

	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		Lfos[Idx].Prepare(InSampleRateHz);
		Lfos[Idx].UseSettings(&LfoSettings[Idx]);
	}

	SetPortamentoTime(GetPortamentoTime());
}

void FFusionSampler::ResetInstrumentStateImpl()
{
	// TODO: Move some reset stuff from ResetPatchRelatedState to here!!!!
	SetMidiChannelGain(1.0, AudioRendering::kMicroFadeSec);
	SetMidiExpressionGain(1.0, AudioRendering::kMicroFadeSec);
	SetMidiChannelMute(false);

	Speed = 1.0f;
	CurrentQuarterNote = 0.0f;
	CurrentTempoBPM = 120.0f;
	RawPitchMultiplier = 1.0f;
	Transposition = 0;
	for (int32 Idx = 0; Idx < kMaxSubstreams; ++Idx)
	{
		SubstreamGain[Idx] = 1.0f;
	}

}

void FFusionSampler::ResetMidiStateImpl()
{
	AllNotesOff();
	SetPitchBend(0.0f);
}

void FFusionSampler::ResetPatchRelatedState()
{
	FScopeLock Lock(&GetBusLock());

	ResetNoteActions(true);

	PanSettings.Reset();

	SetTrimVolume(Defaults::kVolumeDb);
	SetMidiExpressionGain(1.0f);

	SetPitchBend(0.0f);
	PitchBendRamper.SnapToTarget();
	SetMaxPitchBendCents(Defaults::kMaxPitchBend);
	SetMinPitchBendCents(Defaults::kMinPitchBend);

	FilterSettings.ResetToDefaults();

	// set up the default Adsr setting
	
	AdsrVolumeSettings.ResetToDefaults();
	AdsrVolumeSettings.IsEnabled = true;
	AdsrVolumeSettings.Target = EAdsrTarget::Volume;

	AdsrAssignableSettings.ResetToDefaults();
	AdsrAssignableSettings.IsEnabled = false;
	AdsrAssignableSettings.Target = EAdsrTarget::FilterFreq;

	// set up the default Lfo settings
	LfoSettings[0].ResetToDefaults();
	LfoSettings[0].Target = ELfoTarget::Pan;

	LfoSettings[1].ResetToDefaults();
	LfoSettings[1].Target = ELfoTarget::Pitch;	

	// set up the default randomizer settings
	for (uint8 ModulatorIdx = 0; ModulatorIdx < (uint8)EModulatorTarget::Num; ++ModulatorIdx)
	{
		RandomizerSettings[ModulatorIdx] = {};
	}
	
	// set up the default velocity modulator settings
	for (int32 ModulatorIdx = 0; ModulatorIdx < kNumModulators; ++ModulatorIdx)
	{
		VelocityModulatorSettings[ModulatorIdx] = {};
	}
	
	MaxNumVoices = Defaults::kMaxNumVoices;

	SetIsPortamentoEnabled(false);
	PortamentoPitchRamper.SnapTo(60);
	SetPortamentoMode(EPortamentoMode::Legato);
	SetPortamentoTime(Defaults::kPortamentoTimeMs / 1000.0f);

	FMemory::Memset(LastStartLayerSelect, -1, sizeof(int8) * Harmonix::Midi::Constants::GMaxNumNotes);
	FMemory::Memset(LastStopLayerSelect, -1, sizeof(int8) * Harmonix::Midi::Constants::GMaxNumNotes);
	FMemory::Memset(LastVelocity, 0, sizeof(int8) * Harmonix::Midi::Constants::GMaxNumNotes);

	TimeStretchEnvelopeOverride = -1;

	SetPatch(nullptr);
}

int32 FFusionSampler::GetMaxNumVoices() const
{
	if (GetIsPortamentoEnabled())
	{
		return 1;
	}

	return MaxNumVoices;
}

void FFusionSampler::SetMaxNumVoices(int32 Num)
{
	FScopeLock Lock(&GetBusLock());

	MaxNumVoices = Num;
}

bool FFusionSampler::RelinquishVoice(FFusionVoice* InVoice)
{
	FScopeLock Lock(&GetBusLock());

	check(InVoice);

	uint32 NoteNumber = InVoice->GetTargetMidiNote();

	if (auto Node = ActiveVoices.FindNode(InVoice))
	{
		ActiveVoices.RemoveNode(Node);
		FScopeLock LockNoteStatus(&sNoteStatusCritSec);
		NoteStatus[NoteNumber].NumActiveVoices -= 1;
		return true;
	}

	return false;
}

//No improvements to be made here since we have to fill the array with all the voices

//TODO: to test, load up flow dev solution that will build Fusion VST, then in the fusion VST put in some code that calls NoteIsOn and prints the result.
//then load the fusion VST in reaper, go play midi note 10, see it print out yes. then print out something with a release
int32 FFusionSampler::GetNumVoicesInUse(FFusionVoice** VoiceArray) const
{
	// TODO, mimic lock logging
	//HmxThreading::ScopedCritSec scs((const_cast<FusionSampler*>(this)->GetBusLock()), HX_FILE_LINE);

	FScopeLock Lock(&const_cast<FFusionSampler*>(this)->GetBusLock());

	int32 OutNum = 0;
	for (auto Node = ActiveVoices.GetHead(); Node; Node = Node->GetNextNode())
	{
		VoiceArray[OutNum] = Node->GetValue();
		++OutNum;
	}
	return OutNum;
}

void FFusionSampler::SetIsPortamentoEnabled(bool InEnabled)
{
	FScopeLock Lock(&GetBusLock());
	IsPortamentoEnabled = InEnabled;
}

void FFusionSampler::SetPortamentoMode(EPortamentoMode InMode)
{
	FScopeLock Lock(&GetBusLock());
	PortamentoMode = InMode;
}

EPortamentoMode FFusionSampler::GetPortamentoMode() const
{
	return PortamentoMode;
}

void FFusionSampler::SetPortamentoTime(float InTimeSec)
{
	FScopeLock Lock(&GetBusLock());
	PortamentoTimeMs = InTimeSec * 1000.0f;
	PortamentoPitchRamper.SetRampTimeMs(RampCallRateHz, PortamentoTimeMs / Speed);
}

float FFusionSampler::GetPortamentoTime() const
{
	return PortamentoTimeMs * 0.001f;
}

float FFusionSampler::GetCurrentPortamentoPitch() const
{
	return PortamentoPitchRamper.GetCurrent();
}

const FPannerDetails& FFusionSampler::GetPan() const
{
	return PanSettings;
}

void FFusionSampler::SetPan(const FPannerDetails& InPannerDetails)
{
	FScopeLock Lock(&GetBusLock());
	PanSettings = InPannerDetails;
}

void FFusionSampler::SetFineTuneCents(float InCents)
{
	FScopeLock Lock(&GetBusLock());
	FineTuneCents = InCents;
	for (auto Node = ActiveVoices.GetHead(); Node; Node = Node->GetNextNode())
	{
		FFusionVoice* Voice = Node->GetValue();
		check(Voice);
		Voice->SetPitchOffset(FineTuneCents);
	}
}

float FFusionSampler::GetFineTuneCents() const
{
	return FineTuneCents;
}

float FFusionSampler::GetPitchBend(int8 InMidiChannel /*= 0*/) const
{
	// we ignore InMidiChannel because this is a single channel instrument!
	return PitchBendRamper.GetTarget();
}

void FFusionSampler::SetPitchBend(float Value, int8 InMidiChannel /*= 0*/)
{
	FScopeLock Lock(&GetBusLock());
	// we ignore InMidiChannel because this is a single channel instrument!
	check((- 1.0f <= Value) & (Value <= 1.0f));
	PitchBendRamper.SetTarget(Value);
}

float FFusionSampler::GetMinPitchBendCents() const
{
	return MinPitchBendCents;
}

void FFusionSampler::SetMinPitchBendCents(float Value)
{
	FScopeLock Lock(&GetBusLock());
	MinPitchBendCents = Value;
	UpdatePitchBendFactor();
}

float FFusionSampler::GetMaxPitchBendCents() const
{
	return MaxPitchBendCents;
}

void FFusionSampler::SetMaxPitchBendCents(float Value)
{
	// TODO
	//HmxThreading::ScopedCritSec cs(GetBusLock(), HXFILELINE);
	FScopeLock Lock(&GetBusLock());
	MaxPitchBendCents = Value;
	UpdatePitchBendFactor();
}

// returns the multiplicative factor used to change the rate of sample frame increment
// to hit the current pitch bend settings
float FFusionSampler::GetPitchBendFactor() const
{
	return PitchBendFactor;
}

void FFusionSampler::UpdatePitchBendFactor()
{
	FScopeLock Lock(&GetBusLock());
	float bend = GetRampedPitchBend();
	float Cents = 0.0f;
	if (bend > 0.0f)
	{
		Cents = bend * MaxPitchBendCents;
	}
	else
	{
		Cents = -bend * MinPitchBendCents;
	}

	Cents += ExtraPitchBend * 100;
	PitchBendFactor = FMath::Pow(2.0f, Cents * HarmonixDsp::kOctavesPerCent);
}

void FFusionSampler::SetExtraPitchBend(float InSemitones, int8 InChannel)
{
	// we ignore InMidiChannel because this is a single channel instrument!
	ExtraPitchBend = InSemitones;
}

void  FFusionSampler::SetMidiChannelVolume(float InVolume, float InSeconds, int8 InChannel)
{
	FScopeLock Lock(&GetBusLock());
	MidiChannelVolume = InVolume;
	float gain = HarmonixDsp::DBToLinear(MidiChannelVolume);
	float ms = InSeconds * 1000.0f;
	if (ms < AudioRendering::kMicroFadeMs)
	{
		ms = AudioRendering::kMicroFadeMs;
	}
	MidiChannelGainRamper.SetRampTimeMs(RampCallRateHz, ms);
	MidiChannelGainRamper.SetTarget(gain);
}

float FFusionSampler::GetMidiChannelVolume(int8 InMidiChannel) const
{
	float gain = MidiChannelGainRamper.GetCurrent();
	float db = HarmonixDsp::LinearToDB(gain);
	return db;
}

void  FFusionSampler::SetMidiExpressionGain(float gain, float seconds, int8 InMidiChannel)
{
	FScopeLock Lock(&GetBusLock());
	float ms = seconds * 1000.0f;
	if (ms < AudioRendering::kMicroFadeMs)
		ms = AudioRendering::kMicroFadeMs;
	ExpressionGain = gain;
	ExpressionGainRamper.SetRampTimeMs(RampCallRateHz, ms);
	ExpressionGainRamper.SetTarget(ExpressionGain);
}

void  FFusionSampler::SetMidiChannelGain(float gain, float seconds, int8 InMidiChannel)
{
	FScopeLock Lock(&GetBusLock());
	MidiChannelVolume = HarmonixDsp::LinearToDB(gain);
	float ms = seconds * 1000.0f;
	if (ms < AudioRendering::kMicroFadeMs)
		ms = AudioRendering::kMicroFadeMs;
	MidiChannelGainRamper.SetRampTimeMs(RampCallRateHz, ms);
	MidiChannelGainRamper.SetTarget(gain);
}

float FFusionSampler::GetMidiChannelGain(int8 InMidiChannel) const
{
	return MidiChannelGainRamper.GetCurrent();
}

void FFusionSampler::SetMidiChannelMute(bool InMute, int8 InMidiChannel)
{
	if (InMute == MidiChannelMuted) return;

	FScopeLock Lock(&GetBusLock());
	MidiChannelMuted = InMute;
	if (InMute)
	{
		MidiChannelMuteGainRamper.SetTarget(0.0);
	}
	else
	{
		MidiChannelMuteGainRamper.SetTarget(1.0);
	}
}

float FFusionSampler::GetTrimVolume() const
{
	return TrimVolume;
}

void FFusionSampler::SetTrimVolume(float InVolume)
{
	FScopeLock Lock(&GetBusLock());
	TrimVolume = InVolume;
	TrimGain = HarmonixDsp::DBToLinear(TrimVolume);
}

float FFusionSampler::GetTrimGain() const
{
	return TrimGain;
}

void FFusionSampler::SetStartPointMs(float InStartPointOffsetMs)
{
	FScopeLock Lock(&GetBusLock());
	StartPointMs = InStartPointOffsetMs;
}

float FFusionSampler::GetStartPointMs() const
{
	return StartPointMs;
}

void FFusionSampler::PrepareToProcess(uint32 InNumSamples)
{
	ExpressionGainRamper.Ramp();
	PortamentoPitchRamper.Ramp();
	UpdatePitchBendFactor();

	for (int32 LfoIdx = 0; LfoIdx < kNumLfos; ++LfoIdx)
	{
		Lfos[LfoIdx].Advance(static_cast<uint32_t>(InNumSamples * Speed));
	}
}

void FFusionSampler::Process(uint32 InSliceIdx, uint32 InSubSliceIdx, TAudioBuffer<float>& Output)
{
	FScopeLock Lock(&GetBusLock());

	CallPreProcessCallbacks((int32)Output.GetLengthInFrames(), (float)GetSamplesPerSecond(), -1, -1, true);

	// There need to ramp whether or not we are currently playing any voices...
	// so it must go before any 'early outs' below!
	MidiChannelGainRamper.Ramp();
	MidiChannelMuteGainRamper.Ramp();
	PitchBendRamper.Ramp();

	Output.SetIsSilent(true);

	ProcessNoteActions(Output.GetMaxNumFrames());

	// TODO: currently we never early out if we have children.
	if (ActiveVoices.Num() == 0)
	{
		VoiceUsage.Merge(0.0);
		Output.FillData(0);
		return;
	}


	// make sure the temporary voice Output buffer is
	// the same size as the final Output buffer
	uint32 NumChannels = Output.GetMaxNumChannels();
	uint32 NumSamplesToProcess = Output.GetMaxNumFrames();
	checkSlow(NumChannels <= kScratchBufferChannels);
	checkSlow(NumSamplesToProcess <= kScratchBufferFrames);

	if (!VoicePool)
	{
		return;
	}

	Output.ZeroData();

	// call the internal process call
	PrepareToProcess(NumSamplesToProcess);

	for (auto Node = ActiveVoices.GetHead(); Node;)
	{
		FFusionVoice* Voice = Node->GetValue();
		check(Voice);

		// increment iterator here...
		Node = Node->GetNextNode();

		// ... because next call may result in a request back to us to 
		// relinquish the voice....
		uint32 NumSamplesProduced = Voice->Process(InSliceIdx, InSubSliceIdx, VoiceWorkBufferChannels, NumChannels, NumSamplesToProcess, Speed, CurrentTempoBPM, MaintainPitchWhenSpeedChanges);

		if (NumSamplesProduced > 0)
		{
			// accumulate this voice's samples into the channel Output buffer
			Output.Accumulate(VoiceWorkBufferChannels, NumChannels, NumSamplesToProcess);
		}
	}
	
	// de-activate portamento if we're in legato mode
	// and there are no voices playing

	if (PortamentoMode == EPortamentoMode::Legato && ActiveVoices.Num() == 0)
	{
		IsPortamentoActive = false;
	}
}

void FFusionSampler::GetController(Harmonix::Midi::Constants::EControllerID InController, int8& Msb, int8& Lsb, int8 InMidiChannel) const
{
	// we ignore InMidiChannel because this is a single channel instrument!

	float Min;
	float Max;
	float Range;


	float ValueF;

	Lsb = 0; // default case.

	float ValueP;

	using namespace Harmonix::Midi::Constants;
	
	switch (InController)
	{
	case EControllerID::BankSelection:
	{
		break;
	}

	case EControllerID::Volume:
	{
		float Exp = HarmonixDsp::DBToMidiLinear(MidiChannelVolume);
		Msb = (int8)(Exp * 127.0f);
		break;
	}

	case EControllerID::Expression:
	{
		float Exp = HarmonixDsp::LinearToDB(ExpressionGain);
		Exp = HarmonixDsp::DBToMidiLinear(Exp);
		Msb = (int8)(Exp * 127.0f);
		break;
	}
	case EControllerID::PanRight:
	{
		Msb = (int8)((PanSettings.Detail.Pan + 1.0f) * 64.0f);
		break;
	}
	case EControllerID::Release:
	{
		Min = 0.005f;
		Max = 2.000f;
		Range = Max - Min;
		ValueP = AdsrVolumeSettings.ReleaseTime;
		ValueF = ((ValueP - Min) / Range);
		Msb = (int8)(ValueF * 127.0f);
		break;
	}

	case EControllerID::Attack:
	{
		Min = 0.005f;
		Max = 2.000f;
		Range = Max - Min;
		ValueP = AdsrVolumeSettings.AttackTime;
		ValueF = ((ValueP - Min) / Range);
		Msb = (int8)(ValueF * 127.0f);
		break;
	}

	case EControllerID::PortamentoSwitch:
	{
		Msb = GetIsPortamentoEnabled() ? 127 : 0;
		break;
	}

	case EControllerID::PortamentoTime:
	{
		Msb = (int8)(gPortamentoTimeInterp.InverseClamped(PortamentoTimeMs) * 127.0f);
		break;
	}

	case EControllerID::FilterFrequency:
	{
		Msb = (int8)(gFreqInterp.InverseClamped(FilterSettings.Freq) * 127.0f);
		break;
	}

	case EControllerID::FilterQ:
	{
		Msb = (int8)(gFilterQInterp.InverseClamped(FilterSettings.Q) * 127.0f);
		break;
	}

	case EControllerID::CoarsePitchBend:
	{
		Msb = (int8)(GetPitchBend() * 127.0f);
		break;
	}

	case EControllerID::SampleStartTime:
	{
		Msb = (int8)(gStartPointTimeInterp.InverseClamped(GetStartPointMs()) * 127.0f);
		break;
	}

	case EControllerID::LFO0Frequency:
	case EControllerID::LFO1Frequency:
	{
		int32 LfoIdx = (InController == EControllerID::LFO1Frequency);

		float Freq = LfoSettings[LfoIdx].Freq;
		Msb = (int8)(gLfoFreqInterp.InverseClamped(Freq) * 127.0f);
		break;
	}

	case EControllerID::LFO0Depth:
	case EControllerID::LFO1Depth:
	{
		int32 LfoIdx = (InController == EControllerID::LFO1Depth);

		Msb = (int8)(LfoSettings[LfoIdx].Depth * 127.0f);
		break;
	}

	case EControllerID::BitCrushWetMix:
	{
		break;
	}

	case EControllerID::BitCrushLevel:
	{
		break;
	}

	case EControllerID::BitCrushSampleHold:
	{
		break;
	}

	case EControllerID::DelayTime:
	{
		break;
	}

	case EControllerID::DelayDryGain:
	{
		break;
	}

	case EControllerID::DelayWetGain:
	{
		break;
	}

	case EControllerID::DelayFeedback:
	{
		break;
	}
	case EControllerID::DelayEQEnabled:
	{
		break;
	}
	case EControllerID::DelayEQType:
	{
		break;
	}
	case EControllerID::DelayEQFreq:
	{
		break;
	}
	case EControllerID::DelayEQQ:
	{
		break;
	}
	case EControllerID::DelayLFOEnabled:
	{
		break;
	}
	case EControllerID::DelayLFOBeatSync:
	{
		break;
	}
	case EControllerID::DelayLFORate:
	{
		break;
	}
	case EControllerID::DelayLFODepth:
	{
		break;
	}
	case EControllerID::DelayStereoType:
	{
		break;
	}
	case EControllerID::DelayPanLeft:
	{
		break;
	}
	case EControllerID::DelayPanRight:
	{
	}
	default:
		break;
	}
}

void FFusionSampler::SetController(Harmonix::Midi::Constants::EControllerID InController, float InValue)
{
	FScopeLock Lock(&GetBusLock());
	// we ignore InMidiChannel because this is a single channel instrument!

	using namespace Harmonix::Midi::Constants;

	switch (InController)
	{
	case EControllerID::BankSelection: 
	{
		// swap out the fusion patch with the new desired patch

		// ApplyPatchSettings(); 
		break;
	}
	case EControllerID::Volume: 
	{ 
		SetMidiChannelVolume(InValue); 
		break;
	}
	case EControllerID::Expression: 
	{ 
		SetMidiExpressionGain(InValue); 
		break;
	}
	case EControllerID::PanRight: 
	{ 
		PanSettings.Detail.Pan = InValue; 
		break; 
	}
	case EControllerID::Release: 
	{ 
		AdsrVolumeSettings.ReleaseTime = InValue; 
		AdsrVolumeSettings.BuildReleaseTable(); 
		break; 
	}
	case EControllerID::Attack: 
	{ 
		AdsrVolumeSettings.AttackTime = InValue; 
		AdsrVolumeSettings.BuildAttackTable(); 
		break; 
	}
	case EControllerID::PortamentoSwitch: 
	{ 
		SetIsPortamentoEnabled(InValue > 0.0f); 
		break; 
	}
	case EControllerID::PortamentoTime: 
	{ 
		SetPortamentoTime(gPortamentoTimeInterp.EvalClamped(InValue) / 1000);
		break; 
	}
	case EControllerID::FilterFrequency: 
	{ 
		FilterSettings.Freq = gFreqInterp.EvalClamped(InValue); 
		break; 
	}
	case EControllerID::FilterQ: 
	{ 
		FilterSettings.Q = gFilterQInterp.EvalClamped(InValue); 
		break; 
	}
	case EControllerID::CoarsePitchBend: 
	{ 
		SetPitchBend(InValue); 
		break; 
	}
	case EControllerID::SampleStartTime: 
	{ 
		SetStartPointMs(gStartPointTimeInterp.EvalClamped(InValue));
		break; 
	}
	case EControllerID::LFO0Frequency:
	case EControllerID::LFO1Frequency: 
	{ 
		int32 LfoIdx = (InController == EControllerID::LFO1Frequency);

		if (LfoSettings[LfoIdx].Freq != InValue)
		{
			LfoSettings[LfoIdx].Freq = InValue;
			Lfos[LfoIdx].UseSettings(&LfoSettings[LfoIdx]);
			UpdateVoiceLfos();
		}
		break; 
	}
	case EControllerID::LFO0Depth:
	case EControllerID::LFO1Depth: 
	{ 
		int32 LfoIdx = (InController == EControllerID::LFO1Depth);
		if (LfoSettings[LfoIdx].Depth != InValue)
		{
			LfoSettings[LfoIdx].Depth = InValue;
			Lfos[LfoIdx].UseSettings(&LfoSettings[LfoIdx]);
			UpdateVoiceLfos();
		}
		break; 
	}
	case EControllerID::BitCrushWetMix: 
	{
		break; 
	}
	case EControllerID::BitCrushLevel: 
	{
		break; 
	}
	case EControllerID::BitCrushSampleHold: 
	{
		break; 
	}
	case EControllerID::DelayTime: 
	{
		break; 
	}
	case EControllerID::DelayDryGain: 
	{
		break; 
	}
	case EControllerID::DelayWetGain: 
	{
		break; 
	}
	case EControllerID::DelayFeedback: 
	{
		break; 
	}
	case EControllerID::DelayEQEnabled: 
	{
		break; 
	}
	case EControllerID::DelayEQType: 
	{
		break; 
	}
	case EControllerID::DelayEQFreq: 
	{
		break; 
	}
	case EControllerID::DelayEQQ: 
	{
		break; 
	}
	case EControllerID::DelayLFOEnabled: 
	{
		break; 
	}
	case EControllerID::DelayLFOBeatSync: 
	{
		break; 
	}
	case EControllerID::DelayLFORate: 
	{
		break; 
	}
	case EControllerID::DelayLFODepth: 
	{
		break; 
	}
	case EControllerID::DelayStereoType: 
	{
		break; 
	}
	case EControllerID::DelayPanLeft: 
	{
		break; 
	}
	case EControllerID::DelayPanRight: 
	{
		break; 
	}
	case EControllerID::TimeStretchEnvelopeOrder:
	{
		TimeStretchEnvelopeOverride = (int16)InValue;
		UpdateVoicesForEnvelopeOrderChange();
		break;
	}
	case EControllerID::SubStreamVol1:
	{ 
		SetSubstreamMidiGain(0, (int32)InValue); 
		break; 
	}
	case EControllerID::SubStreamVol2: 
	{ 
		SetSubstreamMidiGain(1, (int32)InValue); 
		break; 
	}
	case EControllerID::SubStreamVol3: 
	{
		SetSubstreamMidiGain(2, (int32)InValue); 
		break;
	}
	case EControllerID::SubStreamVol4: 
	{
		SetSubstreamMidiGain(3, (int32)InValue);
		break; 
	}
	case EControllerID::SubStreamVol5: 
	{ 
		SetSubstreamMidiGain(4, (int32)InValue); 
		break;
	}
	case EControllerID::SubStreamVol6:
	{ 
		SetSubstreamMidiGain(5, (int32)InValue); 
		break; 
	}
	case EControllerID::SubStreamVol7: 
	{ 
		SetSubstreamMidiGain(6, (int32)InValue); 
		break;
	}
	case EControllerID::SubStreamVol8:
	{
		SetSubstreamMidiGain(7, (int32)InValue); 
		break; 
	}
	default: 
		break;
	}
}

void FFusionSampler::Set7BitControllerImpl(Harmonix::Midi::Constants::EControllerID InController, int8 InValue, int8 InMidiChannel)
{
	FScopeLock Lock(&GetBusLock());
	// we ignore InMidiChannel because this is a single channel instrument!

	float Min;
	float Max;
	float Range;

	float ValueP = 0.0f;

	using namespace Harmonix::Midi::Constants;

	switch (InController)
	{
	case EControllerID::BankSelection: 
	{ 
		ValueP = (float)InValue; 
		break; 
	}
	case EControllerID::Volume: 
	{ 
		ValueP = HarmonixDsp::MidiLinearToDB(InValue); 
		break; 
	}
	case EControllerID::Expression: 
	{ 
		ValueP = HarmonixDsp::DBToLinear(HarmonixDsp::MidiLinearToDB(InValue)); 
		break; 
	}
	case EControllerID::PanRight: 
	{ 
		ValueP = ((float)InValue / 64.0f) - 1.0f; 
		break; 
	}
	case EControllerID::Release: 
	{ 
		Min = 0.005f; Max = 2.000f; Range = Max - Min;
		ValueP = (float)InValue / 127.0f;
		ValueP = ValueP * Range + Min; 
		break; 
	}
	case EControllerID::Attack: 
	{ 
		Min = 0.005f; Max = 2.000f; Range = Max - Min;
		ValueP = (float)InValue / 127.0f;
		ValueP = ValueP * Range + Min; 
		break; 
	}
	case EControllerID::PortamentoSwitch: 
	{ 
		ValueP = (InValue > 0) ? 1.0f : 0.0f; 
		break; 
	}
	case EControllerID::PortamentoTime:
	case EControllerID::FilterFrequency:
	case EControllerID::FilterQ:
	case EControllerID::CoarsePitchBend:
	case EControllerID::SampleStartTime: 
	{ 
		ValueP = (float)InValue / 127.0f; 
		break; 
	}
	case EControllerID::LFO0Frequency:
	case EControllerID::LFO1Frequency: 
	{ 
		ValueP = gLfoFreqInterp.EvalClamped((float)InValue / 127.0f); 
		break; 
	}
	case EControllerID::LFO0Depth:
	case EControllerID::LFO1Depth: 
	{ 
		ValueP = (float)InValue / 127.0f; 
		break; 
	}
	case EControllerID::BitCrushWetMix:
	{ 
		ValueP = (float)InValue * 100.0f / 127.0f; 
		break; 
	}
	case EControllerID::BitCrushLevel: 
	{ 
		ValueP = (float)InValue * 15.0f / 127.0f; 
		break; 
	}
	case EControllerID::BitCrushSampleHold: 
	{ 
		ValueP = (1.5f + (((float)InValue / 127.0f) * 15));
		break; 
	}
	case EControllerID::DelayTime:
	case EControllerID::DelayDryGain:
	case EControllerID::DelayWetGain:
	case EControllerID::DelayEQEnabled:
	case EControllerID::DelayEQType:
	case EControllerID::DelayEQFreq:
	case EControllerID::DelayEQQ:
	case EControllerID::DelayLFOEnabled:
	case EControllerID::DelayLFOBeatSync:
	case EControllerID::DelayLFORate:
	case EControllerID::DelayLFODepth:
	case EControllerID::DelayStereoType:
	case EControllerID::DelayPanLeft:
	case EControllerID::DelayPanRight:
	case EControllerID::DelayFeedback: 
	{ 
		ValueP = (float)InValue / 127.0f; 
		break; 
	}
	case EControllerID::TimeStretchEnvelopeOrder: 
	{ 
		ValueP = (float)InValue; 
		break; 
	}
	case EControllerID::SubStreamVol1:
	case EControllerID::SubStreamVol2:
	case EControllerID::SubStreamVol3:
	case EControllerID::SubStreamVol4:
	case EControllerID::SubStreamVol5:
	case EControllerID::SubStreamVol6:
	case EControllerID::SubStreamVol7:
	case EControllerID::SubStreamVol8: 
	{ 
		ValueP = (float)InValue; 
		break; 
	}
	default: break;
	}
	SetController(InController, ValueP);
}

void FFusionSampler::Set14BitControllerImpl(Harmonix::Midi::Constants::EControllerID InController, int16 InValue, int8 InMidiChannel)
{
	FScopeLock Lock(&GetBusLock());
	// we ignore InMidiChannel because this is a single channel instrument!

	float Min;
	float Max;
	float Range;

	float ValueP;

	using namespace Harmonix::Midi::Constants;

	switch (InController)
	{
	case EControllerID::BankSelection: 
	{
		ValueP = (float)InValue; 
		break; 
	}
	case EControllerID::Volume: 
	{ 
		ValueP = HarmonixDsp::Midi14BitLinearToDB(InValue); 
		break; 
	}
	case EControllerID::Expression: 
	{ 
		ValueP = HarmonixDsp::DBToLinear(HarmonixDsp::Midi14BitLinearToDB(InValue));
		break; 
	}
	case EControllerID::PanRight: 
	{
		ValueP = ((float)InValue / 8191.0f) - 1.0f;
		break;
	}
	case EControllerID::Release: 
	{ 
		Min = 0.005f; Max = 2.000f; 
		Range = Max - Min;
		ValueP = (float)InValue / 16383.0f;
		ValueP = ValueP * Range + Min; 
		break; 
	}
	case EControllerID::Attack: 
	{ 
		Min = 0.005f; Max = 2.000f; 
		Range = Max - Min;
		ValueP = (float)InValue / 16383.0f;
		ValueP = ValueP * Range + Min; 
		break; 
	}
	case EControllerID::PortamentoSwitch: 
	{ 
		ValueP = (InValue > 0) ? 1.0f : 0.0f; 
		break; 
	}
	case EControllerID::PortamentoTime:
	case EControllerID::FilterFrequency:
	case EControllerID::FilterQ:
	case EControllerID::CoarsePitchBend:
	case EControllerID::SampleStartTime: 
	{ 
		ValueP = (float)InValue / 16383.0f; 
		break; 
	}
	case EControllerID::LFO0Frequency:
	case EControllerID::LFO1Frequency: 
	{ 
		ValueP = gLfoFreqInterp.EvalClamped((float)InValue / 16383.0f); 
		break; 
	}
	case EControllerID::LFO0Depth:
	case EControllerID::LFO1Depth: 
	{ 
		ValueP = (float)InValue / 16383.0f;
		break;
	}
	case EControllerID::BitCrushWetMix: 
	{
		ValueP = (float)InValue * 100.0f / 16383.0f;
		break; 
	}
	case EControllerID::BitCrushLevel:
	{ 
		ValueP = (float)InValue * 15.0f / 16383.0f; 
		break;
	}
	case EControllerID::BitCrushSampleHold: 
	{ 
		ValueP = (1.5f + (((float)InValue / 16383.0f) * 15));
		break; 
	}
	case EControllerID::DelayTime:
	case EControllerID::DelayDryGain:
	case EControllerID::DelayWetGain:
	case EControllerID::DelayEQEnabled:
	case EControllerID::DelayEQType:
	case EControllerID::DelayEQFreq:
	case EControllerID::DelayEQQ:
	case EControllerID::DelayLFOEnabled:
	case EControllerID::DelayLFOBeatSync:
	case EControllerID::DelayLFORate:
	case EControllerID::DelayLFODepth:
	case EControllerID::DelayStereoType:
	case EControllerID::DelayPanLeft:
	case EControllerID::DelayPanRight:
	case EControllerID::DelayFeedback:
	{ 
		ValueP = (float)InValue / 16383.0f;
		break; 
	}
	case EControllerID::TimeStretchEnvelopeOrder: 
	{ 
		ValueP = (float)(InValue >> 7); 
		break; 
	}
	case EControllerID::SubStreamVol1:
	case EControllerID::SubStreamVol2:
	case EControllerID::SubStreamVol3:
	case EControllerID::SubStreamVol4:
	case EControllerID::SubStreamVol5:
	case EControllerID::SubStreamVol6:
	case EControllerID::SubStreamVol7:
	case EControllerID::SubStreamVol8: 
	{ 
		ValueP = (float)(InValue >> 7); 
		break; 
	}
	default:
		ValueP = 0.0f; 
		break;
	}
	SetController(InController, ValueP);
}

void FFusionSampler::UpdateVoiceLfos()
{
	FScopeLock Lock(&GetBusLock());

	for (auto Node = ActiveVoices.GetHead(); Node; Node = Node->GetNextNode())
	{
		// skip voices not assigned to this channel
		FFusionVoice* Voice = Node->GetValue();
		check(Voice);
		for (int32 LfoIdx = 0; LfoIdx < kNumLfos; ++LfoIdx)
		{
			Voice->SetupLfo(LfoIdx, LfoSettings[LfoIdx]);
		}
	}
}

void FFusionSampler::UpdateVoicesForEnvelopeOrderChange()
{
	FScopeLock Lock(&GetBusLock());
	for (auto Node = ActiveVoices.GetHead(); Node; Node = Node->GetNextNode())
	{
		FFusionVoice* Voice = Node->GetValue();
		check(Voice);

		if (TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> Shifter = Voice->GetPitchShifter())
		{
			TMap<FName, FTypedParameter> Options;
			Options.Add("EnvelopeOrder", TimeStretchEnvelopeOverride);
			Shifter->UpdateOptions(Options);
		}
	}
}

FString FFusionSampler::GetPatchPath() const
{
	unimplemented();
	return FString(); // FusionPatch->GetPathName();
}

#ifdef UE_BUILD_DEVELOPMENT
TSet<int32> FFusionSampler::GetActiveKeyzones()
{
	FScopeLock Lock(&GetBusLock());

	check(FusionPatchData);

	const TArray<FKeyzoneSettings>& KeyzonesRef = FusionPatchData->GetKeyzones();

	TSet<int32> OutIndices;
	if (!VoicePool)
	{
		return OutIndices;
	}

	for (FFusionVoice* Voice : ActiveVoices)
	{
		check(Voice);

		for (int32 Idx = 0; Idx < KeyzonesRef.Num(); ++Idx)
		{
			if (Voice->UsesKeyzone(&KeyzonesRef[Idx]))
			{
				OutIndices.Add(Idx);
			}
		}
	}

	return OutIndices;
}
#endif

void FFusionSampler::SetSubstreamMidiGain(int32 Idx, uint8 InMidiGain)
{
	if (Idx >= kMaxSubstreams)
	{
		UE_LOGF(LogFusionSampler, Error, "Substream index out of range!");
		return;
	}

	float VolumeDb = HarmonixDsp::LinearToDB(InMidiGain);
	SubstreamGain[Idx] = HarmonixDsp::DBToLinear(VolumeDb);
}