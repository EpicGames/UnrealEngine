// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/FusionSampler/FusionSamplerConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FusionSamplerConfig)

FFusionVoiceConfig UFusionSamplerConfig::GetVoiceConfigForPoolName(FName PoolName) const
{
	if (PoolName == NAME_None)
	{
		return DefaultVoicePoolConfig;
	}

	const FFusionVoiceConfig* FoundConfig = VoicePoolConfigs.Find(PoolName);
	if (!FoundConfig)
	{
		return DefaultVoicePoolConfig;
	}

	return *FoundConfig;
}
