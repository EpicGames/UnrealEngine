// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/FusionVoice.h"
#include "HarmonixDsp/FusionSampler/FusionSampler.h"
#include "HarmonixDsp/FusionSampler/FusionVoicePool.h"

#include "HarmonixDsp/AudioDataRenderer.h"
#include "HarmonixDsp/AudioData/StreamingAudioRendererV2.h"
#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/StretcherAndPitchShifter.h"

#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"

#include "HAL/IConsoleManager.h"

#include "Sound/SoundWave.h"

DEFINE_LOG_CATEGORY(LogFusionVoice);

extern size_t gAudioSliceCounter;

namespace FusionVoice
{
	static const float kInternalTrimGain = 1.0f; 
	static const float kVsoAmount = 0.01f;
	static const float kVsoThresholdMs = 5.0f;
}

const double FFusionVoice::kMaxPitchOffsetCents = 12 * HarmonixDsp::kCentsPerOctave;

FFusionVoice::FFusionVoice()
{

	ActiveRenderer = MakeShared<FStreamingAudioRendererV2>();

	// For now, set the sample rate to a reasonable default.
	// Sample rate will be set properly when the voice is assigned
	// to a sampler.
	SetSampleRate(48000.0);
}

void FFusionVoice::SetSampleRate(double InSampleRate)
{
	SamplesPerSecond = InSampleRate;
	SecondsPerSample = 1.0 / SamplesPerSecond;

	float fSamplesPerSecond = (float)SamplesPerSecond;

	AdsrAssignable.Prepare(fSamplesPerSecond);
	AdsrVolume.Prepare(fSamplesPerSecond);
	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		Lfo[Idx].Prepare(fSamplesPerSecond);
	}
	
	static float sRampTime = 15.0f;
	FilterCoefsRamper.SetRampTimeMs(fSamplesPerSecond / AudioRendering::kMicroSliceSize, sRampTime);
	FilterGainRamper.SetRampTimeMs(fSamplesPerSecond / AudioRendering::kMicroSliceSize, 10.0f);

	float bufferSize = (MySampler) ? MySampler->GetMaxFramesPerProcessCall() : (float)AudioRendering::kFramesPerRenderBuffer;

	Panner.SetRampTimeMs(fSamplesPerSecond / bufferSize, AudioRendering::kMicroFadeMs);
	BuildGainMatrix(true);
}

bool FFusionVoice::AssignIDs(FFusionSampler* InSampler, const FKeyzoneSettings* InKeyZone, FMidiVoiceId InVoiceID, FRelinquishHandler Handler, TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> Shifter /*= nullptr*/)
{
	check(InSampler);
	check(InKeyZone);
	check(InKeyZone->SoundWaveData);

	KeyZone = InKeyZone;

#if CPUPROFILERTRACE_ENABLED
	const IConsoleVariable* ProfilingAllGraphsCheat = IConsoleManager::Get().FindConsoleVariable(TEXT("au.MetaSound.ProfileAllGraphs"));
	bProfiling = (ProfilingAllGraphsCheat && ProfilingAllGraphsCheat->GetBool());
	if (bProfiling)
	{
		ProfileString = FString::Printf(TEXT("FusionVoice::Process Stretch: %s"), Shifter ? *Shifter->GetFactoryName().ToString() : TEXT("(none)"));
	}
	else
	{
		ProfileString.Empty();
	}
#endif

	if (InKeyZone->TimeStretchConfig.bMaintainTime)
	{
		check(InKeyZone->TimeStretchConfig.OriginalTempo > 0.0f);
	}

	RelinquishHandler = Handler;

	MySampler = InSampler;
	PitchShifter = Shifter;

	if (InKeyZone->bOverrideAdsr)
	{
		AdsrVolume.UseSettings(&(InKeyZone->Adsr));
	}
	else
	{
		AdsrVolume.UseSettings(&(MySampler->AdsrVolumeSettings));
	}
	
	AdsrAssignable.UseSettings(&(MySampler->AdsrAssignableSettings));
	VelocityGain = 0.0f;

	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		Lfo[Idx].UseSettings(&MySampler->LfoSettings[Idx]);
	}

	SetSampleRate(MySampler->GetSamplesPerSecond());

	// If we have a trackMap on this, the panner needs to be set up
	// based on the output buffer we are eventually asked to render into,
	// so in that case set NumInChannels to 0 here and we'll sort this
	// out later...
	// 0 because we need to configure to the output buffer we are asked to fill
	int32 NumInChannels = InKeyZone->TrackMap.Num() == 0 ? InKeyZone->SoundWaveData->GetNumChannels() : 0;
	EAudioBufferChannelLayout ChannelLayout = HarmonixDsp::FAudioBuffer::GetDefaultChannelLayoutForChannelCount(NumInChannels);
	ESpeakerMask::Type ChannelMask = HarmonixDsp::FAudioBuffer::GetChannelMaskForNumChannels(NumInChannels);
	Panner.Setup(InKeyZone->Pan,
		NumInChannels,
		MySampler->GetNumAudioOutputChannels(),
		ChannelLayout,
		ChannelMask,
		1.0f,
		MySampler->GetGainTable());

	// compare the audio file's sample rate to the output sample rate
	// so we can make them match
	double FileSamplesPerSecond = InKeyZone->SoundWaveData->GetSampleRate();
	FileToOutputSampleRatio = FileSamplesPerSecond * SecondsPerSample;

	IAudioDataRenderer::FSettings Settings;
	Settings.Shifter = PitchShifter;
	Settings.ShifterSettings = &InKeyZone->TimeStretchConfig;
	Settings.TrackChannelInfo = &InKeyZone->TrackMap;
	Settings.Sampler = MySampler;
	//TRACE_BOOKMARK(TEXT("Starting Waveform: %s"), *InKeyZone->AudioSample->GetName().ToString());
	ActiveRenderer->SetSoundWaveData(InKeyZone->SoundWaveData.ToSharedRef(), Settings);

	OutputBuffer.SetNumValidChannels(InKeyZone->SoundWaveData->GetNumChannels());

	VoiceID = InVoiceID;
	bWaitingForAttack = true;
	return true;
}

void FFusionVoice::SetupLfo(uint8 Index, const FLfoSettings& InSettings)
{
	check(Index < kNumLfos);

	Lfo[Index].UseSettings(&InSettings);
}

void FFusionVoice::Attack()
{
	check(MySampler);
	checkSlow(KeyZone);
	checkSlow(KeyZone->SoundWaveData);

	AdsrVolume.Attack();
	AdsrAssignable.Attack();
	bWaitingForAttack = false;
	bHasRenderedAnySamples = false;
}

void FFusionVoice::Release()
{
	AdsrVolume.Release();
	AdsrAssignable.Release();
}

void FFusionVoice::FastRelease()
{
	AdsrVolume.FastRelease();
	AdsrAssignable.FastRelease();
}

void FFusionVoice::Kill()
{
	if (VoicePool)
	{
		VoicePool->Lock();
	}

	CachedFilterFrequency = 0.0f;
	MaxAudioLevel = 0.0f;
	VoiceID = FMidiVoiceId::None();

	if (RelinquishHandler)
	{
		RelinquishHandler(this);
	}

	RelinquishHandler = nullptr;

	if (VoicePool && PitchShifter)
	{
		VoicePool->ReleaseShifter(PitchShifter);
	}
	PitchShifter = nullptr;

	AdsrVolume.Kill();
	AdsrAssignable.Kill();
	SamplePos = 0.0f;
	bWaitingForAttack = false;
	PitchOffsetCents = 0.0f;
	EndOfSampleData = 0.0f;
	bIsRendererForAlias = false;
	MySampler = nullptr;
	KeyZone = nullptr;
	ModulatedFilterSettings.ResetToDefaults();

	if (ActiveRenderer.IsValid())
	{
		ActiveRenderer->Reset();
	}

	if (VoicePool)
	{
		VoicePool->Unlock();
	}
}

void FFusionVoice::Init(FFusionVoicePool* InOwner, uint32 InDebugID)
{
	VoicePool = InOwner;
	DebugID = InDebugID;
	VoiceID = FMidiVoiceId::None();
	MySampler = nullptr;
	PitchShifter = nullptr;

	SamplePos = 0.0f;

	if (OutputBuffer.GetMaxConfig().GetNumFrames() < AudioRendering::kMicroSliceSize
		|| OutputBuffer.GetMaxConfig().GetNumChannels() < FAudioBufferConfig::kMaxAudioBufferChannels)
	{
		OutputBuffer.Configure(
			FAudioBufferConfig::kMaxAudioBufferChannels,
			AudioRendering::kMicroSliceSize,
			EAudioBufferCleanupMode::DontDelete);
	}

	bWaitingForAttack = false;
}

void FFusionVoice::SetPitchOffset(double InNumCents)
{
	PitchOffsetCents = InNumCents;
	if (!bWaitingForAttack)
	{
		SetPitchShiftForMidiNote((uint8)TargetMidiNote);
	}
}


void FFusionVoice::SetSampler(FFusionSampler* InSampler)
{
	check(InSampler);

	if (!MySampler || MySampler == InSampler)
	{
		MySampler = InSampler;
		return;
	}

	MySampler = InSampler;
	AdsrVolume.UseSettings(&MySampler->AdsrVolumeSettings, false);
	AdsrAssignable.UseSettings(&MySampler->AdsrAssignableSettings, false);
	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		Lfo[Idx].UseSettings(&MySampler->LfoSettings[Idx]);
	}

	if (ActiveRenderer)
	{
		ActiveRenderer->MigrateToSampler(MySampler);
	}
}


void FFusionVoice::SetPitchShiftForMidiNote(uint8 InNote)
{
	check(KeyZone && KeyZone->SoundWaveData);

	// this is the note (pitch) we are supposedly playing
	// although the exact pitch will vary depending on various modulations
	TargetMidiNote = (float)InNote;

	uint8 RootNote = KeyZone->RootNote;
	int8 SemitoneOffset = InNote - RootNote;

	// ignore root note if it's an unpitched keyzone
	if (KeyZone->bUnpitched)
	{
		SemitoneOffset = 0;
	}
	double NewPitchOffsetCents = (double)SemitoneOffset * 100.0 + PitchModulation * 100 + PitchOffsetCents;

	PitchShift = FMath::Pow(2.0, NewPitchOffsetCents * HarmonixDsp::kOctavesPerCent);
}

#if FUSION_VOICE_DEBUG_DUMP_ENABLED
void FFusionVoice::AttackWithTargetNote(uint8 InMidiNoteNumber, float InGain, int32 InEventTick, int32 InTriggerTick, double InStartPointMs = 0.0f, const char* InFilePath)
#else
void FFusionVoice::AttackWithTargetNote(uint8 InMidiNoteNumber, float InGain, int32 InEventTick, int32 InTriggerTick, double InStartPointMs /*= 0.0f*/)
#endif
{
	//UE_LOGF(LogFusionVoice, Log, "0x%x - Attacking\n", (size_t)this);

#if FUSION_VOICE_DEBUG_DUMP_ENABLED
	// TODO
	//if (mDumpFile)
	//{
	//	mDumpFile->Flush();
	//	delete mDumpFile;
	//	mDumpFile = nullptr;
	//}
	//if (gWriting)
	//{
	//	int32 InSliceIndex = theMusicRenderTargetBaseManager().GetDefaultRenderTarget()->GetSliceIndex();
	//	char fileName[256];
	//	String sampPath(filePath);
	//	String sampName = sampPath.substr(sampPath.find_last_of('/') + 1);
	//	mDumpFileIndex++;
	//	sprintf(fileName, "S%08d-C0x%zx-%d-N%d-%s.raw", InSliceIndex, (size_t)this, mDumpFileIndex, midiNoteNumber, sampName.c_str());
	//	mDumpFile = new WriteCacheFileStream(fileName, FileMode::kWrite, 1024 * 128);
	//}
#endif

	uint8 RootNote = KeyZone->RootNote;
	int8 SemitoneOffset = InMidiNoteNumber - RootNote;

	// ignore root note if it's an unpitched keyzone
	if (KeyZone->bUnpitched)
	{
		SemitoneOffset = 0;
	}

	PitchModulation = 0.0;
	float StartPointMsModulation = 0.0f;
	for (int32 Idx = 0; Idx < FFusionPatchSettings::kNumModulators; ++Idx)
	{
		const FModulatorSettings& Settings = MySampler->RandomizerSettings[Idx];
		if (Settings.Target == EModulatorTarget::Pitch)
		{
			float X = FMath::FRand();
			PitchModulation += FMath::Clamp(FMath::Lerp(-Settings.Range, Settings.Range, X * Settings.Depth), -Settings.Range, Settings.Range);
		}
		else if (Settings.Target == EModulatorTarget::StartPoint)
		{
			float X = FMath::FRand();
			StartPointMsModulation += FMath::Clamp(FMath::Lerp(0, Settings.Range, X * Settings.Depth), 0, Settings.Range);
		}
	}
	
	// this is the note (pitch) we are supposedly playing
	// although the exact pitch will vary depending on various modulations
	TargetMidiNote = (float)InMidiNoteNumber;
	TriggeredMidiNote = InMidiNoteNumber;

	if (InEventTick == InTriggerTick)
	{
		if (KeyZone->TimeStretchConfig.bMaintainTime)
		{
			float EventQuarterNote = (float)InEventTick / MySampler->GetTicksPerQuarterNote();
			float SamplerQuarterNote = MySampler->GetQuarterNote();
			float ErrorBeats = SamplerQuarterNote - EventQuarterNote;
			float ErrorMs = ErrorBeats * 60000.0f / KeyZone->TimeStretchConfig.OriginalTempo;
			if (ErrorMs > 10.0f)
			{
				// gotta let the render code skip forward a bit in process otherwise this voice will be out of sync by a lot!
				StartQuarterNote = (float)InEventTick / Harmonix::Midi::Constants::GTicksPerQuarterNote;
			}
			else
			{
				StartQuarterNote = SamplerQuarterNote;
			}
		}
		else
		{
			StartQuarterNote = MySampler->GetQuarterNote();
		}
	}
	else
	{
		StartQuarterNote = (float)InEventTick / Harmonix::Midi::Constants::GTicksPerQuarterNote;
	}

	PrepareWithPitchOffsetAndGain(SemitoneOffset * 100.0 + PitchModulation * 100.0 + PitchOffsetCents, InGain);

	double OutputSampleOffset = 0.001 * (InStartPointMs + StartPointMsModulation) * MySampler->GetSamplesPerSecond();
	double KeyzoneStartOffset = (KeyZone->SampleStartOffset == -1) ?
		0 :
		KeyZone->SampleStartOffset;
	
	SamplePos = KeyzoneStartOffset + (OutputSampleOffset * FileToOutputSampleRatio);
	StartPos = SamplePos;
	EndOfSampleData = (KeyZone->SampleEndOffset == -1) ?
		KeyZone->SoundWaveData->GetNumFrames() :
		KeyZone->SampleEndOffset;

	bool HasLoopSection = KeyZone->SoundWaveData->GetLoopRegions().Num() > 0;
	if (SamplePos > EndOfSampleData && !HasLoopSection)
	{
		Kill();
		return;
	}

	CurrentVso = 0.0f;
	LastVsoPos = SamplePos;

	Attack();
}

void FFusionVoice::BuildGainMatrix(bool InSnap)
{
	float TotalGain = 0.0f;
	if (MySampler)
	{
		float MixGain = MySampler->GetMidiChannelGain() * MySampler->GetMidiChannelMuteGain();
		float TrimGain = MySampler->GetTrimGain();
		float ExpressionGain = MySampler->GetRampedExpression();
		TotalGain = VelocityGain * MixGain * TrimGain * ExpressionGain * FusionVoice::kInternalTrimGain;
	}

	AggregatePansSetTargetAndRamp(TotalGain, InSnap);
}

void FFusionVoice::PrepareWithPitchOffsetAndGain(double InPitchOffsetCents, float InGain)
{
	//UE_LOGF(LogFusionVoice, Log, "0x%x - Preparing\n", (size_t)this);

	checkSlow(MySampler);
	checkSlow(KeyZone);
	checkSlow(KeyZone->SoundWaveData);

	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		if (Lfo[Idx].GetSettings()->ShouldRetrigger)
		{
			Lfo[Idx].Retrigger();
		}
		else
		{
			Lfo[Idx].SetPhase(MySampler->Lfos[Idx].GetPhase());
		}
	}

	InPitchOffsetCents = FMath::Clamp(InPitchOffsetCents, -kMaxPitchOffsetCents, kMaxPitchOffsetCents);
	PitchShift = FMath::Pow(2.0f, InPitchOffsetCents * HarmonixDsp::kOctavesPerCent);
	ResampleRate = FileToOutputSampleRatio;

	// compute the initial gain
	check(0.0f <= InGain && InGain <= 2.0f);

	VelocityGain = InGain * KeyZone->Gain;

	// TODO: At the moment we will have no more than a stereo input. Even multichannel moggs
	// are mixed down to stereo before being used as a "voice". THIS MUST CHANGE to support 
	// quad, 5.1, 7.1, ambisonic, etc. voices!
	BuildGainMatrix(true);

	FilterGainRamper.SetTarget(1.0f);
	FilterGainRamper.SnapToTarget();

	for (int32 idx = 0; idx < FAudioBufferConfig::kMaxAudioBufferChannels; ++idx)
	{
		Filters[idx].ResetState();
	}

	CachedFilterFrequency = MySampler->FilterSettings.Freq;
	OctaveShift = ComputeOctaveShift();
	ApplyModsToFilter(true);
	FilterCoefsRamper.SnapToTarget();
	FilterGainRamper.SetTarget(1.0f);
	FilterGainRamper.SnapToTarget();

	MaxAudioLevel = 0.0f;

	if (ActiveRenderer)
	{
		ActiveRenderer->SetFrame(0);
	}
}

void FFusionVoice::AggregatePansSetTargetAndRamp(float InTotalGain, bool InSnap /*= false*/)
{
	if (!MySampler)
	{
		Panner.SetPanOffset_Radians(0.0f);
		Panner.SetEdgeProximityOffset(1.0f);
		return;
	}
	
	checkSlow(KeyZone);

	if (InSnap)
	{
		Panner.SetStartingPan(KeyZone->Pan, InTotalGain, true);
	}
	else
	{
		Panner.SetOverallGain(InTotalGain);
	}

	float PanOffset = MySampler->PanSettings.GetBasicPan();
	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		if (Lfo[Idx].GetSettings()->Target == ELfoTarget::Pan)
		{
			PanOffset += Lfo[Idx].GetValue();
		}
	}

	if (KeyZone->Pan.Mode == EPannerMode::LegacyStereo || KeyZone->Pan.Mode == EPannerMode::Stereo)
	{
		while (PanOffset < -1.0f)
		{
			PanOffset += 2.0f;
		}
		while (PanOffset > 1.0f)
		{
			PanOffset -= 2.0f;
		}
		if (InSnap)
		{
			Panner.SetPanOffset_Stereo(-PanOffset);
		}
		else
		{
			Panner.SetPanOffsetTarget_Stereo(-PanOffset);
		}
	}
	else
	{
		if (InSnap)
		{
			Panner.SetPanOffset_Radians(-PanOffset * (float)UE_PI);
		}
		else
		{
			Panner.SetPanOffsetTarget_Radians(-PanOffset * (float)UE_PI);
		}
	}
	Panner.Ramp();
}

bool FFusionVoice::IsWaitingForAttack() const
{
	return bWaitingForAttack;
}

bool FFusionVoice::IsInUse() const
{
	bool IsAdsrIdle = AdsrVolume.GetStage() == Harmonix::Dsp::Modulators::EAdsrStage::Idle;
	bool IsAudioActive = MaxAudioLevel > 0.0f;
	return !IsAdsrIdle || IsAudioActive;
}

float FFusionVoice::ComputeOctaveShift()
{
	static const float kMaxOctaveSwing = 5.0f;

	float OutOctaveShift = 0.0f;

	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		if (Lfo[Idx].GetSettings()->Target == ELfoTarget::FilterFreq)
		{
			OutOctaveShift += kMaxOctaveSwing * Lfo[Idx].GetValue();
		}	
	}

	if (AdsrAssignable.GetSettings()->Target == EAdsrTarget::FilterFreq && AdsrAssignable.GetSettings()->IsEnabled)
	{
		OutOctaveShift += 2 * kMaxOctaveSwing * AdsrAssignable.GetValue();
	}

	return OutOctaveShift;
}

void FFusionVoice::ApplyModsToFilter(bool InForceApply)
{
	// the voice's filter settings come from a combination of the
	// sampler's base filter settings plus an octave shift due
	// to modulation

	//----------------------------------------------------
	// calculate the new octave shift based on modulations
	// and track changes in the shift
	float NewOctaveShift = ComputeOctaveShift();
	float OctaveDiff = FMath::Abs(NewOctaveShift - OctaveShift); // how much has the octave shift changed?
	OctaveShift = NewOctaveShift;
	float ShiftFactor = FMath::Pow(2.0f, OctaveShift);


	//----------------------------------------------------
	// calculate the new filter settings
	// start by getting the original settings...
	// TO DO: DON'T DO THIS! IF THE BASE SETTINGS HAVE CNAGED PUSH THEM FROM THE SAMPLER!
	//        DON'T POLL THEM LIKE THIS!
	ModulatedFilterSettings = MySampler->FilterSettings;
	float NewBaseFreq = ModulatedFilterSettings.Freq;
	float BaseFreqDiff = CachedFilterFrequency / NewBaseFreq; // how much has the base frequency changed?
	CachedFilterFrequency = NewBaseFreq;


	//--------------------------------------------------
	// set the modulated settings based on the new frequency
	float NewFreq = CachedFilterFrequency * ShiftFactor;
	if (NewFreq != CachedFilterFrequency || InForceApply)
	{
		ModulatedFilterSettings.Freq = NewFreq;
	}

	float TargetGain = 1.0f;

	// decide if we should duck the filter temporarily
	bool ShouldDuck = false;
	if (BaseFreqDiff > 1.0f)
	{
		BaseFreqDiff = 1.0f / BaseFreqDiff;
	}

	if (BaseFreqDiff < 0.5f)
	{
		TargetGain *= BaseFreqDiff * BaseFreqDiff;
		ShouldDuck = true;
	}

	if (OctaveDiff > 1.0f)
	{
		TargetGain -= (OctaveDiff - 1.0f);
		if (TargetGain < 0.0f)
		{
			TargetGain = 0.0f;
		}
		ShouldDuck = true;
	}

	if (ShouldDuck)
	{
		// duck the filter output while we update the settings
		if (FilterGainRamper.IsAtTarget())
		{
			FilterGainRamper.SetTarget(TargetGain, &UpdateFilterSettings, this);
		}
	}
	else
	{
		if (FilterGainRamper.IsAtTarget() && FilterCoefsRamper.IsAtTarget())
		{
			FilterCoefsRamper.SetTarget(Harmonix::Dsp::Effects::FBiquadFilterCoefs(ModulatedFilterSettings, (float)SamplesPerSecond));
		}
	}
}

void FFusionVoice::UpdateFilterSettings(void* ThisPtr)
{
	FFusionVoice* ThisVoice = static_cast<FFusionVoice*>(ThisPtr);
	ThisVoice->FilterCoefsRamper.SetTarget(Harmonix::Dsp::Effects::FBiquadFilterCoefs(ThisVoice->ModulatedFilterSettings, (float)ThisVoice->SamplesPerSecond), &RestoreFilterGain, ThisVoice);
}

void FFusionVoice::RestoreFilterGain(void* ThisPtr)
{
	FFusionVoice* ThisVoice = static_cast<FFusionVoice*>(ThisPtr);
	ThisVoice->FilterGainRamper.SetTarget(1.0f);
}

uint32 FFusionVoice::Process(uint32 InSliceIndex, uint32 InSubsliceIndex, float** OutData, uint32 InNumChannels, uint32 InMaxNumSamples, float InSpeed, float InTempoBPM, bool MaintainPitchWhenSpeedChanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(*ProfileString, bProfiling);

	using namespace Harmonix::Dsp::Modulators;

	if (!IsInUse() || !MySampler || !KeyZone || !KeyZone->SoundWaveData)
	{
		return 0;
	}

	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		Lfo[Idx].Advance(static_cast<uint32> (InMaxNumSamples * InSpeed));
	}
	
	// compute the sample increment (ratio of input samples per output sample)
	// get the base increment and then apply pitch bend and keyzone's fine tuning
	double ResampleInc = ResampleRate;
	double PitchBend = MySampler->GetPitchBendFactor();

	double SemiTonesBend = 0;

	if (MySampler->GetIsPortamentoEnabled())
	{
		double PortaPitch = MySampler->GetCurrentPortamentoPitch();
		double PortaDiffInSemitones = PortaPitch - TargetMidiNote;
		SemiTonesBend += PortaDiffInSemitones;
	}

	for (int32 Idx = 0; Idx < kNumLfos; ++Idx)
	{
		if (Lfo[Idx].GetSettings()->Target == ELfoTarget::Pitch)
		{
			SemiTonesBend += 2.0 * Lfo[Idx].GetValue();
		}
	}

	// apply the per keyzone FineTuneCents to the semitone bend.
	// Convert cents to semitones (divide by 100)
	SemiTonesBend += KeyZone->FineTuneCents / 100.0f;

	// convert semitones to frequency
	PitchBend *= FMath::Pow(2.0, SemiTonesBend / 12.0);
	ResampleInc *= PitchBend;
	check(ResampleInc > 0.0);  // for now we only support playing forward

	OutputBuffer.SetAliasedChannelData(OutData, InNumChannels);

	//--------------------------------
	// update filter coefs
	//--------------------------------
	ApplyModsToFilter(false);

	double PitchThisSlice = PitchShift * MySampler->GetRawPitchMultiplier();

	bool bSkipFilter = (FilterCoefsRamper.GetCurrent().IsNoop() &&
		FilterCoefsRamper.GetTarget().IsNoop() &&
		FilterGainRamper.GetCurrent() == 1 &&
		FilterGainRamper.GetTarget() == 1);

	// do the processing in small blocks to give us enough granularity so that we don't need to
	// update all of the ramping data on a sample-by-sample basis.
	{
		uint32 NumProcessed = 0;
		uint32 NumRemaining = InMaxNumSamples;
		uint32 NumToProcessThisPass;

		float RenderSpeed = InSpeed;

		if (KeyZone->TimeStretchConfig.bMaintainTime && KeyZone->TimeStretchConfig.bSyncTempo && InTempoBPM > 0.0f)
		{
			RenderSpeed = (InTempoBPM * InSpeed) / KeyZone->TimeStretchConfig.OriginalTempo;
			MaintainPitchWhenSpeedChanges = true;

			// check for resync...
			float CurrentMidiQuarterNote = MySampler->GetQuarterNote();

			if (CurrentMidiQuarterNote > StartQuarterNote && (SamplePos > LastVsoPos || !bHasRenderedAnySamples))
			{
				LastVsoPos = SamplePos;

				float ElapsedBeats = CurrentMidiQuarterNote - StartQuarterNote;
				float ExpectedElapsedMs = ElapsedBeats * 60000.0f / KeyZone->TimeStretchConfig.OriginalTempo;

				float CurrentSampleFrame = (PitchShifter->HasCurrentSampleFrame() && bHasRenderedAnySamples) ? (float)StartPos + (float)PitchShifter->GetCurrentSampleFrame() : (float)SamplePos;
				if (KeyZone->SampleStartOffset != -1)
				{
					CurrentSampleFrame -= KeyZone->SampleStartOffset;
				}
				
				float ActualElapsedMs = (CurrentSampleFrame * 1000.0f) / FMath::Max(1, KeyZone->SoundWaveData->GetSampleRate());
				float ErrorMs = ActualElapsedMs - ExpectedElapsedMs;

				if (!bHasRenderedAnySamples)
				{
					if (ErrorMs > 40.0f)
					{
						UE_LOGF(LogFusionVoice, Verbose, "Adjusting start time due to error of %f ms", ErrorMs);
					}

					SamplePos = ExpectedElapsedMs / 1000.0f * (float)KeyZone->SoundWaveData->GetSampleRate();
					
					if (SamplePos > 0.0)
					{
						UE_LOGF(LogFusionVoice, Verbose, "SHAVING %f ms from the beginning of %ls", ErrorMs, *KeyZone->SoundWaveData->GetFName().ToString());
					}

					if (SamplePos < 0.0)
					{
						SamplePos = 0.0;
					}
					if (KeyZone->SampleStartOffset != -1)
					{
						SamplePos += KeyZone->SampleStartOffset;
					}
					if (SamplePos > EndOfSampleData && KeyZone->SoundWaveData->GetLoopRegions().IsEmpty())
					{
						Kill();
						return 0;
					}
					StartPos = SamplePos;
					CurrentVso = 0.0f;
				}
				else if (ErrorMs > FusionVoice::kVsoThresholdMs)
				{
					if (ErrorMs > 20.0f)
					{
						UE_LOGF(LogFusionVoice, Verbose, "Fixing %ls offset -> %f\n", *KeyZone->SoundWaveData->GetFName().ToString(), ErrorMs);
					}
					CurrentVso = -(FusionVoice::kVsoAmount * RenderSpeed);
				}
				else if (ErrorMs < -FusionVoice::kVsoThresholdMs)
				{
					if (ErrorMs < -20.0f)
					{
						UE_LOGF(LogFusionVoice, Verbose, "Fixing %ls offset -> %f\n", *KeyZone->SoundWaveData->GetFName().ToString(), ErrorMs);
					}
					CurrentVso = FusionVoice::kVsoAmount * RenderSpeed;
				}
				else
				{
					CurrentVso = 0.0f;
				}

				//----------------------------------------------
				// DEBUG
				/*
				static FusionVoice* track = nullptr;
				static int32 print_rate = 100;
				static int32 print_countdown = 0;
				if (track == this && print_countdown-- < 0)
				{
				   UE_LOGF(LogFusionVoice, Verbose, "offset -> %f (%f)\n", ErrorMs, CurrentVso);
				   print_countdown = print_rate;
				}
				*/
				//----------------------------------------------
			}
			RenderSpeed += CurrentVso;
			if (RenderSpeed < 0.000f)
			{
				RenderSpeed = 0.000f; // not too slow!
			}
		}

		// We probably have a trackmap, and this is the first render call,
		// so we have to configure the panner for the output buffer setup...
		int32 NumInChannels = KeyZone->SoundWaveData->GetNumChannels();
		if (NumInChannels != Panner.GetCurrentGainMatrix().GetNumInChannels())
		{
			Panner.Configure(NumInChannels, OutputBuffer);
			BuildGainMatrix(true);
		}

		// We want to lerp between the last gainmatrix and whatever
		// the new gain is (e.g. if pan is changing)
		FGainMatrix PrevGain = Panner.GetCurrentGainMatrix();
		BuildGainMatrix(false);
		FGainMatrix WorkingGain = Panner.GetCurrentGainMatrix();

		while (NumProcessed < InMaxNumSamples)
		{
			NumToProcessThisPass = FMath::Min((uint32)AudioRendering::kMicroSliceSize, NumRemaining);
			OutputBuffer.SetNumValidFrames(NumToProcessThisPass);
			OutputBuffer.ZeroData();

			// write interpolated sample data into the output buffer
			{
				//TIME_BLOCK(fusion_voice_process_filter);
				AdsrVolume.Advance(static_cast<uint32> (NumToProcessThisPass * InSpeed));
				AdsrAssignable.Advance(static_cast<uint32> (NumToProcessThisPass * InSpeed));
			}

			WorkingGain.Lerp(PrevGain, Panner.GetCurrentGainMatrix(), (float)NumProcessed / (float)InMaxNumSamples);
			WorkingGain *= AdsrVolume.GetValue();

			{
				SamplePos = ActiveRenderer->Render(
					OutputBuffer,
					SamplePos,
					KeyZone->SampleEndOffset,
					ResampleInc,
					PitchThisSlice,
					static_cast<double>(RenderSpeed),
					MaintainPitchWhenSpeedChanges,
					WorkingGain);

				ActiveRenderer->SetFrame((uint32)SamplePos);
				bHasRenderedAnySamples = true;
			}

			// apply filter
			if (!bSkipFilter)
			{
				//TRACE_CPUPROFILER_EVENT_SCOPE(Filter);

				FilterCoefsRamper.Ramp();
				FilterGainRamper.Ramp();

				for (uint32 Ch = 0; Ch < InNumChannels; ++Ch)
				{
					Filters[Ch].Process(OutputBuffer[Ch], OutputBuffer[Ch], NumToProcessThisPass, FilterCoefsRamper.GetCurrent(), FilterGainRamper.GetCurrent());
				}
			}

			//----------------------------------------
			// update buffer data
			//----------------------------------------
			OutputBuffer.IncrementChannelDataPointers(NumToProcessThisPass);
			NumProcessed += NumToProcessThisPass;
			NumRemaining -= NumToProcessThisPass;
		}

	}

	//TRACE_CPUPROFILER_EVENT_SCOPE(PostProc);

	// measure the output level of this block
	float MaxLevel = 0.0f;
	for (uint32 SampleNum = 0; SampleNum < InMaxNumSamples; ++SampleNum)
	{
		for (uint32 Ch = 0; Ch < InNumChannels; ++Ch)
		{
			float Level = FMath::Abs(OutData[Ch][SampleNum]);
			if (Level > MaxLevel)
			{
				MaxLevel = Level;
			}
		}
	}
	MaxAudioLevel = MaxLevel;

	//----------------------------------------------------
	// if there is no data left to play, kill the voice.
	// there could be DSP processing that generates sound
	// for a while after source data is gone (like in the
	// case of a high-q, low-cutoff filter which is oscillating
	// the low frequencies), so check the audio level too.
	// oh yeah, and portamento need to process strictly until
	// a note-off.
	//----------------------------------------------------

	// make sure the channel pitch can update even if there is no sound
	bool IsPortamentoEnabled = MySampler->GetIsPortamentoEnabled(); 
	// could be processing an oscillating filter
	bool IsAudioLevelLow = MaxAudioLevel < 0.0001; 
	bool IsOutOfSampleData = AdsrVolume.GetStage() == EAdsrStage::Idle || SamplePos >= EndOfSampleData;

	if (IsAudioLevelLow && IsOutOfSampleData)
	{
		if (!IsPortamentoEnabled || AdsrVolume.GetStage() == EAdsrStage::Idle)
		{
			Kill();
		}
	}

// TODO
//#if FUSION_VOICE_DEBUG_DUMP_ENABLED
//	if (gWriting && DumpFile)
//	{
//		for (uint32 i = 0; i < InMaxNumSamples; i++)
//		{
//			DumpFile->WriteEndian(&output[0][i], sizeof(float));
//			DumpFile->WriteEndian(&output[1][i], sizeof(float));
//		}
//	}
//#endif

	return InMaxNumSamples;
}

bool FFusionVoice::MatchesIDs(const FFusionSampler* InSampler, FMidiVoiceId InVoiceID, const FKeyzoneSettings* InKeyzone)
{
	if ((VoiceID == InVoiceID || InVoiceID == FMidiVoiceId::Any()) &&
		(MySampler == InSampler) &&
		(InKeyzone == nullptr || KeyZone == InKeyzone))
	{
		return true;
	}

	return false;
}

