// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/FusionVoicePool.h"
#include "HarmonixDsp/FusionSampler/FusionVoice.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "HarmonixDsp/FusionSampler/FusionSampler.h"
#include "HarmonixDsp/FusionSampler/FusionSamplerConfig.h"
#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactory.h"
#include "Sound/SoundWave.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FusionVoicePool)

DEFINE_LOG_CATEGORY(LogFusionVoicePool);

FFusionVoicePool::FPoolMap FFusionVoicePool::GVoicePools;

void FFusionVoicePool::SetSampleRate(float InSampleRate)
{
	if (SampleRate == InSampleRate)
	{
		return;
	}

	FScopeLock Lock(&PoolLock);
	KillVoices();
	FreeVoices();
	SampleRate = InSampleRate;
	if (!DynamicAllocAndFree || ClientSamplers.Num() > 0)
	{
		AllocVoices();
	}
}

void FFusionVoicePool::Lock()
{
	PoolLock.Lock();
}

void FFusionVoicePool::Unlock()
{
	PoolLock.Unlock();
}

FSharedFusionVoicePoolPtr FFusionVoicePool::GetDefault(float InSampleRate)
{
	return GetNamedPool(NAME_None, InSampleRate);
}

FSharedFusionVoicePoolPtr FFusionVoicePool::GetNamedPool(FName InPoolName, float InSampleRate)
{
	static FCriticalSection sVoicePoolLock;
	FScopeLock Lock(&sVoicePoolLock);

	GVoicePools = GVoicePools.FilterByPredicate([](const FPoolMap::ElementType& Pair)
		{
			if (FSharedFusionVoicePoolPtr SharedPool = (Pair.Value).Pin())
			{
				return true;
			}
			return false;
		});

	FPoolMapKey Key = MakeTuple(InPoolName, FMath::FloorToInt32(InSampleRate));
	if (TWeakPtr<FFusionVoicePool, ESPMode::ThreadSafe>* WeakPoolPtr = GVoicePools.Find(Key))
	{
		if (FSharedFusionVoicePoolPtr SharedPool = (*WeakPoolPtr).Pin())
		{
			return SharedPool;
		}
	}

	const UFusionSamplerConfig* FusionConfig = ::GetDefault<UFusionSamplerConfig>();

	FSharedFusionVoicePoolPtr NewPool = Create(FusionConfig->GetVoiceConfigForPoolName(InPoolName), InSampleRate);
	GVoicePools.Add(Key, NewPool.ToWeakPtr());
	return NewPool;
}

FSharedFusionVoicePoolPtr FFusionVoicePool::Create(const FFusionVoiceConfig& InConfig, float InSampleRate)
{
	FSharedFusionVoicePoolPtr NewVoicePool = MakeShared<FFusionVoicePool, ESPMode::ThreadSafe>(InSampleRate);

	NewVoicePool->SetHardVoiceLimit(InConfig.NumTotalVoices);
	NewVoicePool->SetSoftVoiceLimit(InConfig.SoftVoiceLimit);
	NewVoicePool->SetFormantVolumeCorrection(
		InConfig.FormantDbCorrectionPerHalfStepDown, 
		InConfig.FormantDbCorrectionPerHalfStepDown,
		InConfig.FormantDbCorrectionMaxUp,
		InConfig.FormantDbCorrectionMaxDown);
	NewVoicePool->SetIsMultithreading(true);
	return NewVoicePool;
}

FFusionVoicePool::~FFusionVoicePool()
{
	FScopeLock Lock(&PoolLock);

	// copy the list so that we don't care if the client
	// tries to remove him- or her- self as a result of
	// calling VoicePoolWillDestruct
	auto ClientsCopy = ClientSamplers;

	for (FFusionSampler* Sampler : ClientsCopy)
	{
		Sampler->VoicePoolWillDestruct(this);
	}

	//Voices.Reset();
	delete[]Voices;
	Voices = nullptr;
	//mPoolLock.Exit();
}

void FFusionVoicePool::HardAllocatateVoices()
{
	if (!DynamicAllocAndFree)
	{
		return; // already done!
	}

	AllocVoices();
	DynamicAllocAndFree = false;
}

void FFusionVoicePool::ReleaseHardAllocation()
{
	if (DynamicAllocAndFree)
	{
		return; // already done!
	}

	DynamicAllocAndFree = true;

	FScopeLock Lock(&PoolLock);

	if (ClientSamplers.Num() == 0)
	{
		FreeVoices();
	}
}

void FFusionVoicePool::SetIsMultithreading(bool InIsMultithreading)
{
	FScopeLock Lock(&PoolLock);

	IsMultithreading = InIsMultithreading;
}

// let the FusionVoicePool know that you will need voices
void FFusionVoicePool::AddClient(FFusionSampler* InSampler)
{
	FScopeLock Lock(&PoolLock);

	// TODO: Make this a TSet<>?
	// make sure we are not adding a duplicate client
	if (ClientSamplers.Contains(InSampler))
	{
		return;
	}

	ClientSamplers.Add(InSampler);

	if (ClientSamplers.Num() == 1 && NumVoicesSetting > 0 && DynamicAllocAndFree)
	{
		// got our first client.
		// better make sure we have the voices.
		AllocVoices();
	}
}

// let the FusionVoicePool know that you no longer need voices
void FFusionVoicePool::RemoveClient(FFusionSampler* InSampler)
{
	FScopeLock Lock(&PoolLock);

	int32 Index = ClientSamplers.Find(InSampler);
	if (Index == INDEX_NONE)
	{
		return;
	}

	ClientSamplers.RemoveAt(Index);

	if (ClientSamplers.IsEmpty() && DynamicAllocAndFree)
	{
		FreeVoices();
	}
}

uint32 FFusionVoicePool::GetNumVoicesInUse()
{
	FScopeLock Lock(&PoolLock);

	uint32 NumInUse = 0;
	for (uint32 VoiceIdx = 0; VoiceIdx < NumAllocatedVoices; ++VoiceIdx)
	{
		if (Voices[VoiceIdx].IsInUse())
		{
			NumInUse++;
		}
	}

	if (NumInUse > PeakVoiceUsage)
	{
		PeakVoiceUsage = NumInUse;
	}

	return NumInUse;
}

void FFusionVoicePool::SetSoftVoiceLimit(uint32 NewSoftLimit)
{
	FScopeLock Lock(&PoolLock);

	NewSoftLimit = FMath::Clamp(NewSoftLimit, kMinPoolSize, NumVoicesSetting);
	SoftVoiceLimit = NewSoftLimit;

	if (NumAllocatedVoices > 0)
	{
		FastReleaseExcessVoices();
	}
}

void FFusionVoicePool::SetHardVoiceLimit(uint32 NewPolyphony)
{
	FScopeLock Lock(&PoolLock);

	if (NewPolyphony < kMinPoolSize)
	{
		NewPolyphony = kMinPoolSize;
	}

	if (NewPolyphony > kMaxPoolSize)
	{
		NewPolyphony = kMaxPoolSize;
	}

	NumVoicesSetting = NewPolyphony;

	if (SoftVoiceLimit > NumVoicesSetting)
	{
		SoftVoiceLimit = NumVoicesSetting;
	}

	if (ClientSamplers.Num() > 0 || !DynamicAllocAndFree)
	{
		CreateVoices(NumVoicesSetting);
	}
}

void FFusionVoicePool::SetFormantVolumeCorrection(float DBPerHalfStepUp, float DBPerHalfStepDown, float DBMaxUp, float DBMaxDown)
{
	for (IStretcherAndPitchShifterFactory* Factory : IStretcherAndPitchShifterFactory::GetAllRegisteredFactories())
	{
		Factory->SetFormantVolumeCorrection(DBPerHalfStepUp, DBPerHalfStepDown, DBMaxUp, DBMaxDown);
	}
}

void FFusionVoicePool::ReleaseShifter(TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> InShifter)
{
	FScopeLock Lock(&PoolLock);
	IStretcherAndPitchShifterFactory* Factory = IStretcherAndPitchShifterFactory::FindFactory(InShifter->GetFactoryName());
	Factory->ReleasePitchShifter(InShifter);
}

FFusionVoice* FFusionVoicePool::GetFreeVoice(
	FFusionSampler* InSampler, 
	FMidiVoiceId InVoiceID, 
	const FKeyzoneSettings* InKeyzone, 
	TFunction<bool(FFusionVoice*)> Handler,
	IStretcherAndPitchShifterFactory* PitchShifterFactory,
	bool AllowAlias, 
	bool IsRendererForAlias)
{
#if 0
	if (AllowAlias && InKeyzone->IsSingleton())
	{
		return InKeyzone->SingletonFusionVoicePool->AllocateAlias(this, InSampler, InVoiceID, Handler, PitchShifterFactory);
	}
#endif

	FScopeLock Lock(&PoolLock);

	// we could be in the middle of changing our polyphony
	if (NumAllocatedVoices < 1)
	{
		UE_LOGF(LogFusionVoicePool, Warning, "Can't get free voice! NumAllocatedVoices <= 0!");
		return nullptr;
	}

	if (InSampler == nullptr)
	{
		UE_LOGF(LogFusionVoicePool, Warning, "Can't get free voice! Passed in a null sampler!");
		return nullptr;
	}

	if (!InKeyzone->SoundWaveData)
	{
		UE_LOGF(LogFusionVoicePool, Warning, "Asked to allocate a InSampler voice for an invalid sample");
		return nullptr;
	}

	// if maintain time is true and we didn't get a pitch shitfer factory, then we can't do anything!
	if (InKeyzone->TimeStretchConfig.bMaintainTime)
	{
		if (!PitchShifterFactory)
		{
			FName FactoryName = InKeyzone->TimeStretchConfig.PitchShifter.Name;
			UE_LOGF(LogFusionVoicePool, Warning,
				"Attempting to play a fusion InKeyzone that is set to \"maintain time\", but no shifter factory was available for assigned Pitch Shifter: %ls. Check project configuration! (%ls)",
				*FactoryName.ToString(), *InKeyzone->SoundWaveData->GetFName().ToString());
			return nullptr;
		}

		if (InKeyzone->TimeStretchConfig.OriginalTempo <= 0.0f)
		{
			UE_LOGF(LogFusionVoicePool, Warning,
				"Attempting to play a fusion InKeyzone that is set to \"maintain time\", but the original tempo for the keyzone is invalid (OriginalTempo = %.2f)! Unable to perform pitch shifting or time stretching.",
				InKeyzone->TimeStretchConfig.OriginalTempo);
			return nullptr;
		}
	}

	FFusionVoice* Voice = nullptr;
	FFusionVoice* BestVoice = nullptr;
	bool BestVoiceIsInUse = true; // assume it is, clear if it isn't below

	// find an unused voice
	uint32 VoiceIdx = 0;
	for (; VoiceIdx < NumAllocatedVoices; ++VoiceIdx)
	{
		Voice = &Voices[VoiceIdx];

		// Unfortunately we can't just grab any old voice that reports
		// false to "IsInUse". There are other criteria which might make 
		// the voice unusable...

		// don't hand out voices that have just been handed out
		if (Voice->IsWaitingForAttack())
		{
			continue;
		}

		// stop an existing voice if it is playing the same note and InKeyzone on the 
		// same InSampler
		if (Voice->MatchesIDs(InSampler, InVoiceID, InKeyzone) && Voice->IsInUse())
		{
			Voice->Release();
		}

		// don't hand out a voice that is rendering on behalf of an alias,
		// even if it appears to not be "in use". more than one sampler 
		// might depend on it!
		if (Voice->IsRendererForAlias())
		{
			continue;
		}

		// don't steal voice if we are multithreading the audio render
		// and the voice is used by another sampler as that sampler may 
		// be rendering on another thread.
		if (IsMultithreading && Voice->GetSampler() && !Voice->UsesSampler(InSampler))
		{
			continue;
		}

		// OK. Now. We know we aren't waiting for an attack, we know we aren't 
		// a special case for aliases, so if we are not in use now we can be handed 
		// back...
		if (!Voice->IsInUse())
		{
			BestVoiceIsInUse = false;
			BestVoice = Voice;
			// We will early out of this loop, but there
			// is a loop below that will continue cleaning up
			// voices. 
			break;
		}

		// don't steal voices with a higher priority 
		// than the voice we are trying to play
		if (Voice->Priority() == UFusionPatch::kVoicePriorityNoSteal ||
			Voice->Priority() < InKeyzone->Priority)
		{
			continue;
		}
		// since the voice is stealable, if we don't now have a best choice yet
		// this voice must be the current best choice...
		if (!BestVoice)
		{
			BestVoice = Voice;
			continue;
		}

		// first criteria... priority...
		int32 BestPriority = BestVoice->Priority();
		int32 VoicePriority = Voice->Priority();
		if (BestPriority > VoicePriority)
		{
			continue;
		}

		if (BestPriority < VoicePriority)
		{
			BestVoice = Voice;
			continue;
		}

		using namespace Harmonix::Dsp::Modulators;

		// next criteria... Adsr state...
		EAdsrStage BestStage = BestVoice->GetAdsrStage();
		EAdsrStage VoiceStage = Voice->GetAdsrStage();

		// Is only one in release? If so, that one is our best choice...
		if ((BestStage == EAdsrStage::Release && VoiceStage != EAdsrStage::Release) ||
			(BestStage != EAdsrStage::Release && VoiceStage == EAdsrStage::Release))
		{
			if (VoiceStage == EAdsrStage::Release)
			{
				BestVoice = Voice;
			}
			continue;
		}

		// Next criteria is age. Oldest voice loses, or if they
		// are the same age, the quietest loses...
		uint32 BestAge = BestVoice->GetAge();
		uint32 VoiceAge = Voice->GetAge();
		if (VoiceAge > BestAge ||
			(VoiceAge == BestAge && Voice->GetCombinedAudioLevel() < BestVoice->GetCombinedAudioLevel()))
		{
			BestVoice = Voice;
			continue;
		}
	}

	// pickup *one after* where the last loop left off (to finish cleanup)...
	for (++VoiceIdx; VoiceIdx < NumAllocatedVoices; ++VoiceIdx)
	{
		Voice = &Voices[VoiceIdx];
		if (Voice->MatchesIDs(InSampler, InVoiceID, InKeyzone) && Voice->IsInUse())
		{
			Voice->Release();
		}
	}

	// did we find one to use?
	if (!BestVoice)
	{
		UE_LOGF(LogFusionVoicePool, Verbose, "Can't get free voice! Failed to find an available voice.");
		return nullptr;
	}

	// Now we have a best choice. Now... do we need a shifter?
	if (InKeyzone->TimeStretchConfig.bMaintainTime && !BestVoice->GetPitchShifter() && !PitchShifterFactory->HasFreePitchShifters(InKeyzone->TimeStretchConfig))
	{
		UE_LOGF(LogFusionVoicePool, Warning, "Can't get free voice! Voice needs a shifter but we don't have one!");
		return nullptr;
	}

	// here's our best choice
	BestVoice->Kill(); // make sure it's free. Kill will tell current owner to relinquish!
	TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> ShifterToUse = nullptr;
	if (InKeyzone->TimeStretchConfig.bMaintainTime)
	{
		ShifterToUse = PitchShifterFactory->GetFreePitchShifter(InKeyzone->TimeStretchConfig);
		check(ShifterToUse);
	}

	if (BestVoice->AssignIDs(InSampler, InKeyzone, InVoiceID, Handler, ShifterToUse))
	{
		UE_LOGF(LogFusionVoicePool, Verbose, "Successfully returning fusion voice from fusion voice pool.");
		BestVoice->SetIsRendererForAlias(IsRendererForAlias);
		return BestVoice;
	}
	else
	{
		UE_LOGF(LogFusionVoicePool, Warning, "Can't get free voice! Failed to Assign IDs!");
		return nullptr;
	}
}

bool FFusionVoicePool::HasVoice(FFusionSampler* InOwner, FMidiVoiceId InVoiceId)
{
	for (uint32 VoiceIdx = 0; VoiceIdx < GetHardVoiceLimit(); ++VoiceIdx)
	{
		FFusionVoice* Voice = GetVoice(VoiceIdx);
		if (Voice->IsInUse() && Voice->MatchesIDs(InOwner, InVoiceId))
		{
			return true;
		}
	}
	return false;
}

uint32 FFusionVoicePool::FastReleaseExcessVoices(FFusionSampler* InSampler)
{
	FScopeLock Lock(&PoolLock);

	// scan all voices to find active (non-released) voices using this channel
	FFusionVoice* VoicesInUse[kMaxPoolSize];
	uint32 NumVoicesInUse = 0;

	if (InSampler)
	{
		NumVoicesInUse = InSampler->GetNumVoicesInUse(VoicesInUse);
	}
	else
	{
		for (uint32 VoiceIdx = 0; VoiceIdx < GetHardVoiceLimit(); ++VoiceIdx)
		{
			FFusionVoice* Voice = GetVoice(VoiceIdx);

			if (!Voice->IsWaitingForAttack() && !Voice->IsInUse())
			{
				continue;
			}

			VoicesInUse[NumVoicesInUse] = Voice;
			NumVoicesInUse++;
		}
	}

	// When multithreaded rendering we can't 
	// continue below with voice limiting because
	// we could steal voices other threads are currently using!
	if (IsMultithreading)
	{
		return NumVoicesInUse;
	}

	// get the soft limit
	uint32 SoftLimit = 0;
	if (InSampler)
	{
		SoftLimit = InSampler->GetMaxNumVoices();
	}
	else
	{
		SoftLimit = GetSoftVoiceLimit();
	}

	int32 NumVoicesOverLimit = NumVoicesInUse - SoftLimit;

	FFusionVoice* BestVoiceToRelease = nullptr;
	int32 BestVoiceIdx = 0;
	while (NumVoicesOverLimit > 0)
	{
		BestVoiceToRelease = nullptr;

		// scan all in-use voices to figure out which one to release
		for (uint32 VoiceIdx = 0; VoiceIdx < NumVoicesInUse; ++VoiceIdx)
		{
			FFusionVoice* Voice = VoicesInUse[VoiceIdx];

			// ignore voices that have just been cancelled
			if (VoicesInUse[VoiceIdx] == nullptr)
			{
				continue;
			}

			// default "best" is the first one
			if (BestVoiceToRelease == nullptr)
			{
				BestVoiceToRelease = Voice;
				BestVoiceIdx = VoiceIdx;
			}

			// if the current voice is older, it is better to release that one
			if (Voice->Priority() > BestVoiceToRelease->Priority() ||
				(Voice->Priority() == BestVoiceToRelease->Priority() && Voice->GetAge() > BestVoiceToRelease->GetAge()))
			{
				BestVoiceToRelease = Voice;
				BestVoiceIdx = VoiceIdx;
			}
		}

		if (BestVoiceToRelease)
		{
			BestVoiceToRelease->FastRelease();
			// remove the voice from the list, so we don't release it again
			VoicesInUse[BestVoiceIdx] = nullptr;
			--NumVoicesOverLimit;
			BestVoiceToRelease = nullptr;
		}
	}

	return NumVoicesInUse;
}

void FFusionVoicePool::KillVoices()
{
	//CRIT_SEC_REF(mPoolLock);
	FScopeLock Lock(&PoolLock);

	FFusionVoice* Voice = nullptr;
	for (uint32 VoiceIdx = 0; VoiceIdx < NumAllocatedVoices; ++VoiceIdx)
	{
		Voices[VoiceIdx].Kill();
	}
}

void FFusionVoicePool::KillVoices(const FFusionSampler* InSampler, bool NoCallbacks)
{
	check(InSampler != nullptr);
	//CRIT_SEC_REF(mPoolLock);
	FScopeLock Lock(&PoolLock);

	FFusionVoice* Voice = nullptr;
	for (uint32 VoiceIdx = 0; VoiceIdx < NumAllocatedVoices; ++VoiceIdx)
	{
		if (Voices[VoiceIdx].UsesSampler(InSampler))
		{
			if (NoCallbacks)
			{
				Voices[VoiceIdx].SetRelinquishHandler(nullptr);
			}
			Voices[VoiceIdx].Kill();
		}
	}
}


void FFusionVoicePool::KillVoices(const FKeyzoneSettings* InKeyzone)
{
	check(InKeyzone != nullptr);
	//CRIT_SEC_REF(mPoolLock);
	FScopeLock Lock(&PoolLock);

	FFusionVoice* Voice = nullptr;
	for (uint32 VoiceIdx = 0; VoiceIdx < NumAllocatedVoices; ++VoiceIdx)
	{
		if (Voices[VoiceIdx].UsesKeyzone(InKeyzone))
		{
			Voices[VoiceIdx].Kill();
		}
	}
}

FFusionVoice* FFusionVoicePool::GetVoice(uint32 VoiceIdx)
{
	check(VoiceIdx < NumAllocatedVoices);
	return &Voices[VoiceIdx];
}


void FFusionVoicePool::AllocVoices()
{
	CreateVoices(NumVoicesSetting);
	CreateShifters();
}


void FFusionVoicePool::CreateVoices(uint16 InNumToAllocate)
{
	check(InNumToAllocate > 0);

	{
		// make sure process can safely be called without using any voice resources.
		FScopeLock Lock(&PoolLock);
		KillVoices();
		NumAllocatedVoices = 0;
	}

	// get rid of the old voices and create the new ones
	delete[]Voices;
	Voices = nullptr;
	Voices = new FFusionVoice[InNumToAllocate];

	for (uint16 VoiceIdx = 0; VoiceIdx < InNumToAllocate; ++VoiceIdx)
	{
		Voices[VoiceIdx].Init(this, VoiceIdx);
	}

	PeakVoiceUsage = 0;

	// do this last, because we could be
	// in the middle of processing
	// or trying to note-on
	NumAllocatedVoices = InNumToAllocate;
}

void FFusionVoicePool::CreateShifters()
{
	KillVoices();
}

void FFusionVoicePool::FreeVoices()
{
	bool FoundVoiceInUse = false;
	for (uint32 VoiceIdx = 0; VoiceIdx < NumAllocatedVoices; ++VoiceIdx)
	{
		if (Voices[VoiceIdx].IsInUse())
		{
			FoundVoiceInUse = true;
			break;
		}
	}
	check(!FoundVoiceInUse);
	NumAllocatedVoices = 0;
	//SAFE_ARRAY_DELETE(mVoices);
	delete[]Voices;
	Voices = nullptr;
}

