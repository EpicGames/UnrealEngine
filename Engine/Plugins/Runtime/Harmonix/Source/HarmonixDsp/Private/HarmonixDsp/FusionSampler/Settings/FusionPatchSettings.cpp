// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FusionPatchSettings)

FFusionPatchSettings::FFusionPatchSettings()
	: VolumeDb(0.0f)
	, DownPitchBendCents(-200.0f)
	, UpPitchBendCents(200.0f)
	, FineTuneCents(0.0f)
	, StartPointOffsetMs(0.0f)
	, MaxVoices(32)
	, KeyzoneSelectMode(EKeyzoneSelectMode::Layers)
{
	Adsr[0].Target = EAdsrTarget::Volume;
	Adsr[1].Target = EAdsrTarget::FilterFreq;
	Lfo[0].Target = ELfoTarget::Pan;
	Lfo[1].Target = ELfoTarget::Pitch;
}
