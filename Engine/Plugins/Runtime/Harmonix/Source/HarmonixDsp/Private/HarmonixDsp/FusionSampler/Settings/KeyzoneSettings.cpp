// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"
#include "HarmonixDsp/FusionSampler/SingletonFusionVoicePool.h"
#include "HarmonixDsp/AudioUtility.h"

#include "AudioStreaming.h"
#include "Sound/SoundWave.h"

#include "HAL/PlatformTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KeyzoneSettings)

DEFINE_LOG_CATEGORY(LogKeyzoneSettings);

FKeyzoneSettings::FKeyzoneSettings()
{
	Adsr.Target = EAdsrTarget::Volume;
}

FKeyzoneSettings::~FKeyzoneSettings()
{
	if (SoundWaveData.IsValid() && SoundWaveData->GetLoadingBehavior() == ESoundWaveLoadingBehavior::ForceInline)
	{
		IStreamingManager::Get().GetAudioStreamingManager().RemoveForceInlineSoundWave(SoundWaveData.ToSharedRef());
	}
}

void FKeyzoneSettings::InitProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FKeyzoneSettings_InitProxyData);

	UE_LOGF(LogKeyzoneSettings, Verbose, "Initializing new KeyzoneSettings proxy");
	
	// we should only be initializing the proxy data if the sound wave is valid
	if (!ensure(SoundWave))
	{
		return;
	}

	if (SoundWaveData.IsValid() && SoundWaveData->GetLoadingBehavior() == ESoundWaveLoadingBehavior::ForceInline)
	{
		// I don't think we'de ever hit this code, but let's just do this to be sure.
		IStreamingManager::Get().GetAudioStreamingManager().RemoveForceInlineSoundWave(SoundWaveData.ToSharedRef());
	}
	
	TSharedPtr<Audio::IProxyData> AudioProxy = SoundWave->CreateProxyData(InitParams);
	TSharedPtr<FSoundWaveProxy> TempSoundWaveProxy = StaticCastSharedPtr<FSoundWaveProxy>(AudioProxy);
	if (!ensureMsgf(TempSoundWaveProxy, TEXT("FKeyzoneSettings::InitProxyData: CreateProxyData did not return an FSoundWaveProxy for '%s'"), *SoundWave->GetName()))
	{
		return;
	}
	SoundWaveData = TempSoundWaveProxy->GetSoundWaveDataRef();
	check(SoundWaveData);
		
	if (SoundWaveData->GetLoadingBehavior() == ESoundWaveLoadingBehavior::ForceInline)
	{
		IStreamingManager::Get().GetAudioStreamingManager().AddForceInlineSoundWave(SoundWaveData.ToSharedRef());
	}

	// nulling out sound wave asset for this Keyzone
	// now that we are turning into ProxyData
	SoundWave = nullptr;

	if (UseSingletonVoicePool && !SingletonFusionVoicePool)
	{
		SingletonFusionVoicePool = MakeShared<FSingletonFusionVoicePool>(FSingletonFusionVoicePool::kMaxSingletonAliases, *this);
	}
	else if (SingletonFusionVoicePool)
	{
		SingletonFusionVoicePool.Reset();
	}
}

bool FKeyzoneSettings::ContainsNoteAndVelocity(uint8 InNote, uint8 InVelocity) const
{
	return (InNote >= MinNote && InNote <= MaxNote) 
		&& (InVelocity >= MinVelocity && InVelocity <= MaxVelocity);
}

void FKeyzoneSettings::SetVolumeDb(float Db)
{ 
	Db = HarmonixDsp::ClampDB(Db);
	Gain = HarmonixDsp::DBToLinear(Db);
}

float FKeyzoneSettings::GetVolumeDb() const
{
	return HarmonixDsp::LinearToDB(FMath::Clamp(Gain, 0.0f, 1.0f));
}
