// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/Settings/PortamentoSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PortamentoSettings)
