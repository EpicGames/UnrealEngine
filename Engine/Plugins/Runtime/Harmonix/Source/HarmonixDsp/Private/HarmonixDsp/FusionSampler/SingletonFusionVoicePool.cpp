// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/FusionSampler/SingletonFusionVoicePool.h"
#include "HarmonixDsp/FusionSampler/FusionVoicePool.h"
#include "HarmonixDsp/FusionSampler/FusionVoice.h"

const int32 FSingletonFusionVoicePool::kMaxSingletonAliases = 16;

FSingletonFusionVoicePool::FSingletonFusionVoicePool(int32 InMaxInstances, FKeyzoneSettings& InKeyzoneRef)
	: KeyzoneRef(InKeyzoneRef)
{
	PrimaryTracker.CachedBuffer.Configure(
		EAudioBufferChannelLayout::Raw,
		AudioRendering::kFramesPerRenderBuffer,
		EAudioBufferCleanupMode::Delete
	);

	ReleasingTracker.CachedBuffer.Configure(
		EAudioBufferChannelLayout::Raw,
		AudioRendering::kFramesPerRenderBuffer,
		EAudioBufferCleanupMode::Delete
	);

	Aliases.Reserve(InMaxInstances);
	for (int32 idx = 0; idx < InMaxInstances; ++idx)
	{
		Aliases.Add(this);
	}
}


void FSingletonFusionVoicePool::SamplerDisconnecting(const FFusionSampler* Sampler)
{
	for (FAliasFusionVoice& AliasVoice : Aliases)
	{
		if (AliasVoice.IsInUse() && AliasVoice.GetSampler() == Sampler)
		{
			AliasVoice.Kill();
		}
	}
}


FFusionVoice* FSingletonFusionVoicePool::AllocateAlias(
	FFusionVoicePool* InPool, 
	FFusionSampler* InSampler, 
	FMidiVoiceId InVoiceID, 
	TFunction<bool(FFusionVoice*)> Handler,
	IStretcherAndPitchShifterFactory* PitchShifterFactory)
{
	FScopeLock Lock(&PoolLock);

	if (InPool)
	{
		InPool->Lock();
	}

	// In keeping with the FusionVoicePool functionality, we need to see if the requested 
	// is for the same note on the same sampler as any of our aliases, because if so, the existing
	// ones should be stopped..
	for (FAliasFusionVoice& AliasVoice : Aliases)
	{
		if (AliasVoice.IsInUse() && AliasVoice.MatchesIDs(InSampler, InVoiceID, &KeyzoneRef))
		{
			AliasVoice.Kill();
		}
	}

	
	// restart the voice if isOnTimeTrigger is true AND last trigger has produced a few samples ?
	FFusionVoice* PrimaryVoice = PrimaryTracker.Voice;
	bool IsInSetupStage = PrimaryVoice ? PrimaryVoice->GetAdsrStage() == Harmonix::Dsp::Modulators::EAdsrStage::Setup : false;
	bool HasProducedSamples = PrimaryVoice ? PrimaryVoice->GetPlaybackPos() > (PrimaryVoice->GetEndOfSampleData() - (double)AudioRendering::kFramesPerRenderBuffer) : false;
	if (PrimaryVoice && !IsInSetupStage && HasProducedSamples)
	{
		if (ReleasingTracker.Voice)
		{
			ReleasingTracker.Voice->Kill();
		}

		ReleasingTracker = PrimaryTracker;
		PrimaryTracker.Voice = nullptr;
		PrimaryVoice->Release();
		PrimaryVoice = nullptr;
		DriverVoice = nullptr;
	}
	// if we don't have a real voice yet... get one!
	else if (!PrimaryVoice)
	{
		if (!ensure(InPool))
		{
			return nullptr;
		}

		PrimaryVoice = PrimaryTracker.Voice = InPool->GetFreeVoice(
			InSampler,
			InVoiceID,
			&KeyzoneRef,
			[this](FFusionVoice* HandleVoice)
			{
				return Relinquish(HandleVoice);
			},
			PitchShifterFactory,
			false,
			true);

		if (!ensure(PrimaryVoice))
		{
			InPool->Unlock();
			return nullptr;
		}
	}

	for (FAliasFusionVoice& AliasVoice : Aliases)
	{
		if (!AliasVoice.IsInUse())
		{
			AliasVoice.Activate(InSampler, InVoiceID, PrimaryVoice, Handler);
			if (!DriverVoice)
			{
				DriverVoice = &AliasVoice;
			}
			NumActiveAliases++;
			if (InPool)
			{
				InPool->Unlock();
			}

			return &AliasVoice;
		}
	}

	if (InPool)
	{
		InPool->Unlock();
	}

	ensureMsgf(false, TEXT("failed to allocate an alias voice"));
	return nullptr;
}

TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> FSingletonFusionVoicePool::GetPitchShifter() const
{
	return (PrimaryTracker.Voice) ? PrimaryTracker.Voice->GetPitchShifter() : nullptr;
}

Harmonix::Dsp::Modulators::EAdsrStage FSingletonFusionVoicePool::GetAdsrStage(const FAliasFusionVoice* InAlias) const
{
	using namespace Harmonix::Dsp::Modulators;

	// If we don't have an underlying real voice return idle...
	if (!InAlias->GetRealVoice())
	{
		return EAdsrStage::Idle;
	}

	// if the requestor is NOT the driver, AND it has been released we always report that 
	// the ADSR has completed...
	if (InAlias != DriverVoice && InAlias->GetState() == FAliasFusionVoice::EState::Released)
	{
		return EAdsrStage::Idle;
	}

	// otherwise the state of the real voice...
	return InAlias->GetRealVoice()->GetAdsrStage();
}

void FSingletonFusionVoicePool::SetPitchOffset(FAliasFusionVoice* InAlias, double InNumCents)
{
	FFusionVoice* RealVoice = InAlias->GetRealVoice();
	if (!RealVoice)
	{
		return;
	}
	RealVoice->SetPitchOffset(InNumCents);
}


#if FUSION_VOICE_DEBUG_DUMP_ENABLED
void FSingletonFusionVoicePool::AttackWithTargetNote(
	FAliasFusionVoice* InAlias,
	uint8 MidiNoteNumber, 
	float InGain, 
	int32 InEventTick, 
	int32 InTriggerTick, 
	double InStartPointMs,
	const char* InFilePath)
#else
void FSingletonFusionVoicePool::AttackWithTargetNote(
	FAliasFusionVoice* InAlias,
	uint8 MidiNoteNumber, 
	float InGain, 
	int32 InEventTick, 
	int32 InTriggerTick,
	double InStartPointMs)
#endif
{
	FFusionVoice* RealVoice = InAlias->GetRealVoice();
	if (!RealVoice)
	{
		return;
	}

	if (RealVoice->IsWaitingForAttack())
	{
#if FUSION_VOICE_DEBUG_DUMP_ENABLED
		RealVoice->AttackWithTargetNote(
			MidiNoteNumber, 
			InGain, 
			InEventTick, 
			InTriggerTick,
			InStartPointMs, 
			InFilePath);
#else
		RealVoice->AttackWithTargetNote(
			MidiNoteNumber, 
			InGain, 
			InEventTick, 
			InTriggerTick,
			InStartPointMs);
#endif
	}
	else
	{
		RealVoice->SetPitchShiftForMidiNote(MidiNoteNumber);
	}
}

void FSingletonFusionVoicePool::Release(FAliasFusionVoice* InAlias)
{
	if (InAlias != DriverVoice)
	{
		return;
	}

	// See if we can find another alias to take over driver
	// responsibilities. It must be an active alias...
	for (auto& Alias : Aliases)
	{
		if (&Alias != DriverVoice && Alias.IsStartingOrRunning() &&
			Alias.GetRealVoice() == InAlias->GetRealVoice())
		{
			DriverVoice = &Alias;
			InAlias->GetRealVoice()->SetSampler(DriverVoice->GetSampler());
			return;
		}
	}

	// If we get here there was no other alias that could take over as driver
	// so communicate with the real voice...
	InAlias->GetRealVoice()->Release();
}

void FSingletonFusionVoicePool::FastRelease(FAliasFusionVoice* InAlias)
{
	if (InAlias != DriverVoice)
	{
		return;
	}

	// See if we can find another alias to take over driver
	// responsibilities. It must be an active alias...
	for (auto& Alias : Aliases)
	{
		if (&Alias != DriverVoice && Alias.IsStartingOrRunning() &&
			Alias.GetRealVoice() == InAlias->GetRealVoice())
		{
			DriverVoice = &Alias;
			InAlias->GetRealVoice()->SetSampler(DriverVoice->GetSampler());
			return;
		}
	}

	// If we get here there was no other alias that could take over as driver
	// so communicate with the real voice...
	InAlias->GetRealVoice()->FastRelease();
}

void FSingletonFusionVoicePool::Kill(FAliasFusionVoice* InAlias)
{
	check(NumActiveAliases > 0);
	NumActiveAliases--;

	if (InAlias != DriverVoice)
	{
		// Either the requestor is just one of multiple aliases using the "primary" aliased voice,
		// or, it is an alease to a voice that was "auto released". If the later, we need to see 
		// if it was the last one, and if so, we need to kill the real voice...
		if (ReleasingTracker.Voice && InAlias->GetRealVoice() == ReleasingTracker.Voice)
		{
			// anyone but requestor still using the voice?
			for (auto& Alias : Aliases)
			{
				if (&Alias != InAlias && Alias.GetRealVoice() == ReleasingTracker.Voice)
				{
					return;
				}
			}
			// No, time to kill the real voice...
			ReleasingTracker.Voice->Kill();
			ReleasingTracker.Voice = nullptr;
		}
		return;
	}

	if (!InAlias->GetRealVoice())
	{
		return;
	}

	// See if we can find another alias to take over driver
	// responsibilities. It must be an active alias...
	// assume not.
	DriverVoice = nullptr; 
	if (NumActiveAliases)
	{
		for (auto& Alias : Aliases)
		{
			if (&Alias != DriverVoice && Alias.IsStartingOrRunning() &&
				Alias.GetRealVoice() == InAlias->GetRealVoice())
			{
				// got one...
				DriverVoice = &Alias;
				InAlias->GetRealVoice()->SetSampler(DriverVoice->GetSampler());
				return;
			}
		}
	}

	// If we get here there was no other alias that could take over as driver
	// so communicate with the real voice...
	InAlias->GetRealVoice()->Kill();
}

bool FSingletonFusionVoicePool::MatchesIDs(const FFusionSampler* InSampler, FMidiVoiceId InVoiceID, const FKeyzoneSettings* InKeyzone)
{
	bool DoesPrimaryMatch = (PrimaryTracker.Voice) ?
		PrimaryTracker.Voice->MatchesIDs(InSampler, InVoiceID, InKeyzone) : false;
	bool DoesReleasingMatch = (ReleasingTracker.Voice) ?
		ReleasingTracker.Voice->MatchesIDs(InSampler, InVoiceID, InKeyzone) : false;

	return DoesPrimaryMatch || DoesReleasingMatch;
}

bool FSingletonFusionVoicePool::UsesKeyzone(const FKeyzoneSettings* InKeyzone) const
{

	return (&KeyzoneRef == InKeyzone) && (PrimaryTracker.Voice || ReleasingTracker.Voice);
}

void FSingletonFusionVoicePool::SetupLfo(const FAliasFusionVoice* InAlias, int32 Index, const FLfoSettings& InSettings)
{
	if (InAlias != DriverVoice || !PrimaryTracker.Voice)
	{
		return;
	}

	PrimaryTracker.Voice->SetupLfo(Index, InSettings);
}

uint32 FSingletonFusionVoicePool::Process(
	FAliasFusionVoice* InAlias, 
	uint32 SliceIndex,
	uint32 SubsliceIndex, 
	float** Output,
	uint32 InNumChannels, 
	uint32 InMaxNumSamples,
	float InSpeed, 
	float InTempoBPM,
	bool  MaintainPitchWhenSpeedChanges)
{
	FCachedVoiceTracker& VoiceTracker = (InAlias->GetRealVoice() == PrimaryTracker.Voice) ?
		PrimaryTracker : ReleasingTracker;
	if (VoiceTracker.SliceIndex != SliceIndex || VoiceTracker.SubsliceIndex != SubsliceIndex)
	{
		// actually render the slice...
		VoiceTracker.NumFramesLastRendered = VoiceTracker.Voice->Process(
			SliceIndex, 
			SubsliceIndex,
			VoiceTracker.CachedBuffer.GetData(), 
			InNumChannels, 
			InMaxNumSamples, 
			InSpeed, 
			InTempoBPM,
			MaintainPitchWhenSpeedChanges);

		VoiceTracker.SliceIndex = SliceIndex;
		VoiceTracker.SubsliceIndex = SubsliceIndex;
	}

	// use the cached data...
	check(InNumChannels <= (uint32)VoiceTracker.CachedBuffer.GetMaxNumChannels());
	check(InMaxNumSamples <= (uint32)VoiceTracker.CachedBuffer.GetMaxNumFrames());

	for (uint32 ch = 0; ch < InNumChannels; ++ch)
	{
		FMemory::Memcpy(Output[ch], VoiceTracker.CachedBuffer.GetData()[ch], sizeof(float) * InMaxNumSamples);
	}
	return VoiceTracker.NumFramesLastRendered;
}

bool FSingletonFusionVoicePool::Relinquish(FFusionVoice* InVoice)
{
	FScopeLock Lock(&PoolLock);

	FCachedVoiceTracker& VoiceTracker = (InVoice == PrimaryTracker.Voice) ? PrimaryTracker : ReleasingTracker;

	for (auto& Ailas : Aliases)
	{
		if (Ailas.IsInUse() && Ailas.GetRealVoice() == InVoice)
		{
			Ailas.ClearVoice();
		}
	}

	VoiceTracker.Voice = nullptr;
	if (&VoiceTracker == &PrimaryTracker)
	{
		DriverVoice = nullptr;
	}
	return true;
}
