// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/GainMatrix.h"

#include "HarmonixDsp/PannerDetails.h"

#include "Containers/UnrealString.h"

#include "HAL/LowLevelMemTracker.h"

DEFINE_LOG_CATEGORY(LogGainMatrix);

LLM_DEFINE_TAG(Harmonix_GainMatrix);

using namespace HarmonixDsp;

FGainMatrix* FGainMatrix::sUnity = nullptr;

void FGainMatrix::Init()
{
	LLM_SCOPE_BYTAG(Harmonix_GainMatrix);

	void* Mem = FMemory::Malloc(sizeof(FGainMatrix), 16);
	sUnity = new(Mem) FGainMatrix(FAudioBuffer::kMaxChannelsInAudioBuffer, FAudioBuffer::kMaxChannelsInAudioBuffer, EAudioBufferChannelLayout::Raw, 0.0f);

	FGainMatrix& GainMatrix = *sUnity;
	for (int32 Idx = 0; Idx < FAudioBuffer::kMaxChannelsInAudioBuffer; ++Idx)
	{
		GainMatrix[Idx].f[Idx] = 1.0f;
	}
}

void FGainMatrix::SetFromLegacyStereo(float InGain, float InPan)
{
	float RtoR = FMath::Min(1.0f, InPan + 1.0f);
	float RtoL = 1.0f - RtoR;
	float LtoL = FMath::Min(1.0f, 1.0f - InPan);
	float LtoR = 1.0f - LtoL;

	if (NumInChannels > 1)
	{
		// left in to left out
		GainsArray[0].f[0] = InGain * LtoL;
		
		// left in to right out
		GainsArray[0].f[1] = InGain * LtoR; 
		
		// right in to right out
		GainsArray[1].f[1] = InGain * RtoR; 
		
		// right in to left out
		GainsArray[1].f[0] = InGain * RtoL;
	}
	else
	{
		// left and right ins to left out... yuck
		GainsArray[0].f[0] = InGain * LtoL; 

		// left and right ins to right out... yuck
		GainsArray[0].f[1] = InGain * RtoR; 
	}
}

void FGainMatrix::SetFromNewStereo(float InGain, float InPan, float InMax, const FGainTable* InGainTable /*= nullptr*/)
{
	if (NumInChannels == 1)
	{
		// This will result in an equal power pan
		return Set(InGain, -InPan, InGainTable);
	}

	// do a "balance" pan just like it used to
	return SetFromLegacyStereo(InGain, InPan / InMax);       
}

void FGainMatrix::SetFromMinusOneToOneSurroundPan(float InGain, float InPan, const FGainTable* InGainTable /*= nullptr*/)
{
	// pan is -1 = left, 0 = center, 1 = right
	// polar is the opposite, so turn in to 
	// radians AND flip sign...
	InPan *= (float)(-UE_PI);
	Set(InGain, InPan, InGainTable);
}

void FGainMatrix::SetFromPolarDegrees(float InGain, float InPan, const FGainTable* InGainTable /*= nullptr*/)
{
	InPan = (InPan / 180.0f) * (float)(-UE_PI);
	Set(InGain, InPan, InGainTable);
}

void FGainMatrix::Set(float InGain, ESpeakerChannelAssignment InChannelAssignment, const FGainTable* InGainTable /*= nullptr*/)
{
	operator=(0.0f);

	if (!InGainTable)
	{
		InGainTable = &FGainTable::Get();
	}

	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::Center:
	case ESpeakerChannelAssignment::LFE:
	case ESpeakerChannelAssignment::LeftFront:
	case ESpeakerChannelAssignment::RightFront:
	case ESpeakerChannelAssignment::LeftSurround:
	case ESpeakerChannelAssignment::RightSurround:
	case ESpeakerChannelAssignment::LeftRear:
	case ESpeakerChannelAssignment::RightRear:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
		InGainTable->GetGainsForDirectAssignment(InChannelAssignment, GainsArray[0]);
		break;
	case ESpeakerChannelAssignment::FrontPair:
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::LeftFront, GainsArray[0]);
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::RightFront, GainsArray[1]);
		break;
	case ESpeakerChannelAssignment::CenterAndLFE:
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::Center, GainsArray[0]);
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::LFE, GainsArray[1]);
		break;
	case ESpeakerChannelAssignment::SurroundPair:
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::LeftSurround, GainsArray[0]);
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::RightSurround, GainsArray[1]);
		break;
	case ESpeakerChannelAssignment::RearPair:
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::LeftRear, GainsArray[0]);
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::RightRear, GainsArray[1]);
		break;
	case ESpeakerChannelAssignment::AmbisonicWXPair:
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::AmbisonicW, GainsArray[0]);
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::AmbisonicX, GainsArray[1]);
		break;
	case ESpeakerChannelAssignment::AmbisonicYZPair:
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::AmbisonicY, GainsArray[0]);
		InGainTable->GetGainsForDirectAssignment(ESpeakerChannelAssignment::AmbisonicZ, GainsArray[1]);
		break;
	}
}

void FGainMatrix::Set(float InGain, float InPan, const FGainTable* InGainTable /*= nullptr*/)
{
	if (!InGainTable)
	{
		InGainTable = &FGainTable::Get();
	}

	if (NumInChannels == 1 && ChannelMask == ESpeakerMask::UnspecifiedMono)
	{
		GainsArray[0] = InGainTable->GetGains(InPan);
	}
	else
	{
		for (int32 Idx = 0, MaskWalk = 0; Idx < NumInChannels; ++Idx)
		{
			while (MaskWalk < FAudioBuffer::kMaxChannelsInAudioBuffer)
			{
				if (ChannelMask & (1L << MaskWalk))
				{
					break;
				}
				++MaskWalk;
			}
			if (MaskWalk < FAudioBuffer::kMaxChannelsInAudioBuffer)
			{
				// When panning multichannel images we swing the entire represented sound
				// field around by the pan. This next step add the channel's 'standard' azimuth
				// to the pan. For mono sources this does nothing. 
				float ChPan = InPan + InGainTable->GetDirectChannelAzimuthInCurrentLayout(ESpeakerChannelAssignment(MaskWalk));
				MaskWalk++;
				// Now we can use the GainTable to get the normalized pan gains...
				GainsArray[Idx] = InGainTable->GetGains(ChPan);
				// Now multiply by gain to get the final result.
			}
		}
	}

	operator*=(InGain);
}

void FGainMatrix::Set(float InGain, const FPannerDetails& InPannerDetails, const FGainTable* InGainTable /*= nullptr*/)
{
	switch (InPannerDetails.Mode)
	{
	case EPannerMode::LegacyStereo:
	case EPannerMode::Stereo:
		// TODO: this should do something different!
		SetFromLegacyStereo(InGain, InPannerDetails.Detail.Pan);
		break;
	case EPannerMode::Surround:
		SetFromMinusOneToOneSurroundPan(InGain, InPannerDetails.Detail.Pan, InGainTable);
		break;
	case EPannerMode::PolarSurround:
		SetFromPolarDegrees(InGain, InPannerDetails.Detail.Pan, InGainTable);
		break;
	case EPannerMode::DirectAssignment:
		Set(InGain, InPannerDetails.Detail.ChannelAssignment, InGainTable);
		break;
	default:
		UE_LOGF(LogGainMatrix, Warning, "Unrecogized Pan mode!");
		SetFromMinusOneToOneSurroundPan(InGain, 0.0f, InGainTable);
		break;
	}
}


bool FGainMatrix::SpeakerChannelAssignmentMapsToOneSpeaker(ESpeakerChannelAssignment InChannelAssignment, EAudioBufferChannelLayout InChannelLayout)
{
	switch (InChannelLayout)
	{
	case EAudioBufferChannelLayout::Raw:
		return true;
	case EAudioBufferChannelLayout::Mono:
		return true;
	case EAudioBufferChannelLayout::Stereo:
	case EAudioBufferChannelLayout::Quad:
		return InChannelAssignment != ESpeakerChannelAssignment::Center &&
			   InChannelAssignment != ESpeakerChannelAssignment::LFE;
	case EAudioBufferChannelLayout::StereoPointOne:
	case EAudioBufferChannelLayout::QuadPointOne:
		return InChannelAssignment != ESpeakerChannelAssignment::Center &&
			   InChannelAssignment != ESpeakerChannelAssignment::LeftRear &&
			   InChannelAssignment != ESpeakerChannelAssignment::RightRear;
	case EAudioBufferChannelLayout::Five:
		return InChannelAssignment != ESpeakerChannelAssignment::LFE &&
			   InChannelAssignment != ESpeakerChannelAssignment::LeftRear &&
			   InChannelAssignment != ESpeakerChannelAssignment::RightRear;
	case EAudioBufferChannelLayout::FivePointOne:
		return InChannelAssignment != ESpeakerChannelAssignment::LeftRear &&
			InChannelAssignment != ESpeakerChannelAssignment::RightRear;
	case EAudioBufferChannelLayout::SevenPointOne:
		return true;
	case EAudioBufferChannelLayout::AmbisonicOrderOne:
		return true;
	case EAudioBufferChannelLayout::Num:
	case EAudioBufferChannelLayout::UnsupportedFormat:
	default:
		UE_LOGF(LogGainMatrix, Warning, "Unrecognized speaker channel enum value! returning true!");
		return true;
	}
}

int32 FGainMatrix::SpeakerChannelAssignmentToChannelIndex(ESpeakerChannelAssignment InChannelAssignment, EAudioBufferChannelLayout InChannelLayout)
{
	switch (InChannelLayout)
	{
	case EAudioBufferChannelLayout::Raw:
		return (int32)InChannelAssignment;
	case EAudioBufferChannelLayout::Mono:
		return 0;
	case EAudioBufferChannelLayout::Stereo:
		return SpeakerToStereo(InChannelAssignment);
	case EAudioBufferChannelLayout::StereoPointOne:
		return SpeakerToStereoPointOne(InChannelAssignment);
	case EAudioBufferChannelLayout::Quad:
		return SpeakerToQuad(InChannelAssignment);
	case EAudioBufferChannelLayout::QuadPointOne:
		return SpeakerToQuadPointOne(InChannelAssignment);
	case EAudioBufferChannelLayout::Five:
		return SpeakerToFive(InChannelAssignment);
	case EAudioBufferChannelLayout::FivePointOne:
		return SpeakerToFivePointOne(InChannelAssignment);
	case EAudioBufferChannelLayout::SevenPointOne:
		return SpeakerToSevenPointOne(InChannelAssignment);
	case EAudioBufferChannelLayout::AmbisonicOrderOne:
		return (int32)InChannelAssignment - (int32)ESpeakerChannelAssignment::AmbisonicW;
	case EAudioBufferChannelLayout::Num:
	case EAudioBufferChannelLayout::UnsupportedFormat:
	default:
		UE_LOGF(LogGainMatrix, Warning, "Unrecognized speaker channel enum value! returning index 0!");
		return 0;
	}
}

int32 FGainMatrix::SpeakerToStereo(ESpeakerChannelAssignment InChannelAssignment)
{
	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::LeftFront:
	case ESpeakerChannelAssignment::LeftSurround:
	case ESpeakerChannelAssignment::LeftRear:
		return (int32)ESpeakerChannelAssignment::LeftFront;

	case ESpeakerChannelAssignment::RightFront:
	case ESpeakerChannelAssignment::RightSurround:
	case ESpeakerChannelAssignment::RightRear:
		return (int32)ESpeakerChannelAssignment::RightFront;

	case ESpeakerChannelAssignment::Center:
	case ESpeakerChannelAssignment::LFE:
	case ESpeakerChannelAssignment::FrontPair:
	case ESpeakerChannelAssignment::CenterAndLFE:
	case ESpeakerChannelAssignment::SurroundPair:
	case ESpeakerChannelAssignment::RearPair:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
	case ESpeakerChannelAssignment::AmbisonicWXPair:
	case ESpeakerChannelAssignment::AmbisonicYZPair:
	case ESpeakerChannelAssignment::Num:
	case ESpeakerChannelAssignment::Invalid:
		// const FString ChannelName = StaticEnum<ESpeakerChannelAssignment>()->GetNameStringByValue((int64)InChannelAssignment);
		UE_LOGF(LogGainMatrix, Error, "Invalid channel assignment!");
		checkNoEntry();
	}
	return -1;
}

int32 FGainMatrix::SpeakerToStereoPointOne(ESpeakerChannelAssignment InChannelAssignment)
{
	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::LeftFront:
	case ESpeakerChannelAssignment::LeftSurround:
	case ESpeakerChannelAssignment::LeftRear:
		return (int32)ESpeakerChannelAssignment::LeftFront;

	case ESpeakerChannelAssignment::RightFront:
	case ESpeakerChannelAssignment::RightSurround:
	case ESpeakerChannelAssignment::RightRear:
		return (int32)ESpeakerChannelAssignment::RightFront;

	case ESpeakerChannelAssignment::LFE:
		return (int32)ESpeakerChannelAssignment::LFE;

	case ESpeakerChannelAssignment::Center:
	case ESpeakerChannelAssignment::FrontPair:
	case ESpeakerChannelAssignment::CenterAndLFE:
	case ESpeakerChannelAssignment::SurroundPair:
	case ESpeakerChannelAssignment::RearPair:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
	case ESpeakerChannelAssignment::AmbisonicWXPair:
	case ESpeakerChannelAssignment::AmbisonicYZPair:
	case ESpeakerChannelAssignment::Num:
	case ESpeakerChannelAssignment::Invalid:
		// const FString ChannelName = StaticEnum<ESpeakerChannelAssignment>()->GetNameStringByValue((int64)InChannelAssignment);
		UE_LOGF(LogGainMatrix, Error, "Invalid channel assignment!");
		checkNoEntry();
	}
	return -1;
}

int32 FGainMatrix::SpeakerToQuad(ESpeakerChannelAssignment InChannelAssignment)
{
	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::LeftFront:
		return (int32)ESpeakerChannelAssignment::LeftFront;

	case ESpeakerChannelAssignment::LeftSurround:
	case ESpeakerChannelAssignment::LeftRear:
		return (int32)ESpeakerChannelAssignment::LeftSurround;

	case ESpeakerChannelAssignment::RightFront:
		return (int32)ESpeakerChannelAssignment::RightFront;

	case ESpeakerChannelAssignment::RightSurround:
	case ESpeakerChannelAssignment::RightRear:
		return (int32)ESpeakerChannelAssignment::RightSurround;

	case ESpeakerChannelAssignment::Center:
	case ESpeakerChannelAssignment::LFE:
	case ESpeakerChannelAssignment::FrontPair:
	case ESpeakerChannelAssignment::CenterAndLFE:
	case ESpeakerChannelAssignment::SurroundPair:
	case ESpeakerChannelAssignment::RearPair:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
	case ESpeakerChannelAssignment::AmbisonicWXPair:
	case ESpeakerChannelAssignment::AmbisonicYZPair:
	case ESpeakerChannelAssignment::Num:
	case ESpeakerChannelAssignment::Invalid:
		// const FString ChannelName = StaticEnum<ESpeakerChannelAssignment>()->GetNameStringByValue((int64)InChannelAssignment);
		UE_LOGF(LogGainMatrix, Error, "Invalid channel assignment!");
		checkNoEntry();
	}
	return -1;
}

int32 FGainMatrix::SpeakerToQuadPointOne(ESpeakerChannelAssignment InChannelAssignment)
{
	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::LeftFront:
		return (int32)ESpeakerChannelAssignment::LeftFront;

	case ESpeakerChannelAssignment::LeftSurround:
	case ESpeakerChannelAssignment::LeftRear:
		return (int32)ESpeakerChannelAssignment::LeftSurround;

	case ESpeakerChannelAssignment::RightFront:
		return (int32)ESpeakerChannelAssignment::RightFront;

	case ESpeakerChannelAssignment::RightSurround:
	case ESpeakerChannelAssignment::RightRear:
		return (int32)ESpeakerChannelAssignment::RightSurround;

	case ESpeakerChannelAssignment::LFE:
		return (int32)ESpeakerChannelAssignment::LFE;

	case ESpeakerChannelAssignment::Center:
	case ESpeakerChannelAssignment::FrontPair:
	case ESpeakerChannelAssignment::CenterAndLFE:
	case ESpeakerChannelAssignment::SurroundPair:
	case ESpeakerChannelAssignment::RearPair:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
	case ESpeakerChannelAssignment::AmbisonicWXPair:
	case ESpeakerChannelAssignment::AmbisonicYZPair:
	case ESpeakerChannelAssignment::Num:
	case ESpeakerChannelAssignment::Invalid:
		// const FString ChannelName = StaticEnum<ESpeakerChannelAssignment>()->GetNameStringByValue((int64)InChannelAssignment);
		UE_LOGF(LogGainMatrix, Error, "Invalid channel assignment!");
		checkNoEntry();
	}
	return -1;
}

int32 FGainMatrix::SpeakerToFive(ESpeakerChannelAssignment InChannelAssignment)
{
	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::LeftFront:
		return (int32)ESpeakerChannelAssignment::LeftFront;

	case ESpeakerChannelAssignment::LeftSurround:
	case ESpeakerChannelAssignment::LeftRear:
		return (int32)ESpeakerChannelAssignment::LeftSurround;

	case ESpeakerChannelAssignment::RightFront:
		return (int32)ESpeakerChannelAssignment::RightFront;

	case ESpeakerChannelAssignment::RightSurround:
	case ESpeakerChannelAssignment::RightRear:
		return (int32)ESpeakerChannelAssignment::RightSurround;

	case ESpeakerChannelAssignment::Center:
		return (int32)ESpeakerChannelAssignment::Center;

	case ESpeakerChannelAssignment::LFE:
	case ESpeakerChannelAssignment::FrontPair:
	case ESpeakerChannelAssignment::CenterAndLFE:
	case ESpeakerChannelAssignment::SurroundPair:
	case ESpeakerChannelAssignment::RearPair:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
	case ESpeakerChannelAssignment::AmbisonicWXPair:
	case ESpeakerChannelAssignment::AmbisonicYZPair:
	case ESpeakerChannelAssignment::Num:
	case ESpeakerChannelAssignment::Invalid:
		//const FString ChannelName = StaticEnum<ESpeakerChannelAssignment>()->GetNameStringByValue((int64)InChannelAssignment);
		UE_LOGF(LogGainMatrix, Error, "Invalid channel assignment!");
		checkNoEntry();
	}
	return -1;
}

int32 FGainMatrix::SpeakerToFivePointOne(ESpeakerChannelAssignment InChannelAssignment)
{
	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::LeftFront:
		return (int32)ESpeakerChannelAssignment::LeftFront;

	case ESpeakerChannelAssignment::LeftSurround:
	case ESpeakerChannelAssignment::LeftRear:
		return (int32)ESpeakerChannelAssignment::LeftSurround;

	case ESpeakerChannelAssignment::RightFront:
		return (int32)ESpeakerChannelAssignment::RightFront;

	case ESpeakerChannelAssignment::RightSurround:
	case ESpeakerChannelAssignment::RightRear:
		return (int32)ESpeakerChannelAssignment::RightSurround;

	case ESpeakerChannelAssignment::Center:
		return (int32)ESpeakerChannelAssignment::Center;

	case ESpeakerChannelAssignment::LFE:
		return (int32)ESpeakerChannelAssignment::LFE;

	case ESpeakerChannelAssignment::FrontPair:
	case ESpeakerChannelAssignment::CenterAndLFE:
	case ESpeakerChannelAssignment::SurroundPair:
	case ESpeakerChannelAssignment::RearPair:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
	case ESpeakerChannelAssignment::AmbisonicWXPair:
	case ESpeakerChannelAssignment::AmbisonicYZPair:
	case ESpeakerChannelAssignment::Num:
	case ESpeakerChannelAssignment::Invalid:
		//const FString ChannelName = StaticEnum<ESpeakerChannelAssignment>()->GetNameStringByValue((int64)InChannelAssignment);
		UE_LOGF(LogGainMatrix, Error, "Invalid channel assignment!");
		checkNoEntry();
	}
	return -1;
}

int32 FGainMatrix::SpeakerToSevenPointOne(ESpeakerChannelAssignment InChannelAssignment)
{
	switch (InChannelAssignment)
	{
	case ESpeakerChannelAssignment::LeftFront:
		return (int32)ESpeakerChannelAssignment::LeftFront;

	case ESpeakerChannelAssignment::LeftSurround:
		return (int32)ESpeakerChannelAssignment::LeftSurround;

	case ESpeakerChannelAssignment::LeftRear:
		return (int32)ESpeakerChannelAssignment::LeftRear;

	case ESpeakerChannelAssignment::RightFront:
		return (int32)ESpeakerChannelAssignment::RightFront;

	case ESpeakerChannelAssignment::RightSurround:
		return (int32)ESpeakerChannelAssignment::RightSurround;

	case ESpeakerChannelAssignment::RightRear:
		return (int32)ESpeakerChannelAssignment::RightRear;

	case ESpeakerChannelAssignment::Center:
		return (int32)ESpeakerChannelAssignment::Center;

	case ESpeakerChannelAssignment::LFE:
		return (int32)ESpeakerChannelAssignment::LFE;

	case ESpeakerChannelAssignment::FrontPair:
	case ESpeakerChannelAssignment::CenterAndLFE:
	case ESpeakerChannelAssignment::SurroundPair:
	case ESpeakerChannelAssignment::RearPair:
	case ESpeakerChannelAssignment::AmbisonicW:
	case ESpeakerChannelAssignment::AmbisonicX:
	case ESpeakerChannelAssignment::AmbisonicY:
	case ESpeakerChannelAssignment::AmbisonicZ:
	case ESpeakerChannelAssignment::AmbisonicWXPair:
	case ESpeakerChannelAssignment::AmbisonicYZPair:
	case ESpeakerChannelAssignment::Num:
	case ESpeakerChannelAssignment::Invalid:
		//const FString ChannelName = StaticEnum<ESpeakerChannelAssignment>()->GetNameStringByValue((int64)InChannelAssignment);
		UE_LOGF(LogGainMatrix, Error, "Invalid channel assignment!");
		checkNoEntry();
	}
	return -1;
}
