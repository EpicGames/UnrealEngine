// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/GainTable.h"
#include "Math/UnrealMathUtility.h"
#include "HarmonixDsp/AudioUtility.h"

DEFINE_LOG_CATEGORY(LogGainTable);

FGainTable* FGainTable::gGainTable = nullptr;

void FGainTable::Init(EAudioBufferChannelLayout ChannelLayout)
{
	if (gGainTable)
		return;

	gGainTable = new FGainTable();
	check(gGainTable);
	gGainTable->SetChannelLayout(ChannelLayout);
}

void FGainTable::SetupPrimaryGainTable(EAudioBufferChannelLayout ChannelLayout)
{
	gGainTable->SetChannelLayout(ChannelLayout);
}

FGainTable::FGainTable()
{
	SetChannelLayout(EAudioBufferChannelLayout::Stereo);
}

// These arrays represent indices that will be used to index into 
// `kDefaultSpeakerAzimuths`. The number of elements in `kDefaultSpeaukerAzimuths`
// is defined by `EAudioBufferChannelLayout::Num`, which is of type uint8.
// It probably makes more sense for these to be arrays of `EAudioBufferChannelLayout` types
// but for now we can just use `uint8` safely here
const uint8 kStereoRemap[] = { 0, 1 };
const uint8 kQuadRemap[] = { 0, 4, 5, 1 };
const uint8 kFiveRemap[] = { 0, 4, 5, 1 };

// 2 CENTER, 3 is LFE
const uint8 kSevenDotOneRemap[] = { 0, 4, 6, 7, 5, 1 }; 

void FGainTable::SetChannelLayout(EAudioBufferChannelLayout ChannelLayout)
{
	using namespace HarmonixDsp;

	if (ChannelLayout == CurrentLayout)
	{
		return;
	}

	CurrentLayout = ChannelLayout;
	FMemory::Memset(&Entries[0], 0, sizeof(FChannelGains) * kGainTableSize);

	switch (ChannelLayout)
	{
	case EAudioBufferChannelLayout::Raw:
		UE_LOGF(LogGainTable, Error, "Can't pan across 'Raw' channel ChannelLayout!");
		checkNoEntry();
		break;
	case EAudioBufferChannelLayout::Mono:
	{
		SpeakerCount = 1;
		for (uint32 i = 0; i < kGainTableSize; ++i)
		{
			Entries[i].f[(uint8)ESpeakerChannelAssignment::Center] = 1.0f;
		}
	}
	break;
	case EAudioBufferChannelLayout::AmbisonicOrderOne:
		UE_LOGF(LogGainTable, Warning, "Can't currently pan across 'AmbisonicOrderOne' channel ChannelLayout. Stereo panning will be used. Direct mapped ambisonic channels WILL be preserved.");
	case EAudioBufferChannelLayout::Stereo:
	case EAudioBufferChannelLayout::StereoPointOne:
		BuildPanEntriesFromPannableSpeakerAzimuths(2, FAudioBuffer::kDefaultSpeakerAzimuths[(uint8)EAudioBufferChannelLayout::Stereo], kStereoRemap);
		break;
	case EAudioBufferChannelLayout::Quad:
	case EAudioBufferChannelLayout::QuadPointOne:
		BuildPanEntriesFromPannableSpeakerAzimuths(4, FAudioBuffer::kDefaultSpeakerAzimuths[(uint8)EAudioBufferChannelLayout::Quad], kQuadRemap);
		break;
	case EAudioBufferChannelLayout::Five:
	case EAudioBufferChannelLayout::FivePointOne:
		BuildPanEntriesFromPannableSpeakerAzimuths(4, FAudioBuffer::kDefaultSpeakerAzimuths[(uint8)EAudioBufferChannelLayout::Five], kFiveRemap);
		break;
	case EAudioBufferChannelLayout::SevenPointOne:
		BuildPanEntriesFromPannableSpeakerAzimuths(6, FAudioBuffer::kDefaultSpeakerAzimuths[(uint8)EAudioBufferChannelLayout::SevenPointOne], kSevenDotOneRemap);
		break;
	case EAudioBufferChannelLayout::Num:
	case EAudioBufferChannelLayout::UnsupportedFormat:
	default:
		UE_LOGF(LogGainTable, Error, "Unrecognized channel ChannelLayout!");
		checkNoEntry();
		break;
	}
}

void FGainTable::BuildPanEntriesFromPannableSpeakerAzimuths(int32 NumSpeakers, const float* SpeakerAzimuths, const uint8* InRemap)
{
	for (int32 Idx = 0; Idx < kGainTableSize; ++Idx)
	{
		float CurrentRadian = ((float)Idx * (2.0f * (float)UE_PI / (float)kGainTableSize));
		
		if (CurrentRadian > (2.0f * (float)UE_PI))
		{
			CurrentRadian -= (2.0f * (float)UE_PI);
		}

		int32 SpeakerOne = 0;
		while (SpeakerOne < NumSpeakers && CurrentRadian >  SpeakerAzimuths[InRemap[SpeakerOne]])
		{
			SpeakerOne++;
		}

		if (SpeakerOne == NumSpeakers)
		{
			SpeakerOne = 0;
		}
	
		int32 SpeakerTwo = SpeakerOne - 1;
		if (SpeakerTwo < 0)
		{
			SpeakerTwo = NumSpeakers - 1;
		}

		float RadianOne = SpeakerAzimuths[InRemap[SpeakerOne]];
		float RadianTwo = SpeakerAzimuths[InRemap[SpeakerTwo]];

		if (RadianTwo > RadianOne)
		{
			RadianTwo -= (UE_PI * 2.0f);
			if (CurrentRadian > RadianOne)
			{
				CurrentRadian -= (UE_PI * 2.0f);
			}
		}
		if (RadianTwo < 0.0f)
		{
			RadianOne += (RadianTwo * -1.0f);
			CurrentRadian += (RadianTwo * -1.0f);
		}
		else
		{
			RadianOne -= RadianTwo;
			CurrentRadian -= RadianTwo;
		}

		float PanA = FMath::Sin((CurrentRadian / RadianOne) * ((float)UE_HALF_PI));
		float PanB = FMath::Cos((CurrentRadian / RadianOne) * ((float)UE_HALF_PI));

		if (PanA < HarmonixDsp::kTinyGain) 
		{
			PanA = 0.0f;
		}
		
		if (PanB < HarmonixDsp::kTinyGain) 
		{
			PanB = 0.0f;
		}

		Entries[Idx].f[InRemap[SpeakerOne]] = PanA;
		Entries[Idx].f[InRemap[SpeakerTwo]] = PanB;
	}
}

bool FGainTable::CurrentLayoutHasSpeaker(ESpeakerChannelAssignment ChannelAssignment) const
{
	using namespace HarmonixDsp;
	return (FAudioBuffer::kLayoutHasSpeakerMasks[(uint8)CurrentLayout] & (1L << (uint8)ChannelAssignment)) != 0;
}