// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Generate.h"
#include "HarmonixDsp/AudioUtility.h"

namespace HarmonixDsp
{

	void GenerateA220Sawtooth(float* Output, uint32 NumFrames, uint32 SampleRate)
	{
		static float sPhase = 0.0f;
		static float sCyclesPerSample = 220.0f / (float)SampleRate;
		static float sPhaseIncrementPerSample = 2 * sCyclesPerSample;

		float Phase = sPhase;
		for (uint32 FrameNum = 0; FrameNum < NumFrames; ++FrameNum)
		{
			Output[FrameNum] = Phase;
			Phase += sPhaseIncrementPerSample;
			if (Phase >= 1.0)
			{
				Phase -= 2.0;
			}
		}

		sPhase = Phase;
	}

	void GenerateWhiteNoise(float* Output, uint32 NumFrames, float Gain /*= 1.0f*/)
	{
		for (uint32 FrameNum = 0; FrameNum < NumFrames; ++FrameNum)
		{
			Output[FrameNum] = Gain * HarmonixDsp::FRandSample();
		}
	}

	void GenerateWhiteNoise(int16* Output, uint32 NumFrames, float Gain /*= 1.0f*/)
	{
		for (uint32 FrameNum = 0; FrameNum < NumFrames; ++FrameNum)
		{
			Output[FrameNum] = (int16)(Gain * HarmonixDsp::FRandSample() * (float)MAX_int16);
		}
	}

	void GenerateSine(float* Output, const uint32 NumFrames, const float Frequency, const float SampleRate, float& Phase)
	{
		const float PhaseInc = Frequency / SampleRate;
		Phase = FMath::Fmod(Phase, 1.0f);
		
		for (uint32 FrameNum = 0; FrameNum < NumFrames; ++FrameNum)
		{
			const float Sample = FMath::Sin(Phase * UE_TWO_PI);
			Output[FrameNum] = Sample;
			Phase = FMath::Fmod(Phase + PhaseInc, 1.0f);
		}
	}
}