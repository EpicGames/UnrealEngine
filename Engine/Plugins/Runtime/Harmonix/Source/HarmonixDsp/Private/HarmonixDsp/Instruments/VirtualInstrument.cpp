// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Instruments/VirtualInstrument.h"

#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/MidiMsg.h"

bool FVirtualInstrument::bShouldPrintMidiActivity = false;

void FVirtualInstrument::ResetInstrumentState()
{
	FMemory::Memset(LastCcVals, -1, 16 * 128);
	ResetInstrumentStateImpl();
}

void FVirtualInstrument::ResetMidiState()
{
	FMemory::Memset(LastCcVals, -1, 16 * 128);
	ResetMidiStateImpl();
}

void FVirtualInstrument::Set7BitController(Harmonix::Midi::Constants::EControllerID InController, int8 InByteValue, int8 InMidiChannel)
{
	LastCcVals[InMidiChannel][(uint8)InController] = InByteValue;
	Set7BitControllerImpl(InController, InByteValue, InMidiChannel);
}

void FVirtualInstrument::Set14BitController(Harmonix::Midi::Constants::EControllerID InController, int16 InByteValue, int8 InMidiChannel)
{
	int32 LsbIdx;
	int32 MsbIdx;
	if (GetMsbLsbIndexes(InController, MsbIdx, LsbIdx))
	{
		LastCcVals[InMidiChannel][MsbIdx] = (InByteValue >> 7) & 0x7F;
		LastCcVals[InMidiChannel][LsbIdx] = InByteValue & 0x7F;
		Set14BitControllerImpl((Harmonix::Midi::Constants::EControllerID)MsbIdx, InByteValue, InMidiChannel);
	}
}

void FVirtualInstrument::HandleMidiMessage(FMidiVoiceId InVoiceId, int8 InStatus, int8 InData1, int8 InData2, int32 InEventTick, int32 InCurrentTick, float InMsOffset)
{
	using namespace Harmonix::Midi::Constants;
	int8 InChannel = InStatus & 0xF;
	switch (InStatus & 0xF0)
	{
	case GNoteOff:
		NoteOff(InVoiceId, InData1, InChannel);
		break;
	case GNoteOn:
		NoteOn(InVoiceId, InData1, InData2, InChannel, InEventTick, InCurrentTick, InMsOffset);
		break;
	case GPolyPres:
		PolyPressure(InVoiceId, InData1, InData2, InChannel);
		break;
	case GChanPres:
		ChannelPressure(InData1, InData2, InChannel);
		break;
	case GControl:
		SetHighOrLowControllerByte((EControllerID)InData1, InData2, InChannel);
		break;
	case GPitch:
		SetPitchBend(FMidiMsg::GetPitchBendFromData(InData1, InData2), InChannel);
		break;
	}
}

bool FVirtualInstrument::IsHighResController(Harmonix::Midi::Constants::EControllerID InControllerId, bool& bIsHighResLowByte)
{
	uint8 Id = (uint8)InControllerId;
	if (Id < 32)
	{
		bIsHighResLowByte = false;
		return true;
	}
	if (Id < 64)
	{
		bIsHighResLowByte = true;
		return true;
	}
	if (Id == 98 || Id == 100)
	{
		bIsHighResLowByte = true;
		return true;
	}
	if (Id == 99 || Id == 101)
	{
		bIsHighResLowByte = false;
		return true;
	}
	return false;
}

bool FVirtualInstrument::GetMsbLsbIndexes(Harmonix::Midi::Constants::EControllerID InControllerId, int& InMsb, int& InLsb)
{
	uint8 Id = (uint8)InControllerId;
	if (Id < 32)
	{
		InMsb = Id; InLsb = Id + 32;
		return true;
	}
	if (Id < 64)
	{
		InMsb = Id - 32; InLsb = Id;
		return true;
	}
	if (Id == 98 || Id == 100)
	{
		InMsb = Id + 1; InLsb = Id;
		return true;
	}
	if (Id == 99 || Id == 101)
	{
		InMsb = Id; InLsb = Id - 1;
		return true;
	}
	return false;
}

// convert a value in [0,127] to the range expected by SetController
float FVirtualInstrument::ConvertCCValue(Harmonix::Midi::Constants::EControllerID InController, uint8 InValue)
{
	using namespace Harmonix::Midi::Constants;
	check(InValue <= 127);
	switch (InController)
	{
	case EControllerID::PanRight:
	case EControllerID::CoarsePitchBend:
		// map to [-1, 1], making sure 64 maps exactly to 0
		return FMath::Max(-1.0f, ((float)InValue - 64) / 63);
	case EControllerID::PortamentoSwitch:
		// 63 and below are "off", 64 and above are "on"
		return InValue >= 64 ? 1.0f : 0.0f;
	default:
		// map to [0, 1]
		return (float)InValue / 127;
	}
}

void FVirtualInstrument::SetController(Harmonix::Midi::Constants::EControllerID InController, float InValue, int8 InMidiChannel /*= 0*/)
{
	int32 MsbIndex;
	int32 LsbIndex;
	if (GetMsbLsbIndexes(InController, MsbIndex, LsbIndex))
	{
		Set14BitController(InController, (int16)(InValue * 16383.0f), InMidiChannel);
	}
	else
	{
		Set7BitController(InController, (int8)(InValue * 127.0f), InMidiChannel);
	}
}

void FVirtualInstrument::SetHighOrLowControllerByte(Harmonix::Midi::Constants::EControllerID InController, int8 InValue, int8 InMidiChannel)
{
	LastCcVals[InMidiChannel][(uint8)InController] = InValue;
	int32 HighByteIdx;
	int32 LowByteIdx;
	if (GetMsbLsbIndexes(InController, HighByteIdx, LowByteIdx))
	{
		if (LastCcVals[InMidiChannel][LowByteIdx] == -1)
		{
			Set7BitController(InController, InValue, InMidiChannel);
		}
		else if (LastCcVals[InMidiChannel][HighByteIdx] != -1)
		{
			Set14BitController((Harmonix::Midi::Constants::EControllerID)HighByteIdx, (LastCcVals[InMidiChannel][HighByteIdx] << 7 | LastCcVals[InMidiChannel][LowByteIdx]), InMidiChannel);
		}
	}
	else
	{
		Set7BitController(InController, InValue, InMidiChannel);
	}
}
