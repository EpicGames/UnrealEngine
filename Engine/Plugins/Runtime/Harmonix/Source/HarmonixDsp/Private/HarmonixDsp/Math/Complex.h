// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/UnrealMath.h"

template<typename T>
class TComplex
{
	static_assert(std::is_floating_point_v<T>, "T must be a floating point type");
public:
	TComplex() : Real(0), Imag(0) {}

	// X is real part
	// Y is imaginary part
	TComplex(T X, T Y) : Real(X), Imag(Y) {}

	// X is real part
	// imaginary part is 0
	TComplex(T X) : Real(X), Imag(0) {}

	TComplex operator*(const TComplex& Z) const
	{
		return TComplex(Real * Z.Real - Imag * Z.Imag, Real * Z.Imag + Imag * Z.Real);
	}

	TComplex operator*(T S) const
	{
		return (*this) * TComplex<T>(S, 0);
	}

	TComplex operator+(const TComplex& Z) const
	{
		return TComplex(Real + Z.Real, Imag + Z.Imag);
	}

	TComplex operator-(const TComplex& Z) const
	{
		return TComplex(Real - Z.Real, Imag - Z.Imag);
	}

	TComplex operator-() const
	{
		return (*this) * T(-1);
	}

	TComplex operator/(const TComplex& Z) const
	{
		return (*this) * Z.Inverse();
	}

	TComplex& SetRect(T X, T Y) 
	{ 
		Real = X; 
		Imag = Y; 
		return *this; 
	}

	TComplex& SetPolar(T R, T Theta)
	{
		return SetRect(R * FMath::Cos(Theta), R * FMath::Sin(Theta));
	}

	TComplex Conjugate() const
	{
		return TComplex(Real, -Imag);
	}

	TComplex Inverse() const
	{
		T NewMagnitude = T(1) / Magnitude();
		T NewAngle = -Angle();
		return TComplex<T>().SetPolar(NewMagnitude, NewAngle);
	}

	TComplex Sqrt() const
	{
		// computes the principle square root
		T Mag = Magnitude();

		// gamma is always positive
		T Gamma = FMath::Sqrt((Mag + Real) * T(0.5));

		// delta has the same sign as the imaginary part
		T Delta = FMath::Sqrt((Mag - Real) * T(0.5));

		if (Imag < 0)
		{
			Delta *= T(-1.0);
		}

		return TComplex<T>(Gamma, Delta);
	}

	T GetReal() const { return Real; }
	T GetImag() const { return Imag; }
	T X() const { return Real; }
	T Y() const { return Imag; }

	T Magnitude() const { return FMath::Sqrt(Real * Real + Imag * Imag); }
	T Angle() const
	{
		// no quadrant case
		if (Magnitude() == T(0))
		{
			return 0;
		}

		// 1st and 4th quadrants
		if (Real >= T(0))
		{
			return FMath::Atan(Imag / Real);
		}

		// 2nd quadrant
		if (Imag >= T(0))
		{
			return UE_PI + FMath::Atan(Imag / Real);
		}

		// 3rd quadrant
		return FMath::Atan(Imag / Real) - UE_PI;
	}

	bool operator==(const TComplex& Other) const
	{
		return Real == Other.Real && Imag == Other.Imag;
	}

	bool operator !=(const TComplex& Other) const
	{
		return !(operator==(Other));
	}

private:

	T Real;
	T Imag;
};

template<typename T>
TComplex<T> operator*(T S, const TComplex<T>& Z)
{
	return Z * S;
}

