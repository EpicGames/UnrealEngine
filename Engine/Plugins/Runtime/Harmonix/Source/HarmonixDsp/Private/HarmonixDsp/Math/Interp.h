// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/UnrealMathUtility.h"
#include "HAL/PlatformMath.h"

class FInterp
{
public:
	float operator()(float Alpha) { return Eval(Alpha); }

	// returns the interpolation result based on a value between 0 and 1
	// accepts values outside of the range 0 and 1 and will extrapolate
	virtual float Eval(float Alpha) const = 0;

	// returns the inverse of the interpolation given a value between the Min and Max range
	// accepts values outside of the Min and Max range and will extrapolate
	virtual float Inverse(float Value) const = 0;

	// clamps the result to the Min and Max value of the range 
	// for values of Alpha is outside the range 0 and 1
	virtual float EvalClamped(float Alpha) const
	{
		return FMath::Clamp(Eval(Alpha), MinValue, MaxValue);
	}

	// clamps the result between 0 and 1
	// for values outside the Min and Max range
	virtual float InverseClamped(float Value) const
	{
		return FMath::Clamp(Inverse(Value), MinAlpha, MaxAlpha);
	}

protected:
	float MinValue = 0.0f;
	float MaxValue = 0.0f;
	float MinAlpha = 0.0f;
	float MaxAlpha = 1.0f;
};

class FLerp : public FInterp
{
public:

	FLerp(float A, float B)
	{
		MinValue = A;
		MaxValue = B;
	}

	virtual float Eval(float Alpha) const override
	{
		return FMath::Lerp(MinValue, MaxValue, Alpha);
	}

	virtual float Inverse(float Value) const override
	{
		return FMath::GetRangePct<float>(MinValue, MaxValue, Value);
	}
};

// easing function = Pow(Alpha, Exp) (B - A) + A
// previously known as ExpInterpolate
// changed names to correspond to UE version
class FInterpEaseIn : public FInterp
{
public:

	FInterpEaseIn(float A, float B, float InExp)
	{
		MinValue = A;
		MaxValue = B;
		Exp = InExp;
	}

	virtual float Eval(float Alpha) const
	{
		// just use the Unreal implementation
		return FMath::InterpEaseIn(MinValue, MaxValue, Alpha, Exp);
	}

	virtual float Inverse(float Value) const
	{
		// perform inverse ease in function
		float Alpha = FMath::Pow((Value - MinValue) / (MaxValue - MinValue), 1.0f / Exp);
		return Alpha;
	}

private:

	float Exp;
};