// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/Math/Complex.h"
#include "HAL/PlatformMath.h"


/**
 * Computes the roots of a quadratic polynomial.
 * Given a quadratic polynomial in the form:
 *
 *    a*x^2 + b*x + c
 *
 * this function solves for the two values of x (real or complex)
 * that makes the polynomial evaluate to zero.
 * @param InA the coefficient for the quadratic term
 * @param InB the coefficient for the linear term
 * @param InC the coefficient for the constant term
 * @param OutX1 one root of the polynomial
 * @param OutX2 the other root of the polynomial
 * @returns the number of valid roots this quadratic has. zero if InA==0 and InB==0. one if InA==0.
 */

namespace FPolynomial
{
	inline int32 ComputeRootsOfQuadratic(TComplex<double> InA, TComplex<double> InB, TComplex<double> InC,
		TComplex<double>& OutX1, TComplex<double>& OutX2)
	{
		static const TComplex<double> sZero(0, 0);
		static const TComplex<double> sOne(1, 0);

		OutX1.SetRect(0, 0);
		OutX2.SetRect(0, 0);

		if (InA == sZero && InB == sZero)
		{
			return 0;
		}

		if (InA == sZero)
		{
			// this is linear.
			// just one root.
			OutX1 = -InC / InB;
			return 1;
		}

		/* this function uses the quadratic equation:

			   -InB ± sqrt( InB^2 - 4*InA*InC )
		   x = ------------------------
						  2*InA
		*/

		TComplex<double> OneOver2A = sOne / (2.0 * InA);

		// compute the discriminant
		TComplex<double> Discr = (InB * InB) - (4.0 * InA * InC);
		if (Discr == sZero)
		{
			OutX1 = -InB * OneOver2A;
			return 1; // only one distinct root in this case
		}

		// and now the two roots
		TComplex<double> SqrtDiscr = Discr.Sqrt();
		OutX1 = (-InB + SqrtDiscr) * OneOver2A;
		OutX2 = (-InB - SqrtDiscr) * OneOver2A;

		return 2; // there are 2 roots
	}
}