// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Modulators/Adsr.h"

namespace Harmonix::Dsp::Modulators
{

float FAdsr::GetValue() const
{
	float Sample = Gain;
	checkSlow(0.0 <= Sample && Sample <= 1.0);

	float Depth = 0;
	if (Settings->IsEnabled)
	{
		Depth = Settings->Depth;
	}

	// apply Depth... Sample still on [0,1]
	switch (Mode)
	{
	case EAdsrMode::MinUp:
		Sample *= Depth;
		break;

	case EAdsrMode::MaxDown:
		Sample = 1.0f - Sample * Depth + Depth;
		break;

	case EAdsrMode::SustainOut:
		Sample = Sample * Depth - 0.5f * (Depth - 1.0f);
		break;
	}

	check(0.0 <= Sample && Sample <= 1.0);

	return Sample;
}

void FAdsr::Advance(uint32 InNumSamples)
{
	check(Settings != nullptr);
	switch (Stage)
	{
	case EAdsrStage::Idle: 
		return;

	case EAdsrStage::Attack:
	{
		Age += InNumSamples;
		CurveAge += (float)InNumSamples;

		if (Settings->IsAttackLinear())
		{
			Gain += InNumSamples * AttackIncrement;
		}
		else
		{
			float TableIdx = CurveAge / (SamplesPerSecond * Settings->AttackTime) * (float)FAdsrSettings::kCurveTableSize;
			if (TableIdx >= (float)FAdsrSettings::kCurveTableSize)
			{
				Gain = 1.0f;
			}
			else
			{
				Gain = Settings->LerpAttackCurve(TableIdx);
			}
		}

		Gain = FMath::Clamp(Gain, 0.0f, 1.0f);
		int32 CurveEndFrame = Settings->AttackTime * SamplesPerSecond;
		if (CurveAge >= CurveEndFrame)
		{
			if (Settings->HasDecayStage())
			{
				Gain = 1.0;
				CurveAge = 0.0f;
				Stage = EAdsrStage::Decay;
				float InNumSamplesFloat = Settings->DecayTime * SamplesPerSecond;
				if (InNumSamplesFloat < 1.0f)
				{
					InNumSamplesFloat = 1.0f;
				}
				DecayDecrement = (1.0f - (Settings->SustainLevel)) / InNumSamplesFloat;
			}
			else
			{
				Gain = Settings->SustainLevel;
				Stage = EAdsrStage::Sustain;
			}
		}
		break;
	}

	case EAdsrStage::Decay:
	{
		Age += InNumSamples;
		CurveAge += InNumSamples;
		if (Settings->IsDecayLinear())
		{
			Gain -= InNumSamples * DecayDecrement;
		}
		else
		{
			float TableIdx = CurveAge / (SamplesPerSecond * Settings->DecayTime) * (float)FAdsrSettings::kCurveTableSize;
			if (TableIdx >= (float)FAdsrSettings::kCurveTableSize)
			{
				Gain = Settings->SustainLevel;
			}
			else
			{
				Gain = Settings->SustainLevel + ((1.0f - Settings->SustainLevel) * Settings->LerpDecayCurve(TableIdx));
			}
		}
		if (Gain <= Settings->SustainLevel)
		{
			Gain = Settings->SustainLevel;
			if (Gain < 0.00008f)
			{
				Kill();
			}
			else
			{
				Stage = EAdsrStage::Sustain;
			}
		}
		break;
	}

	case EAdsrStage::Sustain:
	{
		Age += InNumSamples;
		break;
	}

	case EAdsrStage::Release:
	{
		Age += InNumSamples;
		CurveAge += InNumSamples;
		if (Settings->IsReleaseLinear() || bIsFastRelease)
		{
			Gain -= InNumSamples * ReleaseDecrement;
		}
		else
		{
			float TableIdx = CurveAge / (SamplesPerSecond * Settings->ReleaseTime) * (float)FAdsrSettings::kCurveTableSize;
			if (TableIdx >= (float)FAdsrSettings::kCurveTableSize)
			{
				Gain = 0.0f;
			}
			else
			{
				Gain = ReleaseStartingGain * Settings->LerpReleaseCurve(TableIdx);
			}
		}
		if (Gain <= 0.0f)
		{
			Kill();
		}
		break;
	}
	}
}

}