// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Modulators/Lfo.h"

#include "HAL/PlatformMath.h"

DEFINE_LOG_CATEGORY(LogHarmonixDsp_Lfo);

namespace Harmonix::Dsp::Modulators
{

FWaveShape::FWaveShapeFunction FWaveShape::sWaveShapeTable[(uint8)EWaveShape::Num] =
{
	FWaveShape::Sine,
	FWaveShape::Square,
	FWaveShape::SawtoothUp,
	FWaveShape::SawtoothDown,
	FWaveShape::Triangle,
	FWaveShape::Noise
};

float FWaveShape::Sine(float x)
{
	// use a phase offset to makes the waveform start low (at x = 0)
	// which matches other wave types
	static const float PhaseOffset = 0.75f;

	float Rad = (x + PhaseOffset) * UE_TWO_PI;
	checkSlow(!FGenericPlatformMath::IsNaN(Rad));
	float Result = (x + 0.75f) * (UE_TWO_PI);
	checkSlow(!FGenericPlatformMath::IsNaN(Result));
	Result = FMath::Sin(Result);
	checkSlow(!FGenericPlatformMath::IsNaN(Result));
	Result = 0.5f + (0.5f * Result);

	return Result;
}

float FWaveShape::Square(float X)
{
	if (X > 0.5f)
	{
		X = 1.0f;
	}
	else
	{
		X = 0.0f;
	}

	return X;
}

float FWaveShape::SawtoothUp(float X)
{
	return X;
}

float FWaveShape::SawtoothDown(float X)
{
	return 1.0f - X;
}

float FWaveShape::Triangle(float X)
{
	X *= 2.0f;
	if (X > 1.0f)
	{
		X = 2.0f - X;
	}

	return X;
}

float FWaveShape::Noise(float X)
{
	return FMath::FRand();
}

//------------------------------------------------
FLfo::FLfo()
	: Phase(0.0f)
	, Range(TInterval<float>(0, 1))
	, Mode(ELfoMode::CenterOut)
	, Settings(nullptr)
	, CyclesPerSample(0)
	, SecondsPerSample(1)
	, TempoWhenCalculated(0)
{
	SetPhase(0.0f);
	SetRangeAndMode(TInterval<float>(-1, 1), ELfoMode::CenterOut);
}

void FLfo::UseSettings(const FLfoSettings* InSettings)
{
	Settings = InSettings;
	ComputeCyclesPerSample();
}

void FLfo::ComputeCyclesPerSample()
{
	if (Settings)
	{
		CyclesPerSample = Settings->Freq * SecondsPerSample;
		if (Settings->BeatSync)
		{
			CyclesPerSample *= (Settings->TempoBPM / 60);
		}
		TempoWhenCalculated = Settings->TempoBPM;
	}
}

void FLfo::SetPhase(double InPhase)
{
	check(0.0 <= InPhase && InPhase <= 1.0);
	Phase = InPhase;
}

double FLfo::GetPhase() const
{
	return Phase;
}

void FLfo::Retrigger()
{
	check(Settings);
	float PhaseDegrees = Settings->InitialPhase;
	check(-180 <= PhaseDegrees && PhaseDegrees <= 180);
	float PhaseNormalized = (PhaseDegrees + 180) / 360;
	SetPhase(PhaseNormalized);
}

TInterval<float> FLfo::GetRange() const
{
	return Range;
}

void FLfo::Advance(uint32 InNumSamples)
{
	check(Settings);
	if (TempoWhenCalculated != Settings->TempoBPM && Settings->BeatSync)
	{
		ComputeCyclesPerSample();
	}
	Phase += InNumSamples * CyclesPerSample;

	while (Phase >= 1.0)
	{
		Phase -= 1.0;
	}
}

void FLfo::Prepare(float InSampleRate)
{
	SecondsPerSample = 1.0f / InSampleRate;
	ComputeCyclesPerSample();
};

float FLfo::GetValue() const
{
	check(0.0 <= Phase && Phase <= 1.0);
	check(Settings);

	// tests for misterious crash on Nexus 7 (android 4.4.2 flash)
	// theses tests fail for all release builds with -O2 or higher. -O0 and -O1 work fine

	// test 1 fail
	float Sample = FWaveShape::Apply(Settings->Shape, float(Phase));

	// test 2 fail
	// Sample = sApplyWaveshapingTbl[mSettings->mShape](float(mPhase));

	// test 3 fail android release build
	//    switch (mSettings->mShape)
	//    {
	//    case Settings::kSine:
	//       Sample = Sine(float(mPhase)); break;
	//    case Settings::kSquare:
	//       Sample = Square(float(mPhase)); break;
	//    case Settings::kSawUp:
	//       Sample = SawtoothUp(float(mPhase)); break;
	//    case Settings::kSawDown:
	//       Sample = SawtoothDown(float(mPhase)); break;
	//    case Settings::kTriangle:
	//       Sample = Triangle(float(mPhase)); break;
	//    default:
	//       Sample = Sine(float(mPhase)); break;
	//    }

	// test 4 fail 
	// Sample = Sine((float)mPhase);

	// test 5 ... this seems to work for all android builds.. I have no idea why the others fail
	// Sample = (float)mPhase;

	check(0.0f <= Sample && Sample <= 1.0f);

	float Depth = 0;

	if (!Settings)
	{
		UE_LOGF(LogHarmonixDsp_Lfo, Log, "Setting is null!!");
	}

	if (Settings && Settings->IsEnabled)
	{
		Depth = Settings->Depth;
	}

	switch (Mode)
	{
	case ELfoMode::MinUp:
		Sample *= Depth;
		break;
	case ELfoMode::MaxDown:
		Sample = 1.0f - Sample * Depth + Depth;
		break;
	case ELfoMode::CenterOut:
		Sample = Sample * Depth - 0.5f * (Depth - 1.0f);
		break;
	}

	check(0.0 <= Sample && Sample <= 1.0);

	return Range.Interpolate(Sample);
}

void FLfo::SetRangeAndMode(const TInterval<float>& InRange, ELfoMode InMode)
{
	Range = InRange;
	Mode = InMode;
}

}