// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Modulators/Modulator.h"
#include "HarmonixDsp/Modulators/ModulatorTarget.h"

namespace Harmonix::Dsp::Modulators
{

FModulator::FModulator()
	: Target(nullptr)
	, Depth(0.5f)
	, RangeMagnitude(1.0f)
	, Range(0, 1)
{
	SetTarget(nullptr);
	SetRangeMagnitude(1.0f);
}

void FModulator::SetTarget(const FModulatorTarget* InTarget)
{
	if (InTarget == nullptr)
	{
		InTarget = &FModulatorTarget::kDummyTarget;
	}

	Target = InTarget;

	SetRangeMagnitude(Target->GetDefaultRangeMagnitude());
}

const FModulatorTarget* FModulator::GetTarget() const
{
	return Target;
}

void FModulator::SetDepth(float InDepth)
{
	Depth = InDepth;
}

float FModulator::GetDepth() const
{
	return Depth;
}

void FModulator::SetRangeMagnitude(float Magnitude)
{
	RangeMagnitude = Magnitude;
	Range = Target->GetRangeWithMagnitude(RangeMagnitude);
}

float FModulator::GetRangeMagnitude() const
{
	return RangeMagnitude;
}

void FModulator::Reset()
{
	Depth = 0.5f;
	SetTarget(nullptr);
}

void FModulator::Modulate(float InputNorm)
{
	if (Target == nullptr)
	{
		return;
	}

	float* TargetFloatPtr = Target->GetTargetFloat();

	float X = Target->ApplyDepthToNormalizedValue(InputNorm, Depth);
	X = Range.Interpolate(X);
	*TargetFloatPtr += X;
}

}