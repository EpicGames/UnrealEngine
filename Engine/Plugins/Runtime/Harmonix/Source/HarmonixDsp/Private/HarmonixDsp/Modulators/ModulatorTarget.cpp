// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Modulators/ModulatorTarget.h"

namespace Harmonix::Dsp::Modulators
{

static float sDummyFloat;
FModulatorTarget FModulatorTarget::kDummyTarget(NAME_None, 0, "", EDepthMode::MinUp, &sDummyFloat);

const FName FModulatorTarget::kPitchName = FName("Pitch");
const FName FModulatorTarget::kStartPointName = FName("StartPoint");

FModulatorTarget::FModulatorTarget(FName InName, float InDefaultRangeMag, const FString& InUnits, EDepthMode InDepthMode, float* ValueToTarget)
	: Name(InName)
	, DefaultRangeMag(InDefaultRangeMag)
	, Units(InUnits)
	, TargetFloat(ValueToTarget)
	, DepthMode(InDepthMode)
{
}

float FModulatorTarget::ApplyDepthToNormalizedValue(float InX, float InDepth) const
{
	ensure(0.0f <= InX && InX <= 1.0f);
	ensure(0.0f <= InDepth && InDepth <= 1.0f);

	switch (DepthMode)
	{
	case EDepthMode::MinUp:
		InX *= InDepth;
		break;
	
	case EDepthMode::MaxDown:
		InX = 1.0f - InX * InDepth;
		break;

	case EDepthMode::CenterOut:
		InX = InX * InDepth - 0.5f * (InDepth - 1.0f);
		break;
	}
	return InX;
}

TInterval<float> FModulatorTarget::GetRangeWithMagnitude(float InMagnitude) const
{
	switch (DepthMode)
	{
	case EDepthMode::MinUp: return TInterval<float>(0.0f, InMagnitude);
	case EDepthMode::MaxDown: return TInterval<float>(-InMagnitude, 0.0f);
	case EDepthMode::CenterOut: return TInterval<float>(-InMagnitude, InMagnitude);
	}
	return TInterval<float>(0.0f, 1.0f);
}

}