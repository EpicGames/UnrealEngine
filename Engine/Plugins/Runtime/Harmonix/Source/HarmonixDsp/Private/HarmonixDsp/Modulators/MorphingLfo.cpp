// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Modulators/MorphingLfo.h"

#include "DSP/FloatArrayMath.h"

namespace Harmonix::Dsp::Modulators
{
	FMorphingLFO::FMorphingLFO(const float InSampleRate)
	{
		Reset(InSampleRate);
	}

	float FMorphingLFO::GetSampleRate() const
	{
		return SampleRate;
	}

	void FMorphingLFO::Reset(const float InSampleRate)
	{
		SampleRate = InSampleRate;
		Phase = 0;
		LastFrequency = -1;
	}

	void FMorphingLFO::Advance(const int32 DeltaFrames, float& Output, const FMusicTimingInfo* MusicTimingInfo)
	{
		if (DeltaFrames <= 0)
		{
			return;
		}

		// cache params in case they change mid-block
		const ETimeSyncOption CachedSyncType = SyncType;
		const float CachedFrequency = Frequency;
		const float CachedShape = Shape;
		const bool CachedInvert = Invert;

		const float FrequencyHz = GetFrequencyHz(
			CachedSyncType,
			CachedFrequency,
			nullptr != MusicTimingInfo ? MusicTimingInfo->Tempo : 0,
			nullptr != MusicTimingInfo ? MusicTimingInfo->Speed : 0);
		const float PhaseInc = FrequencyHz / SampleRate;

		// Get the clock-synced phase if appropriate
		if (CachedSyncType != ETimeSyncOption::None && nullptr != MusicTimingInfo)
		{
			Phase = GetClockSyncedPhase(CachedFrequency, *MusicTimingInfo);
		}
		
		Phase = FMath::Fmod(Phase + PhaseInc * DeltaFrames, 1.0f);

		Output = GetValue(CachedShape, Phase);

		// Invert if desired
		if (CachedInvert)
		{
			Output = 1 - Output;
		}
	}

	void FMorphingLFO::Advance(float* OutputBuffer, int32 NumFrames, const FMusicTimingInfo* MusicTimingInfo)
	{
		check(nullptr != OutputBuffer);
			
		if (NumFrames <= 0)
		{
			return;
		}

		// cache params in case they change mid-block
		const ETimeSyncOption CachedSyncType = SyncType;
		const float CachedFrequency = Frequency;
		const float CachedShape = Shape;
		const bool CachedInvert = Invert;

		const float FrequencyHz = GetFrequencyHz(
			CachedSyncType,
			CachedFrequency,
			nullptr != MusicTimingInfo ? MusicTimingInfo->Tempo : 0,
			nullptr != MusicTimingInfo ? MusicTimingInfo->Speed : 0);
		float PhaseInc = FrequencyHz / SampleRate;

		// If the frequency param changed, ramp to the new frequency to avoid discontinuities
		// (except the very first time we advance)
		if (LastFrequency < 0)
		{
			LastFrequency = CachedFrequency;
		}

		float PhaseIncInc = 0;
		
		if (LastFrequency != CachedFrequency)
		{
			const float LastFrequencyHz = GetFrequencyHz(
				CachedSyncType,
				LastFrequency,
				nullptr != MusicTimingInfo ? MusicTimingInfo->Tempo : 0,
				nullptr != MusicTimingInfo ? MusicTimingInfo->Speed : 0);
			const float LastPhaseInc = LastFrequencyHz / SampleRate;
			PhaseIncInc = (PhaseInc - LastPhaseInc) / NumFrames;
			LastFrequency = CachedFrequency;
		}

		// Get the clock-synced phase if appropriate
		if (CachedSyncType != ETimeSyncOption::None && nullptr != MusicTimingInfo)
		{
			Phase = GetClockSyncedPhase(CachedFrequency, *MusicTimingInfo);
		}

		// Populate the output
		for (int32 Idx = 0; Idx < NumFrames; ++Idx)
		{
			OutputBuffer[Idx] = GetValue(CachedShape, Phase);

			Phase += PhaseInc;

			while (Phase > 1.0f)
			{
				Phase -= 1.0f;
			}

			PhaseInc += PhaseIncInc;
		}

		// invert if desired
		if (CachedInvert)
		{
			const TArrayView<float> View{ OutputBuffer, NumFrames };
			Audio::ArrayMultiplyByConstantInPlace(View, -1);
			Audio::ArrayAddConstantInplace(View, 1);
		}
	}
	
	float FMorphingLFO::GetValue(const float Shape, const float Phase)
	{
		// square
		if (Shape <= 0.001f)
		{
			return Phase < 0.5f ? 0 : 1;
		}
			
		// square -> triangle
		if (Shape <= 1.0f)
		{
			const float Slope = 2 / Shape;

			if (Phase < Shape / 2)
			{
				return 1 - Slope * Phase;
			}
			if (Phase < 0.5f)
			{
				return 0;
			}
			if (Phase < 0.5f + Shape / 2)
			{
				return Slope * (Phase - 0.5f);
			}
			return 1;
		}
			
		// triangle -> sawtooth
		if (Shape < 1.999f)
		{
			const float Slope1 = 2 / Shape;
			const float Slope2 = 2 / (2 - Shape);
			
			if (Phase < Shape / 2)
			{
				return 1 - Slope1 * Phase;
			}

			return 1 - Slope2 * (1 - Phase);
		}
			
		// sawtooth
		return 1 - Phase;
	}

	float FMorphingLFO::GetFrequencyHz(const ETimeSyncOption SyncType, const float Frequency, const float Tempo, const float Speed)
	{
		switch (SyncType)
		{
		case ETimeSyncOption::None:
			return Frequency;
		case ETimeSyncOption::TempoSync:
			// frequency is cycles per quarter note
			return Frequency * (Tempo / 60.0f);
		case ETimeSyncOption::SpeedScale:
			return Frequency * Speed;
		default:
			checkNoEntry();
			return 0;
		}
	}

	float FMorphingLFO::GetClockSyncedPhase(const float CyclesPerQuarter, const FMusicTimingInfo& MusicTimingInfo)
	{
		if (MusicTimingInfo.Timestamp.IsValid())
		{
			const float CyclesPerBeat = CyclesPerQuarter * (4.0f / MusicTimingInfo.TimeSignature.Denominator);
			const float CyclesPerBar = CyclesPerBeat * MusicTimingInfo.TimeSignature.Numerator;
			const float TotalPhase = (MusicTimingInfo.Timestamp.Bar - 1) * CyclesPerBar + (MusicTimingInfo.Timestamp.Beat - 1) * CyclesPerBeat;
			return FMath::Fmod(TotalPhase, 1.0f);
		}

		return 0;
	}
}
