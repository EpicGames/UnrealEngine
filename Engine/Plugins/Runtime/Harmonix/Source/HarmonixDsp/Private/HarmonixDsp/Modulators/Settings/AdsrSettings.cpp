// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Modulators/Settings/AdsrSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AdsrSettings)

void FAdsrSettings::CopyCurveTables(const FAdsrSettings& Other)
{
	FMemory::Memcpy(AttackCurveTable, Other.AttackCurveTable, sizeof(float) * kCurveTableSize);
	FMemory::Memcpy(DecayCurveTable, Other.DecayCurveTable, sizeof(float) * kCurveTableSize);
	FMemory::Memcpy(ReleaseCurveTable, Other.ReleaseCurveTable, sizeof(float) * kCurveTableSize);
}

void FAdsrSettings::BuildCurveTable(float InCurve, bool InDown, float* InTable)
{
	if (IsCurveLinear(InCurve))
	{
		return;
	}
	InCurve = FMath::Clamp(InCurve, -0.99999f, 0.99999f);
	float Coeff = (InCurve + 1.0f) / 2.0f;
	float OneMinusCoeff = 1.0f - Coeff;
	float OneMinus2Coeff = OneMinusCoeff - Coeff;
	float CoeffSquared = Coeff * Coeff;
	float CurveS = OneMinusCoeff / Coeff;
	float CurveSquaredS = CurveS * CurveS;
	float B = CoeffSquared / OneMinus2Coeff;
	const float ConstA = -B;
	// are we taking the natural log here?
	// I assume so since it's getting exponentiated later (e^x)
	const float ConstB = 2.0f * FMath::Loge(CurveS);
	const float ConstC = FMath::Loge(FMath::Abs(B));

	float Sign = (B < 0.0f) ? -1.0f : 1.0f;

	for (int32 Idx = 0; Idx < kCurveTableSize; ++Idx)
	{
		float Position = (float)Idx / (float)kCurveTableSize;
		if (InDown)
		{
			Position = 1.0f - Position;
		}
		InTable[Idx] = ConstA + (Sign * FMath::Exp(ConstB * Position + ConstC));
	}
}

float FAdsrSettings::LerpCurve(float InIndex, bool InDown, const float* InCurve) const
{
	int32 C1 = (int32)InIndex;
	int32 C2 = C1 + 1;
	float R1 = InCurve[C1];
	float R2 = (C2 < kCurveTableSize) ? InCurve[C2] : (InDown) ? 0.0f : 1.0f;
	float Frac = InIndex - (float)C1;
	return R1 * (1.0f - Frac) + R2 * Frac;
}
