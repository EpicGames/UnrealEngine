// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/MusicalAudioBus.h"
#include "HarmonixDsp/AudioBufferConstants.h"
#include "HarmonixDsp/AudioBufferConfig.h"

using namespace HarmonixDsp;

FMusicalAudioBus::FMusicalAudioBus()
	: AudioLevel(0)
	, AudioLevelDecay(0)
	, SamplesPerSecond(0)
	, SecondsPerSample(0)
	, Owner(nullptr)
{
}

FMusicalAudioBus::~FMusicalAudioBus()
{
	if (Owner)
	{
		Owner->BusWillDestruct(this);
	}

	TearDown();
}

void FMusicalAudioBus::Prepare(float InSampleRateHz, uint32 InNumChannels, uint32 InMaxSamples, bool bInAllocateBuffer)
{
	Prepare(InSampleRateHz, InNumChannels, EAudioBufferChannelLayout::Raw, InMaxSamples, bInAllocateBuffer);
}

void FMusicalAudioBus::Prepare(float InSampleRateHz, EAudioBufferChannelLayout InChannelLayout, uint32 InMaxSamples, bool bInAllocateBuffer)
{
	Prepare(InSampleRateHz, FAudioBuffer::GetNumChannelsInChannelLayout(InChannelLayout), InChannelLayout, InMaxSamples, bInAllocateBuffer);
}

void FMusicalAudioBus::Prepare(float InSampleRateHz, uint32 InNumChannels, EAudioBufferChannelLayout InChannelLayout, uint32 InMaxSamples, bool bInAllocateBuffer)
{
	check(InNumChannels <= (uint32)FAudioBuffer::GetNumChannelsInChannelLayout(InChannelLayout));
	check(InSampleRateHz > 0.0f);

	SamplesPerSecond = (double)InSampleRateHz;
	SecondsPerSample = 1.0 / SamplesPerSecond;

	AudioLevel = 0;

	// for now we don't filter the audio level,
	// because we want to give the rawest data
	// possible to the clients of the audio level.
	// they can filter it if necessary.
	AudioLevelDecay = 0.0f;

	MaxSamples = InMaxSamples;
	NumAudioOutputChannels = InNumChannels;
	ChannelLayout = InChannelLayout;

	if (MaxSamples > 0 && bInAllocateBuffer)
	{
		FAudioBufferConfig BufferConfig(ChannelLayout, NumAudioOutputChannels, MaxSamples, InSampleRateHz);
		BusBuffer.Configure(BufferConfig, EAudioBufferCleanupMode::Delete);
	}
}

void FMusicalAudioBus::SetSampleRate(float InSampleRateHz)
{
	check(InSampleRateHz > 0.0f);
	SamplesPerSecond = (double)InSampleRateHz;
	SecondsPerSample = 1.0 / SamplesPerSecond;

	BusBuffer.SetSampleRate(InSampleRateHz);
}

void FMusicalAudioBus::Process(uint32 InSliceIndex, uint32 InSubsliceIndex, TAudioBuffer<float>& OutBuffer)
{
	// Decay the current level
	AudioLevel *= AudioLevelDecay;

	// get the sampler (bus) audio level
	OutBuffer.Saturate(-1.0f, 1.0f);
	float PeakLevel = OutBuffer.GetPeak();
	AudioLevel = FMath::Max(AudioLevel, PeakLevel);
}