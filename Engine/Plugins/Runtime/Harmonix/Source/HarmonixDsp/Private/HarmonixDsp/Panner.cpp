// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Panner.h"

void FPanner::SetPanOffset_Stereo(float InMinusOneToOne)
{
	float LeftRad = -FGainTable::Get().GetDirectChannelAzimuthInCurrentLayout(ESpeakerChannelAssignment::LeftFront);
	float RightRad = -LeftRad;
	OffsetPan = (((InMinusOneToOne + 1.0f) / 2.0f) * (RightRad - LeftRad)) + LeftRad;
	UpdatePanRamper(true);
	UpdateGainMatrix();
}

void FPanner::SetPanOffsetTarget_Stereo(float InMinusOneToOne)
{
	float LeftRad = -FGainTable::Get().GetDirectChannelAzimuthInCurrentLayout(ESpeakerChannelAssignment::LeftFront);
	float RightRad = -LeftRad;
	OffsetPan = (((InMinusOneToOne + 1.0f) / 2.0f) * (RightRad - LeftRad)) + LeftRad;
	UpdatePanRamper(false);
}

void FPanner::UpdatePanRamper(bool Snap)
{
	float newPanTarget = OffsetPan + StartingPolarPan;
	if (Snap)
	{
		AggregatedPanRamper.SnapTo(newPanTarget, OriginalPannerDetails.IsCircular());
	}
	else
	{
		AggregatedPanRamper.SetTarget(newPanTarget, OriginalPannerDetails.IsCircular());
	}
}

void FPanner::UpdateEdgeProximityRamper(bool Snap)
{
	float newEdgeProxTarget = OffsetEdgeProximity * StartingEdgeProximity;
	if (Snap)
	{
		EdgeProximityRamper.SnapTo(newEdgeProxTarget);
	}
	else
	{
		EdgeProximityRamper.SetTarget(newEdgeProxTarget);
	}
}

void FPanner::UpdateGainMatrix()
{
	if (!OriginalPannerDetails.IsPannable())
	{
		CurrentGainMatrix.Set(OverallGain, OriginalPannerDetails, GainTable);
	}
	else if (OriginalPannerDetails.Mode == EPannerMode::Stereo)
	{
		CurrentGainMatrix.SetFromNewStereo(OverallGain, -AggregatedPanRamper.GetCurrent(), AggregatedPanRamper.GetMax(), GainTable);
	}
	else if (OriginalPannerDetails.Mode == EPannerMode::LegacyStereo)
	{
		CurrentGainMatrix.SetFromLegacyStereo(OverallGain, -AggregatedPanRamper.GetCurrent() / AggregatedPanRamper.GetMax());
	}
	else
	{
		// todo: edge proximity!
		CurrentGainMatrix.Set(OverallGain, AggregatedPanRamper.GetCurrent(), GainTable); 
	}
}