// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/PannerDetails.h"

#include "Serialization/Archive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PannerDetails)

bool FPannerDetails::Serialize(FArchive& Ar)
{
	uint8 Version = kVersion;
	Ar << Version;
	Ar << Mode;

	if (Mode == EPannerMode::DirectAssignment)
	{
		Ar << Detail.ChannelAssignment;
	}
	else
	{
		Ar << Detail.Pan;
		Ar << Detail.EdgeProximity;
	}
	return true;
}
