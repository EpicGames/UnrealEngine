// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixDsp/Streaming/TrackChannelInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TrackChannelInfo)

FTrackChannelInfo::FTrackChannelInfo()
{
}

FTrackChannelInfo::FTrackChannelInfo(int32 InRealIndex)
	: RealTrackIndex(InRealIndex)
{
}

FTrackChannelInfo::FTrackChannelInfo(FTrackChannelInfo&& Other)
{
	RealTrackIndex = Other.RealTrackIndex;
	Name = Other.Name;
	Routing = Other.Routing;
	Channels = Other.Channels;
	//TODO: Should this be a memory move operation?
	PresetChannels = Other.PresetChannels;

}

FTrackChannelInfo::FTrackChannelInfo(const FTrackChannelInfo& Other)
{
	RealTrackIndex = Other.RealTrackIndex;
	Name = Other.Name;
	Routing = Other.Routing;
	Channels = Other.Channels;
	PresetChannels = Other.PresetChannels;
}

int32 FTrackChannelInfo::GetNumChannels() const
{
	return Channels.Num();
}

int32 FTrackChannelInfo::GetStreamChannelIndex(int32 ChannelIndex) const
{
	check(Channels.IsValidIndex(ChannelIndex));
	return Channels[ChannelIndex].StreamIndex;
}

bool FTrackChannelInfo::GetStreamIndexesPan(int32 StreamIndex, FPannerDetails& OutPan) const
{
	if (const FStreamingChannelParams* ChannelParams = FindStreamingChannelParams(StreamIndex))
	{
		OutPan = ChannelParams->Pan;
		return true;
	}
	return false;
}

bool FTrackChannelInfo::GetStreamIndexesGain(int32 StreamIndex, float& OutGain) const
{
	if (const FStreamingChannelParams* ChannelParams = FindStreamingChannelParams(StreamIndex))
	{
		OutGain = ChannelParams->Gain;
		return true;
	}
	return false;
}

bool FTrackChannelInfo::GetStreamIndexesPresetGain(FName PresetName, int32 StreamIndex, float& OutGain, bool DefaultIfNotFound /*= true*/) const
{
	// find preset patching stream index
	for (int32 ChannelIdx = 0; ChannelIdx < Channels.Num(); ++ChannelIdx)
	{
		const FStreamingChannelParams& ChannelParams = Channels[ChannelIdx];
		if (ChannelParams.StreamIndex == StreamIndex)
		{
			if (const FStreamingChannelParamsArray* Preset = PresetChannels.Find(PresetName))
			{
				OutGain = Preset->ChannelParams[ChannelIdx].Gain;
				return true;
			}
			else if (!DefaultIfNotFound)
			{
				// no preset, and don't default
				return false;
			}

			// no preset, return default
			OutGain = Channels[ChannelIdx].Gain;
			return true;
		}
	}
	return false;
}

bool FTrackChannelInfo::SetStreamIndexesPan(int32 StreamIndex, float InPan)
{
	if (FStreamingChannelParams* ChannelParams = FindStreamingChannelParams(StreamIndex))
	{
		ChannelParams->Pan = InPan;
		return true;
	}
	return false;
}

bool FTrackChannelInfo::SetStreamIndexesPan(int32 StreamIndex, const FPannerDetails& InPan)
{
	if (FStreamingChannelParams* ChannelParams = FindStreamingChannelParams(StreamIndex))
	{
		ChannelParams->Pan = InPan;
		return true;
	}
	return false;
}

bool FTrackChannelInfo::SetStreamIndexesGain(int32 StreamIndex, float InGain)
{
	if (FStreamingChannelParams* ChannelParams = FindStreamingChannelParams(StreamIndex))
	{
		ChannelParams->Gain = InGain;
		return true;
	}
	return false;
}

bool FTrackChannelInfo::SetStreamIndexesPresetGain(FName PresetName, int32 StreamIndex, float InGain)
{
	for (int32 ChannelIdx = 0; ChannelIdx < Channels.Num(); ++ChannelIdx)
	{
		FStreamingChannelParams& ChannelParams = Channels[ChannelIdx];

		if (ChannelParams.StreamIndex == StreamIndex)
		{
			if (!PresetChannels.Find(PresetName))
			{
				PresetChannels[PresetName] = Channels;
			}

			PresetChannels[PresetName][ChannelIdx].Gain = InGain;
			return true;
		}
	}
	return false;
}

const FStreamingChannelParams* FTrackChannelInfo::FindStreamingChannelParams(int32 StreamIndex) const
{
	for (const FStreamingChannelParams& ChannelParams : Channels)
	{
		if (ChannelParams.StreamIndex == StreamIndex)
		{
			return &ChannelParams;
		}
	}
	return nullptr;
}

FStreamingChannelParams* FTrackChannelInfo::FindStreamingChannelParams(int32 StreamIndex)
{
	return const_cast<FStreamingChannelParams*>(const_cast<const FTrackChannelInfo*>(this)->FindStreamingChannelParams(StreamIndex));
}
