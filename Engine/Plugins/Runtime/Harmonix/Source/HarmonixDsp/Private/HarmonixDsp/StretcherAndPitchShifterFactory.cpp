// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/StretcherAndPitchShifterFactory.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactoryConfig.h"
#include "HarmonixDsp/StretcherAndPitchShifter.h"
#include "HarmonixDsp/PitchShifterName.h"
#include "Algo/Transform.h"
#include "ContentStreaming.h"
#include "Logging/LogMacros.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogStretcherAndPitchShifterFactory, Log, All)

static int32 MaxMemoryUsageKBCvar = -1;
FAutoConsoleVariableRef CvarMaxMemoryUsageKB(
	TEXT("hmx.pitchshifter.MaxMemoryUsageKB"),
	MaxMemoryUsageKBCvar,
	TEXT("Sets the max amount the Pitch Shifters are allowed to \"steal\" from the streaming cache in Kilobytes. This caps the amount calculated by the pitch shifgters.\n")
	TEXT("n: n > 0 sets the value. n < 0 means no maximum, allowing as much memory as requested."),
	ECVF_Default);

static int32 AdditionalMemoryUsageKBCvar = 0;
FAutoConsoleVariableRef CvarAdditionalMemoryUsageKB(
	TEXT("hmx.pitchshifter.AdditionalMemoryUsageKB"),
	AdditionalMemoryUsageKBCvar,
	TEXT("Sets an Additional amount the Pitch Shifters \"steal\" from the streaming cache in Kilobytes. This amount is added to the calculated memory usage of the pitch shifters. Will only apply  when the calculated memory usage > 0.\n")
	TEXT("n: any non-zero value to offset the memory count. Negative values allow \"negative\" padding."),
	ECVF_Default);

static float MemoryUsageScaleFactorCvar = -1;
FAutoConsoleVariableRef CvarMemoryUsageScaleFactor(
	TEXT("hmx.pitchshifter.MemoryUsageScaleFactor"),
	MemoryUsageScaleFactorCvar,
	TEXT("Scales the calculated amount of memory the Pitch Shifters \"steal\" from the streaming cache in.\n")
	TEXT("n: n > 0 sets the value. n < 0 no scale factor"),
	ECVF_Default);

static int32 OverrideMemoryUsageKBCvar = -1;
FAutoConsoleVariableRef CvarOverrideMemoryUsageKB(
	TEXT("hmx.pitchshifter.OverrideMemoryUsageKB"),
	OverrideMemoryUsageKBCvar,
	TEXT("Completely overrides the total memory usage the Pitch Shifters \"steal\" from the streaming cache in Kilobytes.\n")
	TEXT("n: n > 0 sets the value. n < 0 means no override"),
	ECVF_Default);


// pitch shifters can take up as much as 52MB though :/
// and as little as 17MB
// most low end platforms use ~20MB, so start with that
static int32 ReserveMemoryKBCvar = 20480; // default: reserve 20MB of memory
FAutoConsoleVariableRef CvarReserveMemoryKB(
	TEXT("hmx.pitchshifter.ReserveMemoryKB"),
	ReserveMemoryKBCvar,
	TEXT("Amount of memory to \"reserve\". This isn't actual allocated memory. This ammount will be reported to the streaming cache *right before* we actually allocate the pitch shifters.\n"
		 "This gives the streaming cache the chance to \"Trim\" cached audio if needed before we allocate potentially a lot of memory up front\n"
		 "We update the streaming cache to the actual usage of the pitch shifters once the pitch shifters are allocated")
	TEXT("n: Amount of memory to \"reserve\" in the streaming cache, in KB"),
	ECVF_Default);

TArray<IFeatureClient*> IStretcherAndPitchShifterFactory::Clients;
FCriticalSection IStretcherAndPitchShifterFactory::ClientLock;
size_t IStretcherAndPitchShifterFactory::TotalMemoryUsed = 0;
FAudioStreamCacheMemoryHandle IStretcherAndPitchShifterFactory::MemoryHandle(GetModularFeatureName(), 0);

IStretcherAndPitchShifterFactory::IStretcherAndPitchShifterFactory()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
}

IStretcherAndPitchShifterFactory::~IStretcherAndPitchShifterFactory()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

FName IStretcherAndPitchShifterFactory::GetModularFeatureName()
{
	static FName FeatureName = FName(TEXT("StretcherAndPitchShifter Factory"));
	return FeatureName;
}

void IStretcherAndPitchShifterFactory::AddClient(IFeatureClient* Client, float SampleRate)
{
	FScopeLock Lock(&ClientLock);

	if (Clients.Contains(Client))
		return;

	Clients.Add(Client);
	UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Adding client. Num Clients: %d", Clients.Num());

	if (ReserveMemoryKBCvar > 0 && TotalMemoryUsed == 0)
	{
		// we haven't allocated any memory yet!
		// Prepare the Audio Streaming Cache by spoofing/reserving memory usage right before we do a huge allocation.
		const uint64 ReserveMemoryBytes = ReserveMemoryKBCvar * 1024;
		MemoryHandle.ResetMemoryUseInBytes(ReserveMemoryBytes);
	}

	size_t NewMemoryUsed  = 0;
	for (IStretcherAndPitchShifterFactory* Factory : GetAllRegisteredFactories())
	{
		Factory->OnClientAdded(Client, SampleRate);
		const TArray<FName>& SupportedNames = Factory->GetFactoryNames();
		for (FName FactoryName : SupportedNames)
		{
			UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Updating Factory: \"%ls\"", *FactoryName.ToString());
		}
		NewMemoryUsed += Factory->GetMemoryUsedByPool();
	}
	
	if (NewMemoryUsed != TotalMemoryUsed)
	{
		TotalMemoryUsed = NewMemoryUsed;
		ResetMemoryUsageBytes(TotalMemoryUsed);
	}
	else if (ReserveMemoryKBCvar > 0 && NewMemoryUsed == 0)
	{
		// handle the case where we didn't allocate anything and we need to undo our "reserved" amount
		MemoryHandle.ResetMemoryUseInBytes(0);
	}
	
}

void IStretcherAndPitchShifterFactory::RemoveClient(IFeatureClient* Client)
{
	FScopeLock Lock(&ClientLock);

	if (Clients.Remove(Client) == 0)
		return;

	UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Removed client. Num Clients: %d", Clients.Num());

	size_t NewMemoryUsed = 0;
	for (IStretcherAndPitchShifterFactory* Factory : GetAllRegisteredFactories())
	{
		Factory->OnClientRemoved(Client);
		const TArray<FName>& SupportedNames = Factory->GetFactoryNames();
		for (FName FactoryName : SupportedNames)
		{
			UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Updating Factory: \"%ls\"", *FactoryName.ToString());
		}
		NewMemoryUsed += Factory->GetMemoryUsedByPool();
	}
	if (NewMemoryUsed != TotalMemoryUsed)
	{
		TotalMemoryUsed = NewMemoryUsed;
		ResetMemoryUsageBytes(TotalMemoryUsed);
	}
}

bool IStretcherAndPitchShifterFactory::HasClient(IFeatureClient* Client)
{
	FScopeLock Lock(&ClientLock);

	return Clients.Contains(Client);
}

int32 IStretcherAndPitchShifterFactory::GetNumClients()
{
	FScopeLock Lock(&ClientLock);

	return Clients.Num();
}

TArray<IStretcherAndPitchShifterFactory*> IStretcherAndPitchShifterFactory::GetAllRegisteredFactories()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	return IModularFeatures::Get().GetModularFeatureImplementations<IStretcherAndPitchShifterFactory>(GetModularFeatureName());
}

TArray<FName> IStretcherAndPitchShifterFactory::GetAllRegisteredFactoryNames()
{
	TArray<FName> Names;
	TArray<IStretcherAndPitchShifterFactory*> AllFactories = GetAllRegisteredFactories();
	for (IStretcherAndPitchShifterFactory* Factory : AllFactories)
	{
		Names.Append(Factory->GetFactoryNames());
	}
	return Names;
}

IStretcherAndPitchShifterFactory* IStretcherAndPitchShifterFactory::FindBestFactory(const FTimeStretchConfig& InConfig)
{
	UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Finding best factory for FactoryName: \"%ls\"", *InConfig.PitchShifter.Name.ToString());

	const UStretcherAndPitchShifterFactoryConfig* Config = GetDefault<UStretcherAndPitchShifterFactoryConfig>();

	FName FactoryName = InConfig.PitchShifter.Name;
	const TArray<FName>& FactoryPriority = Config->FactoryPriority;
	int32 PriorityIndex = FactoryPriority.Find(FactoryName);

	FName DefaultFactoryName = Config->DefaultFactory;
	if (FactoryName == NAME_None)
	{
		FactoryName = DefaultFactoryName;
		UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "FactoryName is NONE, using DefaultFactoryName: \"%ls\"", *FactoryName.ToString());
	}

	while (FactoryName != NAME_None)
	{
		if (IStretcherAndPitchShifterFactory* Factory = FindFactory(FactoryName))
		{
			if (Factory->HasFreePitchShifters(InConfig))
			{
				UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Found available factory for FactoryName: \"%ls\"", *FactoryName.ToString());
				return Factory;
			}

			UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Factory for FactoryName: \"%ls\" has no free pitch shifters. Falling back to next in line.", *FactoryName.ToString());
		}

		if (FactoryPriority.IsValidIndex(PriorityIndex))
		{
			if (FactoryPriority.IsValidIndex(++PriorityIndex))
			{
				FactoryName = FactoryPriority[PriorityIndex];
				UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Fallback: Finding factory for FactoryName: \"%ls\"", *FactoryName.ToString());
			}
			else // we're at the end of the priority list
			{
				FactoryName = NAME_None;
				UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Fallback: No more factories to fall back to!");
			}
		}
		// The factory name isn't in the priority list, and it's not the default factory
		else if (DefaultFactoryName != FactoryName) 
		{
			FactoryName = DefaultFactoryName;
			UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Fallback: Unable to determine a factory to fallback to (the factory name passed in is not in the Priority List!). Using DefaultFactoryName: \"%ls\"", *FactoryName.ToString());
		}
		else
		{
			// there's nothing left to try
			FactoryName = NAME_None;
			UE_LOGF(LogStretcherAndPitchShifterFactory, Verbose, "Fallback: Unable to determine a factory to fallback to (the factory name passed in is not in the Priority List!), and we already tried the DefaultFactory: \"%ls\", so there's nothing left to try!", *DefaultFactoryName.ToString());
		}
	}

	return nullptr;
}

IStretcherAndPitchShifterFactory* IStretcherAndPitchShifterFactory::FindFactory(const FName InFactoryName)
{
	TArray<IStretcherAndPitchShifterFactory*> Factories = GetAllRegisteredFactories();
	if (IStretcherAndPitchShifterFactory** Found = Factories.FindByPredicate([InFactoryName](IStretcherAndPitchShifterFactory* Factory) { return Factory->GetFactoryNames().Contains(InFactoryName); }))
	{
		return *Found;
	}
	return nullptr;
}

void IStretcherAndPitchShifterFactory::ResetMemoryUsageBytes(uint64 MemoryUsageBytes)
{
	if (GetNumClients() == 0)
	{
		ensureMsgf(MemoryUsageBytes == 0, TEXT("All clients to StretchAndPitchShifterFactory were removed, but memory usage has not gone down to 0!"));
	}
	
	if (MemoryUsageBytes > 0 && OverrideMemoryUsageKBCvar > -1)
	{
		// if we have an override, slam memory usage to that value when any client is active
		MemoryUsageBytes = OverrideMemoryUsageKBCvar * 1024;
	}
	else if (MemoryUsageBytes > 0)
	{
		// only apply modifiers if memory usage > 0
		if (MemoryUsageScaleFactorCvar > -1)
		{
			MemoryUsageBytes *= MemoryUsageScaleFactorCvar;
		}

		// don't subtract to a negative value, since MemoryUsageBytes is unsigned!!
		int32 AdditionalMemoryBytes = AdditionalMemoryUsageKBCvar * 1024;
		if (AdditionalMemoryUsageKBCvar < 0 && FMath::Abs(AdditionalMemoryBytes) >= MemoryUsageBytes)
		{
			MemoryUsageBytes = 0;
		}
		else
		{
			MemoryUsageBytes += AdditionalMemoryBytes;
		}
		
		if (MaxMemoryUsageKBCvar > -1)
		{
			MemoryUsageBytes = FMath::Min(MemoryUsageBytes, static_cast<uint64>(MaxMemoryUsageKBCvar) * 1024);
		}
	}

	MemoryHandle.ResetMemoryUseInBytes(MemoryUsageBytes);
}
