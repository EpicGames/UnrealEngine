// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/StretcherAndPitchShifterFactoryConfig.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactory.h"
#include "HarmonixDsp/Stretchers/SmbPitchShifterFactory.h"
#include "Algo/Count.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StretcherAndPitchShifterFactoryConfig)

UStretcherAndPitchShifterFactoryConfig::UStretcherAndPitchShifterFactoryConfig()
	: UHarmonixDeveloperSettings()
{
	// guarantee that we'll always have a default pitch shifter assigned to "SmbPitchShifter"
	DefaultFactory = FSmbPitchShifterFactory::Name;
}

void UStretcherAndPitchShifterFactoryConfig::PostInitProperties()
{
	Super::PostInitProperties();

	PruneDuplicatesAndAddMissingNames(FactoryPriority);
}

void UStretcherAndPitchShifterFactoryConfig::AddFactoryNameRedirect(FName OldName, FName NewName)
{
	if (!FactoryNameRedirects.FindByPredicate([&OldName](const FPitchShifterNameRedirect& Redirect)
		{
			return Redirect.OldName == OldName;
		}))
	{
		FactoryNameRedirects.Add(FPitchShifterNameRedirect(OldName, NewName));
	}
}


const FPitchShifterNameRedirect* UStretcherAndPitchShifterFactoryConfig::FindFactoryNameRedirect(FName OldName) const
{
	if (OldName == NAME_None)
	{
		return nullptr;
	}

	return FactoryNameRedirects.FindByPredicate([&OldName](const FPitchShifterNameRedirect& Redirect)
		{
			return Redirect.OldName == OldName;
		});
}

void UStretcherAndPitchShifterFactoryConfig::PruneDuplicatesAndAddMissingNames(TArray<FName>& InOutNames)
{
	if (InOutNames.IsEmpty())
	{
		for (FName FactoryName : IStretcherAndPitchShifterFactory::GetAllRegisteredFactoryNames())
		{
			InOutNames.Add(FactoryName);
		}
		return;
	}

	for (FName FactoryName : IStretcherAndPitchShifterFactory::GetAllRegisteredFactoryNames())
	{
		int32 Count = Algo::Count(InOutNames, FactoryName);
		while (Count > 1)
		{
			// iterate backwards and remove duplicates
 			for (int32 Index = InOutNames.Num() - 1; Index >= 0; --Index)
			{
				if (InOutNames[Index] == FactoryName)
				{
					InOutNames.RemoveAt(Index);
					if (--Count == 1)
					{
						break;
					}
				}
			}
		}

		if (Count < 1)
		{
			InOutNames.Add(FactoryName);
		}
	}
}

#if WITH_EDITORONLY_DATA

void UStretcherAndPitchShifterFactoryConfig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	PruneDuplicatesAndAddMissingNames(FactoryPriority);
}


#endif
