// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Stretchers/ConsoleCommands.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactory.h"

#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixDspConsoleCommands, Log, All)

namespace Harmonix::Dsp::ConsoleCommands
{

TArray<FName> FPitchShifter::DisabledPitchShifters;

void FPitchShifter::EnableAll()
{
	DisabledPitchShifters.Empty();
}

void FPitchShifter::DisableAll()
{
	for (FName Name : IStretcherAndPitchShifterFactory::GetAllRegisteredFactoryNames())
	{
		DisabledPitchShifters.AddUnique(Name);
	}
}

bool FPitchShifter::Enable(FName Name)
{
	if (!IStretcherAndPitchShifterFactory::FindFactory(Name))
	{
		return false;
	}

	DisabledPitchShifters.Remove(Name);
	return true;
}

bool FPitchShifter::Disable(FName Name)
{
	if (!IStretcherAndPitchShifterFactory::FindFactory(Name))
	{
		return false;
	}
	DisabledPitchShifters.AddUnique(Name);
	return true;
}

bool FPitchShifter::IsEnabled(FName Name)
{
	return !DisabledPitchShifters.Contains(Name);
}

static FAutoConsoleCommand ConsoleCmd_ListPitchShifters(
	TEXT("hmx.pitchShifters.list"),
	TEXT("List the names of the available pitch shifters.\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args)
		{
			for (const FName& Name : IStretcherAndPitchShifterFactory::GetAllRegisteredFactoryNames())
			{
				UE_LOGF(LogHarmonixDspConsoleCommands, Log, "%ls %ls", 
					   *Name.ToString(), FPitchShifter::IsEnabled(Name) ? TEXT("") : TEXT("(disabled)"));
			}
		}
	)
);

static FAutoConsoleCommand ConsoleCmd_EnablePitchShifters(
	TEXT("hmx.pitchShifters"),
	TEXT("Enable/Disable a list of pitch shifters.\n"
		"If Disabled, prevents pitch shifter factory from returning instances of given pitch shifter name\n"
		"Enter list of names as a space separated list.\n"
		"$hmx.pitchShifters ('enable'|'disable') ('all'|($PitchShifterName )+)"),
	FConsoleCommandWithArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args)
		{
			if (Args.Num() < 2)
			{
				UE_LOGF(LogHarmonixDspConsoleCommands, Warning,
					"USAGE: hmx.pitchShifters ('enable'|'disable') ('all'|($PitchShifterName)+)");
				return;
			}

			const FString& Toggle = Args[0];

			if (!(Toggle == "enable" || Toggle == "disable"))
			{
				UE_LOGF(LogHarmonixDspConsoleCommands, Warning,
					"USAGE: hmx.pitchShifters ('enable'|'disable') ('all'|($PitchShifterName)+)");
				return;
			}

			bool Enable = Toggle == "enable";

			if (Args.Num() == 2 && Args[1] == "all")
			{
				if (Enable)
				{
					FPitchShifter::EnableAll();
				}
				else
				{
					FPitchShifter::DisableAll();
				}

				// listo all the pitch shifter names
				for (const FName& Name : IStretcherAndPitchShifterFactory::GetAllRegisteredFactoryNames())
				{
					UE_LOGF(LogHarmonixDspConsoleCommands, Log, "%ls %ls",
						*Name.ToString(), FPitchShifter::IsEnabled(Name) ? TEXT("(enabled)") : TEXT("(disabled)"));
				}
				return;
			}

			for (int ArgIdx = 1; ArgIdx < Args.Num(); ++ArgIdx)
			{
				const FString& Name = Args[ArgIdx];

				if (Enable)
				{
					if (FPitchShifter::Enable(FName(Name)))
					{
						UE_LOGF(LogHarmonixDspConsoleCommands, Log, "%ls %ls", *Name, TEXT("(enabled)"));
					}
					else
					{
						UE_LOGF(LogHarmonixDspConsoleCommands, Warning,
							"Invalid pitch shifter name: %ls", *Name);
					}
				}
				else
				{
					if (FPitchShifter::Disable(FName(Name)))
					{
						UE_LOGF(LogHarmonixDspConsoleCommands, Log, "%ls %ls", *Name, TEXT("(disabled)"));
					}
					else
					{
						UE_LOGF(LogHarmonixDspConsoleCommands, Warning,
							"invalid name: %ls", *Name);
					}
				}
			}
		}
	)
);

}
