// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace Harmonix::Dsp::ConsoleCommands
{
	struct FPitchShifter
	{
		static void EnableAll();
		static void DisableAll();

		static bool Enable(FName Name);

		static bool Disable(FName Name);

		static bool IsEnabled(FName Name);

	private:

		static TArray<FName> DisabledPitchShifters;

		FPitchShifter() {};
	};
}