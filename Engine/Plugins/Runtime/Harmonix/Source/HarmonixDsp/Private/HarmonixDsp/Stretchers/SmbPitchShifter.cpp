// Copyright Epic Games, Inc. All Rights Reserved.

/****************************************************************************
* NOTE: This is a HEAVILY modified version of the original smbPitchShift.cpp
* that was written by Stephan M Bernsee, and made public available under the
* "Wide Open License." The link to the original code is...
* http://blogs.zynaptiq.com/bernsee/repo/smbPitchShift.cpp
* The original copyright notice and license statement are below.
*
* One more note: the original comment section refers to a web page for
* more information about the "WOL". That link is stale. At the time of this
* writing the correct link is...
* http://www.dspguru.com/wide-open-license
*****************************************************************************/

/****************************************************************************
*
* NAME: SmbPitchShifter.cpp
* VERSION: 1.2
* HOME URL: http://blogs.zynaptiq.com/bernsee
* KNOWN BUGS: none
*
* SYNOPSIS: Routine for doing pitch shifting while maintaining
* duration using the Short Time Fourier Transform.
*
* DESCRIPTION: The routine takes a pitchShift factor value which is between 0.5
* (one octave down) and 2. (one octave up). A value of exactly 1 does not change
* the pitch. InNumSamples tells the routine how many samples in InData[0...
* InNumSamples-1] should be pitch shifted and moved to OutData[0 ...
* InNumSamples-1]. The two buffers can be identical (ie. it can process the
* data in-place). fftFrameSize defines the FFT frame size used for the
* processing. Typical values are 1024, 2048 and 4096. It may be any value <=
* MAX_FRAME_LENGTH but it MUST be a power of 2. osamp is the STFT
* oversampling factor which also determines the overlap between adjacent STFT
* frames. It should at least be 4 for moderate scaling ratios. A value of 32 is
* recommended for best quality. sampleRate takes the sample rate for the signal
* in unit Hz, ie. 44100 for 44.1 kHz audio. The data passed to the routine in
* InData[] should be in the range [-1.0, 1.0), which is also the output range
* for the data, make sure you scale the data accordingly (for 16bit signed integers
* you would have to divide (and multiply) by 32768).
*
* COPYRIGHT 1999-2015 Stephan M. Bernsee <s.bernsee [AT] zynaptiq [DOT] com>
*
* 						The Wide Open License (WOL)
*
* Permission to use, copy, modify, distribute and sell this software and its
* documentation for any purpose is hereby granted without fee, provided that
* the above copyright notice and this license appear in all source copies.
* THIS SOFTWARE IS PROVIDED "AS IS" WITHOUT EXPRESS OR IMPLIED WARRANTY OF
* ANY KIND. See http://www.dspguru.com/wol.htm for more information.
*
*****************************************************************************/

#include "HarmonixDsp/Stretchers/SmbPitchShifter.h"
#include "HarmonixDsp/AudioDataRenderer.h"

#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMath.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Sound/SoundWave.h"

//#include "kiss_fft/kiss_fft.h"

float FSmbPitchShifter::Window128[EWindowSize::k128] = { 0 };
float FSmbPitchShifter::Window256[EWindowSize::k256] = { 0 };
float FSmbPitchShifter::Window512[EWindowSize::k512] = { 0 };
float FSmbPitchShifter::Window1024[EWindowSize::k1024] = { 0 };
float FSmbPitchShifter::Window2048[EWindowSize::k2048] = { 0 };
float FSmbPitchShifter::Window4096[EWindowSize::k4096] = { 0 };
bool  FSmbPitchShifter::WindowsInitialized = false;

void FSmbPitchShifter::FChannelBuffer::Reset()
{
	OutWriteIndex = 0;
	FMemory::Memset(InPCMBuffer, 0, EWindowSize::Max * sizeof(float));
	FMemory::Memset(OutPCMBuffer, 0, EWindowSize::Max * sizeof(float));
	FMemory::Memset(FftBuffer, 0, EWindowSize::MaxImag * sizeof(float));
	FMemory::Memset(LastPhase, 0, EWindowSize::MaxReal * sizeof(float));
	FMemory::Memset(SummedPhase, 0, EWindowSize::MaxReal * sizeof(float));
	FMemory::Memset(OutputAccumulator, 0, EWindowSize::MaxImag * sizeof(float));
}

FSmbPitchShifter::FSmbPitchShifter(FName InFactoryName) 
	: IStretcherAndPitchShifter(InFactoryName)
{
	if (!WindowsInitialized)
	{
		InitializeWindows();
	}
	Reset();
}

FSmbPitchShifter::FSmbPitchShifter(FName InFactoryName, float InSampleRate) 
	: IStretcherAndPitchShifter(InFactoryName)
{
	if (!WindowsInitialized)
	{
		InitializeWindows();
	}
	FSmbPitchShifter::SetSampleRateAndReset(InSampleRate);
}

FSmbPitchShifter::FSmbPitchShifter()
	: FSmbPitchShifter(NAME_None)
{}

FSmbPitchShifter::FSmbPitchShifter(float InSampleRate)
	: FSmbPitchShifter(NAME_None, InSampleRate)
{}

FSmbPitchShifter::~FSmbPitchShifter()
{
	if (ChannelBuffers)
	{
		delete[] ChannelBuffers;
		ChannelBuffers = nullptr;
		NumChannels = 0;
	}
}

void FSmbPitchShifter::Configure(int32 InNumChannels, int32 InWindowSize, int32 InOverlapFactor)
{
	check(EWindowSize::IsValid(InWindowSize));

	MemoryUsed = sizeof(FSmbPitchShifter);

	if (InNumChannels != NumChannels)
	{
		if (ChannelBuffers)
		{
			delete[] ChannelBuffers;
		}
		ChannelBuffers = new FChannelBuffer[InNumChannels + 1]; // +1 for the mono summation for analysis.
		MemoryUsed += sizeof(FChannelBuffer) * (InNumChannels + 1);
		NumChannels = InNumChannels;
	}

	BottomStereoBin = 0;
	FftFrameSize = InWindowSize;
	HalfFftFrameSize = InWindowSize / 2;
	FreqPerFftBin = (double)SampleRate / (double)FftFrameSize;
	OverlapFactor = InOverlapFactor;
	StepSize = InWindowSize / InOverlapFactor;
	ExpectedPhaseDifferent = 2.0 * UE_PI * (double)StepSize / (double)FftFrameSize;
	InFifoLatency = InWindowSize - StepSize;
	if (!WindowsInitialized)
	{
		InitializeWindows();
	}

	switch (InWindowSize)
	{
	case EWindowSize::k128:  Window = Window128; break;
	case EWindowSize::k256:  Window = Window256; break;
	case EWindowSize::k512:  Window = Window512; break;
	case EWindowSize::k1024: Window = Window1024; break;
	case EWindowSize::k2048: Window = Window2048; break;
	case EWindowSize::k4096: Window = Window4096; break;
	}

	Reset();
}

void FSmbPitchShifter::Configure(int32 InNumChannels, int32 InWindowSize, int32 InOverlapFactor, int32 MaxInputSamplesToCache)
{
	Configure(InNumChannels, InWindowSize, InOverlapFactor);
	InputBuffer.Configure(InNumChannels, MaxInputSamplesToCache, EAudioBufferCleanupMode::Delete, 48000, false);
	MemoryUsed += InputBuffer.GetFreeableSize();
	InputFrameOffset = 0;
}

int32 FSmbPitchShifter::GetLatencySamples() const
{
	return InFifoLatency;
}

void FSmbPitchShifter::SetMonoCutoff(float FreqHz)
{
	BottomStereoBin = (int32)(FreqHz / (float)FreqPerFftBin);
}

void FSmbPitchShifter::SetSampleRateAndReset(float InSampleRate)
{
	SampleRate = InSampleRate;
	FreqPerFftBin = (double)InSampleRate / (double)FftFrameSize;
	Reset();
}

void FSmbPitchShifter::SetupAndResetImpl(const FTimeStretchConfig* ShifterSettings)
{
	check(MySoundWaveData);
	SampleRate = MySoundWaveData->GetSampleRate();
	FreqPerFftBin = (double)MySoundWaveData->GetSampleRate() / (double)FftFrameSize;
	Reset();
}

#define INIT_WINDOW(Size)                                                                     \
   do {                                                                                       \
      check(EWindowSize::IsValid(Size))                                                       \
	  for (int32 Idx = 0; Idx < Size; Idx++)                                                  \
      {                                                                                       \
         Window ## Size[Idx] = float(-.5*FMath::Cos(2.*UE_PI*(double)Idx/(double)Size)+.5);   \
      }                                                                                       \
   } while(0)

void FSmbPitchShifter::InitializeWindows()
{
	INIT_WINDOW(128);
	INIT_WINDOW(256);
	INIT_WINDOW(512);
	INIT_WINDOW(1024);
	INIT_WINDOW(2048);
	INIT_WINDOW(4096);
	WindowsInitialized = true;
}

void FSmbPitchShifter::Reset()
{
	if (!ChannelBuffers) return;

	for (int32 ChIdx = 0; ChIdx <= NumChannels; ++ChIdx) // there is +1 buffer for mono analysis
	{
		ChannelBuffers[ChIdx].Reset();
	}
}

double FSmbPitchShifter::Render(TAudioBuffer<float>& OutputData, double InPos, int32 InMaxFrame, double ResampleInc, double InPitchShift, double InSpeed, bool MaintainPitchWhenSpeedChanges, const FGainMatrix& InGain)
{
	double NewPos = MyAudioRenderer->RenderUnshifted(OutputData, InPos, InMaxFrame, ResampleInc * InSpeed, InGain);
	float ActualPitchShift = MaintainPitchWhenSpeedChanges ? (float)(InPitchShift / InSpeed) : InPitchShift;
	int32 NumOutSamples = OutputData.GetNumValidFrames();
	int32 NumOutChannels = OutputData.GetNumValidChannels();

	float* InDataL = OutputData.GetValidChannelData(0);
	float* OutDataL = InDataL;
	float* InDataR = nullptr;
	float* OutDataR = nullptr;
	int32 Stride = 1;
	if (OutputData.GetIsInterleaved())
	{
		Stride = NumOutChannels;
		InDataR = InDataL + 1;
		OutDataR = InDataR;
	}
	else if (NumOutChannels > 1)
	{
		Stride = 1;
		InDataR = OutputData.GetValidChannelData(1);
		OutDataR = InDataR;
	}

	PitchShift(ActualPitchShift, NumOutSamples, InDataL, OutDataL, 0, Stride);
	if (NumOutChannels > 1)
	{
		PitchShift(ActualPitchShift, NumOutSamples, InDataR, OutDataR, 1, Stride);
	}

	return NewPos;
}

bool FSmbPitchShifter::TakeInput(TAudioBuffer<float>& InBuffer)
{
	check(!InBuffer.GetIsInterleaved());
	check(InBuffer.GetNumValidChannels() == InputBuffer.GetNumValidChannels());
	int32 SpaceInCache = InputBuffer.GetMaxNumFrames() - InputFrameOffset;
	int32 FramesToTake = SpaceInCache < InBuffer.GetNumValidFrames() ? SpaceInCache : InBuffer.GetNumValidFrames();
	TAudioBuffer<float> AliasBuffer;
	AliasBuffer.Alias(InputBuffer, InputFrameOffset);
	AliasBuffer.SetNumValidFrames(FramesToTake);
	AliasBuffer.Copy(InBuffer);
	InputFrameOffset += InBuffer.GetNumValidFrames();
	return FramesToTake == InBuffer.GetNumValidFrames();
}

bool FSmbPitchShifter::InputSilence(int32 InNumFrames)
{
	int32 SpaceInCache = InputBuffer.GetMaxNumFrames() - InputFrameOffset;
	int32 DoFrames = InNumFrames > SpaceInCache ? SpaceInCache : InNumFrames;
	TAudioBuffer<float> AliasBuffer;
	AliasBuffer.Alias(InputBuffer, InputFrameOffset);
	AliasBuffer.SetNumValidFrames(DoFrames);
	AliasBuffer.ZeroData();
	InputFrameOffset += InNumFrames;
	return DoFrames == InNumFrames;
}

void FSmbPitchShifter::PitchShift(float InPitchShift, int32 InNumSamples, float* InData, float* OutData, int32 InChannelIdx, int32 InStride)
{
	double Magnitude;
	double Phase;
	double Temp;
	double Real;
	double Image;
	int32 Idx;
	int32 K;
	int32 Qpd;
	int32 Index;

	check(FftFrameSize > 8); // assure some minimum fft size!

	check(InChannelIdx < NumChannels);
	FChannelBuffer& ChannelBuffer = ChannelBuffers[InChannelIdx];

	if (ChannelBuffer.OutWriteIndex == 0)
	{
		ChannelBuffer.OutWriteIndex = InFifoLatency;
	}

	/* main processing loop */
	for (Idx = 0; Idx < InNumSamples; Idx++)
	{
		/* As long as we have not yet collected enough data just read in */
		ChannelBuffer.InPCMBuffer[ChannelBuffer.OutWriteIndex] = InData[Idx * InStride];
		OutData[Idx * InStride] = ChannelBuffer.OutPCMBuffer[ChannelBuffer.OutWriteIndex - InFifoLatency];
		ChannelBuffer.OutWriteIndex++;

		/* now we have enough data for processing */
		if (ChannelBuffer.OutWriteIndex >= FftFrameSize)
		{
			ChannelBuffer.OutWriteIndex = InFifoLatency;

			/* do windowing and re,im interleave */
			for (K = 0; K < FftFrameSize; K++)
			{
				ChannelBuffer.FftBuffer[2 * K] = float(ChannelBuffer.InPCMBuffer[K] * Window[K]);
				ChannelBuffer.FftBuffer[2 * K + 1] = 0.;
			}

			/* ***************** ANALYSIS ******************* */
			/* do transform */
			Fft(ChannelBuffer.FftBuffer, -1);

			/* this is the analysis step */
			int32 StartBin = (InChannelIdx > 0) ? BottomStereoBin : 0;

			for (K = StartBin; K <= HalfFftFrameSize; K++)
			{
				/* de-interlace FFT buffer */
				Real = ChannelBuffer.FftBuffer[2 * K];
				Image = ChannelBuffer.FftBuffer[2 * K + 1];

				/* compute magnitude and phase */
				Magnitude = 2. * FMath::Sqrt(Real * Real + Image * Image);
				Phase = FMath::Atan2(Image, Real);

				/* compute phase difference */
				Temp = Phase - ChannelBuffer.LastPhase[K];
				ChannelBuffer.LastPhase[K] = float(Phase);

				/* subtract expected phase difference */
				Temp -= (double)K * ExpectedPhaseDifferent;

				/* map delta phase into +/- Pi interval */
				Qpd = int32(Temp / UE_PI);
				if (Qpd >= 0)
				{
					Qpd += Qpd & 1;
				}
				else
				{
					Qpd -= Qpd & 1;
				}
				Temp -= UE_PI * (double)Qpd;

				/* get deviation from bin frequency from the +/- Pi interval */
				Temp = OverlapFactor * Temp / (2. * UE_PI);

				/* compute the K-th partials' true frequency */
				Temp = (double)K * FreqPerFftBin + Temp * FreqPerFftBin;

				/* store magnitude and true frequency in analysis arrays */
				MeasuredBinMagnitude[K] = float(Magnitude);
				MeasuredBinFreq[K] = float(Temp);
			}

			/* ***************** PROCESSING ******************* */
			/* this does the actual pitch shifting */
			FMemory::Memset(SynthesizedBinMagnitude, 0, FftFrameSize * sizeof(float));
			FMemory::Memset(SynthesizedBinFreq, 0, FftFrameSize * sizeof(float));
			for (K = 0; K <= HalfFftFrameSize; K++)
			{
				Index = int32(K * InPitchShift);
				if (Index <= HalfFftFrameSize)
				{
					SynthesizedBinMagnitude[Index] += MeasuredBinMagnitude[K];
					SynthesizedBinFreq[Index] = MeasuredBinFreq[K] * InPitchShift;
				}
			}

			/* ***************** SYNTHESIS ******************* */
			/* this is the synthesis step */
			for (K = 0; K <= HalfFftFrameSize; K++)
			{
				/* get magnitude and true frequency from synthesis arrays */
				Magnitude = SynthesizedBinMagnitude[K];
				Temp = SynthesizedBinFreq[K];

				/* subtract bin mid frequency */
				Temp -= (double)K * FreqPerFftBin;

				/* get bin deviation from freq deviation */
				Temp /= FreqPerFftBin;

				/* take osamp into account */
				Temp = 2. * UE_PI * Temp / OverlapFactor;

				/* add the overlap phase advance back in */
				Temp += (double)K * ExpectedPhaseDifferent;

				/* accumulate delta phase to get bin phase */
				ChannelBuffer.SummedPhase[K] += float(Temp);
				Phase = ChannelBuffer.SummedPhase[K];

				/* get Real and imag part and re-interleave */
				ChannelBuffer.FftBuffer[2 * K] = float(Magnitude * FMath::Cos(Phase));
				ChannelBuffer.FftBuffer[2 * K + 1] = float(Magnitude * FMath::Sin(Phase));
			}

			/* zero negative frequencies */
			for (K = FftFrameSize + 2; K < 2 * FftFrameSize; K++)
			{
				ChannelBuffer.FftBuffer[K] = 0.;
			}

			/* do inverse transform */
			Fft(ChannelBuffer.FftBuffer, 1);

			/* do windowing and add to output accumulator */
			for (K = 0; K < FftFrameSize; K++)
			{
				ChannelBuffer.OutputAccumulator[K] += float(2.0 * Window[K] * ChannelBuffer.FftBuffer[2 * K] / (HalfFftFrameSize * OverlapFactor));
			}

			for (K = 0; K < StepSize; K++)
			{
				ChannelBuffer.OutPCMBuffer[K] = ChannelBuffer.OutputAccumulator[K];
			}

			/* shift accumulator */
			FMemory::Memmove(ChannelBuffer.OutputAccumulator, ChannelBuffer.OutputAccumulator + StepSize, FftFrameSize * sizeof(float));

			/* move input FIFO */
			FMemory::Memmove(ChannelBuffer.InPCMBuffer, ChannelBuffer.InPCMBuffer + StepSize, InFifoLatency * sizeof(float));
		}
	}
}

void FSmbPitchShifter::PitchShift(float InPitchShift, float InSpeed, TAudioBuffer<float>& OutBuffer)
{
	check(OutBuffer.GetNumValidChannels() == InputBuffer.GetNumValidChannels());
	check(InSpeed == 1.0f);
	check(OutBuffer.GetNumValidFrames() == InputBuffer.GetNumValidFrames());
	check(FftFrameSize > 8); // assure some minimum fft size!

	double Magnitude;
	double Phase;
	double Temp;
	double Real;
	double Image;
	int32 Idx;
	int32 K;
	int32 Qpd;
	int32 Index;

	if (ChannelBuffers[0].OutWriteIndex == 0)
	{
		ChannelBuffers[0].OutWriteIndex = InFifoLatency;
	}
	int32 InNumSamples = InputBuffer.GetNumValidFrames();
	int32 NumInChannels = InputBuffer.GetNumValidChannels();

	/* main processing loop */
	for (Idx = 0; Idx < InNumSamples; Idx++)
	{
		/* As long as we have not yet collected enough data just read in */
		ChannelBuffers[NumInChannels].InPCMBuffer[ChannelBuffers[0].OutWriteIndex] = 0.0f;
		for (int32 ChIdx = 0; ChIdx < NumInChannels; ++ChIdx)
		{
			ChannelBuffers[ChIdx].InPCMBuffer[ChannelBuffers[0].OutWriteIndex] = InputBuffer.GetData()[ChIdx][Idx];
			ChannelBuffers[NumInChannels].InPCMBuffer[ChannelBuffers[0].OutWriteIndex] += InputBuffer.GetData()[ChIdx][Idx];
			OutBuffer.GetData()[ChIdx][Idx] = ChannelBuffers[ChIdx].OutPCMBuffer[ChannelBuffers[0].OutWriteIndex - InFifoLatency];
		}
		ChannelBuffers[0].OutWriteIndex++;

		/* now we have enough data for processing */
		if (ChannelBuffers[0].OutWriteIndex >= FftFrameSize)
		{
			ChannelBuffers[0].OutWriteIndex = InFifoLatency;

			for (int32 ChIdx = 0; ChIdx < NumInChannels + 1; ChIdx++)
			{
				/* do windowing and re,im interleave */
				for (K = 0; K < FftFrameSize; K++)
				{
					ChannelBuffers[ChIdx].FftBuffer[2 * K] = float(ChannelBuffers[ChIdx].InPCMBuffer[K] * Window[K]);
					ChannelBuffers[ChIdx].FftBuffer[2 * K + 1] = 0.;
				}

				/* ***************** ANALYSIS ******************* */
				/* do transform */
				Fft(ChannelBuffers[ChIdx].FftBuffer, -1);
			}

			for (int32 ChIdx = 0; ChIdx < NumInChannels; ChIdx++)
			{
				FChannelBuffer& ChannelBuffer = ChannelBuffers[ChIdx];

				/* this is the analysis step */
				for (K = 0; K <= HalfFftFrameSize; K++)
				{
					/* de-interlace FFT buffer */
					if (K < BottomStereoBin)
					{
						Real = ChannelBuffers[NumInChannels].FftBuffer[2 * K];
						Image = ChannelBuffers[NumInChannels].FftBuffer[2 * K + 1];
					}
					else
					{
						Real = ChannelBuffer.FftBuffer[2 * K];
						Image = ChannelBuffer.FftBuffer[2 * K + 1];
					}

					/* compute magnitude and phase */
					Magnitude = 2. * FMath::Sqrt(Real * Real + Image * Image);
					Phase = FMath::Atan2(Image, Real);

					/* compute phase difference */
					Temp = Phase - ChannelBuffer.LastPhase[K];
					ChannelBuffer.LastPhase[K] = float(Phase);

					/* subtract expected phase difference */
					Temp -= (double)K * ExpectedPhaseDifferent;

					/* map delta phase into +/- Pi interval */
					Qpd = int32(Temp / UE_PI);
					if (Qpd >= 0)
					{
						Qpd += Qpd & 1;
					}
					else 
					{
						Qpd -= Qpd & 1;
					}
					Temp -= UE_PI * (double)Qpd;

					/* get deviation from bin frequency from the +/- Pi interval */
					Temp = OverlapFactor * Temp / (2. * UE_PI);

					/* compute the K-th partials' true frequency */
					Temp = (double)K * FreqPerFftBin + Temp * FreqPerFftBin;

					/* store magnitude and true frequency in analysis arrays */
					MeasuredBinMagnitude[K] = float(Magnitude);
					MeasuredBinFreq[K] = float(Temp);
				}

				/* ***************** PROCESSING ******************* */
				/* this does the actual pitch shifting */
				FMemory::Memset(SynthesizedBinMagnitude, 0, FftFrameSize * sizeof(float));
				FMemory::Memset(SynthesizedBinFreq, 0, FftFrameSize * sizeof(float));
				for (K = 0; K <= HalfFftFrameSize; K++)
				{
					Index = int32(K * InPitchShift);
					if (Index <= HalfFftFrameSize)
					{
						SynthesizedBinMagnitude[Index] += MeasuredBinMagnitude[K];
						SynthesizedBinFreq[Index] = MeasuredBinFreq[K] * InPitchShift;
					}
				}

				/* ***************** SYNTHESIS ******************* */
				/* this is the synthesis step */
				for (K = 0; K <= HalfFftFrameSize; K++)
				{
					/* get magnitude and true frequency from synthesis arrays */
					Magnitude = SynthesizedBinMagnitude[K];
					Temp = SynthesizedBinFreq[K];

					/* subtract bin mid frequency */
					Temp -= (double)K * FreqPerFftBin;

					/* get bin deviation from freq deviation */
					Temp /= FreqPerFftBin;

					/* take osamp into account */
					Temp = 2. * UE_PI * Temp / OverlapFactor;

					/* add the overlap phase advance back in */
					Temp += (double)K * ExpectedPhaseDifferent;

					/* accumulate delta phase to get bin phase */
					ChannelBuffer.SummedPhase[K] += float(Temp);
					Phase = ChannelBuffer.SummedPhase[K];

					/* get Real and imag part and re-interleave */
					ChannelBuffer.FftBuffer[2 * K] = float(Magnitude * FMath::Cos(Phase));
					ChannelBuffer.FftBuffer[2 * K + 1] = float(Magnitude * FMath::Sin(Phase));
				}

				/* zero negative frequencies */
				for (K = FftFrameSize + 2; K < 2 * FftFrameSize; K++)
				{
					ChannelBuffer.FftBuffer[K] = 0.;
				}

				/* do inverse transform */
				Fft(ChannelBuffer.FftBuffer, 1);

				/* do windowing and add to output accumulator */
				for (K = 0; K < FftFrameSize; K++)
				{
					ChannelBuffer.OutputAccumulator[K] += float(2.0 * Window[K] * ChannelBuffer.FftBuffer[2 * K] / (HalfFftFrameSize * OverlapFactor));
				}

				for (K = 0; K < StepSize; K++)
				{
					ChannelBuffer.OutPCMBuffer[K] = ChannelBuffer.OutputAccumulator[K];
				}

				/* shift accumulator */
				FMemory::Memmove(ChannelBuffer.OutputAccumulator, ChannelBuffer.OutputAccumulator + StepSize, FftFrameSize * sizeof(float));

				/* move input FIFO */
				FMemory::Memmove(ChannelBuffer.InPCMBuffer, ChannelBuffer.InPCMBuffer + StepSize, InFifoLatency * sizeof(float));
			}
			/* move mono input FIFO */
			FMemory::Memmove(ChannelBuffers[NumInChannels].InPCMBuffer, ChannelBuffers[NumInChannels].InPCMBuffer + StepSize, InFifoLatency * sizeof(float));
		}
	}
	InputFrameOffset = 0;
}

void FSmbPitchShifter::StereoPitchShift(float InPitchShift, int32 InNumSamples,
	float* inLeftData, float* inRightData,
	float* outLeftData, float* outRightData)
{
	double Magnitude;
	double Phase;
	double Temp;
	double Real;
	double Image;
	int32 Idx;
	int32 K;
	int32 Qpd;
	int32 Index;

	check(NumChannels == 2);
	check(FftFrameSize > 8); // assure some minimum fft size!

	if (ChannelBuffers[0].OutWriteIndex == 0)
	{
		ChannelBuffers[0].OutWriteIndex = InFifoLatency;
	}

	/* main processing loop */
	for (Idx = 0; Idx < InNumSamples; Idx++)
	{
		/* As long as we have not yet collected enough data just read in */
		ChannelBuffers[0].InPCMBuffer[ChannelBuffers[0].OutWriteIndex] = inLeftData[Idx];
		ChannelBuffers[1].InPCMBuffer[ChannelBuffers[0].OutWriteIndex] = inRightData[Idx];
		ChannelBuffers[2].InPCMBuffer[ChannelBuffers[0].OutWriteIndex] = (inLeftData[Idx] + inRightData[Idx]) * 0.5f;
		outLeftData[Idx] = ChannelBuffers[0].OutPCMBuffer[ChannelBuffers[0].OutWriteIndex - InFifoLatency];
		outRightData[Idx] = ChannelBuffers[1].OutPCMBuffer[ChannelBuffers[0].OutWriteIndex - InFifoLatency];
		ChannelBuffers[0].OutWriteIndex++;

		/* now we have enough data for processing */
		if (ChannelBuffers[0].OutWriteIndex >= FftFrameSize)
		{
			ChannelBuffers[0].OutWriteIndex = InFifoLatency;

			for (int32 ChIdx = 0; ChIdx < 3; ChIdx++)
			{
				/* do windowing and re,im interleave */
				for (K = 0; K < FftFrameSize; K++)
				{
					ChannelBuffers[ChIdx].FftBuffer[2 * K] = float(ChannelBuffers[ChIdx].InPCMBuffer[K] * Window[K]);
					ChannelBuffers[ChIdx].FftBuffer[2 * K + 1] = 0.;
				}

				/* ***************** ANALYSIS ******************* */
				/* do transform */
				Fft(ChannelBuffers[ChIdx].FftBuffer, -1);
			}

			for (int32 ChIdx = 0; ChIdx < 2; ChIdx++)
			{
				FChannelBuffer& ChannelBuffer = ChannelBuffers[ChIdx];

				/* this is the analysis step */
				for (K = 0; K <= HalfFftFrameSize; K++)
				{
					/* de-interlace FFT buffer */
					if (K < BottomStereoBin)
					{
						Real = ChannelBuffers[2].FftBuffer[2 * K];
						Image = ChannelBuffers[2].FftBuffer[2 * K + 1];
					}
					else
					{
						Real = ChannelBuffer.FftBuffer[2 * K];
						Image = ChannelBuffer.FftBuffer[2 * K + 1];
					}

					/* compute magnitude and phase */
					Magnitude = 2. * FMath::Sqrt(Real * Real + Image * Image);
					Phase = FMath::Atan2(Image, Real);

					/* compute phase difference */
					Temp = Phase - ChannelBuffer.LastPhase[K];
					ChannelBuffer.LastPhase[K] = float(Phase);

					/* subtract expected phase difference */
					Temp -= (double)K * ExpectedPhaseDifferent;

					/* map delta phase into +/- Pi interval */
					Qpd = int32(Temp / UE_PI);
					if (Qpd >= 0) Qpd += Qpd & 1;
					else Qpd -= Qpd & 1;
					Temp -= UE_PI * (double)Qpd;

					/* get deviation from bin frequency from the +/- Pi interval */
					Temp = OverlapFactor * Temp / (2. * UE_PI);

					/* compute the K-th partials' true frequency */
					Temp = (double)K * FreqPerFftBin + Temp * FreqPerFftBin;

					/* store magnitude and true frequency in analysis arrays */
					MeasuredBinMagnitude[K] = float(Magnitude);
					MeasuredBinFreq[K] = float(Temp);
				}

				/* ***************** PROCESSING ******************* */
				/* this does the actual pitch shifting */
				FMemory::Memset(SynthesizedBinMagnitude, 0, FftFrameSize * sizeof(float));
				FMemory::Memset(SynthesizedBinFreq, 0, FftFrameSize * sizeof(float));
				for (K = 0; K <= HalfFftFrameSize; K++)
				{
					Index = int32(K * InPitchShift);
					if (Index <= HalfFftFrameSize)
					{
						SynthesizedBinMagnitude[Index] += MeasuredBinMagnitude[K];
						SynthesizedBinFreq[Index] = MeasuredBinFreq[K] * InPitchShift;
					}
				}

				/* ***************** SYNTHESIS ******************* */
				/* this is the synthesis step */
				for (K = 0; K <= HalfFftFrameSize; K++)
				{
					/* get magnitude and true frequency from synthesis arrays */
					Magnitude = SynthesizedBinMagnitude[K];
					Temp = SynthesizedBinFreq[K];

					/* subtract bin mid frequency */
					Temp -= (double)K * FreqPerFftBin;

					/* get bin deviation from freq deviation */
					Temp /= FreqPerFftBin;

					/* take osamp into account */
					Temp = 2. * UE_PI * Temp / OverlapFactor;

					/* add the overlap phase advance back in */
					Temp += (double)K * ExpectedPhaseDifferent;

					/* accumulate delta phase to get bin phase */
					ChannelBuffer.SummedPhase[K] += float(Temp);
					Phase = ChannelBuffer.SummedPhase[K];

					/* get real and imag part and re-interleave */
					ChannelBuffer.FftBuffer[2 * K] = float(Magnitude * FMath::Cos(Phase));
					ChannelBuffer.FftBuffer[2 * K + 1] = float(Magnitude * FMath::Sin(Phase));
				}

				/* zero negative frequencies */
				for (K = FftFrameSize + 2; K < 2 * FftFrameSize; K++)
				{
					ChannelBuffer.FftBuffer[K] = 0.;
				}

				/* do inverse transform */
				Fft(ChannelBuffer.FftBuffer, 1);

				/* do windowing and add to output accumulator */
				for (K = 0; K < FftFrameSize; K++)
				{
					ChannelBuffer.OutputAccumulator[K] += float(2.0 * Window[K] * ChannelBuffer.FftBuffer[2 * K] / (HalfFftFrameSize * OverlapFactor));
				}

				for (K = 0; K < StepSize; K++)
				{
					ChannelBuffer.OutPCMBuffer[K] = ChannelBuffer.OutputAccumulator[K];
				}

				/* shift accumulator */
				FMemory::Memmove(ChannelBuffer.OutputAccumulator, ChannelBuffer.OutputAccumulator + StepSize, FftFrameSize * sizeof(float));

				/* move input FIFO */
				FMemory::Memmove(ChannelBuffer.InPCMBuffer, ChannelBuffer.InPCMBuffer + StepSize, InFifoLatency * sizeof(float));
			}
			/* move mono input FIFO */
			FMemory::Memmove(ChannelBuffers[2].InPCMBuffer, ChannelBuffers[2].InPCMBuffer + StepSize, InFifoLatency * sizeof(float));
		}
	}
}

void FSmbPitchShifter::StereoPitchShift(float InPitchShift, int32 numOutputFrame, float* outLeftData, float* outRightData)
{
	StereoPitchShift(InPitchShift, numOutputFrame, InputBuffer.GetRawChannelData(0), InputBuffer.GetRawChannelData(1), outLeftData, outRightData);
}

void FSmbPitchShifter::Fft(float* FftBuffer, int32 InSign)
{
	// appears to be the  Cooley-Tukey algorithm
	float RealW, ImagW, Angle, *DataPtr1, *DataPtr2, Temp;
	float RealT, ImagT, RealU, ImagU, *DataPtr1R, *DataPtr1I, *DataPtr2R, *DataPtr2I;
	int32 Idx, BitMask, j, le, le2, K;

	for (Idx = 2; Idx < 2 * FftFrameSize - 2; Idx += 2)
	{
		for (BitMask = 2, j = 0; BitMask < 2 * FftFrameSize; BitMask <<= 1)
		{
			if (Idx & BitMask) j++;
			j <<= 1;
		}
		if (Idx < j)
		{
			DataPtr1 = FftBuffer + Idx; 
			DataPtr2 = FftBuffer + j;
			
			Temp = *DataPtr1; 
			
			*(DataPtr1++) = *DataPtr2;
			*(DataPtr2++) = Temp; 
			
			Temp = *DataPtr1;

			*DataPtr1 = *DataPtr2; 
			*DataPtr2 = Temp;
		}
	}
	for (K = 0, le = 2; K < (int32)(FMath::Log2((double)FftFrameSize) + .5); K++)
	{
		le <<= 1;
		le2 = le >> 1;
		RealU = 1.0;
		ImagU = 0.0;
		Angle = UE_PI / (le2 >> 1);
		RealW = FMath::Cos(Angle);
		ImagW = InSign * FMath::Sin(Angle);
		for (j = 0; j < le2; j += 2)
		{
			DataPtr1R = FftBuffer + j; 
			DataPtr1I = DataPtr1R + 1;
			
			DataPtr2R = DataPtr1R + le2; 
			DataPtr2I = DataPtr2R + 1;
			
			for (Idx = j; Idx < 2 * FftFrameSize; Idx += le)
			{
				RealT = *DataPtr2R * RealU - *DataPtr2I * ImagU;
				ImagT = *DataPtr2R * ImagU + *DataPtr2I * RealU;
				*DataPtr2R = *DataPtr1R - RealT; 
				*DataPtr2I = *DataPtr1I - ImagT;
				*DataPtr1R += RealT; 
				*DataPtr1I += ImagT;
				DataPtr1R += le; 
				DataPtr1I += le;
				DataPtr2R += le; 
				DataPtr2I += le;
			}
			RealT = RealU * RealW - ImagU * ImagW;
			ImagU = RealU * ImagW + ImagU * RealW;
			RealU = RealT;
		}
	}
}
