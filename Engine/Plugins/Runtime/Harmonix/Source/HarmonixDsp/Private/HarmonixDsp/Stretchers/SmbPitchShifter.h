// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/****************************************************************************
* NOTE: This is a HEAVILY modified version of the original smbPitchShift.cpp
* that was written by Stephan M Bernsee, and made public available under the
* "Wide Open License." The link to the original code is...
* http://blogs.zynaptiq.com/bernsee/repo/smbPitchShift.cpp
* The original copyright notice and license statement are below.
*
* One more note: the original comment section refers to a web page for
* more information about the "WOL". That link is stale. At the time of this
* writing the correct link is...
* http://www.dspguru.com/wide-open-license
*****************************************************************************/

/****************************************************************************
*
* NAME: SmbPitchShifter.cpp
* VERSION: 1.2
* HOME URL: http://blogs.zynaptiq.com/bernsee
* KNOWN BUGS: none
*
* SYNOPSIS: Routine for doing pitch shifting while maintaining
* duration using the Short Time Fourier Transform.
*
* DESCRIPTION: The routine takes a pitchShift factor value which is between 0.5
* (one octave down) and 2. (one octave up). A value of exactly 1 does not change
* the pitch. numSampsToProcess tells the routine how many samples in indata[0...
* numSampsToProcess-1] should be pitch shifted and moved to outdata[0 ...
* numSampsToProcess-1]. The two buffers can be identical (ie. it can process the
* data in-place). fftFrameSize defines the FFT frame size used for the
* processing. Typical values are 1024, 2048 and 4096. It may be any value <=
* MAX_FRAME_LENGTH but it MUST be a power of 2. osamp is the STFT
* oversampling factor which also determines the overlap between adjacent STFT
* frames. It should at least be 4 for moderate scaling ratios. A value of 32 is
* recommended for best quality. sampleRate takes the sample rate for the signal
* in unit Hz, ie. 44100 for 44.1 kHz audio. The data passed to the routine in
* indata[] should be in the range [-1.0, 1.0), which is also the output range
* for the data, make sure you scale the data accordingly (for 16bit signed integers
* you would have to divide (and multiply) by 32768).
*
* COPYRIGHT 1999-2015 Stephan M. Bernsee <s.bernsee [AT] zynaptiq [DOT] com>
*
* 						The Wide Open License (WOL)
*
* Permission to use, copy, modify, distribute and sell this software and its
* documentation for any purpose is hereby granted without fee, provided that
* the above copyright notice and this license appear in all source copies.
* THIS SOFTWARE IS PROVIDED "AS IS" WITHOUT EXPRESS OR IMPLIED WARRANTY OF
* ANY KIND. See http://www.dspguru.com/wol.htm for more information.
*
*****************************************************************************/

#include "HarmonixDsp/StretcherAndPitchShifter.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HAL/Platform.h"

#define UE_API HARMONIXDSP_API

namespace EWindowSize
{
	static constexpr int32 k128 = 128;
	static constexpr int32 k256 = 256;
	static constexpr int32 k512 = 512;
	static constexpr int32 k1024 = 1024;
	static constexpr int32 k2048 = 2048;
	static constexpr int32 k4096 = 4096;

	static constexpr int32 Max = k4096;
	static constexpr int32 Default = k1024;

	static const int32 MaxImag = Max * 2;
	static const int32 MaxReal = (Max / 2) + 1;

	static constexpr bool IsValid(int32 Size)
	{
		return Size == k128 || Size == k256 || Size == k512 || Size == k1024 || Size == k2048 || Size == k4096;
	}
}

class FSmbPitchShifter : public IStretcherAndPitchShifter
{
public:

	UE_API FSmbPitchShifter(FName InFactoryName);
	UE_API FSmbPitchShifter(FName InFactoryName, float InSampleRate);

	UE_API FSmbPitchShifter();
	UE_API FSmbPitchShifter(float InSampleRate);

	UE_API virtual ~FSmbPitchShifter();

	static UE_API void Init();

	void Configure() { Configure(kDefaultNumChannels, EWindowSize::Default, kDefaultOverlapFactor); }
	UE_API void Configure(int32 InNumChannels, int32 InWindowSize, int32 InOverlapFactor);
	UE_API void Configure(int32 InNumChannels, int32 InWindowSize, int32 InOverlapFactor, int32 MaxInputSamplesToCache);

	UE_API virtual int32 GetLatencySamples() const override;

	UE_API void SetMonoCutoff(float FreqHz);
	UE_API virtual void SetSampleRateAndReset(float InSampleRate) override;
	virtual int32  GetInputFramesNeeded(int32 InNumOutFramesNeeded, float PitchShift, float SpeedShift) override { return InNumOutFramesNeeded; }
	UE_API virtual void StereoPitchShift(float InPitchShift, int32 InNumSampsToProcess,
		float* InLeftData, float* InRightData,
		float* OutLeftData, float* OutRightData) override;

	UE_API virtual double Render(TAudioBuffer<float>& OutputData, double Pos, int32 MaxFrame, double ResampleInc, double PitchShift, double Speed, bool MaintainPitchWhenSpeedChanges, const FGainMatrix& Gain) override;

	UE_API virtual bool TakeInput(TAudioBuffer<float>& InBuffer) override;
	UE_API virtual bool InputSilence(int32 NumFrames) override;
	UE_API virtual void StereoPitchShift(float InPitchShift, int32 InNumOutputFrame, float* OutLeftData, float* OutRightData) override;

	UE_API void PitchShift(float InPitchShift, int32 InNumSampsToProcess, float* InData, float* OutData, int32 ChannelIndex, int32 Stride);

	UE_API virtual void PitchShift(float InPitchShift, float InSpeed, TAudioBuffer<float>& OutBuffer) override;

	static const int32 kDefaultNumChannels = 2;
	static const int32 kDefaultOverlapFactor = 4;

	virtual size_t GetMemoryUsage() const override { return MemoryUsed; }

	virtual void Cleanup() override {}

protected:
	UE_API virtual void SetupAndResetImpl(const FTimeStretchConfig* ShifterSettings) override;

private:
	// A bunch of stereo buffers...
	struct FChannelBuffer
	{
		float InPCMBuffer[EWindowSize::Max];
		float OutPCMBuffer[EWindowSize::Max];
		float FftBuffer[EWindowSize::MaxImag];
		float LastPhase[EWindowSize::MaxReal];
		float SummedPhase[EWindowSize::MaxReal];
		float OutputAccumulator[EWindowSize::MaxImag];
		int32 OutWriteIndex;

		void Reset();
	};
	int32 NumChannels = 0;
	FChannelBuffer* ChannelBuffers = nullptr;

	// Some scratch buffers we don't want to new and delete 
	// every time we need them...
	float MeasuredBinFreq[EWindowSize::Max];
	float MeasuredBinMagnitude[EWindowSize::Max];
	float SynthesizedBinFreq[EWindowSize::Max];
	float SynthesizedBinMagnitude[EWindowSize::Max];

	int32  FftFrameSize;
	int32  HalfFftFrameSize;
	int32  OverlapFactor;
	int32  StepSize;
	int32  InFifoLatency;
	double ExpectedPhaseDifferent;
	double FreqPerFftBin;
	float* Window;
	int32  BottomStereoBin;
	float  SampleRate;
	size_t MemoryUsed = 0;

	TAudioBuffer<float> InputBuffer;
	int32 InputFrameOffset = 0;

	static UE_API float Window128[EWindowSize::k128];
	static UE_API float Window256[EWindowSize::k256];
	static UE_API float Window512[EWindowSize::k512];
	static UE_API float Window1024[EWindowSize::k1024];
	static UE_API float Window2048[EWindowSize::k2048];
	static UE_API float Window4096[EWindowSize::k4096];
	static UE_API bool  WindowsInitialized;

	UE_API void InitializeWindows();
	UE_API void Reset();

	/**
	 * The function Fft is a Fast Fourier Transform (FFT) implementation. 
	 * It takes an array of floats FftBuffer representing the time-domain input signal, 
	 * and an integer sign indicating whether the FFT should be performed in the forward or inverse direction. 
	 * The function modifies the input array FftBuffer to store the output frequency-domain signal.
	 */
	UE_API void Fft(float* FftBuffer, int32 Sign);
};

#undef UE_API
