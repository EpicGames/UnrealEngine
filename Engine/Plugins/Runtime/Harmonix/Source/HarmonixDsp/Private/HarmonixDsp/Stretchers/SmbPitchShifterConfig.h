// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "HarmonixDsp/StretcherAndPitchShifterConfig.h"

#include "SmbPitchShifterConfig.generated.h"

UCLASS(meta = (DisplayName = "SMB Pitch Shifter Settings"))
class USmbPitchShifterConfig : public UStretcherAndPitchShifterConfig
{
	GENERATED_BODY()

public:

	/** If true, allows the stretcher pool to grow beyond DefaultAllocatedStretchers in the editor. */
	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	bool bAllowPoolGrowthInEditor = false;

	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	uint32 DefaultAllocatedStretchers = 12;

	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	uint32 SwitchAllocatedStretchers = 0;

	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	uint32 PS4AllocatedStretchers = 0;

	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	uint32 PS5AllocatedStretchers = 0;

	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	uint32 XboxOneAllocatedStretchers = 0;

	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	uint32 XSXAllocatedStretchers = 0;

	UPROPERTY(config, EditAnywhere, Category = "Pool Config")
	uint32 AndroidAllocatedStretchers = 0;

	uint32 GetNumAllocatedStretchersForPlatform() const
	{
#if defined(PLATFORM_SWITCH) && PLATFORM_SWITCH
		return SwitchAllocatedStretchers;
#elif defined(PLATFORM_PS4) && PLATFORM_PS4
		return PS4AllocatedStretchers;
#elif defined(PLATFORM_PS5) && PLATFORM_PS5
		return PS5AllocatedStretchers;
#elif (defined(PLATFORM_XB1) && PLATFORM_XB1) || (defined(PLATFORM_XBOXONE) && PLATFORM_XBOXONE) ||  (defined(PLATFORM_XBOXONEGDK) && PLATFORM_XBOXONEGDK)
		return XboxOneAllocatedStretchers;
#elif defined(PLATFORM_XSX) && PLATFORM_XSX
		return XSXAllocatedStretchers;
#elif defined(PLATFORM_ANDROID) && PLATFORM_ANDROID
		return AndroidAllocatedStretchers;
#else
		return DefaultAllocatedStretchers;
#endif
	}

	bool ShouldAllowPoolGrowth() const
	{
#if WITH_EDITOR
		return bAllowPoolGrowthInEditor;
#else
		return false;
#endif
	}
};
