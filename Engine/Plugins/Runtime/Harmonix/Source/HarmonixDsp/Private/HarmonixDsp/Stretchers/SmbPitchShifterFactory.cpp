// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Stretchers/SmbPitchShifterFactory.h"
#include "HarmonixDsp/Stretchers/SmbPitchShifterConfig.h"
#include "HarmonixDsp/Stretchers/SmbPitchShifter.h"


static FSmbPitchShifterFactory gSmbPitchShifterFactory;

const FName FSmbPitchShifterFactory::Name = FName(TEXT("SmbPitchShifter"));

const TArray<FName>& FSmbPitchShifterFactory::GetFactoryNames() const
{
	static TArray<FName> FactoryNames;
	if (FactoryNames.IsEmpty())
	{
		FactoryNames.Add(FSmbPitchShifterFactory::Name);
	}
	return FactoryNames;
}

TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> FSmbPitchShifterFactory::CreateStretcherAndPitchShifter(float InSampleRate)
{
	TSharedPtr<FSmbPitchShifter, ESPMode::ThreadSafe> SmbShifter = MakeShared<FSmbPitchShifter>(Name, InSampleRate);
	SmbShifter->Configure();
	return SmbShifter;
}

void FSmbPitchShifterFactory::OnClientAdded(void* Client, float InSampleRate)
{
	if (PitchShifterPool.IsInitialized())
	{
		check(FactorySampleRate == InSampleRate);
		return;
	}

	FactorySampleRate = InSampleRate;
	const USmbPitchShifterConfig* Config = GetDefault<USmbPitchShifterConfig>();
	check(Config);

	if (!PitchShifterPool.IsInitialized() && Config->GetNumAllocatedStretchersForPlatform() > 0)
	{
		auto FactoryMethod = [SampleRate = FactorySampleRate]() -> TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe>
		{
			return CreateStretcherAndPitchShifter(SampleRate);
		};


		PitchShifterPool.Allocate(Config->GetNumAllocatedStretchersForPlatform(), FactoryMethod, Config->ShouldAllowPoolGrowth());
	}
}

void FSmbPitchShifterFactory::OnClientRemoved(void* Client)
{
	if (GetNumClients() == 0)
	{
		PitchShifterPool.Reset();
	}
}

TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> FSmbPitchShifterFactory::GetFreePitchShifter(const FTimeStretchConfig& InConfig)
{
	return PitchShifterPool.GetFreeShared();
}

void FSmbPitchShifterFactory::ReleasePitchShifter(TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> InShifter)
{
	PitchShifterPool.ReturnToPool(InShifter);
}

bool FSmbPitchShifterFactory::HasFreePitchShifters(const FTimeStretchConfig& InConfig) const
{
	return (PitchShifterPool.GetNumFree() > 0) || PitchShifterPool.CanGrow();
}

size_t FSmbPitchShifterFactory::GetMemoryUsedByPool() const
{
	size_t TotalBytes = 0;
	PitchShifterPool.ForEachObject([&TotalBytes](FShifterPool::FSharedObjectPtr Shifter){ TotalBytes += Shifter->GetMemoryUsage();});
	return TotalBytes;
}

