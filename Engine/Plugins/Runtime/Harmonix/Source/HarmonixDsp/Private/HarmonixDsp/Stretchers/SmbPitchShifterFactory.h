// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDsp/StretcherAndPitchShifterFactory.h"
#include "HarmonixDsp/ObjectPool.h"
#include "UObject/NameTypes.h"

class FSmbPitchShifterFactory : public IStretcherAndPitchShifterFactory
{
public:
	static const FName Name;

	virtual const TArray<FName>& GetFactoryNames() const override;

	virtual TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> GetFreePitchShifter(const FTimeStretchConfig& InConfig) override;

	virtual void ReleasePitchShifter(TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> InShifter) override;

	virtual bool HasFreePitchShifters(const FTimeStretchConfig& InConfig) const override;

	virtual size_t GetMemoryUsedByPool() const override;

protected:
	virtual void OnClientAdded(void* Client, float SampleRate) override;

	virtual void OnClientRemoved(void* Client) override;

private:
	using FShifterPool = Harmonix::TDynamicObjectPool<IStretcherAndPitchShifter>;
	FShifterPool PitchShifterPool;

	float FactorySampleRate = 0;

	static TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> CreateStretcherAndPitchShifter(float InSampleRate);
};