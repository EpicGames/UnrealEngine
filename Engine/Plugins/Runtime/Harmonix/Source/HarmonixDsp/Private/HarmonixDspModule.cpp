// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDspModule.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"

#include "HarmonixDsp/GainMatrix.h"
#include "HarmonixDsp/GainTable.h"

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixDspModule, Log, Log)

void FHarmonixDspModule::StartupModule()
{
	FGainMatrix::Init();
	FGainTable::Init();
}

void FHarmonixDspModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHarmonixDspModule, HarmonixDsp);
