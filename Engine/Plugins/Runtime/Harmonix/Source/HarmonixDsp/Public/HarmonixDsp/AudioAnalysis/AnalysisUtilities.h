// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/UnrealMathUtility.h"

struct FHarmonixAudioAnalyzerOutputSettings;

namespace Harmonix::Dsp::AudioAnalysis
{
	/**
	 * @brief Smooth a value based on some input settings
	 * @param TargetEnergyLinear - The current target should be an energy rather than amplitude (squared and non-negative)
	 * @param PreviousEnergyLinear - The previous energy
	 * @param ElapsedMs - The amount of time elapsed since the last call to this function
	 * @param Settings - The settings to use for the scaling and smoothing
	 * @return The smoothed value
	 */
	float SmoothEnergy(
		float TargetEnergyLinear,
		float PreviousEnergyLinear,
		float ElapsedMs,
		const FHarmonixAudioAnalyzerOutputSettings& Settings);

	/**
	@brief Calculates the signal to noise. 96dB is perfect... no noise, all signal.	Big numbers good, small numbers bad
	@param InterleavedInA - Pointer to the first sample in the first buffer. If multichannel, the channels must be interleaved
	@param InterleavedInB - Pointer to the first sample in the second buffer. If multichannel, the channels must be interleaved
	@param NumnChannels - Number of interleaved channels per sample frame
	@param NumSampleFrames - Number of samples frames
	@return Classic PSNR in dB. 96dB is perfect match. Smaller values indicate bigger difference.
	*/
	template <typename T>
	float CalculatePSNR(const T* InterleavedInA, const T* InterleavedInB, const int32 NumChannels, const int32 NumSampleFrames)
	{
		// make sure if this is an integral type it is an int16...
		static_assert(!std::is_integral<T>::value || sizeof(T) == 2);

		double WorstSNR = 96.0f;
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			double ChannelSumSquares = 0.0f;
			const T* SampleSrc1 = InterleavedInA + ChannelIndex;
			const T* SampleSrc2 = InterleavedInB + ChannelIndex;
			for (int32 SampleIndex = 0; SampleIndex < NumSampleFrames; ++SampleIndex)
			{
				if constexpr (std::is_integral<T>::value)
				{
					double S1 = static_cast<double>(*SampleSrc1) / 32768.0;
					double S2 = static_cast<double>(*SampleSrc2) / 32768.0;
					ChannelSumSquares += (double)((S1 - S2) * (S1 - S2));
				}
				else
				{
					double S1 = static_cast<double>(*SampleSrc1);
					double S2 = static_cast<double>(*SampleSrc2);
					ChannelSumSquares += (double)((S1 - S2) * (S1 - S2));
				}
				SampleSrc1 += NumChannels;
				SampleSrc2 += NumChannels;
			}
			ChannelSumSquares /= static_cast<double>(NumSampleFrames);
			double SNR = ChannelSumSquares == 0.0 ? 96.0f : FMath::Clamp(10.0f * FMath::LogX(10.0, 4.0 / ChannelSumSquares), 0.0, 96.0);
			if (SNR < WorstSNR)
			{
				WorstSNR = SNR;
			}
		}
		return WorstSNR;
	}
}
