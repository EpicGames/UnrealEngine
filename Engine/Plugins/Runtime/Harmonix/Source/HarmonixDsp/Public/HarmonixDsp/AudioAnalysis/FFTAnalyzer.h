// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AudioFFT.h"
#include "DSP/SlidingWindow.h"
#include "HarmonixDsp/AudioAnalysis/OutputSettings.h"

#include "FFTAnalyzer.generated.h"

#define UE_API HARMONIXDSP_API

namespace Audio
{
	class IFFTAlgorithm;
}

USTRUCT(BlueprintType)
struct FHarmonixFFTAnalyzerSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Settings")
	int32 FFTSize = 512;

	UPROPERTY(EditAnywhere, Category="Settings")
	float MinFrequencyHz = 20.0f;

	UPROPERTY(EditAnywhere, Category="Settings")
	float MaxFrequencyHz = 5500.0f;

	UPROPERTY(EditAnywhere, Category="Settings")
	bool MelScaleBinning = false;

	UPROPERTY(EditAnywhere, Category="Settings")
	int32 NumResultBins = 256;

	UPROPERTY(EditAnywhere, Category="Settings")
	FHarmonixAudioAnalyzerOutputSettings OutputSettings;
};

USTRUCT(BlueprintType)
struct FHarmonixFFTAnalyzerResults
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category="Results")
	TArray<float> Spectrum;
	
	friend FORCEINLINE uint32 GetTypeHash(const FHarmonixFFTAnalyzerResults& InAnalyzerResults)
	{
		return GetTypeHash(InAnalyzerResults.Spectrum);
	}
};

namespace Harmonix::Dsp::AudioAnalysis
{
	class FFFTAnalyzer
	{
	public:
		inline static constexpr int32 MinFFTSize = 64;
		static_assert(FMath::IsPowerOfTwo(MinFFTSize), "Min FFT size should be a power of two.");
		inline static constexpr int32 MaxFFTSize = 2048;
		static_assert(FMath::IsPowerOfTwo(MaxFFTSize), "Max FFT size should be a power of two.");
		inline static constexpr int32 MaxBins = MaxFFTSize / 2;

		inline static constexpr float MinFrequency = 20.0f;
		inline static constexpr float MaxFrequency = 20000.0f;

		inline static constexpr float MinSmoothingTime = 0.0f;
		inline static constexpr float MaxSmoothingTime = 10000.0f;

		UE_API explicit FFFTAnalyzer(float InSampleRate);

		/**
		 * @brief Set the settings for this analyzer
		 * @param InSettings - The settings to use
		 */
		UE_API void SetSettings(const FHarmonixFFTAnalyzerSettings& InSettings);

		/**
		 * @brief Reset the analyzer to its initial state
		 */
		UE_API void Reset();

		/**
		 * @brief Process a buffer and get the analyzer results
		 * @param InBuffer - The audio buffer to analyze
		 * @param InOutResults - The results from the analysis
		 */
		UE_API void Process(const Audio::FAlignedFloatBuffer& InBuffer, FHarmonixFFTAnalyzerResults& InOutResults);
		
	private:
		const float SampleRate;
		
		FCriticalSection SettingsGuard;
		FHarmonixFFTAnalyzerSettings Settings;
		bool NeedsRecalculate = true;

		struct FState
		{
			TUniquePtr<Audio::IFFTAlgorithm> FFT;
			TUniquePtr<Audio::TSlidingBuffer<float>> SlidingBuffer;
			TUniquePtr<Audio::FWindow> Window;
			TArray<float> WindowedBuffer;
			Audio::AlignedFloatBuffer FFTOutput;
			Audio::AlignedFloatBuffer RawSpectrumOutput;
			TArray<float> RiseTargets;
			TArray<float> BinRanges;
			float FFTScaling = 0.0f;
		};
		FState State;
		
		static UE_API void Recalculate(float SampleRate, FHarmonixFFTAnalyzerSettings& InOutSettings, FState& InOutState);
	};
}

#undef UE_API
