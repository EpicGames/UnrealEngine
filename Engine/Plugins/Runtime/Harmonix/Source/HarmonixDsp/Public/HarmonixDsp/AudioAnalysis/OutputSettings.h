// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OutputSettings.generated.h"

USTRUCT(BlueprintType)
struct FHarmonixAudioAnalyzerOutputSettings
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(UIMin = 0, ClampMin = "0", UIMax = 1000, ClampMax = "1000"))
	float RiseMs = 50.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(UIMin = 0, ClampMin = "0", UIMax = 4000, ClampMax = "4000"))
	float FallMs = 250.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(UIMin = -96, ClampMin = "-96", UIMax = 0, ClampMax = "0"))
	float MaxDecibels = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(UIMin = 3, ClampMin = "3", UIMax = 96, ClampMax = "96"))
	float RangeDecibels = 30.0f;
};
