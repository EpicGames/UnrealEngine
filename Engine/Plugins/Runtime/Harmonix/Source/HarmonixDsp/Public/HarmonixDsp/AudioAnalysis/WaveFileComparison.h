// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Audio.h"

#define UE_API HARMONIXDSP_API

class FString;
class FArchive;

namespace Harmonix::Dsp::AudioAnalysis 
{
	class FWaveFileComparison
	{
	public:
		UE_API bool LoadForCompare(const FString& Wave1FilePath, const FString& Wave2FilePath);
		UE_API bool LoadForCompare(FArchive& Wave1Archive, FArchive& Wave2Archive);

		UE_API float GetPSNR(bool bCommonSizeOnly = true) const;

	private:
		struct FOneWaveFile
		{
			FWaveModInfo Info;
			TArray<uint8> BulkData;

			bool Load(FArchive& Archive);
		};

		FOneWaveFile Wave1;
		FOneWaveFile Wave2;

		bool bOk = false;
	};
}

#undef UE_API
