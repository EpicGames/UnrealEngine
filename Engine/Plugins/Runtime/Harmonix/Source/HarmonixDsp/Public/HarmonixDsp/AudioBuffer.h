// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Harmonix
#include "HarmonixDsp/AudioBufferConfig.h"
#include "HarmonixDsp/AudioBufferConstants.h"
#include "HarmonixDsp/Generate.h"
#include "HarmonixDsp/StridePointer.h"

// UE
#include "Logging/LogMacros.h"
#include "Math/UnrealMath.h"
#include "Math/NumericLimits.h"
#include "Math/VectorRegister.h"

#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "HAL/LowLevelMemTracker.h"


LLM_DECLARE_TAG_API(Harmonix_AudioBuffer, HARMONIXDSP_API);

DECLARE_LOG_CATEGORY_EXTERN(LogAudioBuffer, Log, All);

namespace HarmonixDsp
{
	namespace FAudioBuffer
	{
		void DebugLog(const int16* InData, uint64 InNumSamples);
		void DebugLog(const float* InData, uint64 InNumSamples);
	}
}

enum class EAudioBufferCleanupMode : uint8
{
	Delete,
	DontDelete
};

template <typename DataType>
class TAudioBuffer
{
public:

	TAudioBuffer()
	{
		Initialize();
	}

	void Initialize()
	{
		for (int32 ChannelIndex = 0; ChannelIndex < FAudioBufferConfig::kMaxAudioBufferChannels; ++ChannelIndex)
		{
			MyData[ChannelIndex] = nullptr;
		}

		MaxConfig        = FAudioBufferConfig(EAudioBufferChannelLayout::UnsupportedFormat, 0);
		CleanupMode      = EAudioBufferCleanupMode::DontDelete;
		NumValidChannels = 0;
		ChannelLayout    = EAudioBufferChannelLayout::UnsupportedFormat;
		NumValidFrames   = 0;
		FirstFrame	     = 0;
		NumFrames        = 0;
		LastFrame        = 0;
		IsSilent        = true;
	}

	/**
	 * Clears the buffer and deletes the data if CleanupMode == EAudioBufferCleanupMode::Delete
	 */
	void Reset()
	{
		if (CleanupMode == EAudioBufferCleanupMode::Delete)
		{
			char* DataPtr = (char*)MyData[0];
			check(DataPtr);

			// our buffer has one extra sample at the beginning (and one at the end)
			// to help find buffer over-writes. So we need to back up our pointer by one
			// to make sure we clean up properly

			// we added 16 bytes during configuration to retain alignment
			DataPtr -= 16;

			FMemory::Free(DataPtr);
			MyData[0] = nullptr;
		}
		Initialize();
	}

	void Configure(int32 InNumChannels, int32 InNumFrames, EAudioBufferCleanupMode InCleanupMode, float InSampleRate = 0.0f, bool InInterleaved = false)
	{
		Configure(FAudioBufferConfig(InNumChannels, InNumFrames, InSampleRate, InInterleaved), InCleanupMode);
	}

	void Configure(EAudioBufferChannelLayout InChannelLayout, int32 InNumFrames, EAudioBufferCleanupMode InCleanupMode, float InSampleRate = 0.0f, bool InInterleaved = false)
	{
		Configure(FAudioBufferConfig(InChannelLayout, InNumFrames, InSampleRate, InInterleaved), InCleanupMode);
	}

	void Configure(FAudioBufferConfig InConfig, EAudioBufferCleanupMode InCleanupMode)
	{
		LLM_SCOPE_BYTAG(Harmonix_AudioBuffer);

		Reset();

		check(InConfig.GetNumChannels() > 0);
		check(InConfig.GetNumChannels() <= FAudioBufferConfig::kMaxAudioBufferChannels);

		CleanupMode = InCleanupMode;
		MaxConfig = InConfig;

		check(MaxConfig.GetNumFrames() < MAX_int32);

		if (CleanupMode == EAudioBufferCleanupMode::Delete)
		{
			// We will allocate and own the buffers...

			// Allocate one big buffer for all channels and frames.
			// Allocate 16 extra bytes for 'stomp' detection at the front and 4 
			// extra bytes for stomp detection at the end (hence 20)
			int32 TotalSamples = (MaxConfig.GetNumFrames() * MaxConfig.GetNumChannels());
			//Allocate extras for proper alignment that accounts for stomp detector
			DataType* DataPtr = (DataType*)FMemory::Malloc(sizeof(DataType) * (TotalSamples + 1) + 20, 16); 

			// Set stomp detector...
			{
				uint32* AsWords = reinterpret_cast<uint32*>(DataPtr);
				int32 NumStompWords = 16 / sizeof(uint32);
				for (int32 i = 0; i < NumStompWords; ++i)
				{
					*(AsWords++) = kAudioBufferStompDetector;
				}
				DataPtr = reinterpret_cast<DataType*>(AsWords);

				// Put stomp detector at the end...
				// @NOTE here we're not taking care to ensure 'endAsWords' is 
				// properly aligned to a 4-byte boundary... will that be an 
				// issue on any of the platforms we care about??
				DataType* EndPtr = DataPtr + (MaxConfig.GetNumFrames() * MaxConfig.GetNumChannels());
				uint32* EndAsWords = reinterpret_cast<uint32*>(EndPtr);
				*EndAsWords = (uint32)kAudioBufferStompDetector;
			}

			// Now, how we deal with that buffer depends on whether the data is interleaved or not...
			if (MaxConfig.GetIsInterleaved())
			{
				// one buffer holding interleaved channel data...
				MyData[0] = DataPtr;
			}
			else
			{
				// one buffer for each audio channel...
				for (int32 ch = 0; ch < MaxConfig.GetNumChannels(); ++ch)
				{
					MyData[ch] = DataPtr;
					DataPtr += MaxConfig.GetNumFrames();
				}
			}
		}

		NumValidFrames   = InConfig.GetNumFrames();
		NumValidChannels = InConfig.GetNumChannels();
		ChannelLayout    = InConfig.GetChannelLayout();
		SetFirstFrame(0);
		SetLengthInFrames(NumValidFrames);

		SetIsSilent(false);
	}

	void SetSampleRate(float InSampleRateHz)
	{
		MaxConfig.SetSampleRate(InSampleRateHz);
	}

	bool CheckIntegrity()
	{
		if (CleanupMode == EAudioBufferCleanupMode::Delete)
		{
			DataType ExpectedValue = DataType(kAudioBufferStompDetector);
			DataType* DataPtr = MyData[0];
			int32 Num = MaxConfig.GetNumFrames() * MaxConfig.GetNumChannels();
			check(DataPtr[-1] == ExpectedValue);
			check(DataPtr[Num] == ExpectedValue);
		}
		return true;
	}

	TAudioBuffer(FAudioBufferConfig InConfig, EAudioBufferCleanupMode InCleanupMode)
	{
		Initialize();
		Configure(InConfig, InCleanupMode);
	}

	TAudioBuffer(int32 InNumChannels, int32 InNumFrames, EAudioBufferCleanupMode InCleanupMode)
	{
		Initialize();
		Configure(FAudioBufferConfig(InNumChannels, InNumFrames), InCleanupMode);
	}

	TAudioBuffer(EAudioBufferChannelLayout InChannelLayout, int32 InNumFrames, EAudioBufferCleanupMode InCleanupMode)
	{
		Initialize();
		Configure(FAudioBufferConfig(InChannelLayout, InNumFrames), InCleanupMode);
	}

	~TAudioBuffer()
	{
		Reset();
	}

	bool GetIsSilent() const { return IsSilent; }
	void SetIsSilent(bool InIsSilent) { IsSilent = InIsSilent; }

	bool GetIsInterleaved() const { return MaxConfig.GetIsInterleaved(); }

	int32 GetFreeableSize() const
	{
		if (CleanupMode == EAudioBufferCleanupMode::DontDelete)
		{
			return 0;
		}

		return (int32)(sizeof(DataType) * MaxConfig.GetNumChannels() * MaxConfig.GetNumFrames());
	}

	int32 GetNumValidBytes() const
	{
		return (int32)(sizeof(DataType) * GetNumValidChannels() * GetNumValidFrames());
	}

	static int32 GetBytesPerSample()
	{
		return (int32)sizeof(DataType);
	}

	// frame position assigns a frame number to the first sample in the buffer
	int32 GetFirstFrame() const { return FirstFrame; }
	void SetFirstFrame(int32 InFirstFrame)
	{
		FirstFrame = InFirstFrame;
		LastFrame = FirstFrame + NumFrames - 1;
	}

	// because this buffer could hold multiple frames in a single channel (aka interleaved data)
	// then the channel size does not necessarily represent the number of frames
	int32 GetLengthInFrames() const { return NumFrames; }
	void SetLengthInFrames(int32 InLength)
	{
		NumFrames = InLength;
		LastFrame = FirstFrame + NumFrames - 1;
	}

	size_t GetLastFrame() const { return LastFrame; }

	EAudioBufferCleanupMode GetCleanupMode() const { return CleanupMode; }

	FORCEINLINE bool ContainsFrame(int32 InFrameNumber)
	{
		return GetFirstFrame() <= InFrameNumber && InFrameNumber <= (int32)GetLastFrame();
	}

	void ZeroValidFrames()
	{
		if (NumValidFrames == MaxConfig.GetNumFrames())
		{
			// a little faster
			FillData(0);
		}
		else
		{
			// a little slower
			FillValidFrames(0);
		}

		SetIsSilent(true);
	}

	void ZeroData()
	{
		FillData(0);
		SetIsSilent(true);
	}

	void FillValidFrames(char InChr)
	{
		if (GetIsInterleaved())
		{
			FMemory::Memset(MyData[0], InChr, sizeof(DataType) * NumValidFrames * GetNumValidChannels());
		}
		else
		{
			for (int32 ch = 0; ch < GetNumValidChannels(); ++ch)
			{
				FMemory::Memset(MyData[ch], InChr, sizeof(DataType) * NumValidFrames);
			}
		}
	}

	void Fill(DataType InValue)
	{
		if (GetIsInterleaved())
		{
			DataType* DataPtr = GetValidChannelData(0);
			for (int32 idx = 0; idx < GetNumValidFrames() * GetNumValidChannels(); ++idx)
			{
				DataPtr[idx] = InValue;
			}
		}
		else
		{
			for (int32 ch = 0; ch < GetNumValidChannels(); ++ch)
			{
				DataType* DataPtr = GetValidChannelData(ch);
				for (int32 idx = 0; idx < GetNumValidFrames(); ++idx)
				{
					DataPtr[idx] = InValue;
				}
			}
		}
	}

	void FillData(char InChr)
	{
		if (CleanupMode == EAudioBufferCleanupMode::Delete || MaxConfig.GetIsInterleaved())
		{
			FMemory::Memset(MyData[0], InChr, sizeof(DataType) * MaxConfig.GetNumTotalSamples());
		}
		else
		{
			for (int32 ch = 0; ch < GetNumValidChannels(); ++ch)
			{
				FMemory::Memset(MyData[ch], InChr, sizeof(DataType) * MaxConfig.GetNumFrames());
			}
		}
	}

	/**
	* fill the buffer with white noise
	* this function only works for buffers of floats.
	* @param InGain (optional) the InGain to apply to the white noise. defaults to 1 (0 dBFS).
	* @returns description of the return type
	*/
	void FillWithWhiteNoise(float InGain = 1.0f)
	{
		for (int32 ch = 0; ch < GetNumValidChannels(); ++ch)
		{
			HarmonixDsp::GenerateWhiteNoiseEq(MyData[ch], GetNumValidFrames(), InGain);
		}
	}


	DataType GetPeak(int32 ChannelIndex) const
	{
		check(ChannelIndex < GetNumValidChannels());

		DataType OutPeak = DataType(0);

		if (MaxConfig.GetIsInterleaved())
		{
			DataType* DataPtr = MyData[0 + ChannelIndex];
			for (int32 FrameIdx = 0; FrameIdx < NumValidFrames; ++FrameIdx)
			{
				DataType Sample = FMath::Abs(*DataPtr);
				if (Sample > OutPeak)
				{
					OutPeak = Sample;
				}
				DataPtr += NumValidChannels;
			}
		}
		else
		{
			DataType* DataPtr = MyData[ChannelIndex];
			for (int32 FrameIdx = 0; FrameIdx < NumValidFrames; ++FrameIdx)
			{
				DataType Sample = FMath::Abs(DataPtr[FrameIdx]);
				if (Sample > OutPeak)
				{
					OutPeak = Sample;
				}
			}
		}

		return OutPeak;
	}

	DataType GetPeak() const
	{
		DataType OutPeak = DataType(0);
		for (int32 ChannelIdx = 0; ChannelIdx < NumValidChannels; ++ChannelIdx)
		{
			DataType Sample = GetPeak(ChannelIdx);
			if (Sample > OutPeak)
			{
				OutPeak = Sample;
			}
		}
		return OutPeak;
	}

	void ComputeOverview(uint32 InFramesPerOverviewSample, TAudioBuffer<DataType>& OutOverviewMax, TAudioBuffer<DataType>& OutOverviewMin) const
	{
		check(!MaxConfig.GetIsInterleaved());
		check(OutOverviewMax.GetNumValidChannels() == GetNumValidChannels());
		check(OutOverviewMax.GetNumValidFrames() == OutOverviewMin.GetNumValidFrames());
		check(OutOverviewMax.GetNumValidChannels() == OutOverviewMin.GetNumValidChannels());

		OutOverviewMax.ZeroData();
		OutOverviewMin.ZeroData();

		uint32 NumOverviewFrames = FMath::Min(OutOverviewMax.GetNumValidFrames(), OutOverviewMin.GetNumValidFrames() / InFramesPerOverviewSample);

		for (uint32 ch = 0; ch < (uint32)GetNumValidChannels(); ++ch)
		{
			const DataType* SourcePtr = GetValidChannelData(ch);
			for (uint32 n = 0; n < NumOverviewFrames; ++n)
			{
				DataType MaxSample = SourcePtr[0];
				DataType MinSample = SourcePtr[0];
				for (uint32 idx = 1; idx < InFramesPerOverviewSample; ++idx)
				{
					DataType Sample = SourcePtr[idx];

					if (Sample > MaxSample)
					{
						MaxSample = Sample;
					}

					if (Sample < MinSample)
					{
						MinSample = Sample;
					}
				}

				OutOverviewMax.GetValidChannelData(ch)[n] = MaxSample;
				OutOverviewMin.GetValidChannelData(ch)[n] = MinSample;
				SourcePtr += InFramesPerOverviewSample;
			}
		}
	}

	void SwapEndianness()
	{
		if (CleanupMode == EAudioBufferCleanupMode::Delete || MaxConfig.GetIsInterleaved())
		{
			int32 TotalSamples = MaxConfig.GetNumTotalSamples();
			DataType* DataPtr = MyData[0];
			for (int32 SampleIndex = 0; SampleIndex < TotalSamples; ++SampleIndex)
			{
				EndianSwapEq(DataPtr[SampleIndex]);
			}
		}
		else
		{
			for (int32 ch = 0; ch < NumValidChannels; ++ch)
			{
				DataType* DataPtr = MyData[ch];
				for (int32 SampleIndex = 0; SampleIndex < NumValidFrames; ++SampleIndex)
				{
					EndianSwapEq(DataPtr[SampleIndex]);
				}
			}
		}
	}

	bool Saturate(DataType MinValue, DataType MaxValue)
	{
		if constexpr (_UseSIMD())
		{
			return _SaturateSIMD(MinValue, MaxValue);
		}
		else
		{

			bool OutClipped = false;
			if (GetIsInterleaved())
			{
				for (int32 n = 0; n < NumValidFrames * NumValidChannels; ++n)
				{
					if (MyData[0][n] < MinValue)
					{
						MyData[0][n] = MinValue;
						OutClipped = true;
					}
					else if (MyData[0][n] > MaxValue)
					{
						MyData[0][n] = MaxValue;
						OutClipped = true;
					}
				}
			}
			else
			{
				for (int32 ch = 0; ch < NumValidChannels; ++ch)
				{
					DataType* OutputChannel = MyData[ch];
					for (int32 n = 0; n < NumValidFrames; ++n)
					{
						if (OutputChannel[n] < MinValue)
						{
							OutputChannel[n] = MinValue;
							OutClipped = true;
						}
						else if (OutputChannel[0] > MaxValue)
						{
							OutputChannel[n] = MaxValue;
							OutClipped = true;
						}
					}
				}
			}
			return OutClipped;
		}
	}

	bool SaturateWithNANCheck(DataType MinValue, DataType MaxValue)
	{
		// SIMD will already "clip" nan...
		if constexpr (_UseSIMD())
		{
			return _SaturateSIMD(MinValue, MaxValue);
		}
		else
		{
			bool OutClipped = false;
			if (GetIsInterleaved())
			{
				for (int32 n = 0; n < NumValidFrames * NumValidChannels; ++n)
				{
					if (MyData[0][n] < MinValue)
					{
						MyData[0][n] = MinValue;
						OutClipped = true;
					}
					else if (MyData[0][n] > MaxValue)
					{
						MyData[0][n] = MaxValue;
						OutClipped = true;
					}
					else if (FGenericPlatformMath::IsNaN(MyData[0][n]))
					{
						MyData[0][n] = 0.0f;
						OutClipped = true;
					}
				}
			}
			else
			{
				for (int32 ch = 0; ch < NumValidChannels; ++ch)
				{
					DataType* OutputChannel = MyData[ch];
					for (int32 n = 0; n < NumValidChannels; ++n)
					{
						if (OutputChannel[n] < MinValue)
						{
							OutputChannel[n] = MinValue;
							OutClipped = true;
						}
						else if (OutputChannel[n] > MaxValue)
						{
							OutputChannel[n] = MaxValue;
							OutClipped = true;
						}
						else if (FGenericPlatformMath::IsNaN(OutputChannel[n]))
						{
							OutputChannel[n] = 0.0f;
							OutClipped = true;
						}
					}
				}
			}
			return OutClipped;
		}
	}

	void Scale(float InFactor)
	{
		if constexpr (_UseSIMD())
		{
			_ScaleSIMD(InFactor);
			return;
		}
		else
		{
			check(!GetIsInterleaved());
			for (int32 ch = 0; ch < NumValidChannels; ++ch)
			{
				DataType* OutputChannel = MyData[ch];
				for (int32 n = 0; n < NumValidFrames; ++n)
				{
					OutputChannel[n] = (DataType)((float)OutputChannel[n] * InFactor);
				}
			}
		}
	}

	void SetChannelData(int32 InChannelIndex, const DataType* InData)
	{
		check(InChannelIndex < GetMaxNumChannels());
		check(InChannelIndex == 0 || !MaxConfig.GetIsInterleaved());
		check(CleanupMode == EAudioBufferCleanupMode::DontDelete);

		// sadly can't keep the data const without a const Data member,
		// which would require a whole new class (or maybe possible via the template argument?)
		// not sure we care, though.
		MyData[InChannelIndex] = (DataType*)InData;
	}

	void SetAliasedChannelData(DataType** InData, int32 InNumChannels)
	{
		SetNumValidChannels(InNumChannels);
		for (int32 idx = 0; idx < InNumChannels; ++idx)
			MyData[idx] = InData[idx];
	}

	void AdvanceAliasedDataPointers(int32 InNumFrames)
	{
		check(InNumFrames <= NumValidFrames);
		if (MaxConfig.GetIsInterleaved())
		{
			MyData[0] += (InNumFrames * GetNumValidChannels());
		}
		else
		{
			for (int32 ch = 0; ch < NumValidChannels; ++ch)
			{
				MyData[ch] += InNumFrames;
			}
		}
		SetNumValidFrames(GetNumValidFrames() - InNumFrames);
	}

	DataType* GetValidChannelData(int32 InChannelIndex)
	{
		check(InChannelIndex < GetNumValidChannels());
		check(InChannelIndex == 0 || !MaxConfig.GetIsInterleaved());

		return MyData[InChannelIndex];
	}

	const DataType* GetValidChannelData(int32 ChanneIdx) const
	{
		check(ChanneIdx < GetNumValidChannels());
		check(ChanneIdx == 0 || !MaxConfig.GetIsInterleaved());
		return MyData[ChanneIdx];
	}

	DataType* GetRawChannelData(int32 ChanneIdx)
	{
		check(ChanneIdx < GetMaxNumChannels());
		check(ChanneIdx == 0 || !MaxConfig.GetIsInterleaved());

		return MyData[ChanneIdx];
	}

	const DataType* GetRawChannelData(int32 ChanneIdx) const
	{
		check(ChanneIdx < GetMaxNumChannels());
		check(ChanneIdx == 0 || !MaxConfig.GetIsInterleaved());
		return MyData[ChanneIdx];
	}

	TDynamicStridePtr<DataType> GetStridingChannelDataPointer(int32 InChannelIdx) const
	{
		if (MaxConfig.GetIsInterleaved())
		{
			TDynamicStridePtr<DataType> StridingPtr(MaxConfig.GetNumChannels());
			StridingPtr = MyData[0] + InChannelIdx;
			return StridingPtr;
		}
		TDynamicStridePtr<DataType> StridingPtr(1);
		StridingPtr = MyData[InChannelIdx];
		return StridingPtr;
	}

	void Alias(const TAudioBuffer<DataType>& Other, int32 InFrameOffset = 0)
	{
		Configure(Other.GetMaxConfig(),  EAudioBufferCleanupMode::DontDelete);
		AliasChannelDataPointers(Other);
		if (InFrameOffset)
		{
			AdvanceAliasedDataPointers(InFrameOffset);
		}
	}

	void Alias(const DataType* RawData, int32 InNumFrames, int32 InNumChannels, float SampleRate = 0.0f, bool IsInterleaved = false)
	{
		// only configure if we're resizing the buffer
		if (GetNumValidFrames() != InNumFrames || GetNumValidChannels() != InNumChannels)
		{
			Configure(InNumChannels, InNumFrames, EAudioBufferCleanupMode::DontDelete, SampleRate, IsInterleaved);
		}

		// Have to cast away const-ness to be able to alias e.g. a FAudioBufferReadRef's buffer.
		// Should probably make a whole other API for this in the future.
		MyData[0] = const_cast<DataType*>(RawData);
	}
	
	void AliasChannelDataPointers(
		const TAudioBuffer<DataType>& Other,
		int32 InFirstChannel = 0,
		int32 InNumChannels = MAX_int32)
	{
		if (InNumChannels == MAX_int32)
		{
			InNumChannels = Other.GetNumValidChannels() - InFirstChannel;
		}

		check(InFirstChannel < Other.GetNumValidChannels());
		check(InNumChannels <= Other.GetNumValidChannels());

		SetNumValidChannels(InNumChannels);
		MaxConfig.SetNumChannels(InNumChannels);
		for (int32 ch = 0; ch < InNumChannels; ++ch)
		{
			SetChannelData(ch, Other.GetValidChannelData(ch + InFirstChannel));
		}
	}

	void AliasChannelDataPointers(
		const FAudioBufferConfig& NewValidConfig,
		DataType** DataPtrs)
	{
		check(CleanupMode == EAudioBufferCleanupMode::DontDelete);

		// only configure if we're resizing the buffer
		if (GetMaxNumFrames() < NewValidConfig.GetNumFrames() || GetMaxNumChannels() < NewValidConfig.GetNumChannels())
		{
			Configure(NewValidConfig, EAudioBufferCleanupMode::DontDelete);
		}

		SetNumValidChannels(NewValidConfig.GetNumChannels());
		SetNumValidFrames(NewValidConfig.GetNumFrames());
		for (int32 ch = 0; ch < NumValidChannels; ++ch)
		{
			SetChannelData(ch, DataPtrs[ch]);
		}
	}

	void IncrementChannelDataPointers(int32 InIncrementSize)
	{
		// this function only works for aliased pointers
		check(CleanupMode == EAudioBufferCleanupMode::DontDelete); 
		// this function only works for non-interleaved buffers
		check(!MaxConfig.GetIsInterleaved());             

		for (int32 ch = 0; ch < GetNumValidChannels(); ++ch)
		{
			DataType* data = MyData[ch];
			data += InIncrementSize;
			MyData[ch] = data;
		}
	}

	DataType** GetData() { return MyData; }
	DataType* const* GetData() const { return MyData; }
	const FAudioBufferConfig& GetMaxConfig() const { return MaxConfig; }

	FAudioBufferConfig GetValidConfig() const
	{
		return FAudioBufferConfig(GetChannelLayout(), GetNumValidChannels(), GetNumValidFrames(),
			MaxConfig.GetSampleRate(), MaxConfig.GetIsInterleaved());
	}
	int32 GetMaxNumChannels() const { return MaxConfig.GetNumChannels(); }
	int32 GetMaxNumFrames() const { return MaxConfig.GetNumFrames(); }

	int32 GetNumValidChannels() const { return NumValidChannels; }
	void SetNumValidChannels(int32 InNumChannels)
	{
		check(InNumChannels <= FAudioBufferConfig::kMaxAudioBufferChannels);
		NumValidChannels = InNumChannels;
	}

	EAudioBufferChannelLayout GetChannelLayout() const { return ChannelLayout; }
	void SetChannelLayout(EAudioBufferChannelLayout InChannelLayout)
	{
		ChannelLayout = InChannelLayout;
	}

	uint16 GetChannelMask() const { return ChannelMask; }
	void SetChannelMask(uint16 InMask)
	{
		ChannelMask = InMask;
	}

	int32 GetNumValidFrames() const { return NumValidFrames; }
	void SetNumValidFrames(int32 InNumFrames)
	{
		check(InNumFrames <= MaxConfig.GetNumFrames());
		NumValidFrames = InNumFrames;
	}

	uint32 GetNumTotalValidSamples() const
	{
		return GetNumValidChannels() * GetNumValidFrames();
	}

	// deinterleave the Other data set into this one.
	// this one ends up deinterleaved.
	void Deinterleave(const TAudioBuffer<DataType>& Other)
	{
		check(Other.GetIsInterleaved());
		//check(Other.GetNumValidFrames() <= GetMaxNumFrames());

		int32 inCh = Other.GetNumValidChannels();
		int32 outCh = GetNumValidChannels();
		int32 numChToCopy = inCh < outCh ? inCh : outCh;
		int32 MaxFrames = FMath::Min(Other.GetNumValidFrames(), GetMaxNumFrames());
		DataType* inFrame = Other.MyData[0];
		for (int32 frame = 0; frame < MaxFrames; ++frame)
		{
			for (int32 ch = 0; ch < numChToCopy; ++ch)
			{
				MyData[ch][frame] = inFrame[ch];
			}
			inFrame += inCh;
		}
		SetNumValidFrames(MaxFrames);
		for (int32 ch = numChToCopy; ch < outCh; ++ch)
		{
			FMemory::Memset(MyData[ch], 0, sizeof(DataType) * GetNumValidFrames());
		}

		SetIsSilent(Other.GetIsSilent());
	}

	void Deinterleave(const TAudioBuffer<DataType>& Other, float InGain, DataType InMin, DataType InMax)
	{
		check(Other.GetIsInterleaved());
		check(Other.GetNumValidChannels() <= GetNumValidChannels());
		check(Other.GetNumValidFrames() <= GetMaxNumFrames());

		DataType* InterleavedDataPtr = Other.MyData[0];
		int32 Pos = 0;
		int32 OtherNumFrames = Other.GetNumValidFrames();
		int32 OtherNumChannels = Other.GetNumValidChannels();
		for (int32 frame = 0; frame < OtherNumFrames; ++frame)
		{
			for (int32 ch = 0; ch < OtherNumChannels; ++ch, ++Pos)
			{
				MyData[ch][frame] = (InterleavedDataPtr[Pos] < InMin) ? InMin : ((InterleavedDataPtr[Pos] > InMax) ? InMax : InterleavedDataPtr[Pos]);
			}
		}

		SetNumValidFrames(Other.GetNumValidFrames());
		SetIsSilent(Other.GetIsSilent());
	}

	// interleave the Other data set into this one.
	// this one ends up interleaved.
	void Interleave(const TAudioBuffer<DataType>& Other)
	{
		check(GetIsInterleaved());

		int32 MaxFrames = FMath::Min(Other.GetNumValidFrames(), GetMaxNumFrames());
		int32 inCh = Other.GetNumValidChannels();
		int32 outCh = GetNumValidChannels();
		int32 chToCopy = inCh < outCh ? inCh : outCh;
		DataType* outFrame = MyData[0];
		for (int32 frame = 0; frame < MaxFrames; ++frame)
		{
			int32 ch = 0;
			for (; ch < chToCopy; ++ch)
			{
				outFrame[ch] = Other.MyData[ch][frame];
			}
			for (; ch < outCh; ++ch)
			{
				outFrame[ch] = (DataType)0;
			}
			outFrame += outCh;
		}
		SetNumValidFrames(MaxFrames);
		SetIsSilent(Other.GetIsSilent());
	}

	void Interleave(const TAudioBuffer<DataType>& Other, float InGain, DataType InMin, DataType InMax)
	{
		checkSlow(GetIsInterleaved());
		checkSlow(Other.GetNumValidChannels() == GetNumValidChannels());
		checkSlow(Other.GetNumValidFrames() <= GetMaxNumFrames());

		DataType* InterleavedDataPtr = MyData[0];
		int32 Pos = 0;
		int32 OtherNumFrames = Other.GetNumValidFrames();
		int32 InNumChannels = GetNumValidChannels();
		for (int32 frame = 0; frame < OtherNumFrames; ++frame)
		{
			for (int32 ch = 0; ch < InNumChannels; ++ch)
			{
				DataType sample = (DataType)((float)Other.MyData[ch][frame] * InGain);
				InterleavedDataPtr[Pos++] = (sample < InMin) ? InMin : ((sample > InMax) ? InMax : sample);
			}
		}

		SetNumValidFrames(Other.GetNumValidFrames());
		SetIsSilent(Other.GetIsSilent());
	}

	bool EqualWithTolerance(int32 ThisChannelIdx, const TAudioBuffer<DataType>& Other, int32 OtherChannelIdx, DataType Tolerance) const
	{
		if (ThisChannelIdx >= GetNumValidChannels() || OtherChannelIdx >= Other.GetNumValidChannels())
		{
			return false;
		}

		const int32 NumSamples = GetNumValidFrames();
		
		if (NumSamples != Other.GetNumValidFrames())
		{
			return false;
		}

		TDynamicStridePtr<DataType> ThisChannelData = GetStridingChannelDataPointer(ThisChannelIdx);
		TDynamicStridePtr<DataType> OtherChannelData = Other.GetStridingChannelDataPointer(OtherChannelIdx);

		for (int32 SampleIdx = 0; SampleIdx < NumSamples; ++SampleIdx)
		{
			const DataType AbsDelta = FMath::Abs(ThisChannelData[SampleIdx] - OtherChannelData[SampleIdx]);
			if (AbsDelta > Tolerance)
			{
				return false;
			}
		}

		return true;
	}

	bool FindFirstDiff(const TAudioBuffer<DataType>& Other, int32& OutChannelIndex, int32& OutFrameIndex, DataType InTolerance) const
	{
		check(GetNumValidChannels() == Other.GetNumValidChannels());
		check(GetNumValidFrames() == Other.GetNumValidFrames());
		check(OutChannelIndex);
		check(OutFrameIndex);

		// check each sample
		for (int32 fr = 0; fr < GetNumValidFrames(); ++fr) 
		{
			for (int32 ch = 0; ch < GetNumValidChannels(); ++ch) 
			{

				const DataType* OtherData = Other.GetValidChannelData(ch);
				const DataType* Data = this->GetValidChannelData(ch);

				DataType diff = Data[fr] - OtherData[fr];
				DataType absdiff = FMath::Abs(diff);
				if (absdiff > InTolerance) {
					OutChannelIndex = ch;
					OutFrameIndex = fr;
					return true;
				}
			}
		}

		// didn't find a difference greater than the tolerance
		return false;
	}

	bool EqualWithTolerance(const TAudioBuffer<DataType>& Other, DataType InTolerance) const
	{
		if (GetNumValidChannels() != Other.GetNumValidChannels())
			return false;

		if (GetNumValidFrames() != Other.GetNumValidFrames())
			return false;

		int32 ch;
		int32 fr;
		bool FoundSignificantDifference = FindFirstDiff(Other, ch, fr, InTolerance);

		return !FoundSignificantDifference;
	}

	void AssertEqualWithTolerance(const TAudioBuffer<DataType>& Other, DataType InTolerance) const
	{
		using namespace HarmonixDsp;

		int32 ch;
		int32 fr;
		bool FoundSignificantDifference = FindFirstDiff(Other, ch, fr, InTolerance);

		if (!FoundSignificantDifference)
			return;

		// ok... print the samples in the area, and assert
		UE_LOGF(LogAudioBuffer, Error, "AudioBuffer<T>::AssertEqualWithTolerance failed on channel %d at frame %d\n", ch, fr);

		const static int32 sLength = 5;
		int32 framesRemaining = GetNumValidFrames() - fr;
		int32 length = FMath::Min(framesRemaining, sLength);

		UE_LOGF(LogAudioBuffer, Error, "this buffer channel %d, frame %d through %d:\n", ch, fr, fr + length - 1);
		FAudioBuffer::DebugLog(GetValidChannelData(ch) + fr, length);
		UE_LOGF(LogAudioBuffer, Error, "Other buffer channel %d, frame %d through %d:\n", ch, fr, fr + length - 1);
		FAudioBuffer::DebugLog(Other.GetValidChannelData(ch) + fr, length);

		check(!FoundSignificantDifference);
	}

	bool operator==(const TAudioBuffer<DataType>& Other) const
	{
		return EqualWithTolerance(Other, DataType(0));
	}

	bool operator!=(const TAudioBuffer<DataType>& Other) const 
	{ 
		return !(*this == Other); 
	}

	TAudioBuffer<DataType>& operator*=(const TAudioBuffer<DataType>& Other)
	{
		check(GetNumValidChannels() == Other.GetNumValidChannels());
		check(GetNumValidFrames() == Other.GetNumValidFrames());
		check(GetIsInterleaved() == Other.GetIsInterleaved());

		if (GetIsInterleaved())
		{
			DataType* RawData = GetRawChannelData(0);
			const DataType* OtherRawData = Other.GetRawChannelData(0);
			int32 NumSamples = GetNumValidChannels() * GetNumValidFrames();
			for (int32 fr = 0; fr < NumSamples; ++fr)
			{
				RawData[fr] *= OtherRawData[fr];
			}
		}
		else
		{
			for (int32 ch = 0; ch < GetNumValidChannels(); ++ch)
			{
				DataType* RawData = GetRawChannelData(ch);
				const DataType* OtherRawData = Other.GetRawChannelData(ch);
				for (int32 fr = 0; fr < GetNumValidFrames(); ++fr)
				{
					RawData[fr] *= OtherRawData[fr];
				}
			}
		}

		return *this;
	}

	void Copy(const TAudioBuffer<DataType>& Other)
	{
		if (Other.GetIsInterleaved() == GetIsInterleaved())
		{
			int32 inCh = Other.GetNumValidChannels();
			int32 outCh = GetNumValidChannels();
			int32 NumFramesToCopy = FMath::Min(Other.GetNumValidFrames(), GetMaxNumFrames());
			if (inCh != outCh)
			{
				int32 chToCopy = inCh < outCh ? inCh : outCh;
				if (GetIsInterleaved())
				{
					const DataType* inFrame = Other.MyData[0];
					DataType* outFrame = MyData[0];
					for (int32 i = 0; i < NumFramesToCopy; ++i)
					{
						int32 ch = 0;
						for (; ch < chToCopy; ++ch)
						{
							outFrame[ch] = inFrame[ch];
						}
						for (; ch < outCh; ++ch)
						{
							outFrame[ch] = (DataType)0;
						}
						inFrame += inCh;
						outFrame += outCh;
					}
				}
				else
				{
					int32 ch = 0;
					for (; ch < chToCopy; ++ch)
					{
						FMemory::Memcpy(GetValidChannelData(ch), Other.GetValidChannelData(ch), sizeof(DataType) * NumFramesToCopy);
					}
					for (; ch < outCh; ++ch)
					{
						FMemory::Memset(GetValidChannelData(ch), 0, sizeof(DataType) * NumFramesToCopy);
					}
				}
			}
			else
			{
				if ((outCh == 1) || GetIsInterleaved() || (GetCleanupMode() == EAudioBufferCleanupMode::Delete && Other.GetCleanupMode() == EAudioBufferCleanupMode::Delete))
				{
					// If we're in here, it means we own the buffers, they're interleaved, or both, OR there's only a single channel;
					// they are therefore contiguous, and we can use a single memcpy...
					FMemory::Memcpy(GetValidChannelData(0), Other.GetValidChannelData(0), sizeof(DataType) * (NumFramesToCopy * outCh));
				}
				else
				{
					// If we're in here, we don't own one or both of the buffers, and they're
					// both de-interleaved, so we'll do it the safe way...
					for (int32 channel = 0; channel < outCh; ++channel)
					{
						FMemory::Memcpy(GetValidChannelData(channel), Other.GetValidChannelData(channel), sizeof(DataType) * NumFramesToCopy);
					}
				}
			}
			SetNumValidFrames(NumFramesToCopy);
			SetIsSilent(Other.GetIsSilent());
		}
		else if (Other.GetIsInterleaved())
		{
			Deinterleave(Other);
		}
		else
		{
			Interleave(Other);
		}
	}

	void Copy(const TAudioBuffer<DataType>& Other, int32 InNumFrames, int32 InOffset)
	{
		check(InNumFrames <= GetMaxNumFrames());
		check(InNumFrames + InOffset <= Other.GetMaxNumFrames());
		check(Other.GetIsInterleaved() == GetIsInterleaved());

		int32 inCh = Other.GetNumValidChannels();
		int32 outCh = GetNumValidChannels();
		if (inCh != outCh)
		{
			int32 chToCopy = inCh < outCh ? inCh : outCh;
			if (GetIsInterleaved())
			{
				const DataType* inFrame = Other.MyData[0 + InOffset];
				DataType* outFrame = MyData[0];
				for (int32 i = 0; i < InNumFrames; ++i)
				{
					int32 ch = 0;
					for (; ch < chToCopy; ++ch)
					{
						outFrame[ch] = inFrame[ch];
					}
					for (; ch < outCh; ++ch)
					{
						outFrame[ch] = (DataType)0;
					}
					inFrame += inCh;
					outFrame += outCh;
				}
			}
			else
			{
				int32 ch = 0;
				for (; ch < chToCopy; ++ch)
				{
					FMemory::Memcpy(GetValidChannelData(ch), Other.GetValidChannelData(ch) + InOffset, sizeof(DataType) * InNumFrames);
				}
				for (; ch < outCh; ++ch)
				{
					FMemory::Memset(GetValidChannelData(ch), 0, sizeof(DataType) * InNumFrames);
				}
			}
		}
		else
		{
			for (int32 channel = 0; channel < outCh; ++channel)
			{
				FMemory::Memcpy(GetValidChannelData(channel), Other.GetValidChannelData(channel) + InOffset, sizeof(DataType) * InNumFrames);
			}
		}
		SetNumValidFrames(InNumFrames);
		SetIsSilent(Other.GetIsSilent());
	}

	void Copy(const TAudioBuffer<DataType>& Other, float InGain, DataType InMin, DataType InMax)
	{
		check(Other.GetNumValidChannels() == GetNumValidChannels());
		check(Other.GetNumValidFrames() == GetNumValidFrames());
		int32 InNumChannels = GetNumValidChannels();
		int32 InNumFrames = GetNumValidFrames();


		if (GetIsInterleaved() == Other.GetIsInterleaved())
		{
			if constexpr (_UseSIMD())
			{
				_CopySIMD(Other, InGain, InMin, InMax);
				return;
			}
			else
			{
				if ((InNumChannels == 1) ||
					GetIsInterleaved() ||
					(GetCleanupMode() == EAudioBufferCleanupMode::Delete && Other.GetCleanupMode() == EAudioBufferCleanupMode::Delete))
				{
					const DataType* in = Other.GetValidChannelData(0);
					DataType* out = MyData[0];
					int32 TotalFrames = InNumFrames * InNumChannels;
					for (int32 frame = 0; frame < TotalFrames; ++frame)
					{
						DataType val = (DataType)((float)in[frame] * InGain);
						out[frame] = (val < InMin) ? InMin : ((val > InMax) ? InMax : val);
					}
				}
				else
				{
					for (int32 channel = 0; channel < InNumChannels; ++channel)
					{
						const DataType* in = Other.GetValidChannelData(channel);
						DataType* out = MyData[channel];
						for (int32 frame = 0; frame < InNumFrames; ++frame)
						{
							DataType val = (DataType)((float)in[frame] * InGain);
							out[frame] = (val < InMin) ? InMin : ((val > InMax) ? InMax : val);
						}
					}
				}
			}
		}
		else if (Other.GetIsInterleaved())
		{
			//Not SIMD-optimized!
			Deinterleave(Other, InGain, InMin, InMax);
		}
		else
		{
			//Not SIMD-optimized!
			Interleave(Other, InGain, InMin, InMax);
		}
	}

	DataType* operator[](int32 Index)
	{
		if (Index >= GetNumValidChannels()) return nullptr;
		return MyData[Index];
	}

	void Accumulate(const TAudioBuffer<DataType>& Other)
	{
		if (Other.GetIsSilent())
			return;

		checkSlow(Other.GetNumValidChannels() == GetNumValidChannels());
		checkSlow(Other.GetNumValidFrames() == GetNumValidFrames());
		int32 InNumChannels = GetNumValidChannels();
		int32 InNumFrames = GetNumValidFrames();

		bool DeleteBuffers = GetCleanupMode() == EAudioBufferCleanupMode::Delete && Other.GetCleanupMode() == EAudioBufferCleanupMode::Delete;
		if ((InNumChannels == 1) || ((GetIsInterleaved() == Other.GetIsInterleaved()) && (GetIsInterleaved() || DeleteBuffers)))
		{
			for (int32 ch = 0; ch < InNumChannels; ++ch)
			{
				const DataType* in = Other.GetValidChannelData(ch);
				DataType* out = GetValidChannelData(ch);
				for (int32 frame = 0; frame < InNumFrames; frame++)
				{
					out[frame] += in[frame];
				}
			}
		}
		//We should only ever fall in here if both are deinterleaved but we don't meet the above conditions
		else if (GetIsInterleaved() == Other.GetIsInterleaved())
		{
			const DataType* in;
			DataType* out;
			for (int32 ch = 0; ch < InNumChannels; ++ch)
			{
				in = Other.GetValidChannelData(ch);
				out = GetValidChannelData(ch);
				for (int32 frame = 0; frame < InNumFrames; frame++)
				{
					out[frame] += in[frame];
				}
			}
		}
		// Only the "out" buffer is interleaved
		else if (GetIsInterleaved())
		{
			const DataType* in;
			TDynamicStridePtr<DataType> out(InNumChannels);
			for (int32 ch = 0; ch < InNumChannels; ch++)
			{
				in = Other.GetValidChannelData(ch);
				out = GetValidChannelData(0) + ch;
				for (int32 frame = 0; frame < InNumFrames; frame++)
				{
					out[frame] += in[frame];
				}
			}
		}
		//Only the "in" buffer is interleaved
		else
		{
			TDynamicStridePtr<const DataType> in(InNumChannels);
			DataType* out;
			for (int32 ch = 0; ch < InNumChannels; ch++)
			{
				in = Other.GetValidChannelData(0) + ch;
				out = GetValidChannelData(ch);
				for (int32 frame = 0; frame < InNumFrames; frame++)
				{
					out[frame] += in[frame];
				}
			}
		}

		SetIsSilent(false);
	}

	void Accumulate(DataType** Input, int32 InNumChannels, int32 InNumFrames)
	{
		checkSlow(InNumChannels == GetNumValidChannels());
		checkSlow(InNumFrames == GetNumValidFrames());

		if constexpr (_UseSIMD())
		{
			_AccumulateSIMD(Input, InNumChannels, InNumFrames);
			SetIsSilent(false);
			return;
		}
		else
		{

			if (GetIsInterleaved())
			{
				TDynamicStridePtr<DataType> out(InNumChannels);
				for (int32 ch = 0; ch < InNumChannels; ++ch)
				{
					out = GetValidChannelData(0) + ch;
					const DataType* in = Input[ch];
					for (int32 frame = 0; frame < InNumFrames; ++frame)
					{
						*out += in[frame];
						out++;
					}
				}
			}
			else
			{
				for (int32 ch = 0; ch < InNumChannels; ++ch)
				{
					const DataType* in = Input[ch];
					DataType* out = GetValidChannelData(ch);
					for (int32 frame = 0; frame < InNumFrames; ++frame)
					{
						out[frame] += in[frame];
					}
				}
			}

			SetIsSilent(false);
		}
	}

	// merge the Other to this buffer, and apply the specified InGain,
	// which will change by the delta each sample.
	void MultiplyAccumulate(const TAudioBuffer<DataType>& Other, DataType InGain, DataType InGainDelta)
	{
		if (Other.GetIsSilent())
			return;

		int32 InNumChannels = GetNumValidChannels();
		int32 InNumFrames = GetNumValidFrames();

		checkSlow(Other.GetNumValidChannels() == InNumChannels);
		checkSlow(Other.GetNumValidFrames() == InNumFrames);
		if (!GetIsInterleaved() && !Other.GetIsInterleaved())
		{
			if constexpr (_UseSIMD())
			{
				_MultiplyAccumulateSIMD(Other, InGain, InGainDelta);
				return;
			}
			else
			{
				for (int32 ch = 0; ch < InNumChannels; ++ch)
				{
					const DataType* in = Other.GetValidChannelData(ch);
					DataType* out = GetValidChannelData(ch);
					DataType GainAcc = InGain;
					for (int32 frame = 0; frame < InNumFrames; ++frame, GainAcc += InGainDelta)
					{
						out[frame] += in[frame] * GainAcc;
					}
				}
			}
		}
		else if (GetIsInterleaved() && Other.GetIsInterleaved())
		{
			TDynamicStridePtr<const DataType> in(InNumChannels);
			TDynamicStridePtr<DataType> out(InNumChannels);
			for (int32 ch = 0; ch < InNumChannels; ch++)
			{
				in = Other.GetValidChannelData(0) + ch;
				out = GetValidChannelData(0) + ch;
				DataType GainAcc = InGain;
				for (int32 frame = 0; frame < InNumFrames; ++frame, ++in, ++out, GainAcc += InGainDelta)
				{
					*out = *in * GainAcc;
				}
			}
		}
		else if (GetIsInterleaved())
		{
			const DataType* in;
			TDynamicStridePtr<DataType> out(InNumChannels);
			for (int32 ch = 0; ch < InNumChannels; ch++)
			{
				in = Other.GetValidChannelData(ch);
				out = GetValidChannelData(0) + ch;
				DataType GainAcc = InGain;
				for (int32 frame = 0; frame < InNumFrames; ++frame, ++out, GainAcc += InGainDelta)
				{
					*out = in[frame] * GainAcc;
				}
			}
		}
		else
		{
			TDynamicStridePtr<const DataType> in(InNumChannels);
			DataType* out;
			for (int32 ch = 0; ch < InNumChannels; ch++)
			{
				in = Other.GetValidChannelData(0) + ch;
				out = GetValidChannelData(ch);
				DataType GainAcc = InGain;
				for (int32 FrameIdx = 0; FrameIdx < InNumFrames; ++FrameIdx, ++in, GainAcc += InGainDelta)
				{
					out[FrameIdx] = *in * GainAcc;
				}
			}
		}

		SetIsSilent(false);
	}

	void DebugLog(uint32 InChannel) const
	{
		using namespace HarmonixDsp;
		FAudioBuffer::DebugLog(GetValidChannelData(InChannel), GetNumValidFrames());
	}

private:

	static constexpr bool _UseSIMD() { return false; }
	
	bool _SaturateSIMD(DataType InMinValue, DataType InMaxValue) { return false; }

	// works for all data
	void _ScaleSIMD(float InFactor) { return; }

	// only optimized if this->IsInterleaved() == Other.IsInterleaved()
	void _CopySIMD(const TAudioBuffer<DataType>& Other, float InGain, DataType InMin, DataType InMax) { return; }
	
	// works for all data
	void _AccumulateSIMD(DataType** InData, int32 InNumChannels, int32 InNumFrames) { return; }
	
	// only optimized if both buffers are NOT Interleaved
	void _MultiplyAccumulateSIMD(const TAudioBuffer<DataType>& Other, DataType InGain, DataType InGainDelta) { return; }

	EAudioBufferCleanupMode CleanupMode;
	DataType* MyData[FAudioBufferConfig::kMaxAudioBufferChannels];
	// maximum channel size, initial num valid channels
	FAudioBufferConfig MaxConfig;

	// these should never be larger than what MaxConfig reports
	int32 NumValidChannels;
	EAudioBufferChannelLayout ChannelLayout;
	int32 NumValidFrames;
	uint16 ChannelMask = 0xFFFF;

	// the frame position this buffer is associated with
	int32 FirstFrame;
	int32 NumFrames;
	int32 LastFrame;

	bool IsSilent;

	static constexpr int32 kAudioBufferStompDetector = 0xFEEDF00D;

	TAudioBuffer(const TAudioBuffer& Other);
	void operator=(const TAudioBuffer& Other);
};

namespace HarmonixDsp
{
	namespace FAudioBuffer
	{
		void Convert(const TAudioBuffer<float>& Source, TAudioBuffer<int16>& Destination);
		void Convert(const TAudioBuffer<int16>& Source, TAudioBuffer<float>& Destination);
	}
}

template <typename DataType>
class TConsumableAudioBuffer : public TAudioBuffer<DataType>
{
public:

	TConsumableAudioBuffer() 
		: DataReady(false) 
	{}

	void MarkReady() { DataReady = true; }
	void MarkConsumed() { DataReady = false; }
	bool IsDataReady() { return DataReady; }

private:

	bool DataReady = false;
};

template<>
constexpr bool TAudioBuffer<float>::_UseSIMD()
{
	return !UE_BUILD_DEBUG || SIMD_IN_DEBUG;
}

template<>
FORCEINLINE bool TAudioBuffer<float>::_SaturateSIMD(float InMinValue, float InMaxValue)
{
	VectorRegister4Float VecLow = VectorSetFloat1(InMinValue);
	VectorRegister4Float VecHigh = VectorSetFloat1(InMaxValue);
	VectorRegister4Float VecOrig;
	VectorRegister4Float VecNew;
	bool OutClipped = false;
	if (GetIsInterleaved())
	{
		for (int32 n = 0; n < NumValidFrames * NumValidChannels; n += 4)
		{
			VecOrig = VectorLoadAligned(&MyData[0][n]);
			VecNew = VectorMin(VectorMax(VecOrig, VecLow), VecHigh);
			VecOrig = VectorSubtract(VecOrig, VecNew);
			if (VectorMaskBits(VectorCompareNE(VecOrig, GlobalVectorConstants::FloatZero)))
			{
				OutClipped = true;
			}
			VectorStoreAligned(VecNew, &MyData[0][n]);
		}
		for (int32 n = NumValidFrames - (NumValidFrames % 4); n < NumValidFrames; ++n)
		{
			if (MyData[0][n] < InMinValue)
			{
				MyData[0][n] = InMinValue;
				OutClipped = true;
			}
			else if (MyData[0][n] > InMaxValue)
			{
				MyData[0][n] = InMaxValue;
				OutClipped = true;
			}
		}
	}
	else
	{
		for (int32 ch = 0; ch < NumValidChannels; ++ch)
		{
			float* OutChannel = MyData[ch];
			for (int32 n = 0; n < NumValidFrames; n += 4)
			{
				VecOrig = VectorLoadAligned(&OutChannel[n]);
				VecNew = VectorMin(VectorMax(VecOrig, VecLow), VecHigh);
				VecOrig = VectorSubtract(VecOrig, VecNew);
				if (VectorMaskBits(VectorCompareNE(VecOrig, GlobalVectorConstants::FloatZero)))
				{
					OutClipped = true;
				}
				VectorStoreAligned(VecNew, &OutChannel[n]);
			}
			for (int32 n = NumValidFrames - (NumValidFrames % 4); n < NumValidFrames; ++n)
			{
				if (OutChannel[n] < InMinValue)
				{
					OutChannel[n] = InMinValue;
					OutClipped = true;
				}
				else if (OutChannel[n] > InMaxValue)
				{
					OutChannel[n] = InMaxValue;
					OutClipped = true;
				}
			}
		}
	}
	return OutClipped;
}

template<>
FORCEINLINE void TAudioBuffer<float>::_ScaleSIMD(float InFactor)
{
	VectorRegister4Float factorVec = VectorSetFloat1(InFactor);
	for (int32 ch = 0; ch < NumValidChannels; ++ch)
	{
		float* OutChannel = MyData[ch];
		for (int32 n = 0; n < NumValidFrames; n += 4)
		{
			VectorStoreAligned(VectorMultiply(VectorLoadAligned(&OutChannel[n]), factorVec), &OutChannel[n]);
		}
		for (int32 n = NumValidFrames - (NumValidFrames % 4); n < NumValidFrames; ++n)
		{
			OutChannel[n] *= InFactor;
		}
	}
}

template<>
FORCEINLINE void TAudioBuffer<float>::_CopySIMD(const TAudioBuffer<float>& Other, float InGain, float InMin, float InMax)
{
	check(GetIsInterleaved() == Other.GetIsInterleaved());

	int32 InNumChannels = GetNumValidChannels();
	int32 InNumFrames = GetNumValidFrames();
	VectorRegister4Float InGainVec = VectorSetFloat1(InGain);
	VectorRegister4Float InMinVec = VectorSetFloat1(InMin);
	VectorRegister4Float InMaxVec = VectorSetFloat1(InMax);

	bool DeleteBuffers = (GetCleanupMode() == EAudioBufferCleanupMode::Delete && Other.GetCleanupMode() == EAudioBufferCleanupMode::Delete);
	if ((InNumChannels == 1) || GetIsInterleaved() || DeleteBuffers)
	{
		const float* in = Other.GetValidChannelData(0);
		float* out = MyData[0];
		int32 TotalFrames = InNumFrames * InNumChannels;
		for (int32 frame = 0; frame < TotalFrames; ++frame)
		{
			VectorStoreAligned(VectorMin(VectorMax(VectorMultiply(VectorLoadAligned(&in[frame]), InGainVec), InMinVec), InMaxVec), &out[frame]);
		}
		for (int32 frame = TotalFrames - (TotalFrames % 4); frame < TotalFrames; ++frame)
		{
			float val = InGain * in[frame];
			out[frame] = (val > InMax) ? InMax : ((val < InMin) ? InMin : val);
		}
	}
	else
	{
		for (int32 channel = 0; channel < InNumChannels; ++channel)
		{
			const float* in = Other.GetValidChannelData(channel);
			float* out = MyData[channel];
			for (int32 frame = 0; frame < InNumFrames; ++frame)
			{
				VectorStoreAligned(VectorMin(VectorMax(VectorMultiply(VectorLoadAligned(&in[frame]), InGainVec), InMinVec), InMaxVec), &out[frame]);
			}
			for (int32 frame = InNumFrames - (InNumFrames % 4); frame < InNumFrames; ++frame)
			{
				float val = InGain * in[frame];
				out[frame] = (val > InMax) ? InMax : ((val < InMin) ? InMin : val);
			}
		}
	}
}

template<>
FORCEINLINE void TAudioBuffer<float>::_AccumulateSIMD(float** InputData, int32 InNumChannels, int32 InNumFrames)
{
	//   if (InNumFrames%4) return false;
	//   if (GetIsInterleaved()) return false;

	InNumFrames = InNumFrames >> 2;
	for (int32 ich = 0; ich < InNumChannels; ++ich)
	{
		float* in = InputData[ich];
		float* out = GetValidChannelData(ich);
		for (int32 frame = 0; frame < InNumFrames; ++frame)
		{
			VectorRegister4Float inSamples = VectorLoadAligned(in);
			VectorRegister4Float outSamples = VectorLoadAligned(out);
			outSamples = VectorAdd(inSamples, outSamples);
			VectorStoreAligned(outSamples, out);
			in += 4;
			out += 4;
		}
	}
}

template<>
FORCEINLINE void TAudioBuffer<float>::_MultiplyAccumulateSIMD(const TAudioBuffer<float>& Other, float InGain, float InGainDelta)
{
	check(!GetIsInterleaved() && !Other.GetIsInterleaved());

	int32 InNumChannels = GetNumValidChannels();
	int32 InNumFrames = GetNumValidFrames();

	VectorRegister4Float InGainMultVec = MakeVectorRegisterFloat(1.0f, 2.0f, 3.0f, 4.0f);
	VectorRegister4Float InGainDeltaVec = VectorSetFloat1(InGainDelta);
	VectorRegister4Float InGainOriginalVec = VectorSetFloat1(InGain);
	
	for (int32 ch = 0; ch < InNumChannels; ++ch)
	{
		const float* in = Other.GetValidChannelData(ch);
		float* out = GetValidChannelData(ch);
		for (int32 frame = 0; frame < InNumFrames; frame += 4)
		{
			VectorRegister4Float GainsVec = VectorAdd(VectorMultiply(InGainDeltaVec, VectorAdd(VectorSetFloat1((float)frame), InGainMultVec)), InGainOriginalVec);
			VectorStoreAligned(VectorAdd(VectorMultiply(VectorLoadAligned(&in[frame]), GainsVec), VectorLoadAligned(&out[frame])), &out[frame]);
		}
		for (int32 frame = InNumFrames - (InNumFrames % 4); frame < InNumFrames; ++frame)
		{
			out[frame] += in[frame] * (InGain + InGainDelta * frame);
		}
	}
}