// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Harmonix
#include "HarmonixDsp/AudioBufferConstants.h"


// UE
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "HAL/Platform.h"

#define UE_API HARMONIXDSP_API

DECLARE_LOG_CATEGORY_EXTERN(LogAudioBufferConfig, Log, All);

struct FAudioBufferConfig
{
public:
	static const uint32 kMaxAudioBufferChannels = 8;

	UE_API FAudioBufferConfig();
	UE_API FAudioBufferConfig(int32 InNumChannels, int32 InNumFrames, float InSampleRate = 0.0f, bool InInterleaved = false);
	UE_API FAudioBufferConfig(EAudioBufferChannelLayout InChannelLayout, int32 InNumFrames, float InSampleRate = 0.0f, bool InInterleaved = false);
	UE_API FAudioBufferConfig(EAudioBufferChannelLayout InChannelLayout, int32 InNumChannels, int32 InNumFrames, float InSampleRate = 0.0f, bool InInterleaved = false);

	UE_API int32 GetNumTotalSamples() const;
	UE_API float GetSampleRate() const;
	UE_API void  SetSampleRate(float InSamplesPerSecond);
	
	FORCEINLINE int32 GetNumChannels() const { return NumChannels; }
	UE_API void SetNumChannels(int32 InNumChannels);

	FORCEINLINE int32 GetNumFrames() const { return NumFrames; }
	UE_API void SetNumFrames(int32 InNumFrames);

	UE_API FAudioBufferConfig GetInterleavedConfig() const;

	UE_API FAudioBufferConfig GetDeinterleavedConfig() const;

	FORCEINLINE bool GetIsInterleaved() const { return Interleaved; }

	FORCEINLINE EAudioBufferChannelLayout GetChannelLayout() const { return ChannelLayout; }
	UE_API void SetChannelLayout(EAudioBufferChannelLayout InChannelLayout);

	FORCEINLINE uint32 GetChannelMask() const { return ChannelMask; }
	UE_API void SetChannelMask(uint32 InChannelMask);

	UE_API bool operator==(const FAudioBufferConfig&) const;

private:
	float SampleRate;
	int32 NumChannels;
	int32 NumFrames;
	uint32 ChannelMask;
	bool Interleaved;
	EAudioBufferChannelLayout ChannelLayout;
};

#undef UE_API
