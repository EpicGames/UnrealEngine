// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// UE
#include "Containers/UnrealString.h"
#include "Math/UnrealMathUtility.h"

#include "AudioBufferConstants.generated.h"

enum class EAudioBufferChannelLayout : uint8
{
	Raw,
	Mono,
	Stereo,
	StereoPointOne,
	Quad,
	QuadPointOne,
	Five,
	FivePointOne,
	SevenPointOne,
	AmbisonicOrderOne,
	Num,
	UnsupportedFormat
};

UENUM()
enum class ESpeakerChannelAssignment : uint8
{
	LeftFront,
	RightFront,
	Center,
	LFE,
	LeftSurround,
	RightSurround,
	LeftRear,
	RightRear,
	FrontPair,
	CenterAndLFE,
	SurroundPair,
	RearPair,
	AmbisonicW,
	AmbisonicX,
	AmbisonicY,
	AmbisonicZ,
	AmbisonicWXPair,
	AmbisonicYZPair,
	UnspecifiedMono,
	Num					UMETA(Hidden),
	Invalid				UMETA(Hidden)
};

// namespace enum pattern
// creates scope for the masks
// while also keeping them loosely typed
// The benefits: easier to read, easier type (auto complete and search), ESpeakerMask::Type can be used as parameter
namespace ESpeakerMask
{
	enum Type
	{
		LeftFront = 1 << (uint32)ESpeakerChannelAssignment::LeftFront,
		RightFront = 1 << (uint32)ESpeakerChannelAssignment::RightFront,
		Center = 1 << (uint32)ESpeakerChannelAssignment::Center,
		LFE = 1 << (uint32)ESpeakerChannelAssignment::LFE,
		LeftSurround = 1 << (uint32)ESpeakerChannelAssignment::LeftSurround,
		RightSurround = 1 << (uint32)ESpeakerChannelAssignment::RightSurround,
		LeftRear = 1 << (uint32)ESpeakerChannelAssignment::LeftRear,
		RightRear = 1 << (uint32)ESpeakerChannelAssignment::RightRear,
		AmbisonicWAssigned = 1 << (uint32)ESpeakerChannelAssignment::AmbisonicW,
		AmbisonicXAssigned = 1 << (uint32)ESpeakerChannelAssignment::AmbisonicX,
		AmbisonicYAssigned = 1 << (uint32)ESpeakerChannelAssignment::AmbisonicY,
		AmbisonicZAssigned = 1 << (uint32)ESpeakerChannelAssignment::AmbisonicZ,

		// -- NOTE: Ambisonics live in the first 4 channels of an audio buffer, so here we play a game with the mask!
		AmbisonicWRemapped = 1 << (uint32)ESpeakerChannelAssignment::LeftFront,
		// -- NOTE: Ambisonics live in the first 4 channels of an audio buffer, so here we play a game with the mask!
		AmbisonicXRemapped = 1 << (uint32)ESpeakerChannelAssignment::RightFront,
		// -- NOTE: Ambisonics live in the first 4 channels of an audio buffer, so here we play a game with the mask!
		AmbisonicYRemapped = 1 << (uint32)ESpeakerChannelAssignment::Center,
		// -- NOTE: Ambisonics live in the first 4 channels of an audio buffer, so here we play a game with the mask!
		AmbisonicZRemapped = 1 << (uint32)ESpeakerChannelAssignment::LFE,

		UnspecifiedMono = 1 << (uint32)ESpeakerChannelAssignment::UnspecifiedMono,
		Stereo = LeftFront | RightFront,
		AmbisonicAssigned = AmbisonicWAssigned | AmbisonicXAssigned | AmbisonicYAssigned | AmbisonicZAssigned,
		AmbisonicRemapped = AmbisonicWRemapped | AmbisonicXRemapped | AmbisonicYRemapped | AmbisonicZRemapped,
		FourDotZero = LeftFront | RightFront | LeftSurround | RightSurround,
		FourDotOne = LeftFront | RightFront | LFE | LeftSurround | RightSurround,
		FiveDotZero = LeftFront | Center | RightFront | LeftSurround | RightSurround,
		FiveDotOne = LeftFront | Center | RightFront | LFE | LeftSurround | RightSurround,
		SevenDotOne = LeftFront | Center | RightFront | LFE | LeftSurround | RightSurround | LeftRear | RightRear,
		AllSpeakers = 0xFFFFFFFF,
	};
};

namespace HarmonixDsp
{
namespace FAudioBuffer
{
	static const uint32 kMaxChannelsInAudioBuffer = 8;

	FORCEINLINE constexpr HARMONIXDSP_API int32 GetNumChannelsInChannelLayout(EAudioBufferChannelLayout ChannelLayout)
	{
		switch (ChannelLayout)
		{
		case EAudioBufferChannelLayout::Raw:				return kMaxChannelsInAudioBuffer;
		case EAudioBufferChannelLayout::Mono:				return 1;
		case EAudioBufferChannelLayout::Stereo:				return 2;
		case EAudioBufferChannelLayout::StereoPointOne:		return 3;
		case EAudioBufferChannelLayout::Quad:				return 6;
		case EAudioBufferChannelLayout::QuadPointOne:		return 6;
		case EAudioBufferChannelLayout::Five:				return 6;
		case EAudioBufferChannelLayout::FivePointOne:		return 6;
		case EAudioBufferChannelLayout::SevenPointOne:		return 8;
		case EAudioBufferChannelLayout::AmbisonicOrderOne:	return 4;
		default: return 0;
		}
	}

	FORCEINLINE constexpr HARMONIXDSP_API EAudioBufferChannelLayout GetDefaultChannelLayoutForChannelCount(uint32 ChannelCount)
	{
		switch (ChannelCount)
		{
		case 0:  return EAudioBufferChannelLayout::UnsupportedFormat;
		case 1:  return EAudioBufferChannelLayout::Mono;
		case 2:  return EAudioBufferChannelLayout::Stereo;
		case 3:  return EAudioBufferChannelLayout::StereoPointOne;
		case 4:  return EAudioBufferChannelLayout::Quad;
		case 5:  return EAudioBufferChannelLayout::Five;
		case 6:  return EAudioBufferChannelLayout::FivePointOne;
		case 7:  return EAudioBufferChannelLayout::UnsupportedFormat;
		case 8:  return EAudioBufferChannelLayout::SevenPointOne;
		default: return EAudioBufferChannelLayout::UnsupportedFormat;
		}
	}

	FORCEINLINE constexpr HARMONIXDSP_API ESpeakerMask::Type GetChannelMaskForNumChannels(uint32 NumChannels)
	{
		switch (NumChannels)
		{
		case 1: return ESpeakerMask::UnspecifiedMono;
		case 2: return ESpeakerMask::Stereo;
		case 3: return ESpeakerMask::FiveDotZero;
		case 6: return ESpeakerMask::FiveDotOne;
		case 8: return ESpeakerMask::SevenDotOne;
		case 4:
			return ESpeakerMask::AmbisonicAssigned;
		}

		// Invalid num channels
		checkNoEntry();
		return ESpeakerMask::AllSpeakers;
	}

	const uint32 kLayoutHasSpeakerMasks[] = {
		// Raw,
		0,                      

		// Mono,
		ESpeakerMask::Center, 

		// Stereo,
		ESpeakerMask::LeftFront | ESpeakerMask::RightFront, 

		// StereoPointOne,
		ESpeakerMask::LeftFront | ESpeakerMask::RightFront | ESpeakerMask::LFE, 
								   
		// Quad,
		ESpeakerMask::LeftFront | ESpeakerMask::RightFront | ESpeakerMask::LeftSurround | ESpeakerMask::RightSurround, 

		// QuadPointOne,
		ESpeakerMask::LeftFront | ESpeakerMask::RightFront | ESpeakerMask::LeftSurround | ESpeakerMask::RightSurround | ESpeakerMask::LFE, 

		// Five,
		ESpeakerMask::LeftFront | ESpeakerMask::RightFront | ESpeakerMask::LeftSurround | ESpeakerMask::RightSurround | ESpeakerMask::Center, 

		// FivePointOne,
		ESpeakerMask::LeftFront | ESpeakerMask::RightFront | ESpeakerMask::LeftSurround | ESpeakerMask::RightSurround | ESpeakerMask::Center | ESpeakerMask::LFE, 

		// SevenPointOne,
		ESpeakerMask::LeftFront | ESpeakerMask::RightFront | ESpeakerMask::LeftSurround | ESpeakerMask::RightSurround | ESpeakerMask::LeftRear | ESpeakerMask::RightRear | ESpeakerMask::Center | ESpeakerMask::LFE,

		// AmbisonicOrderOne,
		ESpeakerMask::AmbisonicWAssigned | ESpeakerMask::AmbisonicXAssigned | ESpeakerMask::AmbisonicYAssigned | ESpeakerMask::AmbisonicZAssigned
	};

	static const uint8 ChannelAssignmentSpeakerToMappedChannel[] =
	{  // NOTE: Must correspond to ESpeakerChannelAssignment enum!
	   (uint8)ESpeakerChannelAssignment::LeftFront,
	   (uint8)ESpeakerChannelAssignment::RightFront,
	   (uint8)ESpeakerChannelAssignment::Center,
	   (uint8)ESpeakerChannelAssignment::LFE,
	   (uint8)ESpeakerChannelAssignment::LeftSurround,
	   (uint8)ESpeakerChannelAssignment::RightSurround,
	   (uint8)ESpeakerChannelAssignment::LeftRear,
	   (uint8)ESpeakerChannelAssignment::RightRear,
	   (uint8)ESpeakerChannelAssignment::Center,     // FrontPair,
	   (uint8)ESpeakerChannelAssignment::Center,     // CenterAndLFE,
	   (uint8)ESpeakerChannelAssignment::Center,     // SurroundPair,
	   (uint8)ESpeakerChannelAssignment::Center,     // RearPair,
	   (uint8)ESpeakerChannelAssignment::LeftFront,  // AmbisonicW,
	   (uint8)ESpeakerChannelAssignment::RightFront, // AmbisonicX,
	   (uint8)ESpeakerChannelAssignment::Center,     // AmbisonicY,
	   (uint8)ESpeakerChannelAssignment::LFE,        // AmbisonicZ,
	   (uint8)ESpeakerChannelAssignment::Center,     // AmbisonicWXPair,
	   (uint8)ESpeakerChannelAssignment::Center,     // AmbisonicYZPair,
	   (uint8)ESpeakerChannelAssignment::Center,     // UnspecifiedMono,
	   (uint8)ESpeakerChannelAssignment::Center,     // NumSeakerDefs,
	   (uint8)ESpeakerChannelAssignment::Center,     // InvalidSpeaker,
	};

	static const uint32 ChannelAssignmentSpeakerToMappedMasks[] =
	{ // NOTE: Must correspond to ESpeakerChannelAssignment enum!
	   ESpeakerMask::LeftFront,												// ESpeakerMask::LeftFront,
	   ESpeakerMask::RightFront,											// ESpeakerMask::RightFront,
	   ESpeakerMask::Center,												// ESpeakerMask::Center,
	   ESpeakerMask::LFE,													// ESpeakerMask::LFE,
	   ESpeakerMask::LeftSurround,											// ESpeakerMask::LeftSurround,
	   ESpeakerMask::RightSurround,											// ESpeakerMask::RightSurround,
	   ESpeakerMask::LeftRear,												// ESpeakerMask::LeftRear,
	   ESpeakerMask::RightRear,												// ESpeakerMask::RightRear,
	   ESpeakerMask::LeftFront | ESpeakerMask::RightFront,                  // ESpeakerMask::FrontPair,
	   ESpeakerMask::Center | ESpeakerMask::LFE,                            // ESpeakerMask::CenterAndLFE,
	   ESpeakerMask::LeftSurround | ESpeakerMask::RightSurround,            // ESpeakerMask::SurroundPair,
	   ESpeakerMask::LeftRear | ESpeakerMask::RightRear,                    // ESpeakerMask::RearPair,
	   ESpeakerMask::AmbisonicWRemapped,									// ESpeakerMask::AmbisonicW,
	   ESpeakerMask::AmbisonicXRemapped,									// ESpeakerMask::AmbisonicX,
	   ESpeakerMask::AmbisonicYRemapped,									// ESpeakerMask::AmbisonicY,
	   ESpeakerMask::AmbisonicZRemapped,									// ESpeakerMask::AmbisonicZ,
	   ESpeakerMask::AmbisonicWRemapped | ESpeakerMask::AmbisonicXRemapped, // ESpeakerMask::AmbisonicWXPair,
	   ESpeakerMask::AmbisonicYRemapped | ESpeakerMask::AmbisonicZRemapped, // ESpeakerMask::AmbisonicYZPair,
	};

	const float kLeftSpeakerPos_Stereo = UE_HALF_PI;
	const float kLeftSpeakerPos_Quad = UE_HALF_PI / 2.0f;
	const float kLeftSpeakerPos_Five = 0.523598775598f;  // 30 degrees
	const float kLeftSpeakerPos_Seven = 0.523598775598f;  // 30 degrees
	const float kRightSpeakerPos_Stereo = (2.0f * UE_PI) - kLeftSpeakerPos_Stereo;
	const float kRightSpeakerPos_Quad = (2.0f * UE_PI) - kLeftSpeakerPos_Quad;
	const float kRightSpeakerPos_Five = (2.0f * UE_PI) - kLeftSpeakerPos_Five;   // -30 degrees
	const float kRightSpeakerPos_Seven = (2.0f * UE_PI) - kLeftSpeakerPos_Seven;  // -30 degrees

	const float kLeftSurroundSpeakerPos_Stereo = kLeftSpeakerPos_Stereo;
	const float kLeftSurroundSpeakerPos_Quad = UE_HALF_PI + (UE_HALF_PI / 2.0f);
	const float kLeftSurroundSpeakerPos_Five = 1.9198621771f;  // 110 degrees
	const float kLeftSurroundSpeakerPos_Seven = 1.5707963268f;  // 90 degrees
	const float kRightSurroundSpeakerPos_Stereo = (2.0f * UE_PI) - kLeftSurroundSpeakerPos_Stereo;
	const float kRightSurroundSpeakerPos_Quad = (2.0f * UE_PI) - kLeftSurroundSpeakerPos_Quad;
	const float kRightSurroundSpeakerPos_Five = (2.0f * UE_PI) - kLeftSurroundSpeakerPos_Five;   // -110 degrees
	const float kRightSurroundSpeakerPos_Seven = (2.0f * UE_PI) - kLeftSurroundSpeakerPos_Seven;  // -90 degrees

	const float kLeftRearSpeakerPos_Stereo = kLeftSurroundSpeakerPos_Stereo;
	const float kLeftRearSpeakerPos_Quad = 2.617993878f; // 150 degrees
	const float kLeftRearSpeakerPos_Five = 2.617993878f; //      "
	const float kLeftRearSpeakerPos_Seven = 2.617993878f; //      "
	const float kRightRearSpeakerPos_Stereo = (2.0f * UE_PI) - kLeftRearSpeakerPos_Stereo;
	const float kRightRearSpeakerPos_Quad = (2.0f * UE_PI) - kLeftRearSpeakerPos_Quad;  // -150 degrees
	const float kRightRearSpeakerPos_Five = (2.0f * UE_PI) - kLeftRearSpeakerPos_Five;  //      " 
	const float kRightRearSpeakerPos_Seven = (2.0f * UE_PI) - kLeftRearSpeakerPos_Seven; //      "

	const float kDefaultSpeakerAzimuths[(uint8)EAudioBufferChannelLayout::Num][8] = {

	/*********************/
	{ /* Raw               */
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f
	},
	/*********************/
	{ /* Mono              */
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f,
		0.0f
	},
	/*********************/
	{ /* Stereo            */
		kLeftSpeakerPos_Stereo,
		kRightSpeakerPos_Stereo,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Stereo,
		kRightSurroundSpeakerPos_Stereo,
		kLeftRearSpeakerPos_Stereo,
		kRightRearSpeakerPos_Stereo
	},
	/*********************/
	{ /* StereoPointOne    */
		kLeftSpeakerPos_Stereo,
		kRightSpeakerPos_Stereo,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Stereo,
		kRightSurroundSpeakerPos_Stereo,
		kLeftRearSpeakerPos_Stereo,
		kRightRearSpeakerPos_Stereo
	},
	/*********************/
	{ /* Quad              */
		kLeftSpeakerPos_Quad,
		kRightSpeakerPos_Quad,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Quad,
		kRightSurroundSpeakerPos_Quad,
		kLeftRearSpeakerPos_Quad,
		kRightRearSpeakerPos_Quad
	}, // 45, 125, -125, -45
	/*********************/
	{ /* QuadPointOne      */
		kLeftSpeakerPos_Quad,
		kRightSpeakerPos_Quad,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Quad,
		kRightSurroundSpeakerPos_Quad,
		kLeftRearSpeakerPos_Quad,
		kRightRearSpeakerPos_Quad
	},
	/*********************/
	{ /* Five              */
		kLeftSpeakerPos_Five,
		kRightSpeakerPos_Five,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Five,
		kRightSurroundSpeakerPos_Five,
		kLeftRearSpeakerPos_Five,
		kRightRearSpeakerPos_Five
	},
	/*********************/
	{ /* FivePointOne      */
		kLeftSpeakerPos_Five,
		kRightSpeakerPos_Five,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Five,
		kRightSurroundSpeakerPos_Five,
		kLeftRearSpeakerPos_Five,
		kRightRearSpeakerPos_Five
	},
	/*********************/
	{ /* SevenPointOne     */
		kLeftSpeakerPos_Seven,
		kRightSpeakerPos_Seven,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Seven,
		kRightSurroundSpeakerPos_Seven,
		kLeftRearSpeakerPos_Seven,
		kRightRearSpeakerPos_Seven
	},
	/*********************/
	{ /* AmbisonicOrderOne */
		// We can't yet pan around/render in ambisonic, so use stereo... 
		kLeftSpeakerPos_Stereo,
		kRightSpeakerPos_Stereo,
		0.0f,
		0.0f,
		kLeftSurroundSpeakerPos_Stereo,
		kRightSurroundSpeakerPos_Stereo,
		kLeftRearSpeakerPos_Stereo,
		kRightRearSpeakerPos_Stereo
	}
	};
	}
}