// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

#include "HarmonixDsp/Streaming/TrackChannelInfo.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixAudioDataRenderer, Log, All);

class IStretcherAndPitchShifter;
struct FTrackChannelInfo;
template<typename Type> 
class TAudioBuffer;
class FGainMatrix;
class FFusionSampler;
class FSoundWaveData;
class FSoundWaveProxy;
struct FTimeStretchConfig;

class IAudioDataRenderer : public TSharedFromThis<IAudioDataRenderer>
{
public:

	/** Settings for a AudioDataRenderer. */
	struct FSettings
	{
		TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> Shifter = nullptr;
		const FTimeStretchConfig* ShifterSettings = nullptr;
		const TArray<FTrackChannelInfo>* TrackChannelInfo = nullptr;
		const FFusionSampler* Sampler = nullptr;
	};

	virtual ~IAudioDataRenderer() = default;

	UE_DEPRECATED(5.8, "Mutable get/set is deprecated, please call SetSoundWaveData() instead")
	virtual void SetAudioData(TSharedRef<FSoundWaveProxy> SoundWaveProxy, const FSettings& InSettings) {};

	UE_DEPRECATED(5.8, "Mutable get/set is deprecated, please call GetSoundWaveData() instead")
	virtual const TSharedPtr<FSoundWaveProxy> GetAudioData() const { return {}; };

	virtual void SetSoundWaveData(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const FSettings& InSettings);
	virtual TSharedPtr<const FSoundWaveData> GetSoundWaveData() const;

	virtual void Reset() = 0;
	

	virtual void MigrateToSampler(const FFusionSampler* InSampler) = 0;

	virtual void SetFrame(uint32 InFrameNum) = 0;

	virtual double Render(TAudioBuffer<float>& OutBuffer, double InPos, int32 InMaxFrame, double InResampleInc, double InPitchShift, double InSpeed,
		bool MaintainPitchWhenSpeedChanges, const FGainMatrix& InGain) = 0;

	virtual double RenderUnshifted(TAudioBuffer<float>& OutBuffer, double InPos, int32 InMaxFrame, double InInc, const FGainMatrix& InGain) = 0;

protected:

	struct FLerpData
	{
		uint32 PosA;
		uint32 PosB;

		// pos a and b Relative to start position, and never loop
		uint32 PosARelative;
		uint32 PosBRelative;

		float WeightA;
		float WeightB;
	};

	double CalculateLerpData(FLerpData* LerpArray, int32 InNumPoints, uint32 InNumOutFrames, double InPos, int32 InMaxFrame, double Inc) const;
};
