// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/PlatformMath.h"
#include "Math/UnrealMathUtility.h"

namespace AudioRendering
{
	static const int32 kFramesPerRenderBuffer = 128;
	static const int32 kMicroSliceSize = 4; // TODO HX_AUDIO_MICRO_SLICE_SIZE
	static const float kMicroFadeMs = 25.0f;
	static const float kMicroFadeSec = kMicroFadeMs / 1000.0f;
}

namespace HarmonixDsp
{
	// ridiculously small minimum:
	static const float kDbMin = -10000.0f;
	// reasonable minimum value for volume in Db
	static const float kDbSilence = -96.0f;
	// reasonable maximum value for volume in Db
	static const float kDbMax = 12.0f;

	const float kCentsPerOctave = 1200.0f;
	const float kOctavesPerCent = 1.0f / kCentsPerOctave;

	// TODO?
	// SMALL_NUMBER = 1.e-8f;
	// KINDA_SMALL_NUMBER = 1.e-4f;
	// kTinyGain = 1.e-5f though...
	// could we use either of these?
	const float kTinyGain = 0.00001f;

	inline float Log10(float Value)
	{
		return FMath::LogX(10.0f, Value);
	}

	inline double Log10(double Value)
	{
		return FMath::LogX(10.0, Value);
	}

	inline float ClampDB(float dB)
	{
		return FMath::Clamp(dB, kDbSilence, kDbMax);
	}

	inline float DBToLinear(float dB)
	{
		return FMath::Pow(10.0f, dB / 20.0f);
	}

	inline int8 DBToMidiLinear(float dB)
	{
		const float Max = static_cast<float>(MAX_int8);
		return (int8)(FMath::Pow(10.0f, dB / 40.0f) * Max);
	}

	inline float LinearToDB(float Gain)
	{
		if (Gain == 0.0f)
		{
			return kDbSilence;
		}

		return 20.0f * Log10(Gain);
	}

	inline float MidiLinearToDB(int8 Level /* 0 - 127 (*/)
	{
		if (Level == 0)
		{
			return kDbSilence;
		}
		const float Max = static_cast<float>(MAX_int8);
		return 40.0f * Log10((float)Level / Max);
	}

	inline float Midi14BitLinearToDB(int32 Level /* 0 - 16383 (*/)
	{
		if (Level == 0)
		{
			return kDbSilence;
		}
		const float Max = static_cast<float>(MAX_int16 / 2);
		return 40.0f * Log10((float)Level / Max);
	}

	// decibels relative to full scale
	inline float dBFS(float Linear)
	{
		if (Linear == 0.0f)
		{
			return kDbMin;
		}

		float dB = 20.0f * Log10(Linear);

		if (dB < kDbMin)
		{
			dB = kDbMin;
		}

		return dB;
	}


	inline float FRandSample()
	{
		float X = FMath::FRand();
		X = 2.0f * X - 1.0f;
		checkSlow(-1.0f <= X && X <= 1.0f);
		return X;
	}

	inline void PanToGainsConstantPower(float Pan, float& OutLeftGain, float& OutRightGain)
	{
		float NormalizedPan = (Pan + 1.0f) / 2.0f;
		NormalizedPan *= UE_HALF_PI;
		OutLeftGain = FMath::Cos(NormalizedPan);
		OutRightGain = FMath::Sin(NormalizedPan);
		if (FMath::IsNearlyZero(OutLeftGain, kTinyGain))
		{
			OutLeftGain = 0.0f;
		}
		if (FMath::IsNearlyZero(OutRightGain, kTinyGain))
		{
			OutRightGain = 0.0f;
		}
	}
	
	float MapLinearToDecibelRange(float ValueLinear, float MaxDecibels, float RangeDecibels);
}