// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/Class.h"
#include "Misc/Variant.h"

#include "TypedParameter.generated.h"

#define UE_API HARMONIXDSP_API

/**
 * constrain the types allowed for TypedVariant
 * simplify the blueprint customization and ui/ux
 */
UENUM()
enum class EParameterType : uint8
{
	Bool	UMETA(DisplayName = "bool"),
	Double	UMETA(DisplayName = "double"),
	Float	UMETA(DisplayName = "float"),
	Int8	UMETA(DisplayName = "int8"),
	Int16	UMETA(DisplayName = "int16"),
	Int32	UMETA(DisplayName = "int32"),
	Int64	UMETA(DisplayName = "int64"),
	Name	UMETA(DisplayName = "FName"),
	String	UMETA(DisplayName = "FString"),
	UInt8	UMETA(DisplayName = "uint8"),
	UInt16	UMETA(DisplayName = "uint16"),
	UInt32	UMETA(DisplayName = "uint32"),
	UInt64	UMETA(DisplayName = "uint64"),
	Num		UMETA(Hidden),
	Invalid UMETA(Hidden)
};

/**
 * Wrapper for FVariant, an extensible union of multiple types
 *
 * exposed to blueprints as a "Type Parameter", which in blueprints:
 * - allows the user to select the type the parameter represents
 * - assign the value for the type
 * 
 * We can then retrieve the underlying value by checking what type is stored
 * and reading the value accordingly
 * 
 * Serializes as an FVariant, which serializes the value as a byte array.
 */
USTRUCT()
struct FTypedParameter
{
	GENERATED_BODY()

	// default to int type
	FTypedParameter() : Value(FVariant(int(0))) {}

	template<typename T>
	FTypedParameter(T InValue)
	{
		static_assert(FromVariantType(TVariantTraits<T>::GetType()) != EParameterType::Invalid);
		Value = FVariant(InValue);
	}

	/**
	 * Template method for getting the stored value
	 * asserts that value type is valid and matches
	 *
	 * @return Value as type T
	 */
	template<typename T>
	T GetValue() const
	{
		check(GetType() != EParameterType::Invalid);
		check(GetType() != EParameterType::Num);
		return Value.GetValue<T>();
	}

	/**
	 * Template method for setting the stored value
	 * asserts that the value type is supported
	 *
	 * overwrites current value type with new value type
	 *
	 * @param InValue new value to assign
	 */
	template<typename T>
	void SetValue(T InValue)
	{
		static_assert(FromVariantType(TVariantTraits<T>::GetType()) != EParameterType::Invalid);
		EParameterType NewType = FromVariantType(TVariantTraits<T>::GetType());
		check(NewType != EParameterType::Invalid);
		Value = InValue;
	}

	/**
	 * Template method to check type matches
	 *
	 * @return whether the type T matches the type represented by this parameter
	 */
	template<typename T>
	bool IsType() const
	{
		EParameterType Type = FromVariantType(TVariantTraits<T>::GetType());
		return GetType() == Type;
	}

	/**
	 * Same as this->GetType() == InType;
	 *
	 * @return whether the EParameterType matches the type represented by this parameter
	 */
	bool IsType(EParameterType InType) const
	{
		return GetType() == InType;
	}

	template<typename T>
	void SetType()
	{
		EParameterType Type = FromVariantType(TVariantTraits<T>::GetType());
		SetType(Type);
	}

	/**
	 * Set the type this Parameter represents
	 * will reset the value to the default for the type
	 * only if the type is different
	 *
	 * @param Type - the new type
	 */
	UE_API void SetType(EParameterType Type);

	/**
	 * Gets the type this parameter represents
	 */
	UE_API EParameterType GetType() const;

	// custom struct serialization
	UE_API bool Serialize(FArchive& Ar);

	template<typename T>
	static bool FindMappedValue(const TMap<FName, FTypedParameter>& ParameterMap, FName Name, T& OutValue)
	{
		if (const FTypedParameter* Parameter = ParameterMap.Find(Name))
		{
			if (Parameter->IsType<T>())
			{
				OutValue = Parameter->GetValue<T>();
				return true;
			}
		}
		return false;
	}

private:

	static constexpr EParameterType FromVariantType(EVariantTypes VariantType)
	{
		switch (VariantType)
		{
		case EVariantTypes::Bool: return EParameterType::Bool;
		case EVariantTypes::Double: return EParameterType::Double;
		case EVariantTypes::Float: return EParameterType::Float;
		case EVariantTypes::Int8: return EParameterType::Int8;
		case EVariantTypes::Int16: return EParameterType::Int16;
		case EVariantTypes::Int32: return EParameterType::Int32;
		case EVariantTypes::Int64: return EParameterType::Int64;
		case EVariantTypes::Name: return EParameterType::Name;
		case EVariantTypes::String: return EParameterType::String;
		case EVariantTypes::UInt8: return EParameterType::UInt8;
		case EVariantTypes::UInt16: return EParameterType::UInt16;
		case EVariantTypes::UInt32: return EParameterType::UInt32;
		case EVariantTypes::UInt64: return EParameterType::UInt64;
		default: return EParameterType::Invalid;
		}
	}

	FVariant Value;

	static const uint8 kVersion = 0;
};

template<>
struct TStructOpsTypeTraits<FTypedParameter> : public TStructOpsTypeTraitsBase2<FTypedParameter>
{
	enum
	{
		WithSerializer = true,
	};
};

#undef UE_API
