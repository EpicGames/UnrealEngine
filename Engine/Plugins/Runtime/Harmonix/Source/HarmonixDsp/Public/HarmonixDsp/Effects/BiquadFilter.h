// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/Ramper.h"
#include "HarmonixDsp/StridePointer.h"

#include "HarmonixDsp/Effects/Settings/BiquadFilterSettings.h"

#include "HAL/PlatformMath.h"

template<typename>
class TComplex;

struct FBiquadFilterSettings;

namespace Harmonix::Dsp::Effects
{
	// coefficients for difference function
	//
	//    y(n) = B0*x(n)+B1*x(n-1)+B2*x(n-2)
	//                  -A1*y(n-1)-A2*y(n-2)
	//
	// 
	// 
	// or in the z domain
	//   
	//           Y(z)    B0 + B1*z^-1 + B2*z^-2 
	//    H(z) = ---- =  ----------------------
	//           X(x)     1 + A1*z^-1 + A2*z^-2
	//
	// a0 is assumed to be 1.0.
	class FBiquadFilterCoefs
	{
	public:

		FBiquadFilterCoefs(float B0Gain = 1.0f)
			: B0(B0Gain)
			, B1(0.0f)
			, B2(0.0f)
			, A1(0.0f)
			, A2(0.0f)
			, SR(1.0f)
		{}

		FBiquadFilterCoefs(const FBiquadFilterSettings& InSettings, float Fs)
		{
			SR = Fs;
			MakeFromSettings(InSettings);
		}

		void GetMagnitudeResponse(float const* InFrequencies, int32 InNumFreq, float* OutResopnse);
		void MakeFromSettings(const FBiquadFilterSettings& InSettings);
		void MakeFromSettings(const FBiquadFilterSettings& InSettings, float Fs)
		{
			SR = Fs;
			MakeFromSettings(InSettings);
		}

		void Flatten()
		{
			B0 = 1;
			B1 = 0;
			B2 = 0;
			A1 = 0;
			A2 = 0;
		}

		bool IsNoop() const
		{
			return B0 == 1 && B1 == 0 && B2 == 0 && A1 == 0 && A2 == 0;
		}

		bool operator==(const FBiquadFilterCoefs& Other) const
		{
			return (B0 == Other.B0)
				&& (B1 == Other.B1)
				&& (B2 == Other.B2)
				&& (A1 == Other.A1)
				&& (A2 == Other.A2)
				&& (SR == Other.SR);
		}

		bool operator!=(const FBiquadFilterCoefs& Other) const
		{
			return !(operator==(Other));
		}

		FBiquadFilterCoefs operator+(const FBiquadFilterCoefs& Other) const
		{
			FBiquadFilterCoefs Result;
			Result.B0 = B0 + Other.B0;
			Result.B1 = B1 + Other.B1;
			Result.B2 = B2 + Other.B2;
			Result.A1 = A1 + Other.A1;
			Result.A2 = A2 + Other.A2;
			return Result;
		}

		FBiquadFilterCoefs operator-(const FBiquadFilterCoefs& Other) const
		{
			FBiquadFilterCoefs Result;
			Result.B0 = B0 - Other.B0;
			Result.B1 = B1 - Other.B1;
			Result.B2 = B2 - Other.B2;
			Result.A1 = A1 - Other.A1;
			Result.A2 = A2 - Other.A2;
			return Result;
		}

		FBiquadFilterCoefs operator*(double Scale) const
		{
			FBiquadFilterCoefs Result;
			Result.B0 = B0 * Scale;
			Result.B1 = B1 * Scale;
			Result.B2 = B2 * Scale;
			Result.A1 = A1 * Scale;
			Result.A2 = A2 * Scale;
			return Result;
		}


		bool IsStable() const;

		// feedforward
		double B0;
		double B1;
		double B2;

		// feedback;
		double A1;
		double A2;

		double SR;

	private:
		void GetPoles(TComplex<double>& OutP1, TComplex<double>& OutP2) const;

		void MakeLowPassCoefs(float F0, float Q);
		void MakeHighPassCoefs(float F0, float Q);
		void MakeBandPassCoefs(float F0, float Q);
		void MakePeakingCoefs(float F0, float dbGain, float Q);
		void MakeLowShelfCoefs(float F0, float dbGain, float Q);
		void MakeHighShelfCoefs(float F0, float dbGain, float Q);
	};

	template<typename T>
	struct TBiquadFilter
	{
		TBiquadFilter() { ResetState(); }
		virtual ~TBiquadFilter() = default;

		void ResetState()
		{
			FF1 = 0.0f;
			FF2 = 0.0f;
		}

		float Filter(float In, const FBiquadFilterCoefs& InCoefs)
		{
			T FF0;
			T Minus_A1 = -InCoefs.A1;
			T Minus_A2 = -InCoefs.A2;
			double Out;

			FF0 = (T)In
				+ Minus_A1 * FF1
				+ Minus_A2 * FF2;

			Out = InCoefs.B0 * FF0
				+ InCoefs.B1 * FF1
				+ InCoefs.B2 * FF2;

			FF2 = FF1;
			FF1 = FF0;

			return (float)Out;
		}

		void Process(float const* input,
			float* output, uint32 InNumSamples,
			const FBiquadFilterCoefs& InCoefs,
			double InGain = 1.0)
		{
			T IncFF0;
			T IncFF1 = FF1;
			T IncFF2 = FF2;

			double B0 = InCoefs.B0 * InGain;
			double B1 = InCoefs.B1 * InGain;
			double B2 = InCoefs.B2 * InGain;
			T Minus_A1 = -InCoefs.A1;
			T Minus_A2 = -InCoefs.A2;
			double Out;

			// Direct Form II

			for (uint32 n = 0; n < InNumSamples; ++n)
			{
				IncFF0 = (T)input[n]
					+ Minus_A1 * IncFF1
					+ Minus_A2 * IncFF2;

				Out = B0 * IncFF0
					+ B1 * IncFF1
					+ B2 * IncFF2;

				output[n] = (float)Out;

				IncFF2 = IncFF1;
				IncFF1 = IncFF0;
			}

			FF1 = IncFF1;
			FF2 = IncFF2;
		}

		void ProcessTransposed(float const* input,
			float* output,
			uint32 InNumSamples,
			const FBiquadFilterCoefs& InCoefs,
			double InGain = 1.0)
		{
			T In;
			T Out;
			T d1 = FF1;
			T d2 = FF2;
			double B0 = InCoefs.B0 * InGain;
			double B1 = InCoefs.B1 * InGain;
			double B2 = InCoefs.B2 * InGain;
			T minus_A1 = -InCoefs.A1;
			T minus_A2 = -InCoefs.A2;

			// Direct Form II transposed:
			//    y[n] = B0 * x[n] + d1;
			//    d1 = B1 * x[n] - A1 * y[n] + d2;
			//    d2 = B2 * x[n] - A2 * y[n];

			for (uint32 n = 0; n < InNumSamples; ++n)
			{
				In = (T)input[n];
				Out = In * B0 + d1;
				d1 = B1 * In + minus_A1 * Out + d2;
				d2 = B2 * In + minus_A2 * Out;

				output[n] = (float)Out;
			}

			FF1 = d1;
			FF2 = d2;
		}

		void ProcessInterleaved(TDynamicStridePtr<float>& input,
			TDynamicStridePtr<float>& output,
			int32 InNumSamples,
			const FBiquadFilterCoefs& InCoefs,
			double InGain = 1.0)
		{
			T IncFF0;
			T IncFF1 = FF1;
			T IncFF2 = FF2;
			double B0 = InCoefs.B0 * InGain;
			double B1 = InCoefs.B1 * InGain;
			double B2 = InCoefs.B2 * InGain;
			T minus_A1 = -InCoefs.A1;
			T minus_A2 = -InCoefs.A2;
			double Out;

			for (int32 n = 0; n < InNumSamples; ++n)
			{
				IncFF0 = (T)input[n]
					+ minus_A1 * IncFF1
					+ minus_A2 * IncFF2;

				Out = B0 * IncFF0
					+ B1 * IncFF1
					+ B2 * IncFF2;

				output[n] = (float)Out;

				IncFF2 = IncFF1;
				IncFF1 = IncFF0;
			}

			FF1 = IncFF1;
			FF2 = IncFF2;
		}

		// TODO? 
		//static void UnitTest();

	private:

		// we should keep around the state for each filter
		// that is the stuff that makes it distinct from other filters
		//typedef float T;
		//typedef double T;
		T FF1;
		T FF2;
	};

	// declare FBiquadFilter for compatibility reasons. And saves some typing
	typedef TBiquadFilter<float> FBiquadFilter;

	template<typename T, int32 Num>
	class TMultipassBiquadFilter
	{
	public:

		TMultipassBiquadFilter() { ResetState(); }
		~TMultipassBiquadFilter() = default;

		void ResetState()
		{
			FMemory::Memset(&History[0], 0, sizeof(T) * Num * 2);
		}

		void Process(float const* input,
			float* output,
			uint32 InNumSamples,
			const FBiquadFilterCoefs& InCoefs,
			double InGain = 1.0,
			int32 InNumPasses = Num)
		{
			T ff0;
			T ff1 = History[0];
			T ff2 = History[1];
			double B0 = InCoefs.B0 * InGain;
			double B1 = InCoefs.B1 * InGain;
			double B2 = InCoefs.B2 * InGain;
			T minus_A1 = (T)-InCoefs.A1;
			T minus_A2 = (T)-InCoefs.A2;
			double Out;

			for (uint32 n = 0; n < InNumSamples; ++n)
			{
				ff0 = (T)input[n]
					+ minus_A1 * ff1
					+ minus_A2 * ff2;

				Out = B0 * ff0
					+ B1 * ff1
					+ B2 * ff2;

				output[n] = (float)Out;

				ff2 = ff1;
				ff1 = ff0;
			}

			History[0] = ff1;
			History[1] = ff2;

			for (int32 i = 1; i < InNumPasses; ++i)
			{
				ff1 = History[2 * i];
				ff2 = History[2 * i + 1];

				for (uint32 n = 0; n < InNumSamples; ++n)
				{
					ff0 = (T)output[n]
						+ minus_A1 * ff1
						+ minus_A2 * ff2;

					Out = B0 * ff0
						+ B1 * ff1
						+ B2 * ff2;

					output[n] = (float)Out;

					ff2 = ff1;
					ff1 = ff0;
				}

				History[2 * i] = ff1;
				History[2 * i + 1] = ff2;
			}
		}

		void ProcessInterleaved(TDynamicStridePtr<float>& input,
			TDynamicStridePtr<float>& output,
			int32 InNumSamples,
			const FBiquadFilterCoefs& InCoefs,
			double InGain = 1.0,
			int32 InNumPasses = Num)
		{
			T ff0;
			T ff1 = History[0];
			T ff2 = History[1];
			double B0 = InCoefs.B0 * InGain;
			double B1 = InCoefs.B1 * InGain;
			double B2 = InCoefs.B2 * InGain;
			T minus_A1 = -InCoefs.A1;
			T minus_A2 = -InCoefs.A2;
			double Out;

			for (int32 n = 0; n < InNumSamples; ++n)
			{
				ff0 = (T)input[n]
					+ minus_A1 * ff1
					+ minus_A2 * ff2;

				Out = B0 * ff0
					+ B1 * ff1
					+ B2 * ff2;

				output[n] = (float)Out;

				ff2 = ff1;
				ff1 = ff0;
			}

			History[0] = ff1;
			History[1] = ff2;

			for (int32 i = 1; i < InNumPasses; ++i)
			{
				ff1 = History[2 * i];
				ff2 = History[2 * i + 1];

				for (int32 n = 0; n < InNumSamples; ++n)
				{
					ff0 = (T)output[n]
						+ minus_A1 * ff1
						+ minus_A2 * ff2;

					Out = B0 * ff0
						+ B1 * ff1
						+ B2 * ff2;

					output[n] = (float)Out;

					ff2 = ff1;
					ff1 = ff0;
				}

				History[2 * i] = ff1;
				History[2 * i + 1] = ff2;
			}
		}
	private:
		T History[2 * Num];
	};

	// simple definiton, to save some typing I guess
	//template<int32 Num>
	//class HARMONIXDSP_API FMultipassBiquadFilter : public TMultipassBiquadFilter<float, Num> {};

	template<typename T, int32 COUNT>
	class TMultiChannelBiquadFilter : public TBiquadFilter<T>
	{
	public:
		TMultiChannelBiquadFilter()
		{
			GainRamper.SnapTo(1.0f);
			Flatten(true);
		}

		virtual ~TMultiChannelBiquadFilter() = default;

		//this prepares a filter to the default settings not default coefs...
		void Prepare(float InSampleRate, float InRampTimeMs = -1.0f, float InRampCallsPerSec = -1.0f, bool InAdjustRampTimeByOctaves = false)
		{
			IsFirstTimeThrough = true;
			SampleRate = InSampleRate;
			AdjustRampTimeByOctaves = InAdjustRampTimeByOctaves;
			RampTimePerOctave = InRampTimeMs;
			MaxFreq = (SampleRate / 2.0f);
			MaxFreq *= 0.9f;

			if (InRampCallsPerSec <= 0.0f)
				InRampCallsPerSec = kDefaultRampsPerSec;

			//floor this to the nearest power of 2
			FramesPerRamp = 1;
			while (FramesPerRamp < int32(SampleRate / InRampCallsPerSec) + 1)
			{
				FramesPerRamp <<= 1;
			}
			FramesPerRamp >>= 1;

			TrueRPS = float(SampleRate / FramesPerRamp);

			if (InRampTimeMs <= 0.0f)
				InRampTimeMs = kDefaultRampTimeMs;

			FreqRamper.SetRampTimeMs(TrueRPS, InRampTimeMs);
			QRamper.SetRampTimeMs(TrueRPS, InRampTimeMs);
			GainRamper.SetRampTimeMs(TrueRPS, InRampTimeMs);

			SnapSettings();

			ApplySettingsFromRampers();
		}

		void ProcessInterleaved(float* input, float* output, int32 NumSamples, int32 NumChannels, int32 NumActiveChannels, float InGain = 1.0f)
		{
			check(NumSamples % FramesPerRamp == 0);
			check(FramesPerRamp <= NumSamples);

			if (IsFirstTimeThrough)
			{
				SnapSettings();
				IsFirstTimeThrough = false;
			}

			int32 HopsThisPass = int32(NumSamples / FramesPerRamp);

			for (int32 i = 0; i < HopsThisPass; ++i)
			{
				for (int32 j = 0; j < NumActiveChannels; ++j)
				{
					TDynamicStridePtr<float> inPtr(&input[j], NumChannels);
					TDynamicStridePtr<float> outPtr(&output[j], NumChannels);
					FilterStates[j].ProcessInterleaved(inPtr, outPtr, FramesPerRamp, Coefs, GainRamper.GetCurrent() * InGain);
				}

				input += NumChannels * FramesPerRamp;
				output += NumChannels * FramesPerRamp;

				RampSettings();
			}
		}

		void ProcessInterleavedInPlace(float* InOutData, int32 NumSamples, int32 NumChannels, int32 NumActiveChannels, float InGain = 1.0f)
		{
			check(NumSamples % FramesPerRamp == 0);
			check(FramesPerRamp <= NumSamples);

			if (IsFirstTimeThrough)
			{
				SnapSettings();
				IsFirstTimeThrough = false;
			}

			int32 rampsThisPass = int32(NumSamples / FramesPerRamp);

			for (int32 i = 0; i < rampsThisPass; ++i)
			{
				for (int32 j = 0; j < NumActiveChannels; ++j)
				{
					TDynamicStridePtr<float> inPtr(&InOutData[j], NumChannels);
					FilterStates[j].ProcessInterleaved(inPtr, inPtr, FramesPerRamp, Coefs, GainRamper.GetCurrent() * InGain);
				}

				InOutData += NumChannels * FramesPerRamp;

				RampSettings();
			}
		}

		void SnapSettings()
		{
			FreqRamper.SnapToTarget();
			QRamper.SnapToTarget();
			GainRamper.SnapToTarget();
			ApplySettingsFromRampers();
		}

		void Flatten(bool Snap)
		{
			FreqRamper.SetTarget(20.0f);
			QRamper.SetTarget(1.0f);
			GainRamper.SetTarget(1.0f);
			Settings.Type = EBiquadFilterType::HighPass;

			if (Snap)
			{
				SnapSettings();
			}

			UpdateCoefsFromSettings();
		}

		void ResetState()
		{
			IsFirstTimeThrough = true;
			for (int32 i = 0; i < COUNT; ++i)
			{
				FilterStates[i].ResetState();
			}
		}

		const FBiquadFilterSettings& GetSettings() const { return Settings; }

		void SetTargetGain(float InGain, bool Snap)
		{
			if (Snap)
			{
				GainRamper.SnapTo(InGain);
				return;
			}

			if (InGain != GainRamper.GetTarget())
			{
				Settings.DesignedDBGain = HarmonixDsp::LinearToDB(InGain);
				if (AdjustRampTimeByOctaves)
				{
					float startingDb = HarmonixDsp::LinearToDB(GainRamper.GetCurrent());
					float rampTime = FMath::Abs((startingDb - Settings.DesignedDBGain) / 6.0f);
					GainRamper.SetRampTimeMs(RampTimePerOctave * rampTime);
				}
				GainRamper.SetTarget(InGain);
			}
		}

		void SetFreqTarget(float InTarget)
		{
			InTarget = FMath::Clamp(InTarget, kMinFreq, MaxFreq);
			if (InTarget != FreqRamper.GetTarget())
			{
				if (AdjustRampTimeByOctaves)
				{
					float startFreq = FreqRamper.GetCurrent();
					if (startFreq != 0.0f && InTarget != 0.0f)
					{
						float octaves = FMath::Abs(FMath::Log2(InTarget / startFreq));
						FreqRamper.SetRampTimeMs(RampTimePerOctave * octaves);
					}
				}
				FreqRamper.SetTarget(InTarget);
			}
		}

		void SetQTarget(float InTarget)
		{
			if (InTarget != QRamper.GetTarget())
			{
				if (AdjustRampTimeByOctaves)
				{
					float startQ = QRamper.GetCurrent();
					if (startQ != 0.0f && InTarget != 0.0f)
					{
						float octaves = FMath::Abs(FMath::Log2(InTarget / startQ));
						QRamper.SetRampTimeMs(RampTimePerOctave * octaves);
					}
				}
				QRamper.SetTarget(InTarget);
			}
		}

		void  SetEnabled(bool InEnabled) { Settings.IsEnabled = InEnabled; }
		void  SetType(EBiquadFilterType InType) { Settings.Type = InType; }
		float GetGainTarget() { return GainRamper.GetTarget(); }
		float GetFreqTarget() { return FreqRamper.GetTarget(); }
		float GetQTarget() { return QRamper.GetTarget(); }
		bool  GetIsEnabled() { return Settings.IsEnabled; }
		EBiquadFilterType GetType() { return Settings.Type; }


		void UpdateCoefsFromSettings()
		{
			FBiquadFilterCoefs NewCoefs(Settings, SampleRate);
			Coefs = NewCoefs;
		}

		void SetSettingsTarget(const FBiquadFilterSettings& InTarget)
		{
			FreqRamper.SetTarget(InTarget.Freq);
			QRamper.SetTarget(InTarget.Q);
			Settings.IsEnabled = InTarget.IsEnabled;
			Settings.Type = InTarget.Type;
			Settings.DesignedDBGain = InTarget.DesignedDBGain;
			Settings.Pad = InTarget.Pad;
			UpdateCoefsFromSettings();
		}

	private:

		void ApplySettingsFromRampers()
		{
			Settings.Freq = FreqRamper.GetCurrent();
			Settings.Q = QRamper.GetCurrent();
			UpdateCoefsFromSettings();
		}

		void RampSettings()
		{
			bool HasChanged = false;
			HasChanged = FreqRamper.Ramp() || HasChanged;
			HasChanged = QRamper.Ramp() || HasChanged;
			HasChanged = GainRamper.Ramp() || HasChanged;
			if (HasChanged)
			{
				ApplySettingsFromRampers();
			}
		}

		float SampleRate = 44100.0f;
		float MaxFreq = 20000.0f;
		int32 FramesPerRamp;
		float RampTimePerOctave;
		bool AdjustRampTimeByOctaves;
		float TrueRPS;
		bool IsFirstTimeThrough = true;
		TBiquadFilter<T> FilterStates[COUNT];

		TLinearRamper<float> FreqRamper;
		TLinearRamper<float> QRamper;
		TLinearRamper<float> GainRamper;
		FBiquadFilterSettings Settings;

		FBiquadFilterCoefs Coefs;

		// 20 samples at 48k
		const float kDefaultRampsPerSec = 4800.0f;

		// 120 increments at 48k
		const float kDefaultRampTimeMs = 80.0f;
		const float kMinFreq = 20.0f;
	};

	// simple definiton to save on type, maintain compatibility
	//template<int32 COUNT>
	//class FMultiChannelBiquadFilter : public TMultiChannelBiquadFilter<float, COUNT> {};

}
