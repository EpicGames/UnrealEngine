// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/ForEach.h"
#include "DSP/MultichannelBuffer.h"
#include "HarmonixDsp/Ramper.h"
#include "HarmonixDsp/TimeSyncOption.h"
#include "HarmonixDsp/Effects/Settings/DelaySettings.h"
#include "HarmonixDsp/Effects/BiquadFilter.h"
#include "HarmonixDsp/Effects/Settings/BiquadFilterSettings.h"
#include "HarmonixDsp/Modulators/Lfo.h"

namespace Harmonix::Dsp::Effects
{
	// forward declare test class
	namespace Delay::Tests
	{
		class FTestDelay;
	}

	class FDelay
	{
	public:
		
		// make Tests a friend class to acces private variables
		friend Delay::Tests::FTestDelay;

		// The max number of channels we expect to support with the delay
		inline static constexpr int32 AbsoluteMaxChannels = 12;
		
		HARMONIXDSP_API FDelay();

		HARMONIXDSP_API void Prepare(float InSampleRate, uint32 InMaxChannels, float InMaxDelayTimeMs);

		HARMONIXDSP_API void Unprepare();

		HARMONIXDSP_API virtual ~FDelay();

		void Clear()
		{
			// zero the delay line
			if (DelayLineInterleaved.Num() > 0)
			{
				FMemory::Memzero(DelayLineInterleaved.GetData(), DelayLineInterleaved.Num() * sizeof(float));
			}
			
			DelayPos = 0;
			CanSlamParams = true;
		}

		void SetParamsToTargets()
		{
			DelayRamper.SnapToTarget();
			FeedbackRamper.SnapToTarget();
			WetRamper.SnapToTarget();
			DryRamper.SnapToTarget();

			DelaySpreadLeft.SnapToTarget();
			DelaySpreadRight.SnapToTarget();
		}

		/*

		The "universal comb filter" as seen in DAFX (Zolzer) pg 66.
		Parameters: time [0,max_time], feedback [0,1], wet [0,1], and dry [0,1]
		Additional parameters added to process the wet signal: Filters, Modulator, Panning

						  dry
			  .----------->(*)--------------------.
			  |                                   |
			  |            time                   |
			  |             /                     |
			  | xf(n)    .----------.      wet    v
		x(n) ---+->(+)-->|delay line|--+-->(*)-->(+)--> y(n)
				 ^       '----------'  |
				 |                     |
				 |                     | xd(n)
				 '-------(*)<----------'
						feedback

		*/

		HARMONIXDSP_API void Process(Audio::FMultichannelBufferView& InOutBuffer);

		HARMONIXDSP_API float CalculateSecsToIdle();

		float GetSampleRate() const { return SampleRate; }
		
		// units are either seconds or beats, depending on beat-sync setting
		HARMONIXDSP_API void  SetDelaySeconds(float Seconds);
		float GetDelaySeconds() const { return DelayTimeSeconds; }

		// in beat-sync mode, delays are specified in beats, not seconds, and the
		//  delay must be notified when the tempo changes.
		HARMONIXDSP_API void  SetTimeSyncOption(ETimeSyncOption Option);
		ETimeSyncOption  GetTimeSyncOption() const { return TimeSyncOption; }

		HARMONIXDSP_API void  SetTempo(float Bpm);
		float GetTempo() const { return TempoBpm; }

		HARMONIXDSP_API void  SetSpeed(float Speed);
		float GetSpeed() const { return Speed; }

		float GetOutputGain() const { return OutputGain; }
		void SetOutputGain(const float InGain) { OutputGain = InGain; }

		HARMONIXDSP_API void SetFeedbackGain(float Gain);
		HARMONIXDSP_API float GetFeedbackGain() const;

		HARMONIXDSP_API void SetWetGain(float Gain);
		HARMONIXDSP_API float GetWetGain() const;

		HARMONIXDSP_API void SetDryGain(float Gain);
		HARMONIXDSP_API float GetDryGain() const;

		HARMONIXDSP_API void SetWetFilterEnabled(bool Enabled);
		bool GetWetFilterEnabled() const { return WetFilters.GetSettings().IsEnabled; }

		HARMONIXDSP_API void SetFeedbackFilterEnabled(bool Enabled);
		bool GetFeedbackFilterEnabled() const { return FeedbackFilters.GetSettings().IsEnabled; }
		
		HARMONIXDSP_API void SetFilterFreq(float Freq);
		float GetFilterFreq() const { return WetFilters.GetSettings().Freq; }

		HARMONIXDSP_API void SetFilterQ(float Q);
		float GetFilterQ() const { return WetFilters.GetSettings().Q; }

		HARMONIXDSP_API void SetFilterType(EDelayFilterType Type);
		HARMONIXDSP_API void SetFilterType(EBiquadFilterType Type);
		EBiquadFilterType GetFilterType() const { return WetFilters.GetSettings().Type; }

		HARMONIXDSP_API void SetLfoEnabled(bool bEnabled);
		bool GetLfoEnabled() const { return LfoSettings.IsEnabled; }

		HARMONIXDSP_API void SetLfoTimeSyncOption(ETimeSyncOption Option);
		ETimeSyncOption GetLfoTimeSyncOption() const { return LfoSyncOption; }

		HARMONIXDSP_API void SetLfoFreq(float Freq);
		float GetLfoFreq() const { return LfoSettings.Freq; }

		HARMONIXDSP_API void SetLfoDepth(float InDepth);
		HARMONIXDSP_API float GetLfoDepth() const;

		HARMONIXDSP_API void SetStereoSpreadLeft(float Spread);
		float GetStereoSpreadLeft() const { return DelaySpreadLeft.GetCurrent(); }

		HARMONIXDSP_API void SetStereoSpreadRight(float Spread);
		float GetStereoSpreadRight() const { return DelaySpreadRight.GetCurrent(); }

		HARMONIXDSP_API void SetStereoType(EDelayStereoType Type);
		EDelayStereoType GetStereoType() const { return DelayType; }

	private:

		struct FDelayOutput
		{
			float Delay;
			float Feedback;
		};

		void FreeUpMemory();

		void ApplyNewParams();

		void RampValues(int32 FrameNum);
		int32 GetPongCh(int32 ChannelIndex, int32 NumChannels) const;

		FORCEINLINE void ProcessInternal(float const X, const TDynamicStridePtr<float>& DelayLinePtr, FDelayOutput& OutDelay) const;

		int32 MaxChannels = 0;
		int32 ActiveChannels = 0;
		int32 SampleRate = 44100;

		uint32 MaxBlockSize = 0;

		uint32 Length = 0;
		Audio::FAlignedFloatBuffer DelayLineInterleaved;
		Audio::FAlignedFloatBuffer WetChannelInterleaved;
		uint32 DelayPos = 0;
		uint32 PosMask;

		float FeedbackGain = 0.0f;
		// also known as feedforward
		float WetGain = 1.0f;
		float DryGain = 1.0f;
		float DelayTimeSeconds = 0.5f;

		ETimeSyncOption TimeSyncOption = ETimeSyncOption::None;
		ETimeSyncOption LfoSyncOption = ETimeSyncOption::None;

		TLinearRamper<float> WetRamper;
		TLinearRamper<float> DryRamper;
		TLinearRamper<float> FeedbackRamper;
		TLinearRamper<float> DelayRamper;

		TMultiChannelBiquadFilter<double, 8> WetFilters;
		TMultiChannelBiquadFilter<double, 8> FeedbackFilters;

		TLinearRamper<float>     DelaySpreadLeft;
		TLinearRamper<float>     DelaySpreadRight;
		EDelayStereoType         DelayType = EDelayStereoType::Default;

		Harmonix::Dsp::Modulators::FLfo Lfo;
		FLfoSettings LfoSettings;
		float LfoBaseFrequency = UE_SMALL_NUMBER;

		// info for bouncing between speakers
		// belongs here rather than GainTable.cpp because this is specific to the
		// ping pong delay effect
		int32 FiveOneSurroundRotation[6] = { 1, 5, -1, -1, 0, 4 };
		int32 SevenOneSurroundRotation[8] = { 1, 5, -1, -1, 0, 7, 4, 6 };
		int32 SevenOneFourSurroundRotation[12] = { 8, 9, -1, -1, -1, -1, 10, 11, 1, 7, 6, 0 };
		int32 FiveOneSurroundLRForce[6] = { 1, 0, -1, -1, 5, 4 };
		int32 SevenOneSurroundLRForce[8] = { 1, 0, -1, -1, 5, 4, 7, 6 };
		int32 SevenOneFourSurroundLRForce[12] = { 1, 0, -1, -1, 5, 4, 7, 6, 9, 8, 11, 10 };
		
		float MaxDelayInSamples{};
		float MaxDelayInMs{};
		float DelayInSamples{};
		float OutputGain{ 1.f };
		float TempoBpm{ 120.f };
		float Speed{ 1.f };
		bool  CanSlamParams{ true };

		static constexpr int32 HopNum = 32;
	};

};