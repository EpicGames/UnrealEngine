// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/AlignedBuffer.h"
#include "HarmonixDsp/Ramper.h"
#include "HarmonixDsp/Effects/Settings/DistortionSettings.h"
#include "HarmonixDsp/Effects/BiquadFilter.h"
#include "HarmonixDsp/Effects/FirFilter.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"

#define UE_API HARMONIXDSP_API

namespace Harmonix::Dsp::Effects
{

	class FDistortionV2
	{
	public:
		UE_API FDistortionV2(uint32 SampleRate = 48000, uint32 MaxRenderBufferSize = 128);
		UE_API void Setup(uint32 SampleRate, uint32 MaxRenderBufferSize);
		virtual ~FDistortionV2() = default;

		UE_API void Process(const TArray<TArrayView<const float>>& InBuffer, const TArray<TArrayView<float>>& OutBuffer);

		UE_API void  SetInputGainDb(float InGainDb, bool Snap = false);
		UE_API float GetInputGainDb() const;

		UE_API void SetInputGain(float InGain, bool Snap = false);
		float GetInputGain() const { return InputGain; }

		UE_API void  SetOutputGainDb(float InGainDb, bool Snap = false);
		UE_API float GetOutputGainDb() const;

		UE_API void SetOutputGain(float InGain, bool Snap = false);
		float GetOutputGain() const { return OutputGain; }

		UE_API void  SetDryGainDb(float InGainDb, bool Snap = false);
		UE_API float GetDryGainDb() const;

		UE_API void SetDryGain(float InGain, bool Snap = false);
		float GetDryGain() const { return DryGain; }

		UE_API void  SetWetGainDb(float InGainDb, bool Snap = false);
		UE_API float GetWetGainDb() const;

		UE_API void SetWetGain(float InGain, bool Snap = false);
		float GetWetGain() const { return WetGain; }

		UE_API void  SetMix(float Mix, bool Snap = false);
		UE_API void  SetDCOffset(float Offset, bool Snap = false);

		float GetDCOffset() const { return DCAdjust; }
		UE_API void  SetType(uint8 InType);
		UE_API void  SetType(EDistortionTypeV2 Type);
		EDistortionTypeV2 GetType() const { return Type; }
		UE_API void SetupFilter(int32 Index, const FDistortionFilterSettings& InSettings);
		int32  GetFilterPasses(int32 Index) { return FilterPasses[Index]; }
		UE_API void SetOversample(bool Oversample, int32 RenderBufferSizeFrames);
		bool GetOversample() const { return DoOversampling; }
		UE_API void SetSampleRate(uint32 SampleRate);

		UE_API void Setup(const FDistortionSettingsV2& Settings, uint32 SampleRate, uint32 RenderBufferSizeFrames, bool Snap);

		UE_API void Reset();

		static const int32 kMaxChannels = 8;
		static const int32 kMaxFilterPasses = 3;
		static const int32 kRampHops = 16;

	private:
		EDistortionTypeV2	  Type;
		TLinearRamper<float>  InputGain;
		TLinearRamper<float>  OutputGain;
		TLinearRamper<float>  DCAdjust;
		TLinearRamper<float>  DryGain;
		TLinearRamper<float>  WetGain;
		bool		          FilterPreClip[FDistortionSettingsV2::kNumFilters];
		FBiquadFilterSettings FilterSettings[FDistortionSettingsV2::kNumFilters];
		TLinearRamper<float>  FilterGain[FDistortionSettingsV2::kNumFilters];

		TLinearRamper<FBiquadFilterCoefs> FilterCoefs[FDistortionSettingsV2::kNumFilters];

		uint32					FilterPasses[kMaxFilterPasses];

		TMultipassBiquadFilter<double, kMaxFilterPasses> Filter[FDistortionSettingsV2::kNumFilters][kMaxChannels];

		FFirFilter32			OversampleFilterUp[kMaxChannels];
		FFirFilter32			OversampleFilterDown[kMaxChannels];
		bool					DoOversampling;
		Audio::FAlignedFloatBuffer UpsampleBuffer;
		uint32					SampleRate;

		static const uint32   kNumFilterTaps = 32;
		static UE_API const float    kOversamplingFilterTaps[kNumFilterTaps];

		mutable FCriticalSection SettingsLock;
	};
}

#undef UE_API
