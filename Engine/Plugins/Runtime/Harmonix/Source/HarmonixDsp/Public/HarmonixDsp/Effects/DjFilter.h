// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Filter.h"

#include "HarmonixDsp/Parameters/Parameter.h"

#define UE_API HARMONIXDSP_API

namespace Harmonix::Dsp::Effects
{
	/**
	 * @brief A filter that cross-fades between a low-pass filter, the dry signal, and a high-pass filter on one "knob".
	 */
	class FDjFilter
	{
	public:
		/**
		 * @brief The knob position.
		 * -1 to the dead zone will low-pass the signal.
		 * Dead zone to 1 will high-pass the signal.
		 * Within the dead zone will fade to dry.
		 */
		Parameters::TParameter<float> Amount{ -1.0f, 1.0f, 0.0f };

		/**
		 * @brief The filters' resonance
		 */
		Parameters::TParameter<float> Resonance{ 0.01f, 10.0f, 1.0f };

		/**
		 * @brief The frequency the low-pass will be at when Amount is set to -1
		 */
		Parameters::TParameter<float> LowPassMinFrequency{ 20.0f, 20000.0f, 20.0f };

		/**
		 * @brief The frequency the low-pass will be at when Amount is set to -DeadZoneSize
		 */
		Parameters::TParameter<float> LowPassMaxFrequency{ 20.0f, 20000.0f, 10000.0f };

		/**
		 * @brief The frequency the high-pass will be at when Amount is set to DeadZoneSize
		 */
		Parameters::TParameter<float> HighPassMinFrequency{ 20.0f, 20000.0f, 20.0f };

		/**
		 * @brief The frequency the high-pass will be at when Amount is set to 1
		 */
		Parameters::TParameter<float> HighPassMaxFrequency{ 20.0f, 20000.0f, 10000.0f };

		/**
		 * @brief The portion of the Amount that will cross-fade between the filtered signal and the dry signal
		 */
		Parameters::TParameter<float> DeadZoneSize{ 0.01f, 0.5f, 0.1f };

		UE_API explicit FDjFilter(float InSampleRate);
		
		UE_API void Reset(float InSampleRate);

		UE_API void Process(const Audio::FAlignedFloatBuffer& InBuffer, Audio::FAlignedFloatBuffer& OutBuffer);

	private:
		Audio::FStateVariableFilter Filter;
		float LastFrequency;
		float LastQ;
		float LastDryGain;
		float LastWetGain;
	};
}

#undef UE_API
