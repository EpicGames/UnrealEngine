// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

namespace Harmonix::Dsp::Effects
{
	/**
	 * Finite Impulse Response filter, or (FIR) Filter, that can be used to 
	 * perform filtering and other signal processing operations. 
	 * The filter is defined by a set of "taps" which are coefficients that
	 * determine the shape of the filter's frequency response.
	 */
	class FFirFilter
	{
	public:
		FFirFilter();

		HARMONIXDSP_API virtual ~FFirFilter();

		/**
		* Initializes the filter with the specified taps, which must have 
		* InTapCount elements. 
		*
		* If InCopyTaps is true, the taps will be copied into a new buffer 
		* owned by the filter; otherwise, the filter will 
		* simply store a pointer to the input array.
		* 
		* Any existing tap or history buffers will be freed.
		*
		* @param InTaps			the array of filter taps
		* @param InTapCount		the number of filter taps
		* @param InCopyTaps		whether to copy the taps into a new buffer
		*/
		virtual void Init(const float* InTaps, int32 InTapCount, bool InCopyTaps);

		/**
		* Adds the specified input data to the filter's history buffer. 
		* InCount elements will be added to the buffer, starting at the 
		* current history index.
		*
		* @param Input			the input data to add
		* @param InCount		the number of input samples to add
		*/
		virtual void AddData(float* Input, int32 InCount);

		/**
		* Adds a single data point to the FIR filter, starting at the 
		* current history index
		*
		* @param Input			the input data to add
		*/
		virtual void AddData(float Input);

		/**
		* Computes a filtered sample using the FIR filter.
		*
		* @return the filtered sample
		*/
		virtual float GetSample();

		/**
		* Resets the FIR filter.
		*/
		void Reset();

	protected:
		const float* Taps;
		int32        TapCount;
		bool         OwnTaps;
		float*		 History;
		uint32       HistoryIndex;
	};

	/**
	* A FIR Filter optimized for 32-tap coefficients. 
	*/
	class FFirFilter32 : public FFirFilter
	{
	public:

		/**
		* Initializes the 32-tap FIR filter with the given taps.
		*
		* @param InTaps			the filter taps
		* @param InTapCount		the number of filter taps (must be 32)
		* @param InCopyTaps		whether to copy the taps or not
		*/
		virtual void Init(const float* InTaps, int32 InTapCount, bool InCopyTaps) override;

		virtual void AddData(float* Input, int32 InCount) override;

		virtual void AddData(float Input) override;

		virtual float GetSample() override;

		/**
		* Inserts a new input sample to the filter, 
		* and generates 4 up-sampled output samples
		*
		* @param InSample		the new input sample to insert to the filter
		* @param OutSamples		an array of length 4 to store the generated output samples
		*/
		void Upsample4x(float InSample, float* OutSamples);
	};

};