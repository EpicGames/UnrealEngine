// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/VectorRegister.h"
#include "Logging/LogMacros.h"
#include "HAL/PlatformMath.h"
#include "HAL/Platform.h"

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixFourBiquads, Log, All);

namespace Harmonix::Dsp::Effects
{

	class FFourBiquads
	{
	public:
		FFourBiquads()
			: B0(VectorSetFloat1(1.0f))
			, B1(VectorSetFloat1(0.0f))
			, B2(VectorSetFloat1(0.0f))
			, A1(VectorSetFloat1(0.0f))
			, A2(VectorSetFloat1(0.0f))
			, S1(VectorSetFloat1(0.0f))
			, S2(VectorSetFloat1(0.0f))
		{
		}

		void SetFilter(
			uint32 Index,
			float  InB0,
			float  InB1,
			float  InB2,
			float  InA1,
			float  InA2
		)
		{
			check(0 <= Index && Index < 4);
			if (FMath::Abs(InA2) > 0.9999995f || FMath::Abs(InA1) > 1.9999995f)
			{
				UE_LOGF(LogHarmonixFourBiquads, Log, "SetFilter: filter may become unstable!");
			}

			SetComponent(B0, Index, InB0);
			SetComponent(B1, Index, InB1);
			SetComponent(B2, Index, InB2);
			SetComponent(A1, Index, InA1);
			SetComponent(A2, Index, InA2);
		}

		VectorRegister4Float ProcessOne(VectorRegister4Float Sample)
		{
			// process one 4-vector of samples, using transposed direct form II
			VectorRegister4Float Y;
			Y = VectorMultiply(B0, Sample);
			Y = VectorAdd(Y, S1);
			S1 = VectorMultiply(B1, Sample);
			S1 = VectorAdd(S1, S2);
			S1 = VectorSubtract(S1, VectorMultiply(A1, Y));
			S2 = VectorMultiply(B2, Sample);
			S2 = VectorSubtract(S2, VectorMultiply(A2, Y));
			return Y;
		}

		void Process(TArray<VectorRegister4Float>& Samples)
		{
			// process a block of 4-vector of samples, e.g. representing
			//  4 interleaved channels of audio data
			VectorRegister4Float Y, X, S1Temp, S2Temp, B0Temp, B1Temp, B2Temp, A1Temp, A2Temp;
			S1Temp = S1;
			S2Temp = S2;
			B0Temp = B0;
			B1Temp = B1;
			B2Temp = B2;
			A1Temp = A1;
			A2Temp = A2;

			uint32 numSamps = (uint32)Samples.Num();
			for (uint32 Idx = 0; Idx < numSamps; ++Idx)
			{
				X = Samples[Idx];
				Y = VectorMultiply(B0Temp, X);
				Y = VectorAdd(Y, S1Temp);
				S1Temp = VectorMultiply(B1Temp, X);
				S1Temp = VectorAdd(S1Temp, S2Temp);
				S1Temp = VectorSubtract(S1Temp, VectorMultiply(A1Temp, Y));
				S2Temp = VectorMultiply(B2Temp, X);
				S2Temp = VectorSubtract(S2Temp, VectorMultiply(A2Temp, Y));
				Samples[Idx] = Y;
			}

			S1 = S1Temp;
			S2 = S2Temp;
		}

		void Process(const float* Input, int32 FrameCount, float** Output)
		{
			VectorRegister4Float X;
			union
			{
				VectorRegister4Float Y;
				float YComponents[4];
			};

			for (int32 Idx = 0; Idx < FrameCount; ++Idx)
			{
				X = VectorSetFloat1(Input[Idx]);
				Y = VectorMultiply(B0, X);
				Y = VectorAdd(Y, S1);
				S1 = VectorMultiply(B1, Y);
				S1 = VectorAdd(S1, S2);
				S1 = VectorSubtract(S1, VectorMultiply(A1, Y));
				S2 = VectorMultiply(B2, X);
				S2 = VectorSubtract(S2, VectorMultiply(A2, Y));
				for (int32 Ch = 0; Ch < 4; Ch++)
				{
					Output[Ch][Idx] = YComponents[Ch];
				}
			}
		}

	private:

		void SetComponent(VectorRegister4Float& OutVector, uint32 Index, float Value)
		{
			check(0 <= Index && Index < 4);
			MS_ALIGN(16) float GCC_ALIGN(16) StackVector[4];
			VectorStoreAligned(OutVector, StackVector);
			StackVector[Index] = Value;
			OutVector = VectorLoadAligned(StackVector);
		}


		VectorRegister4Float B0;
		VectorRegister4Float B1;
		VectorRegister4Float B2;
		VectorRegister4Float A1;
		VectorRegister4Float A2;
		VectorRegister4Float S1;
		VectorRegister4Float S2;
	};

};