// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BiquadFilterSettings.generated.h"

UENUM()
enum class EBiquadFilterType
{
	LowPass		UMETA(Json="low-pass"),
	HighPass	UMETA(Json="high-pass"),
	BandPass	UMETA(Json="band-pass"),
	Peaking		UMETA(Json="peaking"),
	LowShelf	UMETA(Json="low-shelf"),
	HighShelf	UMETA(Json="high-shelf"),
	Num			UMETA(Hidden),
	None		UMETA(Hidden)
};

USTRUCT()
struct FBiquadFilterSettings
{
	GENERATED_BODY()
public:
	typedef int16 BFSPad;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool IsEnabled = true;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EBiquadFilterType Type = EBiquadFilterType::LowPass;

	// what's this?
	//UPROPERTY(EditDefaultsOnly, Category = "Settings")
	BFSPad Pad = 0;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "Freq (Hz)", Meta = (ClampMin = "20", ClampMax = "20000", UIMin = "20", UIMax = "20000"))
	float Freq = 630.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "0", ClampMax = "10", UIMin = "0", UIMax = "10"))
	float Q = 0.0f; 

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "-96", ClampMax = "12", UIMin = "-96", UIMax = "12"))
	float DesignedDBGain = 0.0f;

	FBiquadFilterSettings() { ResetToDefaults(); }
	FBiquadFilterSettings(EBiquadFilterType InType, float InFreq, float InQ, float InGain = 1.0f, bool InEnabled = true)
		: IsEnabled(InEnabled)
		, Type(InType)
		, Freq(InFreq)
		, Q(InQ)
		, DesignedDBGain(InGain)
	{
	}

	void ResetToDefaults()
	{
		Freq = 630.0f;
		Q = 1.0f;
		IsEnabled = false;
		Type = EBiquadFilterType::LowPass;
		DesignedDBGain = 0.0f;
	}

	void GetMagnitudeResponse(float const* FrequenciesOfInterest, int32 NumFrequencies, float* OutMagnitudeResponse, float Fs);

	bool operator==(const FBiquadFilterSettings& Other)
	{
		return Type == Other.Type
			&& IsEnabled == Other.IsEnabled
			&& Freq == Other.Freq
			&& Q == Other.Q
			&& DesignedDBGain == Other.DesignedDBGain;
	}
};
