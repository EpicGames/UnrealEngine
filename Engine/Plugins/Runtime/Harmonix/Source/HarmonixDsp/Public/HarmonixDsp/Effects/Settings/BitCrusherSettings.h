// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BitCrusherSettings.generated.h"

USTRUCT()
struct FBitCrusherSettings
{
	GENERATED_BODY()
public:

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP")
	bool IsEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1"))
	float InputGain = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1"))
	float OutputGain = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1"))
	float WetGain = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", DisplayName="Crush (Bit Depth)", Meta=(ClampMin="0", ClampMax = "15", UIMin="0", UIMax="15"))
	uint16 Crush = 4;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (ClampMin = "0", ClampMax = "16", UIMin = "0", UIMax = "16"))
	uint16 SampleHoldFactor = 5;
};