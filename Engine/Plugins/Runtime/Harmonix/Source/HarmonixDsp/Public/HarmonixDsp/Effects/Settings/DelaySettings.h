// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/TimeSyncOption.h"

#include "DelaySettings.generated.h"

UENUM()
enum class EDelayStereoType : uint8
{
	Default,
	CustomSpread,
	PingPongForceLR,
	PingPongSum,
	PingPongIndividual,
	Num					UMETA(Hidden)
};

UENUM()
enum class EDelayFilterType : uint8
{
	LowPass,
	HighPass,
	BandPass,
	Num			UMETA(Hidden)
};

USTRUCT()
struct FDelaySettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool IsEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	ETimeSyncOption TimeSyncOption = ETimeSyncOption::None;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "0", ClampMax = "100", UIMin = "0", UIMax = "100"))
	float TimeSeconds = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "1", ClampMax = "240", UIMin = "0", UIMax = "240"))
	float Tempo = 120.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float DryGain = 0.5f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float WetGain = 0.5f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float FeedbackGain = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool EQEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EDelayFilterType EQType = EDelayFilterType::LowPass;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "EQ Freq (Hz)", Meta = (ClampMin = "20", ClampMax = "20000", UIMin = "20", UIMax = "20000"))
	float EQFreq = 600.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "0", ClampMax = "10", UIMin = "0", UIMax = "10"))
	float EQQ = 1.0f; 

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool LfoEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	ETimeSyncOption LfoTimeSyncOption = ETimeSyncOption::None;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "LFO Rate (Hz)", Meta = (ClampMin = "0.01", ClampMax = "40", UIMin = "0.01", UIMax = "40"))
	float LfoRate = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "LFO Depth", Meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float LfoDepth = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EDelayStereoType StereoType = EDelayStereoType::Default;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "Pan Left (%)", Meta = (ClampMin = "0", ClampMax = "100", UIMin = "0", UIMax = "100"))
	float PanLeft = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "Pan Right (%)", Meta = (ClampMin = "0", ClampMax = "100", UIMin = "0", UIMax = "100"))
	float PanRight = 1.0f;
};