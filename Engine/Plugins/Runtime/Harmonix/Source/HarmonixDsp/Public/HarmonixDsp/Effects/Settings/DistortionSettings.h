// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/Effects/Settings/BiquadFilterSettings.h"

#include "DistortionSettings.generated.h"

USTRUCT()
struct FDistortionFilterSettings
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool FilterPreClip = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FBiquadFilterSettings Filter;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "1", UIMax = "3", ClampMin = "1", ClampMax = "3"))
	int32 NumPasses = 0;
};

UENUM()
enum class EDistortionTypeV1
{
	Clean,
	Warm,
	Dirty,
	Soft,
	Asymmetric,
	Num				UMETA(Hidden)
};

USTRUCT()
struct FDistortionSettingsV1
{
	GENERATED_BODY()

public:

	static const uint32 kNumFilters = 3;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool IsEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta =(UIMin="-96.0", UIMax="12.0", ClampMin="-96.0", ClampMax="12"))
	float InputGainDb = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "-96.0", UIMax = "12.0", ClampMin = "-96.0", ClampMax="12"))
	float OutputGainDb = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "-1.0", UIMax = "1.0", ClampMin = "-1.0", ClampMax="1.0"))
	float DCAdjust = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EDistortionTypeV1 Type = EDistortionTypeV1::Clean;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool Oversample = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FDistortionFilterSettings Filters[kNumFilters];
};

UENUM()
enum class EDistortionTypeV2
{
	Clean,
	Warm,
	Clip,
	Soft,
	Asymmetric,
	Cruncher,
	CaptCrunch,
	Rectifier,
	Num			UMETA(Hidden)
};

USTRUCT()
struct FDistortionSettingsV2
{
	GENERATED_BODY()

public:

	static const uint32 kNumFilters = 3;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool IsEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "-96.0", UIMax = "12.0", ClampMin = "-96.0", ClampMax = "12"))
	float InputGainDb = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "-96.0", UIMax = "12.0", ClampMin = "-96.0", ClampMax = "12"))
	float OutputGainDb = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "0.0", UIMax = "0.0", ClampMin = "0.0", ClampMax = "1"))
	float DryGain = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1"))
	float WetGain = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "-1.0", UIMax = "1.0", ClampMin = "-1.0", ClampMax = "1.0"))
	float DCAdjust = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EDistortionTypeV2 Type = EDistortionTypeV2::Clean;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool Oversample = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FDistortionFilterSettings Filters[kNumFilters];
};

