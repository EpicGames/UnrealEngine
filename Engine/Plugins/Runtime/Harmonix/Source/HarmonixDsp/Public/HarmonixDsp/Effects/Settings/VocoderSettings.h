// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VocoderSettings.generated.h"

UENUM()
enum class EVocoderBandConfig : uint8
{
	k4 = 0		UMETA(DisplayName = "4"),
	k8			UMETA(DisplayName = "8"),
	k16			UMETA(DisplayName = "16"),
	k32			UMETA(DisplayName = "32"),
	k64			UMETA(DisplayName = "64"),
	k128		UMETA(DisplayName = "128"),
	k256		UMETA(DisplayName = "256"),
	Num			UMETA(Hidden),
	None		UMETA(Hidden),
};

USTRUCT()
struct FVocoderBandConfig
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	int32 BandCount = 0;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	float FrequencyRatio = 0.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FName Name;

	FVocoderBandConfig() {}

	FVocoderBandConfig(int32 InBandCount, float InFrequencyRatio, FName InName)
		: BandCount(InBandCount)
		, FrequencyRatio(InFrequencyRatio)
		, Name(InName)
	{
	}

	static const FVocoderBandConfig& Get(uint8 Index)
	{
		check(Index >= 0 && Index < (uint8)EVocoderBandConfig::Num);
		return sBandConfigs[Index];
	}

	static const FVocoderBandConfig& Get(EVocoderBandConfig BandConfig)
	{
		uint8 Index = (uint8)(BandConfig);
		check(Index >= 0 && Index < (uint8)EVocoderBandConfig::Num);
		return sBandConfigs[Index];
	}

	static const FVocoderBandConfig sBandConfigs[(uint8)EVocoderBandConfig::Num];
};

#define VOCODER_DEFAULT_BAND_CONFIG EVocoderBandConfig::k64;

USTRUCT()
struct FVocoderBand
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", UIMax = "1", ClampMin="0", ClampMax="1"))
	float Gain = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP")
	bool Solo = false;
};

USTRUCT()
struct FVocoderSettings
{
	GENERATED_BODY()

	FVocoderSettings();

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP")
	bool IsEnabled = true;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", ClampMin = "0"))
	int32 ModulatorIndex = 0;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "1", ClampMin = "1", UIMax = "8", ClampMax = "8"))
	int32 ChannelCount = 1;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "40000", ClampMin = "40000", UIMax = "200000", ClampMax = "200000"))
	float SampleRate = 0.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP")
	EVocoderBandConfig BandConfig = EVocoderBandConfig::k64;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP")
	bool Soloing = false;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", ClampMin = "0"))
	int32 FrameCount = 0;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float CarrierGain = 1.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float ModulatorGain = 1.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float CarrierThin = 1.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float ModulatorThin = 1.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", ClampMin = "0"))
	float Attack = 1.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", ClampMin = "0"))
	float Release = 0.0008f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP")
	float HighEmphasis = 0.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float Wet = 1.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float OutputGain = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP")
	TArray<FVocoderBand> Bands;

	const FVocoderBandConfig& GetBandConfig() const { return FVocoderBandConfig::Get(BandConfig); }
};