// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/Ramper.h"

namespace Harmonix::Dsp::Effects
{
	class FTapeStopStart
	{
	public:

		enum class EState
		{
			Play,
			Stop
		};

		enum class EPhase
		{
			SpeedingUp,
			Playing,
			Stopped,
			SlowingDown
		};

		HARMONIXDSP_API FTapeStopStart();

		HARMONIXDSP_API void Prepare(float SampleRate, int32 MaxNumChannels, float MaxDecelerationSeconds);

		void Clear()
		{
			BufferWritePos = 0;
			BufferReadPos = 0.0f;
			Buffer.ZeroData();
		}

		HARMONIXDSP_API EPhase Process(const TArray<const float*>& InChannelData, const TArray<float*>& OutChannelData, const int32 NumFrames);

		HARMONIXDSP_API void  SnapState(EState State);
		HARMONIXDSP_API void  SetState(EState State, float DurationSeconds);
		EState GetState() const { return CurrentState; }
		float GetSpeed() const { return SRRamper.GetCurrent(); }

	private:
		void CheckForStateUpdates();
		void SpeedUp(int32& FrameIndex, const TArray<const float*>& InChannelData, const TArray<float*>& OutChannelData, const int32 NumFrames);
		void SlowDown(int32& FrameIndex, const TArray<const float*>& InChannelData, const TArray<float*>& OutChannelData, const int32 NumFrames);

		int32 MaxNumChannels;
		int32 ActiveNumChannels;
		float SampleRate;

		TAudioBuffer<float>   Buffer;
		int32 BufferWritePos;
		float BufferReadPos;

		int32 NumSamplesInFullRamp;
		int32 RampProgressInSamples;
		TArray<float> CrossfadeFloor;

		TLinearRamper<float> SRRamper;

		EState CurrentState;
		EState DesiredState;
		float RampTimeSeconds;

		static const int32 kHopNum = 4;
		static const int32 kCrossfadeSamps = 512;
	};
}
