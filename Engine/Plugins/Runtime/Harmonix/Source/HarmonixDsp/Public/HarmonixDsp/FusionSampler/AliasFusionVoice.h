// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/FusionSampler/FusionVoice.h"

class FSingletonFusionVoicePool;

class FAliasFusionVoice : public FFusionVoice
{
public:

	enum class EState
	{
		Idle,
		Starting, 
		Running,
		Released
	};

public:

	FAliasFusionVoice(FSingletonFusionVoicePool* InPool);
	FAliasFusionVoice(const FAliasFusionVoice& Other);

	virtual TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> GetPitchShifter() const override;

	virtual Harmonix::Dsp::Modulators::EAdsrStage GetAdsrStage() const override;
	virtual void SetPitchOffset(double InNumCents) override;
#if FUSION_VOICE_DEBUG_DUMP_ENABLED
	virtual void AttackWithTargetNote(
		uint8 MidiNoteNumber, 
		float InGain, 
		int32 InEventTick, 
		int32 InTriggerTick, 
		double InStartPointMs = 0.0f, 
		const char* InFilePath = nullptr) override;
#else
	virtual void AttackWithTargetNote(
		uint8 MidiNoteNumber, 
		float InGain, 
		int32 InEventTick, 
		int32 InTriggerTick, 
		double InStartPointMs = 0.0f) override;
#endif

	virtual uint32 Process(
		uint32 SliceIndex, 
		uint32 SubsliceIndex, 
		float** Output, uint32 InNumChannels, 
		uint32 InMaxNumSamples, 
		float InSpeed = 1.0f, 
		float InTempoBPM = 120.0f,
		bool MaintainPitchWhenSpeedChanges = false) override;

	virtual bool IsInUse() const override { return State != EState::Idle; }
	virtual void Release() override;
	virtual void FastRelease() override;
	virtual void Kill() override;
	virtual bool MatchesIDs(const FFusionSampler* InSampler, FMidiVoiceId InVoiceID, const FKeyzoneSettings* keyzone = nullptr) override;
	virtual bool UsesKeyzone(const FKeyzoneSettings* keyzone) const override;
	virtual void SetupLfo(uint8 Index, const FLfoSettings& InSettings) override;

	bool IsStartingOrRunning() const { return State == EState::Starting || State == EState::Running; }
	EState GetState() const { return State; }

	void Activate(FFusionSampler* InSampler, FMidiVoiceId InVoiceID, FFusionVoice* InRealVoice, FRelinquishHandler Handler);

	FFusionVoice* GetRealVoice() const { return MyVoice; }
	void ClearVoice() { MyVoice = nullptr; }


	virtual bool IsAlias() const override { return true; }
	virtual bool IsRendererForAlias() const override { return false; }
	virtual bool HasBeenRelinquished() const override { return !RelinquishHandler; }

private:

	EState State = EState::Idle;
	FSingletonFusionVoicePool* VoicePool = nullptr;
	// Real voice doing the rendering
	FFusionVoice* MyVoice = nullptr;
	FFusionVoice::FRelinquishHandler RelinquishHandler;
};