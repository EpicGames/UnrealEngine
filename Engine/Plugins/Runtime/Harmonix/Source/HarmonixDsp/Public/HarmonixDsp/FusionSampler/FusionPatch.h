// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"
#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"

#include "Harmonix/AudioRenderableProxy.h"
#include "IAudioProxyInitializer.h"

#include "FusionPatch.generated.h"

#define UE_API HARMONIXDSP_API

class UAudioSample;
class FFusionSampler;

// Empty base class for data that will be passed along to the sampler
struct FFusionPatchUserData
{
	virtual ~FFusionPatchUserData() {}

	virtual void OnSamplerConnected(const FFusionSampler* Sampler) {}
	virtual void OnSamplerDisconnected(const FFusionSampler* Sampler) {}
};

// User facing data of FusionPatch
USTRUCT(BlueprintType)
struct FFusionPatchData
{
	GENERATED_BODY()

public:
	friend class UFusionPatch;

	IMPL_AUDIORENDERABLE_PROXYABLE(FFusionPatchData)

	FFusionPatchData() {};
	const TArray<FKeyzoneSettings>& GetKeyzones() const { return Keyzones; };
	const FFusionPatchSettings& GetSettings() const { return Settings; }
	UE_API void InitProxyData(const Audio::FProxyDataInitParams& InitParams);

	UE_API void ConnectSampler(const FFusionSampler* Sampler);
	UE_API void DisconnectSampler(const FFusionSampler* Sampler);

private:

	UPROPERTY(EditAnywhere, Category="Keyzones")
	TArray<FKeyzoneSettings> Keyzones;

	UPROPERTY(EditAnywhere, Category="Settings")
	FFusionPatchSettings Settings;

	UPROPERTY()
	TArray<FFusionPatchSettings> Presets_DEPRECATED;

	TSharedPtr<FFusionPatchUserData> UserData;
};

// This next macro does a few things. 
//   wrap "FFusionPatchData" in a proxy named "FFusionPatchDataProxy" 
//   this proxy class can then be used as the "guts" for the metasound later
USING_AUDIORENDERABLE_PROXY(FFusionPatchData, FFusionPatchDataProxy)

UENUM(BlueprintType)
enum class EFusionPatchAudioLoadResult : uint8
{
	Success,
	Fail,
	Cancelled
};

/**
* Called when a load request for a sound has completed.
*/
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnFusionPatchLoadComplete, const class UFusionPatch*, LoadedFusionPatch, EFusionPatchAudioLoadResult, LoadResult);

UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Fusion Patch"))
class UFusionPatch : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:
	// IAudioProxyDataFactory
	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams);
	
	static const int32 kVoicePriorityNoSteal = 0;

	UE_API UFusionPatch();

	UE_API void UpdatePatch(const FFusionPatchData& InPatchData);

	const FFusionPatchSettings& GetSettings() const { return FusionPatchData.GetSettings(); }
	UE_API void UpdateSettings(const FFusionPatchSettings& InSettings);

	const TArray<FKeyzoneSettings>& GetKeyzones() const { return FusionPatchData.GetKeyzones(); }
	int32 GetNumKeyzones() const { return FusionPatchData.GetKeyzones().Num(); }
	UE_API void UpdateKeyzones(const TArray<FKeyzoneSettings>& NewKeyzones);

	TSharedPtr<FFusionPatchUserData> GetUserData() { return FusionPatchData.UserData; }
	UE_API void SetUserData(TSharedRef<FFusionPatchUserData> UserData);

#if WITH_EDITORONLY_DATA

	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	// save off the samples dir used when importing samples
	UPROPERTY()
	FString SamplesImportDir;

	
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#endif

#if WITH_EDITOR

	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

#endif

	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;

protected:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Fusion Patch")
	FFusionPatchData FusionPatchData;
private:
	// Notice here that we cache a pointer the Proxy's "Queue" so we can...
	// 1 - Supply it to all instances of Metasound nodes rendering this data. How?
	//     CreateNewProxyData instantiates a NEW unique ptr to an FFusionPatchDataProxy
	//     every time it is called. All of those unique proxy instances refer to the same
	//     queue... this one that we have cached.
	// 2 - Modify that data in response to changes to this class's UPROPERTIES
	//     so that we can hear data changes reflected in the rendered audio.
	TSharedPtr<FFusionPatchDataProxy::QueueType, ESPMode::ThreadSafe> RenderableFusionPatchData;

	UE_API void UpdateRenderableForNonTrivialChange();

	enum class ELoadingState
	{
		None,
		Loading
	};

	ELoadingState LoadingState = ELoadingState::None;
};

#undef UE_API
