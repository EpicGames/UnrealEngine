// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/Instruments/VirtualInstrument.h"

#include "Containers/List.h"
#include "HarmonixDsp/Effects/BiquadFilter.h"

#include "HarmonixDsp/Modulators/Settings/AdsrSettings.h"
#include "HarmonixDsp/Modulators/Lfo.h"

#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"

#include "HarmonixMidi/MidiConstants.h"

#include "HAL/CriticalSection.h"

#include "Templates/SharedPointer.h"

#define UE_API HARMONIXDSP_API

DECLARE_LOG_CATEGORY_EXTERN(LogFusionSampler, Log, All);

class FFusionVoice;
struct FFusionPatchData;
class FFusionVoicePool;
using FSharedFusionVoicePoolPtr = TSharedPtr<FFusionVoicePool, ESPMode::ThreadSafe>;

// Make Fusion Sampler derive directly from virtual instrument
class FFusionSampler : public FVirtualInstrument
{
public:
	UE_API FFusionSampler(float InSampleRate, bool InUsePitchShifters = false);
	UE_API virtual ~FFusionSampler();

	UE_API void ResetPatchRelatedState();

	// configure this patch
	UE_API virtual void Prepare(float InSampleRateHz, EAudioBufferChannelLayout InChannelLayout, uint32 InMaxSamples, bool bInAllocateBuffer = true) override;
	UE_API void SetGainTable(const FGainTable* GainTable);
	const FGainTable* GetGainTable() { return GainTable; }

	//*************************************************************************
	// These are overrides from VirtualInstrument that deal with MIDI message
	// handling...
	UE_API virtual void  NoteOn(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InVelocity, int8 InMidiChannel = 0, int32 InEventTick = 0, int32 InCurrentTick = 0, float InOffsetMs = 0.0f) override;
	UE_API virtual void  NoteOnWithFrameOffset(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InVelocity, int8 InMidiChannel = 0, int32 InNumFrames = 0) override;
	UE_API virtual bool  NoteIsOn(int8 InMidiNoteNumber, int8 InMidiChannel = 0) override;
	UE_API virtual void  NoteOff(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InMidiChannel = 0) override;
	UE_API virtual void  NoteOffWithFrameOffset(FMidiVoiceId InVoiceId, int8 InMidiNoteNumber, int8 InMidiChannel = 0, int32 InNumFrames = 0) override;
	UE_API virtual void  SetExtraPitchBend(float Semitones, int8 InMidiChannel = 0) override;
	UE_API virtual float GetPitchBend(int8 InMidiChannel = 0) const override;
	UE_API virtual void  SetPitchBend(float Value, int8 InMidiChannel = 0) override;
	UE_API virtual void  GetController(Harmonix::Midi::Constants::EControllerID InController, int8& OutMsb, int8& OutLsb, int8 InMidiChannel = 0) const override;

	// Mix Volume (dB) and associated linear gain (0,1]
	UE_API virtual void  SetMidiChannelVolume(float InVolume, float InSeconds = 0.0f, int8 InMidiChannel = 0) override;
	UE_API virtual float GetMidiChannelVolume(int8 InMidiChannel = 0) const override;
	UE_API virtual void  SetMidiChannelGain(float InGain, float InSeconds = 0.0f, int8 InMidiChannel = 0) override;
	UE_API virtual float GetMidiChannelGain(int8 InMidiChannel = 0) const override;
	UE_API virtual void  SetMidiChannelMute(bool InMute, int8 InMidiChannel = 0)  override;
	virtual bool  GetMidiChannelMute(int8 InMidiChannel = 0) const override { return MidiChannelMuted; }

	UE_API virtual int32 GetMaxNumVoices() const override;
	virtual int32 GetNumVoicesInUse() const override { return (int32)(ActiveVoices.Num()); }
	UE_API virtual int32 GetNumVoicesInUse(FFusionVoice** vArray) const;
	UE_API virtual void  KillAllVoices() override;
	UE_API virtual void  AllNotesOff() override;
	UE_API virtual void  AllNotesOffWithFrameOffset(int32 InNumFrames = 0) override;

	// Sets modulation and sample playback speed when maintainPitch is false. Otherwise it's 1.0.
	UE_API virtual void  SetSpeed(float Speed, bool MaintainPitch = false) override;                
	
	// Gets the current speed
	virtual float GetSpeed(bool* MaintainPitchOut = nullptr) override                           
	{
		if (MaintainPitchOut != nullptr)
		{
			*MaintainPitchOut = MaintainPitchWhenSpeedChanges;  
		}
		return Speed;
	}

	UE_API virtual void SetTempo(float BPM) override;
	UE_API virtual void SetQuarterNote(float QuarterNote) override;
	float GetQuarterNote() const { return CurrentQuarterNote; }

	float GetSubtsreamGain(int32 Index)
	{
		if (Index >= kMaxSubstreams)
		{
			return 1.0f;
		}
		
		return SubstreamGain[Index];
	}

	UE_API virtual void SetSampleRate(float InSampleRateHz) override;

	float GetSubstreamGain(int32 Index) const
	{
		if (Index >= kMaxSubstreams)
		{
			return 1.0f;
		}
		return SubstreamGain[Index];
	}

#ifdef UE_BUILD_DEVELOPMENT
	UE_API TSet<int32> GetActiveKeyzones();
#endif

	virtual void SetRawTransposition(int32 SemiTones) override { Transposition = SemiTones; }
	virtual int32 GetRawTransposition() const override { return Transposition; }
	virtual void SetRawPitchMultiplier(float RawPitch) override { RawPitchMultiplier = RawPitch; }
	virtual float GetRawPitchMultiplier() const override { return RawPitchMultiplier; }

	void SetTicksPerQuarterNote(int32 InTicksPerQuarterNote) { TicksPerQuarterNote = InTicksPerQuarterNote; }
	int32 GetTicksPerQuarterNote() const { return TicksPerQuarterNote; }

protected:

	//*************************************************************************
	// These are overrides from VirtualInstrument
	UE_API virtual void  ResetInstrumentStateImpl() override;
	UE_API virtual void  ResetMidiStateImpl() override;
	UE_API virtual void  Set7BitControllerImpl(Harmonix::Midi::Constants::EControllerID InController, int8 Value, int8 InMidiChannel = 0) override;
	UE_API virtual void  Set14BitControllerImpl(Harmonix::Midi::Constants::EControllerID InController, int16 Value, int8 InMidiChannel = 0) override;
	
	// NOTE: This next function's InValue is NOT some standard midi value. By the time it is called it is assumed 
	// the units and range of InValue are correct for the given controller id. So, for example, if InController 
	// is MidiConstants::LFO0Frequency than InValue is treated as Hz!
	UE_API void SetController(Harmonix::Midi::Constants::EControllerID InController, float InValue);

	UE_API void SetVoicePool(FSharedFusionVoicePoolPtr InPool, bool NoCallbacks);
	FSharedFusionVoicePoolPtr GetVoicePool() const { return VoicePool; }
	UE_API void VoicePoolWillDestruct(const FFusionVoicePool* InPool);

	UE_API virtual void Process(uint32 InSliceIndex, uint32 InSubSliceIndex, TAudioBuffer<float>& OutBuffer) override;
	bool CanProcessFromWorkerThread() const override { return true; }

	UE_API void SetPatch(FFusionPatchDataProxy::NodePtr PatchData);

	UE_API FString GetPatchPath() const;

	// apply all settings from the patch.
	// call whenever the patch is set
	UE_API void ApplyPatchSettings();

	UE_API void  SetFineTuneCents(float cents);
	UE_API float GetFineTuneCents() const;

private:
	friend class FFusionVoice;
	friend class FFusionVoicePool;
	friend class FFusionVstEffect;



	UE_API void            SetPortamentoMode(EPortamentoMode InMode);
	UE_API EPortamentoMode GetPortamentoMode() const;
	UE_API void            SetIsPortamentoEnabled(bool InEnabled);
	bool            GetIsPortamentoEnabled() const { return IsPortamentoEnabled; }
	UE_API void            SetPortamentoTime(float InTimeSec);
	// returns time in seconds
	UE_API float           GetPortamentoTime() const;
	UE_API float           GetCurrentPortamentoPitch() const;

	UE_API void  SetStartPointMs(float InStartPointOffsetMs);
	UE_API float GetStartPointMs() const;

	// This is a soft limit, so old voices will be released instead of killed
	UE_API void SetMaxNumVoices(int32); 
	UE_API bool RelinquishVoice(FFusionVoice* InVoice);

	virtual bool ProcessCallWillProduceSilence() const override { return ActiveVoices.Num() == 0; }

	UE_API void  SetPan(const FPannerDetails& pan);
	UE_API const FPannerDetails& GetPan() const;

	//float RandomStartPointMs = 0.0f;
	//float RandomPitchCents = 0.0f;

	UE_API float GetMinPitchBendCents() const;
	UE_API void  SetMinPitchBendCents(float Value);
	UE_API float GetMaxPitchBendCents() const;
	UE_API void  SetMaxPitchBendCents(float Value);

	// returns the multiplicative factor used to change the rate of sample frame increment
	// to hit the current pitch bend settings
	UE_API float GetPitchBendFactor() const;

	float GetMidiChannelMuteGain() const { return MidiChannelMuteGainRamper.GetCurrent(); }

	UE_API float GetTrimVolume() const;
	UE_API void  SetTrimVolume(float Value);
	UE_API float GetTrimGain() const;

	float GetRampedExpression() const { return ExpressionGainRamper.GetCurrent(); }
	float GetRampedPitchBend() const { return PitchBendRamper.GetCurrent(); }

	UE_API void  SetMidiExpressionGain(float InGain, float InSeconds = 0.0f, int8 InMidiChannel = 0);

	// Updates time-based state.
	UE_API void PrepareToProcess(uint32 numSamples);

	UE_API void UpdatePitchBendFactor();
	UE_API void UpdateVoiceLfos();

	// Pending note actions...
	UE_API void ProcessNoteActions(int32 InNumFrames);
	UE_API void ResetNoteActions(bool ClearNotes);
	UE_API void ResetNoteStatus();

	// internal note on/off functions
	// only called on audio thread
	UE_API int32 GatherMatchingKeyzones(uint8 TransposedNote, uint8 InVelocity, bool IsNoteOn, const TArray<FKeyzoneSettings>& InKeyzones, uint16* InMatchingZones);
	UE_API int32 StartNote(FMidiVoiceId InVoiceId, uint8 InMidiNoteNumber, uint8 InVelocity, bool IsNoteOn, int32 InEventTick, int32 InTriggerTick, float InOffsetMs);
	// Note: StopNote returns a TArray of midi notes that where stopped so the called can check to see if there are any key-off keyzoned that need to be triggered
	UE_API TArray<uint8> StopNote(FMidiVoiceId InVoiceId, uint8 LastVelocity);

	UE_API bool TryKeyOnZone(FMidiVoiceId InVoiceId, uint8 InTriggeredNote, uint8 InTransposedNote, uint8 InVelocity, int32 InEventTick, int32 InTriggerTick, float InOffsetMs, const FKeyzoneSettings* keyzone);

private:

	float RampCallRateHz = 375.0f;

	float Speed = 1.0f;
	bool  MaintainPitchWhenSpeedChanges = false;
	float CurrentQuarterNote = 0.0f;

	//---------------------------------------------
	// Gain Settings

	// channel's mix setting... persists independently of the patch
	float MidiChannelVolume = 0.0f;
	TLinearRamper<float> MidiChannelGainRamper;

	// channel's mute setting... persists independently of the patch
	bool MidiChannelMuted = false;

	// ramper to smooth muting
	TLinearRamper<float> MidiChannelMuteGainRamper;

	// the patch's trim setting... read-only outside of patch editor.
	float TrimVolume = 0.0f;
	float TrimGain = 1.0f;

	//the CC 11 expression value [0,1]
	float ExpressionGain = 1.0f;
	
	// expression [0,1] mapped to linear gain (a la the MIDI recommendations)
	TLinearRamper<float> ExpressionGainRamper;

	FPannerDetails PanSettings;

	//----------------------------------------------
	// PORTAMENTO
	bool IsPortamentoEnabled = false;

	EPortamentoMode PortamentoMode = EPortamentoMode::None;
	float PortamentoTimeMs = 300.0f;

	// True if the portmanto pitch should be considered (false before any note ons for example)
	bool IsPortamentoActive = false;
	
	// midi note number (fractional) that represents the actual pitch of sounding notes
	TLinearRamper<float> PortamentoPitchRamper;

	//---------------------------------------------
	// PITCH BEND

	// on range [-1, 1]
	TLinearRamper<float> PitchBendRamper;
	
	// extra pitch bend in semitones
	float ExtraPitchBend = 0.0f;
	float PitchBendFactor = 1.0f;

	float FineTuneCents = 0.0f;

	float StartPointMs = 0.0f;

	FBiquadFilterSettings FilterSettings;


	//---------------------------------------------
	// Modulators
	FAdsrSettings AdsrVolumeSettings;
	FAdsrSettings AdsrAssignableSettings;
	static constexpr int32 kNumLfos = 2;
	FLfoSettings LfoSettings[kNumLfos];
	Harmonix::Dsp::Modulators::FLfo Lfos[kNumLfos];
	static constexpr int32 kNumModulators = 2;
	FModulatorSettings RandomizerSettings[kNumModulators];
	FModulatorSettings VelocityModulatorSettings[kNumModulators];

	// soft limit, may be over max if some voices are in release stage
	uint32 MaxNumVoices = 100;

	TDoubleLinkedList<FFusionVoice*> ActiveVoices;

	float MinPitchBendCents = -200.0f;
	float MaxPitchBendCents = 200.0f;

	FSharedFusionVoicePoolPtr VoicePool;
	const bool bUsePitchShifters;

	// if we get note-ons or note-offs on the main
	// thread, or if we get multiple requests for
	// the same note number, then this data helps
	// us track and filter these requests
	FCriticalSection sNoteActionCritSec;
	FCriticalSection sNoteStatusCritSec;
	static const int8 kNoteIgnore = -1;
	static const int8 kNoteOff = 0;
	static const int32 kMaxLayersPerNote = 128;

	struct FPendingNoteAction
	{
		int8  MidiNote = 0;
		int8  Velocity = 0;
		int32 EventTick = 0;
		int32 TriggerTick = 0;
		float OffsetMs = 0.0f;
		int32 FrameOffset = 0;
		FMidiVoiceId VoiceId;
	};

	struct FMIDINoteStatus
	{
		// is the key pressed down?
		bool KeyedOn = false;

		// is there any sound coming out of this note? (release could mean key off but voices active)
		int32 NumActiveVoices = 0;
	};

	TArray<FPendingNoteAction> PendingNoteActions;
	FMIDINoteStatus NoteStatus[Harmonix::Midi::Constants::GMaxNumNotes];

	FFusionPatchDataProxy::NodePtr FusionPatchData = nullptr;

	int8 LastStartLayerSelect[Harmonix::Midi::Constants::GMaxNumNotes];
	int8 LastStopLayerSelect[Harmonix::Midi::Constants::GMaxNumNotes];
	int8 LastVelocity[Harmonix::Midi::Constants::GMaxNumNotes];

	// "scratch pad" used during process to get each voice's output audio
	static const int32 kScratchBufferFrames = 2048;
	static const int32 kScratchBufferChannels = FAudioBufferConfig::kMaxAudioBufferChannels;
	static const int32 kScratchBufferSamples = (kScratchBufferFrames * kScratchBufferChannels);
	static const int32 kScratchBufferBytes = (kScratchBufferSamples * sizeof(float));

	// +4 because we can't count on 'alignas' because we allocate an array of these classes. That type of allocation ignores the alignas attribute :-(
	float VoiceWorkBuffer[kScratchBufferSamples + 4]; 
	float* VoiceWorkBufferChannels[kScratchBufferChannels];

protected:

	// we'll keep track of the 'current tempo' as it is
	// needed by any 'beat sync' effects...
	float CurrentTempoBPM = 120.0f;

	float RawPitchMultiplier = 0.0f;
	int32 Transposition = 0;

	EKeyzoneSelectMode KeyzoneSelectMode = EKeyzoneSelectMode::Layers;

private:
	int16 TimeStretchEnvelopeOverride = -1;
	UE_API void UpdateVoicesForEnvelopeOrderChange();

	static const int32 kMaxSubstreams = 8;
	float SubstreamGain[kMaxSubstreams];
	UE_API void SetSubstreamMidiGain(int32 InIndex, uint8 InMidiGain);

	int32 TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt;

	const FGainTable* GainTable = nullptr;
};

#undef UE_API
