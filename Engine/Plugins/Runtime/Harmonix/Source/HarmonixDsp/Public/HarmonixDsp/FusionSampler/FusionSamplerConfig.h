// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Harmonix/HarmonixDeveloperSettings.h"
#include "HarmonixDsp/FusionSampler/FusionVoicePool.h"

#include "FusionSamplerConfig.generated.h"

#define UE_API HARMONIXDSP_API

UCLASS(MinimalAPI, Config = Engine, defaultconfig, meta = (DisplayName = "Fusion Sampler Settings"))
class UFusionSamplerConfig : public UHarmonixDeveloperSettings
{
	GENERATED_BODY()

public:

	UPROPERTY(config, EditDefaultsOnly, Category = "Voice Pools")
	FFusionVoiceConfig DefaultVoicePoolConfig;

	UPROPERTY(config, EditDefaultsOnly, Category = "Voice Pools")
	TMap<FName, FFusionVoiceConfig> VoicePoolConfigs;

	UE_API FFusionVoiceConfig GetVoiceConfigForPoolName(FName PoolName) const;
};

#undef UE_API
