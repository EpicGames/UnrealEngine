// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/Panner.h"
#include "HarmonixDsp/Ramper.h"

#include "HarmonixDsp/Modulators/Adsr.h"
#include "HarmonixDsp/Modulators/Lfo.h"
#include "HarmonixDsp/Modulators/ModulatorTarget.h"
#include "HarmonixDsp/Effects/BiquadFilter.h"
#include "HarmonixDsp/Effects/Settings/BiquadFilterSettings.h"

#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"

#include "HarmonixMidi/MidiVoiceId.h"

#include "Templates/SharedPointer.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

DECLARE_LOG_CATEGORY_EXTERN(LogFusionVoice, Log, All);

class IAudioDataRenderer;
class IStretcherAndPitchShifter;
class FFusionSampler;
class FFusionVoicePool;
struct FKeyzoneSettings;
struct FLfoSettings;

// Hacky base class to avoid 8 bytes of padding after the vtable
class FFusionVoiceFixLayout
{
public:
	virtual ~FFusionVoiceFixLayout() = default;
};

class HARMONIXDSP_API alignas(16) FFusionVoice : FFusionVoiceFixLayout
{
public:

	FFusionVoice();
	virtual ~FFusionVoice() = default;

	void Init(FFusionVoicePool* InOwner, uint32 InDebugID);

	static const double kMaxPitchOffsetCents;

	static const uint32 kInvalidId = 0xffffffff;

	bool IsWaitingForAttack() const;

	virtual bool IsInUse() const;

	using FRelinquishHandler = TFunction<bool(FFusionVoice*)>;

	bool AssignIDs(FFusionSampler* InSampler, const FKeyzoneSettings* KeyZone, FMidiVoiceId NoteID, FRelinquishHandler Handler, TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> Shifter = nullptr);

	virtual TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> GetPitchShifter() const { return PitchShifter;  }

	virtual Harmonix::Dsp::Modulators::EAdsrStage GetAdsrStage() const { return AdsrVolume.GetStage(); }

	uint32 GetAge() const { return AdsrVolume.GetAge(); }
	FMidiVoiceId GetVoiceID() const { return VoiceID; }
	uint8 GetTriggeredMidiNote() const { return TriggeredMidiNote; }

	double GetPlaybackPos() const { return SamplePos; }
	double GetEndOfSampleData() const { return EndOfSampleData; }
	void SetRelinquishHandler(FRelinquishHandler handler) { RelinquishHandler = handler; }

	/**
	 * offset the pitch of this voice by a (fractional) number of cents
	 * call this only before calling attack.
	 * @param numCents number of cents to move pitch
	 */
	virtual void SetPitchOffset(double numCents);

#if FUSION_VOICE_DEBUG_DUMP_ENABLED
	virtual void AttackWithTargetNote(uint8 InMidiNoteNumber, float InGain, int32 InEventTick, int32 InTriggerTick, double InStartPointMs = 0.0f, const char* InFilePath = nullptr);
#else
	virtual void AttackWithTargetNote(uint8 InMidiNoteNumber, float InGain, int32 InEventTick, int32 InTriggerTick, double InStartPointMs = 0.0f);
#endif

	void PlaySampleSlice(float start, float end, float seconds);

	// this function is called on the audio thread to have the voice fill up the given buffer
	virtual uint32 Process(uint32 sliceIndex, uint32 subsliceIndex, float** output, uint32 numChannels, uint32 maxNumSamples, float speed = 1.0f, float tempo = 120.0f, bool MaintainPitchWhenSpeedChanges = false);

	// puts the voice into release stage, which lets it ramp down.
	// the processor will free up this voice after ramp down
	// is complete.
	virtual void Release();

	// ramps down using a microfade to free up the voice as soon as possible
	// without an audible pop
	virtual void FastRelease();

	// kill (and free up) the voice immediately with no ramp down.
	// this should only be called if the voice is known
	// to be silent anyway, or you ran out of sample data,
	// or you have no other choice.
	virtual void Kill();

	// if keyzone is NULL (or not passed in), then all voices with the channel and note id will match
	virtual bool MatchesIDs(const FFusionSampler* InSampler, FMidiVoiceId InVoiceID, const FKeyzoneSettings* keyzone = nullptr);
	int32 GetTargetMidiNote() { return int32(TargetMidiNote); }

	bool UsesSampler(const FFusionSampler* InSampler) const 
	{ 
		return InSampler == MySampler; 
	}
	FFusionSampler* GetSampler() { return MySampler; }
	virtual bool UsesKeyzone(const FKeyzoneSettings* InKeyzone) const
	{
		return InKeyzone == KeyZone;
	}
	virtual const FKeyzoneSettings* GetKeyzone() const { return KeyZone; }

	float GetCombinedAudioLevel() { return MaxAudioLevel; }

	virtual void SetupLfo(uint8 Index, const FLfoSettings& InSettings);

	uint8 Priority()
	{
		return KeyZone ? KeyZone->Priority : 255;
	}

	// used by the voice aliasing system if this real voice
	// is passed around amoung virtual voices...
	void SetSampler(FFusionSampler* InSampler);

	void SetPitchShiftForMidiNote(uint8 InNote);

	virtual bool IsAlias() const { return false; }
	virtual bool IsRendererForAlias() const { return bIsRendererForAlias; }
	void SetIsRendererForAlias(bool InIsRenderer) { bIsRendererForAlias = InIsRenderer; }
	virtual bool HasBeenRelinquished() const { return !RelinquishHandler; }

protected:

	FMidiVoiceId VoiceID;
	float TargetMidiNote = 0.0f;
	FFusionSampler* MySampler = nullptr;
	bool bIsRendererForAlias = false;
	uint8 TriggeredMidiNote = 0;

private:

	void SetSampleRate(double InSampleRate);

	void  PrepareWithPitchOffsetAndGain(double InPitchOffsetCents, float InGain);
	void  ApplyModsToFilter(bool forceSet);
	void  AggregatePansSetTargetAndRamp(float InTotalGain, bool InSnap = false);
	// the number of octaves to shuft the filter frequency
	float ComputeOctaveShift();

	// start the note (call after preparing the voice)
	void Attack();

	static void UpdateFilterSettings(void* thisPointer);
	static void RestoreFilterGain(void* thisPointer);

	void BuildGainMatrix(bool InSnap);

	bool bWaitingForAttack = false;
	bool bHasRenderedAnySamples = false;

	// just for debugging
	uint32 DebugID;

	float VelocityGain = 0.0f;
	float MaxAudioLevel = 0.0f;

	const FKeyzoneSettings* KeyZone = nullptr;

	TSharedPtr<IAudioDataRenderer, ESPMode::ThreadSafe> ActiveRenderer = nullptr;

	TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> PitchShifter = nullptr;

	Harmonix::Dsp::Modulators::FAdsr AdsrVolume;
	Harmonix::Dsp::Modulators::FAdsr AdsrAssignable;
	static const int32 kNumLfos = 2;
	Harmonix::Dsp::Modulators::FLfo Lfo[kNumLfos];

	FPanner Panner;

	Harmonix::Dsp::Effects::FBiquadFilter Filters[FAudioBufferConfig::kMaxAudioBufferChannels];
	TLinearRamper<Harmonix::Dsp::Effects::FBiquadFilterCoefs> FilterCoefsRamper;
	float CachedFilterFrequency = 0.0f;
	FBiquadFilterSettings ModulatedFilterSettings;

	// this is the number we use to modulate the filter frequency.
	// we would call this "frequency", but we want
	// the units to be "octaves" so it can be linear.
	// the actual frequency is  20*(2^octave)
	float OctaveShift = 0.0f;
	Harmonix::Dsp::Modulators::FModulatorTarget FilterOctaveTarget;

	TLinearRamper<float> FilterGainRamper;

	TAudioBuffer<float> OutputBuffer;

	double FileToOutputSampleRatio = 0.0;

	double SamplesPerSecond = 0.0;
	double SecondsPerSample = 0.0;


	double SamplePos = 0.0;
	double PitchShift = 0.0;
	double ResampleRate = 0.0;

	double PitchOffsetCents = 0.0;
	double PitchModulation = 0.0;
	
	double EndOfSampleData = 0.0;
	
	FFusionVoicePool* VoicePool = nullptr;

	// needed for tempo sync'd keyzones...
	float StartQuarterNote = 0.0f;
	float CurrentVso = 0.0f;
	double LastVsoPos = 0.0f;
	double StartPos = 0.0f;

	FRelinquishHandler RelinquishHandler;

#if CPUPROFILERTRACE_ENABLED
	bool bProfiling{false};
	FString ProfileString;
#endif
};