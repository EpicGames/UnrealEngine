// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactory.h"
#include "HarmonixMidi/MidiVoiceId.h"

#include "FusionVoicePool.generated.h"

#define UE_API HARMONIXDSP_API

struct FKeyzoneSettings;
class FFusionSampler;
class FFusionVoice;
class IStretcherAndPitchShifter;
class FFusionVoicePool;
using FSharedFusionVoicePoolPtr = TSharedPtr<FFusionVoicePool, ESPMode::ThreadSafe>;

DECLARE_LOG_CATEGORY_EXTERN(LogFusionVoicePool, Log, All);

USTRUCT()
struct FFusionVoiceConfig
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Voice Pool")
	uint32 NumTotalVoices = 32;

	UPROPERTY(EditAnywhere, Category = "Voice Pool")
	uint32 SoftVoiceLimit = 24;

	UPROPERTY(EditAnywhere, Category = "Pitch Shifter")
	float  FormantDbCorrectionPerHalfStepUp = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Pitch Shifter")
	float  FormantDbCorrectionPerHalfStepDown = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Pitch Shifter")
	float  FormantDbCorrectionMaxUp = 12.0f;

	UPROPERTY(EditAnywhere, Category = "Pitch Shifter")
	float  FormantDbCorrectionMaxDown = -12.0f;

	FFusionVoiceConfig() {}

	FFusionVoiceConfig(
		uint32 InNumTotalVoices,
		uint32 InSoftVoiceLimit)
		: NumTotalVoices(InNumTotalVoices)
		, SoftVoiceLimit(InSoftVoiceLimit)
	{}
};

class FFusionVoicePool
{
public:

	static UE_API FSharedFusionVoicePoolPtr GetDefault(float InSampleRate);
	static UE_API FSharedFusionVoicePoolPtr GetNamedPool(FName InPoolName, float InSampleRate);
	static UE_API FSharedFusionVoicePoolPtr Create(const FFusionVoiceConfig& InConfig, float InSampleRate);

	FFusionVoicePool(float InSampleRate) 
		: Voices(nullptr)
		, NumAllocatedVoices(0)
		, NumVoicesSetting(kDefaultPoolSize)
		, SoftVoiceLimit(kDefaultPoolSize)
		, PeakVoiceUsage(0)
		, SampleRate(InSampleRate)
	{};

	UE_API virtual ~FFusionVoicePool();

	UE_API void HardAllocatateVoices();
	UE_API void ReleaseHardAllocation();

	UE_API void SetIsMultithreading(bool InIsMultithreaded);

	// let the FusionVoicePool know that you will need voices
	UE_API void AddClient(FFusionSampler* InSampler);

	// let the FusionVoicePool know that you no longer need voices
	UE_API void RemoveClient(FFusionSampler* InSampler);

	// Global Voice Pool
	static const uint32 kMinPoolSize = 1;
	static const uint32 kMaxPoolSize = 256;
	static const uint32 kDefaultPoolSize = 16;


	UE_API uint32 GetNumVoicesInUse();

	/**
	 * The number of voices allocated.
	 * May be different than the hard limit if voices have not been allocated yet
	 * (or if there are no clients for this voice pool)
	 * @returns the number of voices currently allocated.
	 */
	uint32 GetNumVoicesAvailable() { return NumAllocatedVoices; }

	UE_API bool HasVoice(FFusionSampler* InOwner, FMidiVoiceId InVoiceId);

	/**
	 * The number of voices to allocate for this pool.
	 * @param limit the maximum number of voices that this pool can handle. (the number of voices to allocate)
	 */
	UE_API void SetHardVoiceLimit(uint32 InLimit);

	/**
	 * @returns the maximum number of voices this pool can handle.
	 */
	uint32 GetHardVoiceLimit() const { return NumVoicesSetting; }

	/**
	 * The number of voices to allow before automatically releasing excess voices.
	 * @param limit the maximum number of voices to allow before fast-releasing
	 */
	UE_API void SetSoftVoiceLimit(uint32 InLimit);

	/**
	 * @returns the maximum number of voices this pool will allow before automatically releasing voices.
	 */
	uint32 GetSoftVoiceLimit() const { return SoftVoiceLimit; }

	UE_API void SetFormantVolumeCorrection(float dBperHalfStepUp, float dBperHalfStepDown, float dBMaxUp, float  dBMaxDown);

	uint32 GetPeakVoiceUsage() 
	{ 
		GetNumVoicesInUse(); 
		return PeakVoiceUsage; 
	}
	
	void ResetPeakVoiceUsage() 
	{ 
		PeakVoiceUsage = 0; 
		GetNumVoicesInUse(); 
	}

	// THIS PROBABLY SHOULD BE PRIVATE. IT IS USED BY NOTE ON.
	// NOT AN IDEAL ARGUMENT SIGNATURE FOR PUBLIC CONSUMPTION.
	// pass in a channel and note id to assign to the voice.
	// also pass in the keyzone that the voice should use.
	// any active voices with a matching set of IDs will be put into release stage.
	// a voice might get killed (stopped instantly) if there are not enough free voices.
	// returns null if there is no patch assigned to the channel.
	UE_API FFusionVoice* GetFreeVoice(
		FFusionSampler* InSampler, 
		FMidiVoiceId InVoiceID, 
		const FKeyzoneSettings* InKeyzone, 
		TFunction<bool(FFusionVoice*)> Handler,
		IStretcherAndPitchShifterFactory* PitchShifterFactory,
		bool AllowAlias = true, 
		bool IsRendererForAlias = false);

	UE_API void ReleaseShifter(TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> InShifter);
	
	/**
	 * Fast releases voices that have exceeded the soft limit for the pool (or the channel).
	 * If no channel is specified, then voices assigned to any channel are candidates for
	 * being fast-released. If a specific channel is specified, then only voices being used by that
	 * channel may be fast released.
	 * @param channel (optional) restrict the voice candidates for release to the ones uses by a specific channel.
	 * @returns the number of voices currently being used by the pool (or by the specified channel)
	 */
	UE_API uint32 FastReleaseExcessVoices(FFusionSampler* InSampler = nullptr);

	// instantly stops all voices with no release stage
	UE_API void KillVoices();
	//void KillVoices(const MultiInstrument*);
	UE_API void KillVoices(const FFusionSampler* InSampler, bool NoCallbacks);
	UE_API void KillVoices(const FKeyzoneSettings* InKeyzoneSettings);

	UE_API FFusionVoice* GetVoice(uint32 VoiceIdx);

	float GetSampleRate() const { return SampleRate; }
	UE_API void SetSampleRate(float InSampleRate);

	UE_API void Lock();
	UE_API void Unlock();

private:

	// Key is a tuple of PoolName and SampleRate
	// Value is a weak ptr, but shared ptrs are returned so it gets automatically destroyed when nothing references it anymore
	using FPoolMapKey = TTuple<FName, int32>;
	using FPoolMap = TMap<FPoolMapKey, TWeakPtr<FFusionVoicePool, ESPMode::ThreadSafe>>;
	static UE_API FPoolMap GVoicePools;

	FFusionVoice* Voices;

	uint32     NumAllocatedVoices = 0;
	uint32     NumVoicesSetting = 0;

	uint32     SoftVoiceLimit = 0;

	uint32     PeakVoiceUsage = 0;

	UE_API void AllocVoices();
	UE_API void CreateVoices(uint16 InMaxPolyphony);
	UE_API void CreateShifters();
	UE_API void FreeVoices();

	TArray<FFusionSampler*> ClientSamplers;

	FCriticalSection PoolLock;

	float SampleRate = 0.0f;

	bool DynamicAllocAndFree = true;

	bool IsMultithreading = false;
};

#undef UE_API
