// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/PannerDetails.h"
#include "HarmonixDsp/Effects/Settings/BiquadFilterSettings.h"
#include "HarmonixDsp/Effects/Settings/BitCrusherSettings.h"
#include "HarmonixDsp/Effects/Settings/DelaySettings.h"
#include "HarmonixDsp/Effects/Settings/DistortionSettings.h"
#include "HarmonixDsp/Effects/Settings/VocoderSettings.h"
#include "HarmonixDsp/Modulators/Settings/AdsrSettings.h"
#include "HarmonixDsp/Modulators/Settings/LfoSettings.h"
#include "HarmonixDsp/Modulators/Settings/ModulatorSettings.h"
#include "HarmonixDsp/Modulators/ModulatorTarget.h"
#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"
#include "HarmonixDsp/FusionSampler/Settings/PortamentoSettings.h"


#include "FusionPatchSettings.generated.h"

#define UE_API HARMONIXDSP_API

UENUM()
enum class EKeyzoneSelectMode : uint8
{
	Layers UMETA(Json="layers"),
	Random UMETA(Json="random"),
	RandomWithRepetition UMETA(Json="random_with_repititon"),
	Cycle UMETA(Json="cycle"),
	Num UMETA(Hidden),
	Invalid UMETA(Hidden)
};

USTRUCT()
struct FFusionPatchSettings
{
	GENERATED_BODY()

public:

	UE_API FFusionPatchSettings();
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "-96", ClampMax = "12", UIMin = "-96", UIMax = "12"))
	float VolumeDb = 0.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FPannerDetails PannerDetails;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "-2400", ClampMax = "-200", UIMin = "-2400", UIMax = "-200"))
	float DownPitchBendCents = -200.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "200", ClampMax = "2400", UIMin = "200", UIMax = "2400"))
	float UpPitchBendCents = 200.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "-100", ClampMax = "100", UIMin = "-100", UIMax = "100"))
	float FineTuneCents = 0.0f;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "Start Point Offset (Ms)", Meta = (ClampMin = "0", ClampMax = "1000", UIMin = "0", UIMax = "1000"))
	float StartPointOffsetMs = 0.0f;
	
	UPROPERTY()
	int32 MaxVoices = 32;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EKeyzoneSelectMode KeyzoneSelectMode = EKeyzoneSelectMode::Layers;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FPortamentoSettings Portamento;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FBiquadFilterSettings Filter;

	static constexpr int32 kNumAdsrs = 2;
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FAdsrSettings Adsr[kNumAdsrs];

	static constexpr int32 kNumLfos = 2;
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FLfoSettings Lfo[kNumLfos];

	static constexpr int32 kNumRandomizers = 2;
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FModulatorSettings Randomizer[kNumRandomizers];

	static constexpr int32 kNumModulators = 2;
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FModulatorSettings VelocityModulator[kNumModulators];
	
	UPROPERTY()
	FAdsrSettingsArray Adsrs_DEPRECATED;

	UPROPERTY()
	FLfoSettingsArray Lfos_DEPRECATED;

	UPROPERTY()
	FModulatorSettingsArray Randomizers_DEPRECATED;

	UPROPERTY()
	FModulatorSettingsArray VelocityModulators_DEPRECATED;

};

#undef UE_API
