// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/Modulators/Settings/AdsrSettings.h"
#include "HarmonixDsp/PannerDetails.h"
#include "HarmonixDsp/StretcherAndPitchShifter.h"
#include "HarmonixDsp/Streaming/TrackChannelInfo.h"

#include "Logging/LogMacros.h"

#include "IAudioProxyInitializer.h"

#include "KeyzoneSettings.generated.h"

#define UE_API HARMONIXDSP_API

class FSingletonFusionVoicePool;
class USoundWave;

DECLARE_LOG_CATEGORY_EXTERN(LogKeyzoneSettings, Log, All);

struct FKeyzoneArgs
{
	FString SamplePath;
	int8 RootNote = 60;
	int8 MinNote = 0;
	int8 MaxNote = 127;
	int8 MinVelocity = 0;
	int8 MaxVelocity = 127;
	bool bUnpitched = false;
	bool bVelocityToVolume = true;
	bool bMaintainTime = false;
	FName PitchShifter;
	bool bSyncTempo = false;
	float OriginalTempo = 120.0f;
};

USTRUCT()
struct FKeyzoneSettings
{
	friend class FFusionPatchJsonImporter;

	GENERATED_BODY()

	UE_API FKeyzoneSettings();

	FKeyzoneSettings(const FKeyzoneArgs& InArgs)
	{
		ApplyArgs(InArgs);
	}

	UE_API ~FKeyzoneSettings();

	void ApplyArgs(const FKeyzoneArgs& InArgs)
	{
		RootNote = InArgs.RootNote;
		MinNote = InArgs.MinNote;
		MaxNote = InArgs.MaxNote;
		MinVelocity = InArgs.MinVelocity;
		MaxVelocity = InArgs.MaxVelocity;
		bUnpitched = InArgs.bUnpitched;
		bVelocityToGain = InArgs.bVelocityToVolume;

		TimeStretchConfig.bMaintainTime = InArgs.bMaintainTime;
		TimeStretchConfig.PitchShifter = InArgs.PitchShifter;
		TimeStretchConfig.bSyncTempo = InArgs.bSyncTempo;
		TimeStretchConfig.OriginalTempo = InArgs.OriginalTempo;
	}

public:
	// Not null when this keyzone is not being proxied
	UPROPERTY(EditDefaultsOnly, Category = "Audio Sample")
	TObjectPtr<USoundWave> SoundWave;

	TSharedPtr<const FSoundWaveData> SoundWaveData;


	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin=0, UIMax=127, ClampMin=0, ClampMax=127))
	int8 RootNote = 60;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = 0, UIMax = 127, ClampMin = 0, ClampMax = 127))
	int8 MinNote = 0;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = 0, UIMax = 127, ClampMin = 0, ClampMax = 127))
	int8 MaxNote = 127;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = 0, UIMax = 127, ClampMin = 0, ClampMax = 127))
	int8 MinVelocity = 0;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = 0, UIMax = 127, ClampMin = 0, ClampMax = 127))
	int8 MaxVelocity = 127;
	
	// on disk this is the index into the sample file name table
	UPROPERTY(EditDefaultsOnly, Category="Settings")
	uint8 SampleIndex = 0;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings")
	uint8 Priority = 50;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings")
	FPannerDetails Pan;	

	UPROPERTY(EditDefaultsOnly, Category="Settings")
	bool bOverrideAdsr = false;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings", meta=(EditCondition="bOverrideAdsr", EditConditionHides=true))
	FAdsrSettings Adsr;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = -100, UIMax = 100, ClampMin = -100, ClampMax = 100))
	float FineTuneCents = 0.0f;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta=(PostEditType="Trivial", UIMin="0.0", UIMax="1.0", ClampMin="0.0", ClampMax="1.0"))
	float Gain = 1.0f;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings")
	bool bUnpitched = false;

	UPROPERTY(EditDefaultsOnly, Category="Settings")
	bool bVelocityToGain = true;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings")
	FTimeStretchConfig TimeStretchConfig;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings")
	bool bIsNoteOffZone = false;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = 0, UIMax = 100, ClampMin = 0, ClampMax = 100))
	float RandomWeight = 50.0f;

	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = -1, ClampMin = -1))
	int32 SampleStartOffset = -1;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = -1, ClampMin = -1))
	int32 SampleEndOffset = -1;

	UPROPERTY(EditDefaultsOnly, Category="Settings")
	TArray<FTrackChannelInfo> TrackMap;

	UPROPERTY(EditDefaultsOnly, Category="Settings")
	bool UseSingletonVoicePool = false;

	TSharedPtr<FSingletonFusionVoicePool> SingletonFusionVoicePool = nullptr;

	UE_API bool ContainsNoteAndVelocity(uint8 InNote, uint8 InVelocity) const;
	UE_API void SetVolumeDb(float Db);
	UE_API float GetVolumeDb() const;
	bool IsSingleton() const {  return SingletonFusionVoicePool != nullptr; }
	bool IsNoteOnZone() const { return !bIsNoteOffZone; }

	UE_API void InitProxyData(const Audio::FProxyDataInitParams& InitParams);

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta=(EditCondition="false"))
	FString SamplePath;
#endif

};

#undef UE_API
