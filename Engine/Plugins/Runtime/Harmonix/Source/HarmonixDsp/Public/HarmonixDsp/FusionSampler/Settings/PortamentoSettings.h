// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PortamentoSettings.generated.h"

UENUM()
enum class EPortamentoMode : uint8
{
	Legato			UMETA(Json="legato"),
	Persistent		UMETA(Json="persistent"),
	Num				UMETA(Hidden),
	None			UMETA(Hidden)
};

USTRUCT()
struct FPortamentoSettings
{
	GENERATED_BODY()

public:

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool IsEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EPortamentoMode Mode = EPortamentoMode::Legato;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (ClampMin = "0", ClampMax = "10", UIMin = "0", UIMax = "10"))
	float Seconds = 1.0f;
};