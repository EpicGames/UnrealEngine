// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/Modulators/Adsr.h"
#include "HarmonixDsp/FusionSampler/AliasFusionVoice.h"
#include "HarmonixMidi/MidiMsg.h"

#include <limits>

#include "HAL/CriticalSection.h"

struct FKeyzoneSettings;
struct FLfoSettings;
class FAliasFusionVoice;
class FFusionVoice;
class FFusionVoicePool;
class FFusionSampler;
class IStretcherAndPitchShifter;
class IStretcherAndPitchShifterFactory;

//================================================================================================
// SingletonFusionVoicePool - This class allocates a REAL fusion voice when first asked to 
// allocate a voice. Then, it manages some number of "aliases" to that voice as other note-ons
// on any number of samplers referencing the same patch/keyzone come in. 
//================================================================================================
class FSingletonFusionVoicePool
{
public:
	static const int32 kMaxSingletonAliases;

	FSingletonFusionVoicePool(int32 InMaxInstances, FKeyzoneSettings& InKeyzoneRef);
	~FSingletonFusionVoicePool() = default;

	FFusionVoice* AllocateAlias(
		FFusionVoicePool* InPool,
		FFusionSampler* InSampler,
		FMidiVoiceId InVoiceID,
		TFunction<bool(FFusionVoice*)> Handler,
		IStretcherAndPitchShifterFactory* PitchShifterFactory);
	bool IsDriver(FAliasFusionVoice* InVoice) const { return InVoice == DriverVoice; }

	void SamplerDisconnecting(const FFusionSampler* Sampler);

	const FKeyzoneSettings& GetKeyzone() const { return KeyzoneRef; }

	TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> GetPitchShifter() const;

	Harmonix::Dsp::Modulators::EAdsrStage GetAdsrStage(const FAliasFusionVoice* Requestor) const;
	void SetPitchOffset(FAliasFusionVoice* Requestor, double NumCents);

#if FUSION_VOICE_DEBUG_DUMP_ENABLED
	void AttackWithTragetNote(
		FAliasFusionVoice* Requestor, 
		uint8 MidiNoteNumber, 
		float InGain,
		int32 InEventTick,
		int32 InTriggerTick,
		double StartPointMs = 0.0f,
		const char* FilePath = nullptr)
#else
	void AttackWithTargetNote(
		FAliasFusionVoice* Requestor, 
		uint8 MidiNoteNumber,
		float InGain,
		int32 InEventTick, 
		int32 InTriggerTick, 
		double StartPointMs = 0.0f);
#endif

	void Release(FAliasFusionVoice* Requestor);
	void FastRelease(FAliasFusionVoice* Requestor);
	void Kill(FAliasFusionVoice* Requestor);
	bool MatchesIDs(
		const FFusionSampler* InSampler, 
		FMidiVoiceId InVoiceID,
		const FKeyzoneSettings* InKeyzone = nullptr);

	bool UsesKeyzone(const FKeyzoneSettings* InKeyzone) const;

	void SetupLfo(const FAliasFusionVoice* Requestor, int32 Index, const FLfoSettings& InSettings);
	uint32 Process(
		FAliasFusionVoice* Requestor, 
		uint32 SliceIndex, 
		uint32 SubsliceIndex,
		float** Output, 
		uint32 InNumChannels,
		uint32 InMaxNumSamples, 
		float InSpeed = 1.0f,
		float InTempoBPM = 120.0f,
		bool  MaintainPitchWhenSpeedChanges = false);

	FCriticalSection& GetLock() { return PoolLock; };

private:

	mutable FCriticalSection PoolLock;

	struct FCachedVoiceTracker
	{
		FFusionVoice* Voice = nullptr;

		// the last rendered frame info
		uint32 SliceIndex = std::numeric_limits<uint32>::max(); 
		uint32 SubsliceIndex = std::numeric_limits<uint32>::max();

		uint32 NumFramesLastRendered = 0;

		// cached rendered audio
		TAudioBuffer<float> CachedBuffer;

		FCachedVoiceTracker& operator=(const FCachedVoiceTracker& Other)
		{
			Voice = Other.Voice;
			SliceIndex = Other.SliceIndex;
			SubsliceIndex = Other.SubsliceIndex;
			NumFramesLastRendered = Other.NumFramesLastRendered;
			return *this;
		}
	};

	FCachedVoiceTracker PrimaryTracker;
	FCachedVoiceTracker ReleasingTracker;
	FKeyzoneSettings& KeyzoneRef;
	TArray<FAliasFusionVoice> Aliases;

	FAliasFusionVoice* DriverVoice = nullptr;

	int32 NumActiveAliases = 0;

	bool Relinquish(FFusionVoice* InVoice);
};
