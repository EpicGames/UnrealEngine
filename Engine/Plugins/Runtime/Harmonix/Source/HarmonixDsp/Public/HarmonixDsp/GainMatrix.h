// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/AudioBufferConstants.h"
#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/GainTable.h"

#include "HAL/LowLevelMemTracker.h"

LLM_DECLARE_TAG_API(Harmonix_GainMatrix, HARMONIXDSP_API);

DECLARE_LOG_CATEGORY_EXTERN(LogGainMatrix, Log, All);

struct FPannerDetails;

class alignas(16) FGainMatrix
{
public:

	static HARMONIXDSP_API void Init();

	FGainMatrix()
	{
		check(((size_t)GainsArray & 0xF) == 0);
		Set(1, 1, EAudioBufferChannelLayout::Raw, ESpeakerMask::UnspecifiedMono, 0.0f);
	}

	FGainMatrix(EAudioBufferChannelLayout layout) 
		: ChannelMask(0)
	{
		check(((size_t)GainsArray & 0xF) == 0);
	}
	FGainMatrix(EAudioBufferChannelLayout layout, float InGain)
	{
		using namespace HarmonixDsp;
		check(((size_t)GainsArray & 0xF) == 0);
		Set(FAudioBuffer::kMaxChannelsInAudioBuffer, FAudioBuffer::kMaxChannelsInAudioBuffer, layout, ESpeakerMask::AllSpeakers, InGain);
	}

	FGainMatrix(int32 InNumInChannels, int32 InNumOutChannels, EAudioBufferChannelLayout InChannelLayout)
	{
		check(((size_t)GainsArray & 0xF) == 0);
		Set(InNumInChannels, InNumOutChannels, InChannelLayout, ESpeakerMask::AllSpeakers, 0.0f);
	}

	FGainMatrix(int32 InNumInChannels, int32 InNumOutChannels, EAudioBufferChannelLayout InChannelLayout, uint32 InChannelMask)
	{
		check(((size_t)GainsArray & 0xF) == 0);
		Set(InNumInChannels, InNumOutChannels, InChannelLayout, InChannelMask, 0.0f);
	}

	FGainMatrix(int32 InNumInChannels, int32 InNumOutChannels, EAudioBufferChannelLayout InChannelLayout, float InGain)
	{
		check(((size_t)GainsArray & 0xF) == 0);
		Set(InNumInChannels, InNumOutChannels, InChannelLayout, ESpeakerMask::AllSpeakers, InGain);
	}

	FGainMatrix(int32 InNumInChannels, int32 InNumOutChannels, EAudioBufferChannelLayout InChannelLayout, uint32 InChannelMask, float InGain)
	{
		check(((size_t)GainsArray & 0xF) == 0);
		Set(InNumInChannels, InNumOutChannels, InChannelLayout, InChannelMask, InGain);
	}

	int32 GetNumInChannels() const { return NumInChannels; }
	int32 GetNumOutChannels() const { return NumOutChannels; }
	uint32 GetChannelMask() const { return ChannelMask; }
	EAudioBufferChannelLayout GetChannelLayout() const { return ChannelLayout; }

	// pan -1 to 1
	HARMONIXDSP_API void SetFromMinusOneToOneSurroundPan(float InGain, float InPan, const FGainTable* InGainTable = nullptr);
	HARMONIXDSP_API void SetFromPolarDegrees(float InGain, float InPan, const FGainTable* InGainTable = nullptr);
	// pan in radians
	HARMONIXDSP_API void Set(float InGain, float InPanValue, const FGainTable* InGainTable = nullptr);;
	// pan -1 to 1
	HARMONIXDSP_API void SetFromLegacyStereo(float InGain, float InPan);
	HARMONIXDSP_API void SetFromNewStereo(float InGain, float InPan, float InMax, const FGainTable* InGainTable = nullptr);
	HARMONIXDSP_API void Set(float InGain, ESpeakerChannelAssignment InChannelAssignment, const FGainTable* InGainTable = nullptr);
	HARMONIXDSP_API void Set(float InGain, const FPannerDetails& InPannerDetails, const FGainTable* InGainTable = nullptr);

	static const FGainMatrix& Unity() { return *sUnity; }

	void Configure(int32 InNumInChannels, const TAudioBuffer<float>& OutputBuffer)
	{
		NumInChannels = InNumInChannels;
		NumOutChannels = (int32)OutputBuffer.GetNumValidChannels();
		ChannelLayout = OutputBuffer.GetChannelLayout();
		ChannelMask = OutputBuffer.GetChannelMask();
	}

	void ConfigureForOutputBuffer(const TAudioBuffer<float>& OutputBuffer)
	{
		NumOutChannels = (int32)OutputBuffer.GetNumValidChannels();
		ChannelLayout = OutputBuffer.GetChannelLayout();
		ChannelMask = OutputBuffer.GetChannelMask();
	}

	void ApplyToBuffer(float* src, int32 inChannel, float* dest, int32 outChannel, int32 numFrames, bool clear = false) const
	{
		checkSlow(inChannel < NumInChannels);
		checkSlow(outChannel < NumOutChannels);

		if (GainsArray[inChannel].f[outChannel] <= HarmonixDsp::kTinyGain)
		{
			if (clear)
			{
				FMemory::Memset(dest, 0, sizeof(float) * numFrames);
			}
			return;
		}

		VectorRegister4Float gain = VectorSetFloat1(GainsArray[inChannel].f[outChannel]);
		VectorRegister4Float sample;
		int32 slices = numFrames >> 2;
		for (int32 i = 0; i < slices; ++i)
		{
			sample = VectorLoadAligned(src);
			sample = VectorMultiply(gain, sample);
			VectorStoreAligned(sample, dest);
			dest += 4;
			src += 4;
		}
	}

	void ApplyAndAccumulateToBuffer(float* src, int32 InChannel, float* dest, int32 OutChannel, int32 InNumFrames) const
	{
		checkSlow(InChannel < NumInChannels);
		checkSlow(OutChannel < NumOutChannels);
		checkSlow(src != dest);

		if (GainsArray[InChannel].f[OutChannel] <= HarmonixDsp::kTinyGain)
		{
			return;
		}

		VectorRegister4Float gain = VectorSetFloat1(GainsArray[InChannel].f[OutChannel]);
		VectorRegister4Float origSample;
		VectorRegister4Float newSample;
		int32 slices = InNumFrames >> 2;
		for (int32 i = 0; i < slices; ++i)
		{
			newSample = VectorLoadAligned(src);
			newSample = VectorMultiply(gain, newSample);
			origSample = VectorLoadAligned(dest);
			newSample = VectorAdd(newSample, origSample);
			VectorStoreAligned(newSample, dest);
			dest += 4;
			src += 4;
		}
	}

	void PanSample(float sample, int32 InChannel, float* out, int32 InNumOutChannels)
	{
		checkSlow(InNumOutChannels <= NumOutChannels);
		for (int32 ch = 0; ch < InNumOutChannels; ++ch)
		{
			if (GainsArray[InChannel].f[ch] <= HarmonixDsp::kTinyGain)
			{
				continue;
			}

			out[ch] = sample * GainsArray[InChannel].f[ch];
		}
	}

	FGainMatrix& Set(int32 InNumInChannels, int32 InNumOutChannels, EAudioBufferChannelLayout layout, uint32 InChannelMask, float InGain)
	{
		NumInChannels = InNumInChannels;
		NumOutChannels = InNumOutChannels;
		ChannelLayout = layout;
		ChannelMask = InChannelMask;
		operator=(0.0f);
		for (int32 i = 0; i < NumInChannels; ++i)
		{
			GainsArray[i].f[i % NumOutChannels] = InGain;
		}
		return *this;
	}

	FGainMatrix(const FGainMatrix& start, const FGainMatrix& end, float lerpPos)
	{
		Lerp(start, end, lerpPos);
	}

	FGainMatrix& Lerp(const FGainMatrix& start, const FGainMatrix& end, float lerpPos)
	{
		NumInChannels = (start.NumInChannels < end.NumInChannels) ? start.NumInChannels : end.NumInChannels;
		NumOutChannels = (start.NumOutChannels < end.NumOutChannels) ? start.NumOutChannels : end.NumOutChannels;
#ifdef HX_SIMD_NONE
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			for (int32 och = 0; och < NumOutChannels; ++och)
			{
				GainsArray[ich].f[och] = start.GainsArray[ich].f[och] * (1.0f - lerpPos) + end.GainsArray[ich].f[och] * lerpPos;
			}
		}
#else
		VectorRegister4Float lerpA = VectorSetFloat1(1.0f - lerpPos);
		VectorRegister4Float lerpB = VectorSetFloat1(lerpPos);
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			VectorRegister4Float s = VectorMultiply(start.GainsArray[ich].simd[0], lerpA);
			VectorRegister4Float e = VectorMultiply(end.GainsArray[ich].simd[0], lerpB);
			GainsArray[ich].simd[0] = VectorAdd(s, e);
			s = VectorMultiply(start.GainsArray[ich].simd[1], lerpA);
			e = VectorMultiply(end.GainsArray[ich].simd[1], lerpB);
			GainsArray[ich].simd[1] = VectorAdd(s, e);
		}
#endif
		return *this;
	}

	FGainMatrix operator*(const FGainMatrix& Other) const
	{
		int32 numIch = (NumInChannels < Other.NumInChannels) ? NumInChannels : Other.NumInChannels;
		int32 numOch = (NumOutChannels < Other.NumOutChannels) ? NumOutChannels : Other.NumOutChannels;
		FGainMatrix result(numIch, numOch, ChannelLayout, ChannelMask);
		for (int32 ich = 0; ich < numIch; ++ich)
		{
			for (int32 och = 0; och < numOch; ++och)
			{
				result[ich].f[och] = GainsArray[ich].f[och] * Other.GainsArray[ich].f[och];
			}
		}
		return result;
	}

	FGainMatrix operator*(const float& m) const
	{
		FGainMatrix result = *this;
#ifdef HX_SIMD_NONE
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			for (int32 och = 0; och < NumOutChannels; ++och)
			{
				result[ich].f[och] = GainsArray[ich].f[och] * m;
			}
		}
#else
		VectorRegister4Float g = VectorSetFloat1(m);
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			result[ich].simd[0] = VectorMultiply(GainsArray[ich].simd[0], g);
			result[ich].simd[1] = VectorMultiply(GainsArray[ich].simd[1], g);
		}
#endif
		return result;
	}

	FGainMatrix& operator*=(float gain)
	{
#ifdef HX_SIMD_NONE
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			for (int32 och = 0; och < NumOutChannels; ++och)
			{
				GainsArray[ich].f[och] *= gain;
			}
		}
#else
		VectorRegister4Float g = VectorSetFloat1(gain);
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			GainsArray[ich].simd[0] = VectorMultiply(GainsArray[ich].simd[0], g);
			GainsArray[ich].simd[1] = VectorMultiply(GainsArray[ich].simd[1], g);
	}
#endif
		return *this;
}

	FGainMatrix& operator=(float gain)
	{
#ifdef HX_SIMD_NONE
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			// make for the same as the SIMD implementatione
			for (int32 och = 0; och < kMaxChannelsInAudioBuffer; ++och)
			{
				GainsArray[ich].f[och] = gain;
			}
		}
#else
		for (int32 ich = 0; ich < NumInChannels; ++ich)
		{
			GainsArray[ich].simd[0] = VectorSetFloat1(gain);
			GainsArray[ich].simd[1] = VectorSetFloat1(gain);
		}
#endif
		return *this;
	}


	const FChannelGains& operator[](int32 index) const
	{
		checkSlow(index < NumInChannels);
		return GainsArray[index];
	}

	FChannelGains& operator[](int32 index)
	{
		checkSlow(index < NumInChannels);
		return GainsArray[index];
	}

	static HARMONIXDSP_API bool  SpeakerChannelAssignmentMapsToOneSpeaker(ESpeakerChannelAssignment InChannelrAssignment, EAudioBufferChannelLayout InChannelLayout);
	static HARMONIXDSP_API int32 SpeakerChannelAssignmentToChannelIndex(ESpeakerChannelAssignment InChannelAssignment, EAudioBufferChannelLayout InChannelLayout);

private:

	alignas(16) FChannelGains GainsArray[HarmonixDsp::FAudioBuffer::kMaxChannelsInAudioBuffer] = {};

	static HARMONIXDSP_API FGainMatrix* sUnity;

	uint8 NumInChannels = HarmonixDsp::FAudioBuffer::kMaxChannelsInAudioBuffer;
	uint8 NumOutChannels = HarmonixDsp::FAudioBuffer::kMaxChannelsInAudioBuffer;

	EAudioBufferChannelLayout ChannelLayout = EAudioBufferChannelLayout::Raw;

	uint32 ChannelMask = ESpeakerMask::AllSpeakers;

	static int32 SpeakerToStereo(ESpeakerChannelAssignment InChannelAssignment);
	static int32 SpeakerToStereoPointOne(ESpeakerChannelAssignment InChannelAssignment);
	static int32 SpeakerToQuad(ESpeakerChannelAssignment InChannelAssignment);
	static int32 SpeakerToQuadPointOne(ESpeakerChannelAssignment InChannelAssignment);
	static int32 SpeakerToFive(ESpeakerChannelAssignment InChannelAssignment);
	static int32 SpeakerToFivePointOne(ESpeakerChannelAssignment InChannelAssignment);
	static int32 SpeakerToSevenPointOne(ESpeakerChannelAssignment InChannelAssignment);
};