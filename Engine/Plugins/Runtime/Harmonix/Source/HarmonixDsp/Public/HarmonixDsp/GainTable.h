// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBufferConstants.h"

#include "Logging/LogMacros.h"
#include "Math/VectorRegister.h"
#include "HAL/Platform.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGainTable, Log, All);

// anonymous namespace to hide this from externals
namespace Harmonix::Dsp::GainTable
{
	static const int32 kMaxSpeakers = 8;
}

using FChannelGains = MS_ALIGN(16) union 
{ 
	VectorRegister4Float simd[Harmonix::Dsp::GainTable::kMaxSpeakers / 4]; 
	float f[Harmonix::Dsp::GainTable::kMaxSpeakers]; 
} GCC_ALIGN(16);

class FGainTable
{
public:
	static HARMONIXDSP_API void Init(EAudioBufferChannelLayout ChannelLayout = EAudioBufferChannelLayout::Stereo);
	static HARMONIXDSP_API void SetupPrimaryGainTable(EAudioBufferChannelLayout ChannelLayout);

	static FORCEINLINE FGainTable& Get();

	HARMONIXDSP_API FGainTable();

	HARMONIXDSP_API void SetChannelLayout(EAudioBufferChannelLayout ChannelLayout);

	const FChannelGains& GetGains(float PolarAngle) const;
	void PanSample(float InSample, float PolarAngle, FChannelGains& OutGains) const;

	HARMONIXDSP_API FORCEINLINE float GetDirectChannelAzimuthInCurrentLayout(ESpeakerChannelAssignment ChannelAssignment) const;

	HARMONIXDSP_API FORCEINLINE void GetGainsForDirectAssignment(ESpeakerChannelAssignment ChannelAssignment, FChannelGains& OutGains) const;

private:
	EAudioBufferChannelLayout CurrentLayout = EAudioBufferChannelLayout::UnsupportedFormat;
	int32 SpeakerCount = 0;
	uint32 SpeakerMask = 0;

	static const uint32 kGainTableSize = 1024;
	FChannelGains Entries[kGainTableSize];

	HARMONIXDSP_API void BuildPanEntriesFromPannableSpeakerAzimuths(int32 NumSpeakers, const float* SpeakerAzimuths, const uint8* InRemap);
	HARMONIXDSP_API bool CurrentLayoutHasSpeaker(ESpeakerChannelAssignment ChannelAssignment) const;

	static HARMONIXDSP_API FGainTable* gGainTable;
};

#include "HarmonixDsp/GainTable.inl"
