// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBufferConstants.h"

extern FGainTable* gGainTable;

FORCEINLINE FGainTable& FGainTable::Get()
{
	checkSlow(gGainTable);
	return *gGainTable;
}

FORCEINLINE float FGainTable::GetDirectChannelAzimuthInCurrentLayout(ESpeakerChannelAssignment InChannelAssignment) const
{
	using namespace HarmonixDsp;
	if (InChannelAssignment >= ESpeakerChannelAssignment::FrontPair)
	{
		return 0.0f;
	}
	return FAudioBuffer::kDefaultSpeakerAzimuths[(int)CurrentLayout][(int)InChannelAssignment];
}


FORCEINLINE const FChannelGains& FGainTable::GetGains(float PolarAngle) const
{
	while (PolarAngle < 0.0f)
	{
		PolarAngle += (2.0f * (float)UE_PI);
	}

	while (PolarAngle >= (2.0f * (float)UE_PI))
	{
		PolarAngle -= (2.0f * (float)UE_PI);
	}

	int index = (int)(((PolarAngle / (2.0f * (float)UE_PI)) * (float)kGainTableSize) + 0.5f);
	index %= kGainTableSize;
	return Entries[index];
}

FORCEINLINE void FGainTable::PanSample(float InSample, float PolarAngle, FChannelGains& OutGains) const
{
	const FChannelGains& vols = GetGains(PolarAngle);
	VectorRegister4Float s = VectorSetFloat1(InSample);
	OutGains.simd[0] = VectorMultiply(s, vols.simd[0]);
	OutGains.simd[1] = VectorMultiply(s, vols.simd[1]);
}

FORCEINLINE void FGainTable::GetGainsForDirectAssignment(ESpeakerChannelAssignment InChannelAssignment, FChannelGains& OutGains) const
{
	using namespace HarmonixDsp;
	if (CurrentLayoutHasSpeaker(InChannelAssignment))
	{
		OutGains.f[FAudioBuffer::ChannelAssignmentSpeakerToMappedChannel[(int)InChannelAssignment]] = 1.0f;
		return;
	}
	OutGains = GetGains(GetDirectChannelAzimuthInCurrentLayout(InChannelAssignment));
}