// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

namespace HarmonixDsp
{
	// This function is stateful. Doesn't really work for multi channels
	void GenerateA220Sawtooth(float* Output, uint32 NumFrames, uint32 SampleRate);

	/**
	 * fill an output buffer with random sample
	 * samples are in the range [-gain, gain]
	 * @param output the buffer of floating point samples to write to
	 * @param numFrames the number of samples to write
	 * @param gain (optional) the upper range of the samples. defaults to one.
	 */
	HARMONIXDSP_API void GenerateWhiteNoise(float* Output, uint32 NumFrames, float Gain = 1.0f);
	HARMONIXDSP_API void GenerateWhiteNoise(int16* Output, uint32 NumFrames, float Gain = 1.0f);

	template<class T>
	void GenerateWhiteNoiseEq(T* Output, uint32 NumFrames, float Gain)
	{
		GenerateWhiteNoise(Output, NumFrames, Gain);
	}

	/**
	 * Fill a mono output buffer with a sine wave
	 * @param Output - The output buffer
	 * @param NumFrames - The number of sample frames to generate
	 * @param Frequency - The frequency of the sine wave
	 * @param SampleRate - The sample rate at which to generate the sine wave
	 * @param Phase - The phase to start (and to use for future calls if desired)
	 */
	HARMONIXDSP_API void GenerateSine(
		float* Output,
		uint32 NumFrames,
		float Frequency,
		float SampleRate,
		float& Phase);
}