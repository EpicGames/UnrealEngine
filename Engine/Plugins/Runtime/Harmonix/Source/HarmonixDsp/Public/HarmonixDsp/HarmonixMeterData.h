// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

#define UE_API HARMONIXDSP_API

class FHarmonixMeterData
{
public:
	FHarmonixMeterData(double InMin = 0.0)
	{
		MinimumData = InMin;
		CurrentData = MinimumData;
		PeakData = MinimumData;
		bShouldClearDataBeforeNextMerge = true;
	}


	/**
	 * Get the metering data on the main thread.
	 * returns the current value, and the peak.
	 * @param current the current data
	 * @param peak the peak value since the last cal to ClearPeak
	 * @see ClearPeak
	 */
	void GetLatest(double& OutCurrent, double& OutPeak) const
	{
		//CRIT_SEC_REF(sAudioMeterCritSec);
		FScopeLock Lock(&sAudioMeterLock);
		PeekAtLatest(OutCurrent, OutPeak);
		bShouldClearDataBeforeNextMerge = true;
	}

	/**
	 * Get the metering data on the main thread without triggering a data clearing
	 * returns the current value, and the peak.
	 * @param current the current data
	 * @param peak the peak value since the last cal to ClearPeak
	 * @see ClearPeak
	 */
	void PeekAtLatest(double& OutCurrent, double& OutPeak) const
	{
		//CRIT_SEC_REF(sAudioMeterCritSec);
		FScopeLock Lock(&sAudioMeterLock);

		if (CurrentData > PeakData)
			PeakData = CurrentData;

		OutCurrent = CurrentData;
		OutPeak = PeakData;
	}

	/**
	 * Clears the peak data.
	 * On the next call to get latest, peak will match current
	 */
	void ClearPeak()
	{
		PeakData = CurrentData;
	}

	/**
	 * Merge in new data on the audio thread.
	 * @param newData the newData to merge into the existing data
	 */
	void Merge(double InNewData)
	{
		//CRIT_SEC_REF(sAudioMeterCritSec);
		FScopeLock Lock(&sAudioMeterLock);

		if (bShouldClearDataBeforeNextMerge)
		{
			CurrentData = MinimumData;
			bShouldClearDataBeforeNextMerge = false;
		}

		if (InNewData > CurrentData)
		{
			CurrentData = InNewData;
		}
	}

	/**
	* Merge in the data to minimum on the audio thread.
	* This happens without taking a lock.
	*/
	FORCEINLINE void MergeMinimum()
	{
		if (bShouldClearDataBeforeNextMerge)
		{
			CurrentData = MinimumData;
			bShouldClearDataBeforeNextMerge = false;
		}
	}

private:
	//HX_DLLAPI static HmxThreading::CritSec sAudioMeterCritSec;

	static UE_API FCriticalSection sAudioMeterLock;

	mutable bool bShouldClearDataBeforeNextMerge;
	double MinimumData;
	double CurrentData;
	mutable double PeakData;
};

#undef UE_API
