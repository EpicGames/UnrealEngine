// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/PlatformMemory.h"

#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/MidiVoiceId.h"
#include "HarmonixDsp/HarmonixMeterData.h"
#include "HarmonixDsp/MusicalAudioBus.h"

#define UE_API HARMONIXDSP_API

class FVirtualInstrument : public FMusicalAudioBus
{
public:

	FVirtualInstrument(FName InName = NAME_None)
	{
		Name = InName;
		FMemory::Memset(LastCcVals, -1, 16 * 128);
	}

	virtual ~FVirtualInstrument() = default;

	//*************************************************************************
	// Implementation of AudioBus virtual. We will make it pure here, because
	// all virtual instruments must handle this themselves. 
	// virtual void Prepare(float sampleRateHz, uint32 numChannels, uint32 maxSamples, bool allocateBuffer = true) override = 0;

	//*************************************************************************
	// Any subclass of VirtualInstrument will also have to implement the
	// AudioBus pure virtual function... 
	// virtual void Process(AudioBuffer<float>& output) = 0;

	virtual void CallPreProcessCallbacks(int32 NumSamples, float SampleRateHz, int32 SliceIndex, int32 Subslice, bool LastSubslice) {};

	// override this (as an optimization) if calling process may only produce zero samples
	virtual bool ProcessCallWillProduceSilence() const { return false; }

	//*************************************************************************
	// Pure virtuals that deal with instrument state and MIDI message handling...
	UE_API void  ResetInstrumentState();
	UE_API void  ResetMidiState();
	virtual void  NoteOn(FMidiVoiceId InVoiceId, int8 MidiNoteNumber, int8 Velocity, int8 MidiChannel = 0, int32 EventTick = 0, int32 CurrentTick = 0, float MsOffset = 0.0f) = 0;
	virtual void  NoteOnWithFrameOffset(FMidiVoiceId InVoiceId, int8 MidiNoteNumber, int8 Velocity, int8 MidiChannel = 0, int32 NumFrames = 0) = 0;
	virtual bool  NoteIsOn(int8 MidiNoteNumber, int8 MidiChannel = 0) = 0;
	virtual void  NoteOff(FMidiVoiceId InVoiceId, int8 MidiNoteNumber, int8 MidiChannel = 0) = 0;
	virtual void  NoteOffWithFrameOffset(FMidiVoiceId InVoiceId, int8 MidiNoteNumber, int8 MidiChannel = 0, int32 NumFrames = 0) = 0;
	virtual void  PolyPressure(FMidiVoiceId InVoiceId, int8 InData1, int8 InData2, int8 InChannel = 0) {}
	virtual void  ChannelPressure(int8 InData1, int8 InData2, int8 InChannel = 0) {}

	// code-driven pitch bend, in semitones
	virtual void  SetExtraPitchBend(float Semitones, int8 MidiChannel = 0) {}
	
	// pitch bend value, on range [-1.0, 1.0]
	virtual void  SetPitchBend(float InValue, int8 InMidiChannel = 0) = 0;

	// pitch bend value, on range [-1.0, 1.0]
	virtual float GetPitchBend(int8 InMidiChannel = 0) const = 0;

	virtual void SetRawTransposition(int32 SemiTones) = 0;
	virtual int32 GetRawTransposition() const = 0;
	virtual void SetRawPitchMultiplier(float RawPitch) = 0;
	virtual float GetRawPitchMultiplier() const = 0;

	UE_API void Set7BitController(Harmonix::Midi::Constants::EControllerID InController, int8 InByteValue, int8 InMidiChannel = 0);
	UE_API void Set14BitController(Harmonix::Midi::Constants::EControllerID InController, int16 InByteValue, int8 InMidiChannel = 0);

	virtual void GetController(Harmonix::Midi::Constants::EControllerID InController, int8& InMsb, int8& InLsb, int8 InMidiChannel = 0) const = 0;
	UE_API void SetController(Harmonix::Midi::Constants::EControllerID InController, float InValue, int8 InMidiChannel = 0);
	UE_API void SetHighOrLowControllerByte(Harmonix::Midi::Constants::EControllerID InController, int8 InValue, int8 InMidiChannel = 0);

	virtual float GetController(Harmonix::Midi::Constants::EControllerID InController, int8 InMidiChannel = 0)
	{
		int8 msb;
		int8 lsb;
		GetController(InController, msb, lsb, InMidiChannel);
		return (float)(((msb & 0x7F) << 7) | (lsb & 0x7F)) / (float)0x3FFFF;
	}

	virtual void KillAllVoices() = 0;
	virtual void AllNotesOff() = 0;
	virtual void AllNotesOffWithFrameOffset(int32 InNumFrames = 0) = 0;

	virtual void SetSpeed(float InSpeed, bool MaintainPitch = false) = 0;
	virtual float GetSpeed(bool* MaintainPitch = nullptr) = 0;

	// update the effect with the current beat/tempo
	virtual void SetQuarterNote(float QuarterNote) {}
	virtual void SetTempo(float Bpm) {}

	virtual void SetSampleRate(float InSampleRateHz) = 0;

	UE_API virtual void HandleMidiMessage(FMidiVoiceId InVoiceId, int8 InStatus, int8 InData1, int8 InData2, int32 InEventTick = 0, int32 InCurrentTick = 0, float InMsOffset = 0.0f);

	/**
	 * get the max number of note-ons this baby can handle before auto-release.
	 * note that the actual number of in-use voices may go over this max when voices are in release stage
	 * @returns the maximum number of voices this instrument can play before auto-releasing voices
	 * @see GetNumVoicesInUse
	 */
	virtual int32 GetMaxNumVoices() const { return -1; } // may not make sense for some instruments
	virtual int32 GetNumVoicesInUse() const = 0;
	void GetVoiceUsage(int32& OutCurrent, int32& OutPeak) const
	{
		double Current, Peak;
		VoiceUsage.GetLatest(Current, Peak);
		OutCurrent = (int32)Current;
		OutPeak = (int32)Peak;
	}

	void ClearPeakVoiceUsage() { VoiceUsage.ClearPeak(); }

	static bool ShouldPrintMidiActivity() { return bShouldPrintMidiActivity; }
	static void SetShouldPrintMidiActivity(bool bInShouldPrint) { bShouldPrintMidiActivity = bInShouldPrint; }

	void SetName(FName InName) { Name = InName; }
	FName GetName() const { return Name; }

	void   GetProcessingTimeMs(double& OutCurrent, double& OutPeak) const { TimingData.GetLatest(OutCurrent, OutPeak); }
	void   ClearPeakProcessingTime() { TimingData.ClearPeak(); }
	double PeekAtPeakProcessingTime() const { double c, p; TimingData.PeekAtLatest(c, p); return p; }

	virtual void  SetMidiChannelVolume(float InVolume, float InSeconds = 0.0f, int8 InMidiChannel = 0) = 0;
	virtual float GetMidiChannelVolume(int8 InMidiChannel = 0) const = 0;
	virtual void  SetMidiChannelGain(float InGain, float InSeconds = 0.0f, int8 InMidiChannel = 0) = 0;
	virtual float GetMidiChannelGain(int8 InMidiChannel = 0) const = 0;
	virtual void  SetMidiChannelMute(bool InMute, int8 InMidiChannel = 0) = 0;
	virtual bool  GetMidiChannelMute(int8 InMidiChannel = 0) const = 0;

	static UE_API bool IsHighResController(Harmonix::Midi::Constants::EControllerID ControllerId, bool& bIsHighResLowByte);
	static UE_API bool GetMsbLsbIndexes(Harmonix::Midi::Constants::EControllerID ControllerId, int& InMsb, int& InLsb);

protected:
	virtual void Set7BitControllerImpl(Harmonix::Midi::Constants::EControllerID InController, int8 InValue, int8 InMidiChannel = 0) = 0;
	virtual void Set14BitControllerImpl(Harmonix::Midi::Constants::EControllerID InController, int16 InValue, int8 InMidiChannel = 0) = 0;
	virtual void ResetInstrumentStateImpl() = 0; // heavy weight! should completely resets the instrument.
	virtual void ResetMidiStateImpl() = 0;       // less heavy weight. should do all notes off, reset pitch bend, etc.

	// convert a value in [0,127] to the range expected by SetController
	static UE_API float ConvertCCValue(Harmonix::Midi::Constants::EControllerID InController, uint8 InValue);

	FName Name;
	FHarmonixMeterData TimingData;
	FHarmonixMeterData VoiceUsage;
	static UE_API bool bShouldPrintMidiActivity;

	FCriticalSection BusLock;
	int8 LastCcVals[16][128];

};

#undef UE_API
