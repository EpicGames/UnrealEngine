// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/Modulators/Settings/AdsrSettings.h"

#define UE_API HARMONIXDSP_API

namespace Harmonix::Dsp::Modulators
{

enum class EAdsrStage
{
	Attack,
	Decay,
	Sustain,
	Release,
	Idle,
	Setup
};

enum class EAdsrMode
{
	MinUp,
	MaxDown,
	SustainOut
};

struct FAdsr
{
public:

	FAdsr()
	{
		Settings = nullptr;
		Kill();
		Mode = EAdsrMode::MinUp;
	}

	void UseSettings(const FAdsrSettings* InSettings, bool Reset = true)
	{
		check(InSettings != nullptr);
		if (Reset)
		{
			Stage = EAdsrStage::Setup;
		}

		Settings = InSettings;
	}

	void Prepare(float InSamplesPerSecond)
	{
		SamplesPerSecond = InSamplesPerSecond;
	}

	const FAdsrSettings* GetSettings() const
	{
		return Settings;
	}

	UE_API float GetValue() const;
	EAdsrStage GetStage() const { return Stage; }
	uint32 GetAge() const { return Age; }

	UE_API void Advance(uint32 InNumSamples);

	void Attack()
	{
		check(Settings != nullptr);

		if (Stage != EAdsrStage::Idle && Stage != EAdsrStage::Setup)
		{
			return;
		}

		CurveAge = 0.0f;

		check(Gain == 0.0f);

		if (Settings->AttackTime > 0.0f)
		{
			Stage = EAdsrStage::Attack;
			float NumSamples = Settings->AttackTime * SamplesPerSecond;
			if (NumSamples == 0.0f)
			{
				NumSamples = 1.0f;
			}
			AttackIncrement = 1.0f / NumSamples;
		}
		else
		{
			if (Settings->HasDecayStage())
			{
				Gain = 1.0f;
				Stage = EAdsrStage::Decay;
				float NumSamples = Settings->DecayTime * SamplesPerSecond;
				if (NumSamples == 0)
				{
					NumSamples = 1;
				}

				DecayDecrement = (1.0f - (Settings->SustainLevel)) / NumSamples;
			}
			else
			{
				Gain = Settings->SustainLevel;
				Stage = EAdsrStage::Sustain;
			}
		}
	}

	void Release()
	{
		if (Stage != EAdsrStage::Idle && Stage != EAdsrStage::Release)
		{
			check(Settings != nullptr);
			if (Settings->ReleaseTime > 0.0f)
			{
				Stage = EAdsrStage::Release;
				CurveAge = 0.0f;
				ReleaseStartingGain = Gain;
				float NumSamples = Settings->ReleaseTime * SamplesPerSecond;
				ReleaseDecrement = Gain / NumSamples;
			}
			else
			{
				Kill();
			}
		}
	}

	void FastRelease()
	{
		if (Stage != EAdsrStage::Idle && Stage != EAdsrStage::Setup)
		{
			Stage = EAdsrStage::Release;
			bIsFastRelease = true;
			float FastReleaseSec = FMath::Min(0.25f, Settings->ReleaseTime * Settings->SustainLevel);
			float NumSamples = FastReleaseSec * SamplesPerSecond;
			if (NumSamples > 0.0f)
			{
				ReleaseDecrement = 1.0f / NumSamples;
			}
			else
			{
				Kill();
			}
		}
	}

	void Kill()
	{
		Gain = 0.0f;
		Stage = EAdsrStage::Idle;
		Age = 0;
		CurveAge = 0.0f;
		bIsFastRelease = false;
		ReleaseStartingGain = 1.0f;
	}

private:

	const FAdsrSettings* Settings = nullptr;

	float SamplesPerSecond = 0.0f;
	float Gain = 0.0f;
	EAdsrStage Stage = EAdsrStage::Idle;
	uint32 Age = 0.0f;
	float CurveAge = 0.0f;
	
	float AttackIncrement = 0.0f;
	float DecayDecrement = 0.0f;
	float ReleaseDecrement = 0.0f;

	bool bIsFastRelease = false;
	float ReleaseStartingGain = 0.0f;

	EAdsrMode Mode = EAdsrMode::MinUp;
};
}

#undef UE_API
