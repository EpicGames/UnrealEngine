// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Interval.h"
#include "HarmonixDsp/Modulators/Settings/LfoSettings.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixDsp_Lfo, Log, All);

namespace Harmonix::Dsp::Modulators
{

enum class ELfoMode : uint8
{ 
	MinUp, 
	MaxDown, 
	CenterOut 
};

struct FWaveShape
{
	typedef float (*FWaveShapeFunction)(float);


	static float Sine(float x);
	static float Square(float x);
	static float SawtoothUp(float x);
	static float SawtoothDown(float x);
	static float Triangle(float x);
	static float Noise(float x);

	static FWaveShapeFunction GetFunction(EWaveShape WaveShape)
	{
		uint8 Index = (uint8)WaveShape;
		check(Index < (uint8)EWaveShape::Num);
		return sWaveShapeTable[Index];
	}

	static float Apply(EWaveShape WaveShape, float x)
	{
		return GetFunction(WaveShape)(x);
	}

private:
	
	static FWaveShapeFunction sWaveShapeTable[(uint8)EWaveShape::Num];
};


// this low-frequency oscillator (LFO) class is considered low-frequency
// because it doesn't do anything about aliasing, which is audible if the
// frequency is in the audible range. infrasonic frequencies are the target
// for this implementation.
struct FLfo
{
public:

	HARMONIXDSP_API FLfo();

	HARMONIXDSP_API void Prepare(float InSampleRate);

	//  input is [0,1], returns [0,1]
	typedef float (*FWaveShapeFunction)(float);

	HARMONIXDSP_API void UseSettings(const FLfoSettings* InSettings);

	const FLfoSettings* GetSettings() const 
	{ 
		return Settings; 
	}

	HARMONIXDSP_API void SetPhase(double InPhase);

	HARMONIXDSP_API double GetPhase() const;

	// sets phase to the initial phase indicated
	// by the user setting (and possibly by the shape)
	HARMONIXDSP_API void Retrigger();

	HARMONIXDSP_API TInterval<float> GetRange() const;
	HARMONIXDSP_API void SetRangeAndMode(const TInterval<float>& InRange, ELfoMode InMode);

	// advances the phase of the oscillator
	HARMONIXDSP_API void Advance(uint32 InNumSamples);

	HARMONIXDSP_API float GetValue() const;
private:
	void ComputeCyclesPerSample();

	double Phase;
	//WaveShapeFunction mApplyWaveshaping;

	TInterval<float> Range;
	ELfoMode Mode;

	// patch-level settings
	// used as long as these settings
	// are not overridden by and ADSR.
	const FLfoSettings* Settings;

	double CyclesPerSample;
	float  SecondsPerSample;
	float  TempoWhenCalculated;
};

}