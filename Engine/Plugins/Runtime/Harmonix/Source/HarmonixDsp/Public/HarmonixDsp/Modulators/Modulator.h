// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Interval.h"

namespace Harmonix::Dsp::Modulators
{

struct FModulatorTarget;

class FModulator
{
public:
	
	FModulator();

	/**
	 * Set the modulator target.
	 * This class does not take ownership of the modulator target,
	 * and will never delete it.
	 * @param target the target to modulate when Modulate is called.
	 * @returns description of the retumrn type
	 * @see Modulate()
	 */
	void SetTarget(const FModulatorTarget* InTarget);

	/**
	 * Get the modulator target.
	 * @returns description of the return type
	 */
	const FModulatorTarget* GetTarget() const;

	/**
	 * gets the depth (efficacy) of the randomization effect.
	 * @returns the depth on the range [0,1]
	 */
	float GetDepth() const;

	/**
	 * set the depth (efficacy) of the randomization effect
	 * @param depth a value on in the range [0,1] that describes the degree to which randomization occurs
	 * @returns description of the return type
	 */
	void SetDepth(float InDepth);

	void SetRangeMagnitude(float Magnitude);
	float GetRangeMagnitude() const;


	/**
	 * resets to defaults.
	 */
	void Reset();

	/**
	 * modulates the target value based on the input value
	 * @param normalized number in range [0,1] to map to range and to modulate the target
	 */
	void Modulate(float InputNorm);

protected:

	const FModulatorTarget* Target;
	float Depth;
	float RangeMagnitude;
	TInterval<float> Range;
};

}