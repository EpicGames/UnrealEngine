// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/Modulators/Settings/ModulatorSettings.h"

namespace Harmonix::Dsp::Modulators
{

enum class EDepthMode : uint8
{
	// Modulate to the minimum end of the range for 0% depth
	MinUp,

	// Modulate to the maximum end of the range for 0% depth
	MaxDown,

	// Modulate to the center of the range for 0% depth
	CenterOut
};

/**
 * a struct that contains data about a modulator target.
 * including its name, range of modulation, etc.
 */
struct FModulatorTarget
{

public:
	static const FName kStartPointName;
	static const FName kPitchName;

public:
	FModulatorTarget() {}
	FModulatorTarget(FName InName, float InDefaultRangeMag, const FString& InUnits, EDepthMode DepthMode, float* ValueToTarget);

	/**
	 * take a number on [0,1] and transform it based on a depth
	 * @param x the number to map
	 * @param depth the depth to use in mapping
	 * @returns a transformed version of x mapped to the target's range
	 */
	float ApplyDepthToNormalizedValue(float x, float depth) const;

	FName GetName() const { return Name; }
	const FString& GetUnits() const { return Units; }
	float GetDefaultRangeMagnitude() const { return DefaultRangeMag; }
	TInterval<float> GetRangeWithMagnitude(float InMagnitude) const;
	
	float* GetTargetFloat() const { return TargetFloat; }

	static FModulatorTarget kDummyTarget;

private:
	
	// Name of the modulator target
	FName Name;
	
	// hint for reasonable range
	float DefaultRangeMag = 0.0f;
	
	// units the range is expressed in
	FString Units;
	
	// reference to the value to modulate
	float* TargetFloat = nullptr;

	// how to apply a depth of less than 100%
	EDepthMode DepthMode = EDepthMode::MinUp;
};

}