// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDsp/TimeSyncOption.h"
#include "HarmonixDsp/Parameters/Parameter.h"

#include "HarmonixMidi/BarMap.h"

#define UE_API HARMONIXDSP_API

namespace Harmonix::Dsp::Modulators
{
	/**
	 * @brief A clock-sync-able LFO with a morphing waveform
	 */
	class FMorphingLFO
	{
	public:
		UE_API explicit FMorphingLFO(float InSampleRate);

		UE_API float GetSampleRate() const;
		
		/**
		 * @brief Determines how the frequency parameter behaves
		 */
		Parameters::TParameter<ETimeSyncOption> SyncType{ ETimeSyncOption::None, ETimeSyncOption::SpeedScale, ETimeSyncOption::None };

		/**
		 * @brief If LfoSyncType is TempoSync, the unit is cycles per quarter note. Otherwise, it's Hz.
		 */
		Parameters::TParameter<float> Frequency{ UE_SMALL_NUMBER, 40.0f, 1.0f };

		/**
		 * @brief If set, the output will be inverted
		 */
		Parameters::TParameter<bool> Invert{ false, true, false };

		/**
		 * @brief Determines the shape of the output 0.0-1.0 morphs from square to triangle, 1.0-2.0 morphs from triangle to sawtooth
		 */
		Parameters::TParameter<float> Shape{ 0.0f, 2.0f, 1.0f };

		/**
		 * @brief Reset the LFO to its initial state
		 * @param InSampleRate - The sample rate
		 */
		UE_API void Reset(float InSampleRate);

		/**
		 * @brief Contains the information necessary to do music clock sync
		 */
		struct FMusicTimingInfo
		{
			FMusicTimestamp Timestamp;
			FTimeSignature TimeSignature;
			float Tempo = 0;
			float Speed = 0;
		};

		/**
		 * @brief Advance and output the last value
		 * @param DeltaFrames - The amount of time in audio frames to advance
		 * @param Output - The LFO output
		 * @param MusicTimingInfo - (optional) The music timing info to use if appropriate
		 */
		UE_API void Advance(const int32 DeltaFrames, float& Output, const FMusicTimingInfo* MusicTimingInfo = nullptr);

		/**
		 * @brief Advance and output a buffer of LFO values
		 * @param OutputBuffer - The buffer of LFO outputs
		 * @param NumFrames - The number of frames to advance, which should match the size of the buffer
		 * @param MusicTimingInfo - (optional) The music timing info to use if appropriate
		 */
		UE_API void Advance(float* OutputBuffer, int32 NumFrames, const FMusicTimingInfo* MusicTimingInfo = nullptr);

		/**
		 * @brief Get the LFO value for a given shape and phase.
		 * @param Shape - The shape of the LFO, range [0.0, 2.0]. See Shape field for more info.
		 * @param Phase - The current phase, range [0.0, 1.0]
		 * @return 
		 */
		static UE_API float GetValue(const float Shape, const float Phase);
		
	private:
		static float GetFrequencyHz(const ETimeSyncOption SyncType, const float Frequency, const float Tempo, const float Speed);
		static float GetClockSyncedPhase(const float CyclesPerQuarter, const FMusicTimingInfo& MusicTimingInfo);
		
		float SampleRate;
		float Phase;
		float LastFrequency ;
	};
}

#undef UE_API
