// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AdsrSettings.generated.h"

#define UE_API HARMONIXDSP_API

UENUM(BlueprintType)
enum class EAdsrTarget : uint8
{
	Volume      UMETA(Json="volume"),
	FilterFreq  UMETA(Json="filter_freq"),
	Num			UMETA(Hidden),
	Invalid		UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FAdsrSettings
{
	GENERATED_BODY()

public:
	static UE_API const float kMinTimeSec;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EAdsrTarget Target = EAdsrTarget::Volume;
	
	UPROPERTY(EditDefaultsOnly, Category="Settings")
	bool IsEnabled = false;

	UPROPERTY(EditDefaultsOnly, Category="Settings", DisplayName = "Attack (Seconds)", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10"))
	float AttackTime = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category="Settings", DisplayName = "Decay (Seconds)", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10"))
	float DecayTime = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin="0",  UIMax="1", ClampMin="0", ClampMax="1"))
	float SustainLevel = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category="Settings", DisplayName = "Release (Seconds)", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10"))
	float ReleaseTime = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category="Settings", Meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1"))
	float Depth = 0.5f;

	/*
	 * Control the shape of the attack curve:
	 * 0: Linear
	 * <0: Exponential
	 * >0: Logarithmic
	 **/
	UPROPERTY(EditDefaultsOnly, Category="Settings", DisplayName = "Attack Curve", Meta = (UIMin = "-1", UIMax = "1", ClampMin = "-1", ClampMax = "1"))
	float AttackCurve = 0.0f;

	/*
	* Control the shape of the attack curve:
	* 0: Linear
	* <0: Exponential
	* >0: Logarithmic
	**/
	UPROPERTY(EditDefaultsOnly, Category="Settings", DisplayName = "Decay Curve", Meta = (UIMin = "-1", UIMax = "1", ClampMin = "-1", ClampMax = "1"))
	float DecayCurve = 0.0f;

	/*
	 * Control the shape of the attack curve:
	 * 0: Linear
	 * <0: Exponential
	 * >0: Logarithmic
	 **/
	UPROPERTY(EditDefaultsOnly, Category="Settings", DisplayName = "Release Curve", Meta = (UIMin = "-1", UIMax = "1", ClampMin = "-1", ClampMax = "1"))
	float ReleaseCurve = 0.0f;

	static const uint32 kCurveTableSize = 20;

	float AttackUsedForTable = -1.0f;
	float AttackCurveTable[kCurveTableSize];

	float DecayUsedForTable = -1.0f;
	float DecayCurveTable[kCurveTableSize];

	float ReleaseUsedForTable = -1.0f;
	float ReleaseCurveTable[kCurveTableSize];

	FAdsrSettings() { ResetToDefaults(); }

	void ResetToDefaults()
	{
		IsEnabled = false;
		AttackTime = 0.0f;
		DecayTime = 0.0f;
		SustainLevel = 0.0f;
		ReleaseTime = 1.0f;
		Depth = 0.5f;
		AttackCurve = 0.0f;
		DecayCurve = 0.0f;
		ReleaseCurve = 0.0f;
		AttackUsedForTable = 10.0f;
		DecayUsedForTable = 10.0f;
		ReleaseUsedForTable = 10.0f;

		Calculate();
	}

	void Calculate()
	{
		BuildAttackTable();
		BuildDecayTable();
		BuildReleaseTable();
	}

	void CopySettings(const FAdsrSettings& Other)
	{
		Target = Other.Target;
		IsEnabled = Other.IsEnabled;
		AttackTime = Other.AttackTime;
		DecayTime = Other.DecayTime;
		SustainLevel = Other.SustainLevel;
		ReleaseTime = Other.ReleaseTime;
		Depth = Other.Depth;
		AttackCurve = Other.AttackCurve;
		DecayCurve = Other.DecayCurve;
		ReleaseCurve = Other.ReleaseCurve;
	}
	UE_API void CopyCurveTables(const FAdsrSettings& Other);

	bool HasDecayStage() const { return (SustainLevel < 1.0f) && (0.0f < DecayTime); }

	bool IsAttackLinear() const { return IsCurveLinear(AttackCurve); }
	bool IsDecayLinear() const { return IsCurveLinear(DecayCurve); }
	bool IsReleaseLinear() const { return IsCurveLinear(ReleaseCurve); }
	bool IsCurveLinear(float InCurve) const { return InCurve >= -0.001f && InCurve <= +0.001f; }

	void BuildAttackTable()
	{
		if (AttackCurve != AttackUsedForTable)
		{
			BuildCurveTable(AttackCurve, false, AttackCurveTable);  AttackUsedForTable = AttackCurve;
		}
	}

	void BuildDecayTable()   
	{ 
		if (DecayCurve != DecayUsedForTable)
		{
			BuildCurveTable(DecayCurve, true, DecayCurveTable);
		}
		DecayUsedForTable = DecayCurve; 
	}
	
	void BuildReleaseTable() 
	{ 
		if (ReleaseCurve != ReleaseUsedForTable)
		{
			BuildCurveTable(ReleaseCurve, true, ReleaseCurveTable);
		}
		ReleaseUsedForTable = ReleaseCurve; 
	}

	float LerpAttackCurve(float index) const { return LerpCurve(index, false, AttackCurveTable); }
	float LerpDecayCurve(float index) const { return LerpCurve(index, true, DecayCurveTable); }
	float LerpReleaseCurve(float index) const { return LerpCurve(index, true, ReleaseCurveTable); }
	UE_API float LerpCurve(float index, bool down, const float* curve) const;

private:

	UE_API void BuildCurveTable(float curve, bool down, float* table);
};

USTRUCT()
struct FAdsrSettingsArray
{
	GENERATED_BODY()

	UPROPERTY()
	FAdsrSettings Array[2];
};

#undef UE_API
