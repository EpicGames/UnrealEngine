// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "LfoSettings.generated.h"

UENUM(BlueprintType)
enum class EWaveShape : uint8
{
	Sine		UMETA(Json="sine"),
	Square		UMETA(Json="square"),
	SawUp		UMETA(Json="saw_up"),
	SawDown		UMETA(Json="saw_down"),
	Triangle	UMETA(Json="triangle"),
	Random		UMETA(Json="random"),
	Num			UMETA(Hidden),
	None		UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ELfoTarget : uint8
{
	Pan			UMETA(Json="pan"),
	Pitch		UMETA(Json="pitch"),
	FilterFreq	UMETA(Json="filter_freq"),
	Num			UMETA(Hidden),
	None		UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FLfoSettings
{
	GENERATED_BODY()

public:
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	ELfoTarget Target = ELfoTarget::None;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool IsEnabled = false;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	EWaveShape Shape = EWaveShape::Sine;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool ShouldRetrigger = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool BeatSync = false;
	
	UPROPERTY(EditDefaultsOnly, Category = "Settings", DisplayName = "Freq (Hz)", Meta = (UIMin = "0.01", UIMax = "40.0", ClampMin = "0.01", ClampMax = "40.0"))
	float Freq = 4.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Depth = 0.5f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "0.0", UIMax = "0.99", ClampMin = "0.0", ClampMax = "0.99"))
	float InitialPhase = 0.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = "1", UIMax = "240", ClampMin = "1", ClampMax = "240"))
	float TempoBPM = 120.0f;

	FLfoSettings() { ResetToDefaults(); }
	FLfoSettings(ELfoTarget InTarget)
		: Target(InTarget)
	{
		Shape = EWaveShape::Sine;
		IsEnabled = false;
		ShouldRetrigger = false;
		BeatSync = false;
		Freq = 4.0f;
		Depth = 0.5f;
		InitialPhase = 0.0f;
		TempoBPM = 120.0f;
	}

	void ResetToDefaults()
	{
		Shape = EWaveShape::Sine;
		Target = ELfoTarget::None;
		IsEnabled = false;
		ShouldRetrigger = false;
		BeatSync = false;
		Freq = 4.0f;
		Depth = 0.5f;
		InitialPhase = 0.0f;
		TempoBPM = 120.0f;
	}

	void CopySettings(const FLfoSettings& Other)
	{
		Shape = Other.Shape;
		Target = Other.Target;
		IsEnabled = Other.IsEnabled;
		ShouldRetrigger = Other.ShouldRetrigger;
		BeatSync = Other.BeatSync;
		Freq = Other.Freq;
		Depth = Other.Depth;
		InitialPhase = Other.InitialPhase;
		TempoBPM = Other.TempoBPM;
	}
};

USTRUCT()
struct FLfoSettingsArray
{
	GENERATED_BODY()

	UPROPERTY()
	FLfoSettings Array[2];
};
