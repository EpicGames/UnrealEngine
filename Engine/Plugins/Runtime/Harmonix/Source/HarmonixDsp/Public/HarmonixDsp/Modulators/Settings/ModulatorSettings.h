// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/EnumRange.h"

#include "ModulatorSettings.generated.h"

#define UE_API HARMONIXDSP_API

UENUM(BlueprintType)
enum class EModulatorTarget : uint8
{
	StartPoint	UMETA(Json = "Start Point"),
	Pitch		UMETA(Json = "Pitch"),
	Num			UMETA(Hidden),
	None		UMETA(Hidden, Json = "None")
};

USTRUCT(BlueprintType)
struct FModulatorSettings
{
	GENERATED_BODY()

public:
	UE_API FModulatorSettings();

	UPROPERTY(EditDefaultsOnly, Category="Harmonix|DSP")
	EModulatorTarget Target = EModulatorTarget::None;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0.0", UIMax = "1000000.0", ClampMin = "0.0", ClampMax = "1000000.0"))
	float Range = 1.0f;

	UPROPERTY(EditDefaultsOnly, Category = "Harmonix|DSP", Meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Depth = 0.5f;
};

USTRUCT()
struct FModulatorSettingsArray
{
	GENERATED_BODY()

	UPROPERTY()
	FModulatorSettings Array[2];
};

#undef UE_API
