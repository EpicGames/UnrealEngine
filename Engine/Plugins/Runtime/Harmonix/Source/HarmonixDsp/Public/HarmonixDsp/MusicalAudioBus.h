// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixMidi/MidiConstants.h"

#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/AudioBuffer.h"

#include "HAL/CriticalSection.h"

#define UE_API HARMONIXDSP_API

// virtual interface for owners of musical audio buses
class IMusicalAudioBusOwner
{
public:
	virtual ~IMusicalAudioBusOwner() = default;

	virtual void BusWillDestruct(const class FMusicalAudioBus*) = 0;
};

class FMusicalAudioBus
{
public:
	UE_API FMusicalAudioBus();
	UE_API virtual ~FMusicalAudioBus();

	void SetOwner(class IMusicalAudioBusOwner* InOwner) { Owner = InOwner; }

	UE_API virtual void Prepare(float InSampleRateHz, uint32 InNumChannels, uint32 InMaxSamples, bool bInAllocateBuffer = true);
	UE_API virtual void Prepare(float InSampleRateHz, EAudioBufferChannelLayout InChannelLayout, uint32 InMaxSamples, bool bInAllocateBuffer = true);
	UE_API virtual void Prepare(float InSampleRateHz, uint32 InNumChannels, EAudioBufferChannelLayout InChannelLayout, uint32 InMaxSamples, bool bInAllocateBuffer = true);
	UE_API virtual void SetSampleRate(float InSampleRateHz);

	virtual void Process(uint32 InSliceIndex, uint32 InSubsliceIndex, TAudioBuffer<float>& OutBuffer) = 0;

	virtual void LockBus() { BusLock.Lock(); }
	virtual bool TryLockBus() { return BusLock.TryLock(); }
	virtual void UnlockBus() { BusLock.Unlock(); }
	virtual FCriticalSection& GetBusLock() { return BusLock; }

	virtual void TearDown() {};

	virtual bool CanProcessFromWorkerThread() const { return false; }

	double GetSamplesPerSecond() const { return SamplesPerSecond; }
	double GetSecondsPerSample() const { return SecondsPerSample; }
	int32 GetMaxFramesPerProcessCall() const { return MaxSamples; }

	int32 GetNumAudioOutputChannels() const { return NumAudioOutputChannels; }
	EAudioBufferChannelLayout GetChannelLayout() const { return ChannelLayout; }

	float GetAudioLevel() const
	{
		float dB = HarmonixDsp::dBFS(AudioLevel);
		return FMath::Max(dB, HarmonixDsp::kDbSilence);
	}
protected:

	TAudioBuffer<float> BusBuffer;

private:

	float AudioLevel;
	float AudioLevelDecay;

	int32 MaxSamples;
	int32 NumAudioOutputChannels;
	EAudioBufferChannelLayout ChannelLayout;

	double SamplesPerSecond;
	double SecondsPerSample;

	IMusicalAudioBusOwner* Owner;

	FCriticalSection BusLock;
};

#undef UE_API
