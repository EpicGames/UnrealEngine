// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <atomic>

#include "HAL/Platform.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"


namespace Harmonix
{
	/**
	 * A generic pool used to dynamically allocate objects
	 * Objects are stored as SharedPtrs
	 * Objects are returned as SharedPtrs
	 * So resetting the pool won't auto-delete items created by the pool
	 * 
	 * Implemented to be thread safe
	 */
	template<typename TObject>
	class TDynamicObjectPool
	{
	public:

		using FSharedObjectPtr = TSharedPtr<TObject, ESPMode::ThreadSafe>;
		using FWeakObjectPtr = TWeakPtr<TObject, ESPMode::ThreadSafe>;

		/**
		 * Attempts to allocate Num number of objects, given a factory method
		 * Allows the FactoryMethod to return a nullptr, which will get culled from the pool.
		 * This means the number of instances allocated can be less than the number requested. 
		 * 
		 * @return	number of instances successfully created and pooled. Can be fewer than Num requested.
		 */
		FORCEINLINE int32 TryAllocate(int32 Num, TFunction<FSharedObjectPtr()> InFactoryMethod, bool bInAllowGrowth = false)
		{
			checkf(Num > 0, TEXT("Num to allocate must be greater than 0"));

			FScopeLock Lock(&PoolLock);

			checkf(Objects.Num() == 0, TEXT("Objects already allocated. Check \"IsInitialized() == false\" before allocating."));

			bAllowGrowth = bInAllowGrowth;
			Objects.Reset(Num);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				if (FSharedObjectPtr NewObject = InFactoryMethod())
				{
					Objects.Add(NewObject);
				}
			}

			if (bAllowGrowth)
			{
				FactoryMethod = InFactoryMethod;
			}

			return Objects.Num();
		}

		/**
		 * allocates Num number of objects, given a factory method
		 */
		void Allocate(int32 Num, TFunction<FSharedObjectPtr()> InFactoryMethod, bool bInAllowGrowth = false)
		{
			checkf(Num > 0, TEXT("Num to allocate must be greater than 0"));

			FScopeLock Lock(&PoolLock);

			checkf(Objects.Num() == 0, TEXT("Objects already allocated. Check \"IsInitialized() == false\" before allocating."));

			bAllowGrowth = bInAllowGrowth;
			Objects.Reset(Num);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				Objects.Add(InFactoryMethod());
				checkSlow(Objects[Index].IsValid());
			}

			if (bAllowGrowth)
			{
				FactoryMethod = InFactoryMethod;
			}
		}

		/**
		 * Resets the internal storage to no longer track allocated objects
		 * This does not immediately delete the allocated objects
		 * Deletion is managed by the SharedPtrs
		 */
		void Reset()
		{
			FScopeLock Lock(&PoolLock);

			NumInUse = 0;
			bAllowGrowth = false;
			Objects.Reset();
			FactoryMethod.Reset();
		}

		/**
		 * Get a "free" object from the pool, that isn't in use
		 * return value is TSharedPtr, so it'll stay valid even after the pool resets
		 * return null if there is no free instance.
		 */
		FSharedObjectPtr GetFreeShared()
		{
			FScopeLock Lock(&PoolLock);

			// pool should be initialized!
			checkf(Objects.Num() > 0, TEXT("There should be objects in the pool before trying to access the pool"));

			if (NumInUse < Objects.Num())
			{
				int32 FreeIndex = NumInUse;
				++NumInUse;
				return Objects[FreeIndex];
			}
			else if (bAllowGrowth)
			{
				if (FSharedObjectPtr NewObject = FactoryMethod())
				{
					Objects.Add(NewObject);
					++NumInUse;
					return NewObject;
				}
			}

			return nullptr;

		}

		/**
		 * Get a "free" object from the pool, that isn't in use
		 * return value is TWeakPtr, so it's not guaranteed to be valid after the pool resets
		 * return null if there is no free instance.
		 */
		FWeakObjectPtr GetFreeWeak()
		{
			return GetFreeShared();
		}

		void ReturnToPool(TSharedPtr<TObject> Object)
		{
			checkf(Object, TEXT("Must return a valid object to the pool"));
			FScopeLock Lock(&PoolLock);

			int32 ShifterIndex = INDEX_NONE;
			for (int32 Index = 0; Index < NumInUse; ++Index)
			{
				if (Object == Objects[Index])
				{
					ShifterIndex = Index;
					break;
				}
			}

			if (ensureAlwaysMsgf(ShifterIndex != INDEX_NONE, TEXT("Attempting to return object to the pool that wasn't in use or not in this pool")))
			{
				// swap this shifter with the last shifter in use
				// decrement the number in use
				int32 LastUsedIndex = NumInUse - 1;
				--NumInUse;
				Objects.Swap(ShifterIndex, LastUsedIndex);
			}
		}

		bool IsInitialized() const
		{
			FScopeLock Lock(&PoolLock);

			return Objects.Num() > 0;
		}

		bool CanGrow() const
		{
			return bAllowGrowth;
		}

		int32 GetNumAllocated() const
		{
			FScopeLock Lock(&PoolLock);

			return Objects.Num();
		}

		int32 GetNumInUse() const
		{
			FScopeLock Lock(&PoolLock);

			return NumInUse;
		}

		int32 GetNumFree() const
		{
			FScopeLock Lock(&PoolLock);

			return Objects.Num() - NumInUse;
		}

		void ForEachObject(TUniqueFunction<void(FSharedObjectPtr)> DoWork) const
		{
			for (FSharedObjectPtr Obj : Objects)
			{
				DoWork(Obj);
			}
		}

	private:

		mutable FCriticalSection PoolLock;

		int32 NumInUse = 0;
		bool bAllowGrowth = false;
		TArray<FSharedObjectPtr> Objects;
		TFunction<FSharedObjectPtr()> FactoryMethod;
	};
};