// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/AudioBufferConstants.h"
#include "HarmonixDsp/GainMatrix.h"
#include "HarmonixDsp/GainTable.h"
#include "HarmonixDsp/PannerDetails.h"
#include "HarmonixDsp/Ramper.h"

struct HARMONIXDSP_API alignas(16) FPanner
{
public:
	FPanner()
		: AggregatedPanRamper((float)(-UE_PI), (float)UE_PI)
	{
		AggregatedPanRamper.SnapTo(0.0f, false);
		EdgeProximityRamper.SnapTo(1.0f);
		UpdateRampers(true);
	}

	void Reset()
	{
		AggregatedPanRamper.SetMinMax((float)(-UE_PI), (float)UE_PI, false);
		AggregatedPanRamper.SnapTo(0.0f, false);
		EdgeProximityRamper.SnapTo(1.0f);
		UpdateRampers(true);
	}

	void Setup(const FPannerDetails& InPannerDetails, int32 InNumInChannels, int32 InNumOutChannels, EAudioBufferChannelLayout InChannelLayout, uint32 InChannelMask, float InitialGain, const FGainTable* InGainTable = nullptr)
	{
		using namespace HarmonixDsp;
		GainTable = InGainTable;
		if (InPannerDetails.Mode == EPannerMode::DirectAssignment)
		{
			CurrentGainMatrix.Set((uint8)InNumInChannels, (uint8)InNumOutChannels, InChannelLayout, FAudioBuffer::ChannelAssignmentSpeakerToMappedMasks[(uint8)InPannerDetails.Detail.ChannelAssignment], InitialGain);
		}
		else
		{
			CurrentGainMatrix.Set((uint8)InNumInChannels, (uint8)InNumOutChannels, InChannelLayout, InChannelMask, InitialGain);
		}
		SetStartingPan(InPannerDetails, InitialGain, true);
	}

	void SetupSimpleMonoPanner(const FPannerDetails& InPannerDetails, int32 InNumOutChannels, EAudioBufferChannelLayout InChannelLayout, const FGainTable* InGainTable = nullptr)
	{
		GainTable = InGainTable;
		CurrentGainMatrix.Set(1, (uint8)InNumOutChannels, InChannelLayout, ESpeakerMask::UnspecifiedMono, 1.0f);
		SetStartingPan(InPannerDetails, 1.0f, true);
	}

	void SetStartingPan(const FPannerDetails& InPannerDetails, float InGain, bool bInSnap)
	{
		OriginalPannerDetails = InPannerDetails;
		OverallGain = InGain;
		OffsetPan = 0.0f;
		OffsetEdgeProximity = 0.0f;
		float minPan = 0.0f;
		float maxPan = 0.0f;
		InPannerDetails.ToPolarRadiansAndEdgeProximity(StartingPolarPan, minPan, maxPan, StartingEdgeProximity);
		AggregatedPanRamper.SetMinMax(minPan, maxPan, InPannerDetails.Mode != EPannerMode::Stereo && InPannerDetails.Mode != EPannerMode::LegacyStereo);
		UpdateRampers(bInSnap);
		UpdateGainMatrix();
	}

	const FPannerDetails& GetOriginalPannerDetails() const { return OriginalPannerDetails; }

	void SetRampTimeMs(float InNumCallsPerSec, float InRampTimeMs)
	{
		AggregatedPanRamper.SetRampTimeMs(InNumCallsPerSec, InRampTimeMs);
		EdgeProximityRamper.SetRampTimeMs(InNumCallsPerSec, InRampTimeMs);
	};

	void  SetPanOffset_Radians(float rad) 
	{ 
		OffsetPan = rad; UpdatePanRamper(true);  
		UpdateGainMatrix(); 
	}
	void  SetPanOffsetTarget_Radians(float rad) 
	{ 
		OffsetPan = rad; 
		UpdatePanRamper(false); 
	}
	float GetAggregatedPan_Radians() const 
	{ 
		return AggregatedPanRamper.GetCurrent(); 
	}
	void  SetPanOffset_Degrees(float deg) 
	{ 
		OffsetPan = (deg / 180.0f) * (float)UE_PI; 
		UpdatePanRamper(true); 
		UpdateGainMatrix(); 
	}
	void  SetPanOffsetTarget_Degrees(float deg) 
	{ 
		OffsetPan = (deg / 180.0f) * (float)UE_PI; 
		UpdatePanRamper(false); 
	}
	float GetAggregatedPan_Degrees() const 
	{ 
		return (AggregatedPanRamper.GetCurrent() / (float)UE_PI) * 180; 
	}
	void  SetPanOffset_Stereo(float minusOneToOne);
	void  SetPanOffsetTarget_Stereo(float minusOneToOne);
	void  SetEdgeProximityOffset(float ep) 
	{ 
		OffsetEdgeProximity = ep; UpdateEdgeProximityRamper(true); 
		UpdateGainMatrix(); 
	}
	void  SetEdgeProximityOffsetTarget(float ep) 
	{ 
		OffsetEdgeProximity = ep; 
		UpdateEdgeProximityRamper(false); 
	}
	float GetCurrentEdgeProximityOffset() const 
	{ 
		return EdgeProximityRamper.GetCurrent(); 
	}


	void SetOverallGain(float gain) 
	{ 
		if (gain != OverallGain) 
		{ 
			OverallGain = gain; 
			UpdateGainMatrix(); 
		} 
	}
	float GetOverallGain() const { return OverallGain; }

	void Ramp()
	{
		bool somethingChanged = AggregatedPanRamper.Ramp();
		somethingChanged = EdgeProximityRamper.Ramp() || somethingChanged;
		if (somethingChanged)
		{
			UpdateGainMatrix();
		}
	}

	void ConfigureForOutputBuffer(const TAudioBuffer<float>& buff)
	{
		CurrentGainMatrix.ConfigureForOutputBuffer(buff);
	}
	void Configure(int32 inChCount, const TAudioBuffer<float>& buff)
	{
		CurrentGainMatrix.Configure(inChCount, buff);
	}
	const FGainMatrix& GetCurrentGainMatrix() { return CurrentGainMatrix; }

private:

	alignas(16) FGainMatrix CurrentGainMatrix;
	FPannerDetails OriginalPannerDetails;
	float StartingPolarPan = 0.0f;
	float StartingEdgeProximity = 1.0f;
	float OffsetPan = 0.0f;
	float OffsetEdgeProximity = 1.0f;
	float OverallGain = 1.0f;
	TLinearCircularRamper<float> AggregatedPanRamper;
	TLinearRamper<float>         EdgeProximityRamper;
	const FGainTable* GainTable = nullptr;

	void UpdateRampers(bool bInSnap)
	{
		UpdatePanRamper(bInSnap);
		UpdateEdgeProximityRamper(bInSnap);
	}
	void UpdatePanRamper(bool bInSnap);
	void UpdateEdgeProximityRamper(bool bInSnap);
	void UpdateGainMatrix();

};