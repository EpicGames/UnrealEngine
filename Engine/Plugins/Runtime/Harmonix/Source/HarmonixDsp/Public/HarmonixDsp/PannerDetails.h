// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBufferConstants.h"
#include "HarmonixDsp/GainTable.h"
#include "UObject/Class.h"

#include "PannerDetails.generated.h"

#define UE_API HARMONIXDSP_API

class FArchive;


UENUM(BlueprintType)
enum class EPannerMode : uint8
{
	LegacyStereo		UMETA(Json="legacy_stereo"),
	Stereo				UMETA(Json="stereo"),
	Surround			UMETA(Json="surround"),
	PolarSurround		UMETA(Json="polar"),
	DirectAssignment	UMETA(Json="direct_assign"),
	Num					UMETA(Hidden),
	Invalid				UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FPannerDetails
{
	GENERATED_BODY()

public:
	struct FDetail
	{
		union {
			struct { float Pan; float EdgeProximity; };
			ESpeakerChannelAssignment ChannelAssignment;
		};

		void Reset()
		{
			Pan = 0.0f;
			EdgeProximity = 0.0f;
		}
	};

public:

	static const uint8 kVersion = 0;
	UE_API bool Serialize(FArchive& Ar);

	EPannerMode Mode;

	FDetail Detail;

	FPannerDetails() 
		: Mode(EPannerMode::LegacyStereo)
	{
		Detail.Pan = 0.0f;
		Detail.EdgeProximity = 1.0f;
	}
	FPannerDetails(float InPan)
		: Mode(EPannerMode::LegacyStereo)
	{
		Detail.Pan = InPan;
		Detail.EdgeProximity = 0.0f;
	}

	FPannerDetails(ESpeakerChannelAssignment InChannelAssignment)
		: Mode(EPannerMode::DirectAssignment)
	{
		Detail.ChannelAssignment = InChannelAssignment;
	}
	FPannerDetails(float InPan, float InEdgeProximity)
		: Mode(EPannerMode::Surround)
	{
		Detail.Pan = InPan;
		Detail.EdgeProximity = InEdgeProximity;
	}
	FPannerDetails(EPannerMode InMode, float InPan, float InEdgeProximity)
		: Mode(InMode)
	{
		Detail.Pan = InPan;
		Detail.EdgeProximity = InEdgeProximity;
	}

	void Reset()
	{
		Mode = EPannerMode::Stereo;
		Detail.Reset();
	}

	float GetBasicPan()
	{
		if (Mode == EPannerMode::DirectAssignment)
		{
			return 0.0f;
		}
		return Detail.Pan;
	}

	bool operator==(const FPannerDetails& Other) const
	{
		if (Mode != Other.Mode)
		{
			return false;
		}
		else if (Mode == EPannerMode::DirectAssignment)
		{
			return Detail.ChannelAssignment == Other.Detail.ChannelAssignment;
		}
		return Detail.Pan == Other.Detail.Pan && Detail.EdgeProximity == Other.Detail.EdgeProximity;
	}

	bool operator!=(const FPannerDetails& Other) const
	{
		return !(operator==(Other));
	}

	bool IsCircular() const
	{
		return Mode == EPannerMode::PolarSurround
			|| Mode == EPannerMode::Surround
			|| Mode == EPannerMode::DirectAssignment;
	}

	bool IsPannable() const
	{
		return Mode != EPannerMode::DirectAssignment || 
			(  Detail.ChannelAssignment != ESpeakerChannelAssignment::LFE
			&& Detail.ChannelAssignment != ESpeakerChannelAssignment::Center
			&& Detail.ChannelAssignment != ESpeakerChannelAssignment::CenterAndLFE
			&& Detail.ChannelAssignment != ESpeakerChannelAssignment::AmbisonicW);
	}

	void ToPolarRadiansAndEdgeProximity(float& OutRadians, float& OutMinRadians, float& OutMaxRadians, float& OutEdgeProximity) const
	{
		switch (Mode)
		{
		case EPannerMode::LegacyStereo:
		case EPannerMode::Stereo:
		{
			OutEdgeProximity = 1.0f;
			OutMinRadians = -FGainTable::Get().GetDirectChannelAzimuthInCurrentLayout(ESpeakerChannelAssignment::LeftFront);
			OutMaxRadians = -OutMinRadians;
			OutRadians = -((((Detail.Pan + 1.0f) / 2.0f) * (OutMaxRadians - OutMinRadians)) + OutMinRadians);
		}
		break;
		case EPannerMode::Surround:
		{
			OutEdgeProximity = Detail.EdgeProximity;
			OutRadians = Detail.Pan * (float)-UE_PI;
			OutMinRadians = (float)-UE_PI;
			OutMaxRadians = (float)UE_PI;
		}
		break;
		case EPannerMode::PolarSurround:
		{
			OutEdgeProximity = Detail.EdgeProximity;
			OutRadians = (Detail.Pan / 180.0f) * (float)-UE_PI;
			OutMinRadians = (float)-UE_PI;
			OutMaxRadians = (float)UE_PI;
		}
		break;
		case EPannerMode::DirectAssignment:
			OutEdgeProximity = 1.0f;
			OutRadians = FGainTable::Get().GetDirectChannelAzimuthInCurrentLayout(Detail.ChannelAssignment);
			if (OutRadians > UE_PI) OutRadians -= (2 * UE_PI);
			if (OutRadians < -UE_PI) OutRadians += (2 * UE_PI);
			OutMinRadians = (float)-UE_PI;
			OutMaxRadians = (float)UE_PI;
			break;
		case EPannerMode::Num:
		case EPannerMode::Invalid:
		default:
			// Bad Panning Mode!
			checkNoEntry();
			break;
		}
	}
};

template<>
struct TStructOpsTypeTraits<FPannerDetails> : public TStructOpsTypeTraitsBase2<FPannerDetails>
{
	enum
	{
		WithSerializer = true,
	};
};

#undef UE_API
