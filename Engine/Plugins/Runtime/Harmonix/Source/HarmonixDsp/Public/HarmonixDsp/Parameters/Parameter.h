// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "Math/UnrealMathUtility.h"

namespace Harmonix::Dsp::Parameters
{
	// helpers to avoid some compile errors and to optimize bool parameters
	template<typename T>
	struct TIsBoolean { static const bool Value = false; };

	template<>
	struct TIsBoolean<bool> { static const bool Value = true; };

	/**
	 * An atomic config value for something that needs configuring
	 * @tparam T The underlying type of the parameter.
	 */
	template<typename T>
	class TParameter
	{
	public:
		TParameter(T Min, T Max, T Default, T Step = static_cast<T>(0)) :
		Min(FMath::Min(Min, Max)),
		Max(FMath::Max(Min, Max)),
		Default(FMath::Clamp(Default, this->Min, this->Max)),
		Step(Step),
		Value(this->Default)
		{}

		// constructor for simple parameter with min, max, and step = 0;
		// Default value is set to Min
		TParameter(T Min, T Max) : 
		TParameter(Min, Max, Min)
		{}
		
		const T Min;
		const T Max;
		const T Default;
		const T Step;

		void Set(T InValue)
		{
			if constexpr (TIsBoolean<T>::Value)
			{
				Value.store(InValue);
			}
			else
			{
				// clamp
				InValue = FMath::Clamp(InValue, Min, Max);
			
				// snap
				if constexpr (!TIsEnum<T>::Value)
				{
					if (Step > static_cast<T>(0)) {
						const T Delta = InValue - Min;
						InValue = Min + Step * FMath::Floor(Delta / Step + static_cast<T>(0.5));
					}
				}
			
				// assign
				Value.store(InValue);
			}
		}

		T Get() const
		{
			return Value.load();
		}

		// copy constructor
		TParameter(const TParameter& Other)
			: Min(Other.Min), Max(Other.Max), Default(Other.Default), Step(Other.Step)
		{
			Set(Other.Get());
		}

		// assignment operator
		TParameter& operator=(const TParameter& Other)
		{
			Set(Other.Get());
			return *this;
		}

		bool operator==(const TParameter& Other) const
		{
			return Get() == Other.Get();
		}

		bool operator!=(const TParameter& Other) const
		{
			return !(*this == Other);
		}

		bool operator<(const TParameter& Other) const
		{
			return Get() < Other.Get();
		}

		bool operator>(const TParameter& Other) const
		{
			return Get() > Other.Get();
		}

		bool operator<=(const TParameter& Other) const
		{
			return !(*this > Other);
		}

		bool operator>=(const TParameter& Other) const
		{
			return !(*this < Other);
		}

		inline TParameter& operator=(const T& InValue)
		{
			Set(InValue);
			return *this;
		}

		TParameter& operator+=(const T& InValue)
		{
			Set(Get() + InValue);
			return *this;
		}

		TParameter& operator-=(const T& InValue)
		{
			Set(Get() - InValue);
			return *this;
		}
		
		TParameter& operator*=(const T& InValue)
		{
			Set(Get() * InValue);
			return *this;
		}

		TParameter& operator/=(const T& InValue)
		{
			Set(Get() / InValue);
			return *this;
		}

		operator T() const
		{
			return Get();
		}

		bool operator==(const T& InValue) const
		{
			return Get() == InValue;
		}

	private:
		std::atomic<T> Value;
	};

	// Helper to make bool parameters easier to work with
	class FBoolParameter : public TParameter<bool>
	{
	public:
		explicit FBoolParameter(const bool Default) : TParameter(false, true, Default, true)
		{}
	};
}
