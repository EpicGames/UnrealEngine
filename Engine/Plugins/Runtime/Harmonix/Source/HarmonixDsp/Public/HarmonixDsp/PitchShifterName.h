// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

#include "PitchShifterName.generated.h"

// struct with detail customization for drop down selection of registered factories
USTRUCT()
struct FPitchShifterName
{
	GENERATED_BODY()

	FPitchShifterName(const FName& InName) : Name(InName) {}
	FPitchShifterName() : Name(NAME_None) {}

	UPROPERTY(EditAnywhere, Category = "Pitch Shifter")
	FName Name;

	operator FName() const { return Name; }
	FPitchShifterName& operator=(const FName& InName)
	{
		Name = InName;
		return *this;
	}

	bool operator==(const FName& InName) const { return Name == InName; }
	bool operator!=(const FName& InName) const { return Name != InName; }
};
