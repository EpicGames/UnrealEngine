// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/UnrealMath.h"
#include "Misc/AssertionMacros.h"

enum class ERamperMode : uint8
{
	Linear,
	Log,
	Exp
};

class FRamper
{
public:

	typedef void (*FCallback)(void*);
	typedef float Type;

	FRamper(float InNumRampCallsPerSecond = 1.0f)
		: Current(0.0f)
		, NumRampCallsPerSecond(InNumRampCallsPerSecond)
		, Callback(nullptr)
	{
		IsSettingTarget = false;
		SetRampTimeMs(NumRampCallsPerSecond, 1000.0f, ERamperMode::Linear);
		SetTarget(0.0f);
		SnapToTarget();
	}

	void SetRampTimeMs(float InNumRampCallsPerSecond, float InRampTimeMs, ERamperMode InMode)
	{
		NumRampCallsPerSecond = InNumRampCallsPerSecond;
		Mode = InMode;

		NormalIncrement = 1000.0f / (NumRampCallsPerSecond * InRampTimeMs);
	}

	float GetRampTimeMs() const
	{
		return 1000.0f / (NormalIncrement * NumRampCallsPerSecond);
	}

	void SetTarget(Type InTarget, FCallback InCallback = nullptr, void* InData = nullptr)
	{
		switch (Mode)
		{
		case ERamperMode::Linear: 
			SetTargetLinear(InTarget, InCallback, InData); 
			break;
		case ERamperMode::Log:
			SetTargetLog(InTarget, InCallback, InData);
			break;
		case ERamperMode::Exp:
			SetTargetExponential(InTarget, InCallback, InData);
			break;
		default:
			checkNoEntry();
			break;
		}
	}

	void SnapTo(Type InValue)
	{
		SetTarget(InValue);
		SnapToTarget();
	}

	Type GetCurrent() const { return Current; }
	Type GetTarget() const { return Target; }

	void Ramp()
	{
		switch (Mode)
		{
		case ERamperMode::Linear: 
			LinearRamp();
			break;
		case ERamperMode::Log:
			LogRamp();
			break;
		case ERamperMode::Exp:
			ExponentialRamp();
			break;
		default:
			checkNoEntry();
		}
	}

	void SnapToTarget()
	{
		if (IsSettingTarget)
		{
			return;
		}

		// hit target first
		Current = Target;

		// then set the increment (in case we're linear)
		Increment = 0.0f;
		
		// and the multiplier (in case we're log)
		Multiplier = 1.0f;

		if (Callback != nullptr)
		{
			(*Callback)(Data);
		}

		Callback = nullptr;
		Data = nullptr;
	}

private:

	void SetTargetLinear(Type InTarget, FCallback InCallback, void* InData)
	{
		Type DeltaTarget = Target - Current;

		{
			IsSettingTarget = true;
			Increment = DeltaTarget * NormalIncrement;
			Target = InTarget;
			IsSettingTarget = false;
		}

		Callback = InCallback;
		Data = InData;

		check( (DeltaTarget == 0.0f && Increment == 0.0f)
			 || (DeltaTarget <  0.0f && Increment <  0.0f)
			 || (DeltaTarget >  0.0f && Increment >  0.0f));
	}

	void LinearRamp()
	{
		if (Current == Target)
		{
			return;
		}

		Type DeltaTarget = Target - Current;

		if (DeltaTarget < 0.0f)
		{
			// ramp down
			if (Increment <= DeltaTarget)
			{
				SnapToTarget();
				return;
			}
		}
		else
		{
			// ramp up
			if (DeltaTarget <= Increment)
			{
				SnapToTarget();
				return;
			}
		}

		// without some kind of thread-safety,
		// _increment might be zero even though
		// we haven't hit the target yet.
		if (!IsSettingTarget && Increment == 0.0f)
		{
			SnapToTarget();
		}

		Current += Increment;
		if (Current == Target)
		{
			SnapToTarget();
		}
	}

	void SetTargetLog(Type InTarget, FCallback InCallback, void* InData)
	{
		check(Current != 0.0f);
		check(InTarget != 0.0f);
		check((0.0f < Current && 0.0f < InTarget) || (Current < 0.0f && InTarget < 0.0f));

		{
			IsSettingTarget = true;
			Type Ratio = FMath::Log2(Target / Current);
			float RampTimeMs = GetRampTimeMs();
			float RampCallsPerMs = NumRampCallsPerSecond / 1000.0f;
			float NumCalls = RampTimeMs * RampCallsPerMs;
			Multiplier = FMath::Pow(2.0f, Ratio / NumCalls);
			Target = InTarget;
			IsSettingTarget = false;
		}

		Callback = InCallback;
		Data = InData;

		check(Multiplier > 0.0f);
	}

	void LogRamp()
	{
		if (Current == Target)
		{
			return;
		}

		bool bRampingUp = 1.0f < Multiplier && 0.0f < Current;

		float NextValue = Current * Multiplier;

		if (bRampingUp)
		{
			if (Target < NextValue)
			{
				SnapToTarget();
				return;
			}
		}
		else
		{
			if (Target > NextValue)
			{
				SnapToTarget();
				return;
			}
		}

		// without some kind of thread-safety,
		// _multiplier might be 1.0f even though
		// we haven't hit the target yet.
		if (!IsSettingTarget && Multiplier == 1.0f)
		{
			SnapToTarget();
		}

		Current = NextValue;
		if (Current == Target)
		{
			SnapToTarget();
		}
	}

	void SetTargetExponential(float InTarget, FCallback InCallback = nullptr, void* InData = nullptr)
	{
		// Log(0) is not defined. Make sure values ar non zero
		if (Current == 0.0f)
		{
			// KINDA_SMALL_NUMBER = 0.0001f; ?
			Current = 0.001f;
		}

		if (InTarget == 0.0f)
		{
			InTarget = 0.001f;
		}

		float LogMultiplier = 1.0f + (FMath::Loge(InTarget) - FMath::Loge(Current)) * (NormalIncrement);

		{
			IsSettingTarget = true;
			Multiplier = LogMultiplier;
			Target = InTarget;
			IsSettingTarget = false;
		}

		Callback = InCallback;
		Data = InData;
	}

	void ExponentialRamp()
	{
		if (Current == Target)
		{
			return;
		}

		if (Current <= 0.0f)
		{
			Current = 0.001f;
		}
		
		bool bRampingUp = 1.0f < Multiplier;

		float NextValue = Current * Multiplier;

		if (bRampingUp)
		{
			if (Target < NextValue)
			{
				SnapToTarget();
				return;
			}
		}
		else
		{
			if (NextValue < Target)
			{
				SnapToTarget();
				return;
			}
		}

		if (!IsSettingTarget && Multiplier == 1.0f)
		{
			SnapToTarget();
		}

		Current = NextValue;
		if (Current == Target)
		{
			SnapToTarget();
		}
	}

private:

	Type Current;
	Type Target;
	ERamperMode Mode;

	Type Increment;
	Type NormalIncrement;

	Type Multiplier;

	float NumRampCallsPerSecond;
	volatile bool IsSettingTarget;

	FCallback Callback;
	void* Data;
};

namespace Harmonix::Dsp::Ramper::Tests
{
	class FTestRamper;
}

template<typename T>
class TRamper
{
public:
	friend class Harmonix::Dsp::Ramper::Tests::FTestRamper;

	TRamper(float InNumRampCallsPerSecond = 1.0f)
		: ProgressPct(1.0f)
		, NumRampCallsPerSecond(InNumRampCallsPerSecond)
		, IsSettingTarget(false)
		, Callback(nullptr)
	{
		InitData();
		SetRampTimeMs(NumRampCallsPerSecond, 1000.0f);
	}

	void SetRampTimeMs(float InRampTimeMs)
	{
		if (InRampTimeMs == 0.0f)
		{
			NormalIncrement = 1.0f;
			return;
		}

		// this means an increment should be done in a call
		NormalIncrement = 1000.0f / (NumRampCallsPerSecond * InRampTimeMs);
	}

	void SetRampTimeMs(float InNumRampCallsPerSecond, float InRampTimeMs)
	{
		NumRampCallsPerSecond = InNumRampCallsPerSecond;
		SetRampTimeMs(InRampTimeMs);
	}

	float GetRampTimeMs() const
	{
		if (NormalIncrement >= 1.0f)
		{
			return 0.0f;
		}

		return 1000.0f / (NormalIncrement * NumRampCallsPerSecond);
	}

	const T& GetCurrent() const { return CurrentValue; }
	const T& GetTarget() const { return TargetValue; }

	operator T() const { return CurrentValue; }

	bool IsAtTarget() const
	{
		return ProgressPct == 1.0f;
	}

	float GetProgressPct() const
	{
		return ProgressPct;
	}

	void SnapToTarget()
	{
		if (IsSettingTarget)
		{
			return;
		}

		CurrentValue = TargetValue;

		// cache off our callback so we guarantee they won't be cleared
		FRamper::FCallback CacheCallback = Callback;
		void* CacheData = Data;

		ProgressPct = 1.0f;

		if (CacheCallback)
		{
			(*CacheCallback)(Data);
		}

		Callback = nullptr;
		Data = nullptr;
	}

protected:

	void InitData() {}

	T CurrentValue;
	T TargetValue;
	T Increment;
	// the increment to use if the difference between the target and current value is 1.0f
	float NormalIncrement;
	// in range [0.0f, 1.0f]
	float ProgressPct;

	float NumRampCallsPerSecond;
	volatile bool IsSettingTarget;

	// wonder if I should use an event here instead :thinking:
	FRamper::FCallback Callback;
	void* Data;
};

template<>
FORCEINLINE void TRamper<float>::InitData() { CurrentValue = 0.0f; TargetValue = 0.0f; Increment = 0.0f; }
template<> 
FORCEINLINE void TRamper<int32>::InitData() { CurrentValue = 0; TargetValue = 0; Increment = 0; }
template<>
FORCEINLINE void TRamper<int64>::InitData() { CurrentValue = 0; TargetValue = 0; Increment = 0; }
template<>
FORCEINLINE void TRamper<uint32>::InitData() { CurrentValue = 0; TargetValue = 0; Increment = 0; }
template<>
FORCEINLINE void TRamper<uint16>::InitData() { CurrentValue = 0; TargetValue = 0; Increment = 0; }

template<typename T>
class TLinearRamper : public TRamper<T>
{
public:

	explicit TLinearRamper(float InNumRampCallsPerSecond = 1.0f)
		: TRamper<T>(InNumRampCallsPerSecond)
	{}

	void SetTarget(T InTarget, FRamper::FCallback InCallback = nullptr, void* InData = nullptr)
	{
		// write the diff like this to restrict the operator overloading
		// that we require from the template type to
		//     operator+(Type)
		// and
		//     operator*(float)
		T DeltaTarget = InTarget + (this->CurrentValue * (-1));
		T NewIncrement = DeltaTarget * (this->NormalIncrement);

		{
			this->IsSettingTarget = true;
			this->Increment = NewIncrement;
			this->TargetValue = InTarget;
			this->ProgressPct = 0.0f;
			this->IsSettingTarget = false;
		}

		this->Callback = InCallback;
		this->Data = InData;
	}

	bool Ramp()
	{
		if (this->ProgressPct == 1.0f)
		{
			return false;
		}

		this->ProgressPct = this->ProgressPct + this->NormalIncrement;
		this->CurrentValue = this->CurrentValue + this->Increment;

		if (this->ProgressPct >= 1.0f)
		{
			this->SnapToTarget();
		}

		return true;
	}

	void SnapTo(const T& InValue)
	{
		this->SetTarget(InValue);
		this->SnapToTarget();
	}

	TLinearRamper<T>& operator=(const T InValue)
	{
		SetTarget(InValue);
		return *this;
	}
};

template<typename T>
class TLinearCircularRamper : public TRamper<T>
{
public:
	TLinearCircularRamper(T InMin, T InMax, float InNumRampCallsPerSecond = 1.0f)
		: TRamper<T>(InNumRampCallsPerSecond)
	{
		Min = InMin;
		Max = InMax;
	}
	
	void SetMinMax(T InMin, T InMax, bool bCircular)
	{
		Min = InMin;
		Max = InMax;
		this->SnapTo(this->CurrentValue, bCircular);
	}

	void SetTarget(T InTarget, bool bCircular, FRamper::FCallback InCallback = nullptr, void* InData = nullptr)
	{
		if (InTarget == this->TargetValue)
		{
			return;
		}

		T DeltaTarget;
		if (bCircular)
		{
			while (InTarget > Max)
			{
				InTarget -= (Max - Min);
			}
			while (InTarget < Min)
			{
				InTarget += (Max - Min);
			}
			if (InTarget == this->TargetValue)
			{
				return;
			}
			T Forward;
			T Backward;
			if (InTarget > this->CurrentValue)
			{
				Forward = InTarget - this->CurrentValue;
				Backward = (this->CurrentValue - Min) + (Max - this->TargetValue);
			}
			else
			{
				Forward = (InTarget - Min) + (Max - this->CurrentValue);
				Backward = this->CurrentValue - InTarget;
			}

			DeltaTarget = (Backward < Forward) ? -Backward : Forward;
		}
		else
		{
			InTarget = FMath::Clamp(InTarget, Min, Max);
			if (InTarget == this->TargetValue)
			{
				return;
			}
			DeltaTarget = InTarget + (this->CurrentValue * (-1));
		}

		T NewIncrement = DeltaTarget * this->NormalIncrement;

		{
			this->IsSettingTarget = true;
			this->Increment = NewIncrement;
			this->TargetValue = InTarget;
			this->ProgressPct = 0.0f;
			this->IsSettingTarget = false;
		}

		this->Callback = InCallback;
		this->Data = InData;
	}

	bool Ramp()
	{
		if (this->ProgressPct == 1.0f)
		{
			return false;
		}

		this->ProgressPct = this->ProgressPct + this->NormalIncrement;
		this->CurrentValue = this->CurrentValue + this->Increment;

		if (this->CurrentValue < Min)
		{
			this->CurrentValue += (Max - Min);
		}

		if (this->CurrentValue >= Max)
		{
			this->CurrentValue = Min + (this->CurrentValue - Max);
		}

		if (this->ProgressPct >= 1.0f)
		{
			this->SnapToTarget();
		}

		return true;
	}

	void SnapTo(T InValue, bool Circular)
	{
		if (Circular)
		{
			while (InValue > Max)
			{
				InValue -= (Max - Min);
			}
			while (InValue < Min)
			{
				InValue += (Max - Min);
			}
		}
		else
		{
			InValue = FMath::Clamp(InValue, Min, Max);
		}

		this->TargetValue = InValue;
		this->SnapToTarget();
	}

	T GetMin() const { return Min; }
	T GetMax() const { return Max; }

private:
	T Min;
	T Max;
};

template <typename T>
class TTimeBasedRamper
{
public:

	TTimeBasedRamper()
		: Callback(nullptr)
	{
		ProgressPct = 1.0f;
		IsSettingTarget = false;
		SetRampTimeMs(1000.0f);
	}

	void SetRampTimeMs(float InRampTimeMs)
	{
		if (InRampTimeMs == 0.0f)
		{
			SnapToTarget();
			return;
		}
		DurationMs = InRampTimeMs;
	}

	float GetRampTimeMs() const
	{
		return DurationMs;
	}

	void SetTarget(T InTarget, FRamper::FCallback InCallback = nullptr, void* InData = nullptr)
	{
		SetTragetLinear(InTarget, InCallback, InData);
	}

	void SnapTo(T InValue)
	{
		SetTarget(InValue);
		SnapToTarget();
	}

	const T& GetCurrent() const { return CurrentValue; }
	const T& GetTarget() const { return TargetValue; }

	bool Ramp(float InMs)
	{
		return LinearRamp(InMs);
	}

	bool IsAtTarget() const
	{
		return ProgressPct == 1.0f;
	}

	float GetProgressPct() const
	{
		return ProgressPct;
	}

	void SnapToTarget()
	{
		if (IsSettingTarget)
		{
			return;
		}

		CurrentValue = TargetValue;

		FRamper::FCallback CacheCallback = Callback;
		void* CacheData = Data;

		ProgressPct = 1.0f;

		if (CacheCallback != nullptr)
		{
			(*CacheCallback)(CacheData);
		}

		Callback = nullptr;
		Data = nullptr;
	}
private:

	void SetTargetLinear(T InTarget, FRamper::FCallback InCallback, void* InData)
	{

		{
			IsSettingTarget = true;
			StartValue = CurrentValue;
			TargetValue = InTarget;
			ProgressPct = 0.0f;
			PassedMs = 0.0f;
			IsSettingTarget = false;
		}

		Callback = InCallback;
		Data = InData;
	}

	bool LinearRamp(float InMs)
	{
		if (ProgressPct == 1.0f)
		{
			return false;
		}

		PassedMs += InMs;
		ProgressPct = PassedMs / DurationMs;

		if (ProgressPct >= 1.0f)
		{
			SnapToTarget();
			ProgressPct = 1.0f;
		}
		else
		{
			CurrentValue = StartValue + ((TargetValue - StartValue) * ProgressPct);
		}
		return true;
	}

private:

	T StartValue;
	T CurrentValue;
	T TargetValue;
	float DurationMs;
	float ProgressPct;
	float PassedMs;

	volatile bool IsSettingTarget;

	FRamper::FCallback Callback;
	void* Data;
};
