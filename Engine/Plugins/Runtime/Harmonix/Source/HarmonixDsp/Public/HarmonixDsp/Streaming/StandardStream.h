// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/AudioBufferConstants.h"
#include "HarmonixDsp/PannerDetails.h"

//#include "HarmonixDsp/Streaming/HarmonixStream.h"

#include "StandardStream.generated.h"

USTRUCT(BlueprintType)
struct FStreamingChannelParams
{
	GENERATED_BODY()

public:
	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	int32 StreamIndex;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	float Gain;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FPannerDetails Pan;


	FStreamingChannelParams()
		: StreamIndex(-1)
		, Gain(1.0f)
	{}
	FStreamingChannelParams(int32 InIndex, float InGain, float InPan)
		: StreamIndex(InIndex)
		, Gain(InGain)
		, Pan(InPan)
	{}
	FStreamingChannelParams(int32 InIndex, float InGain, ESpeakerChannelAssignment ChannelAssignment)
		: StreamIndex(InIndex)
		, Gain(InGain)
		, Pan(ChannelAssignment)
	{}
	FStreamingChannelParams(int32 InIndex, float InGain, float InPan, float InEdgeProximity)
		: StreamIndex(InIndex)
		, Gain(InGain)
		, Pan(InPan, InEdgeProximity)
	{}
};

USTRUCT()
struct FStreamingTrackParams
{
	GENERATED_BODY()

	float Pan;
	float SlipSpeed;
	bool  Slipped;
	float CurrentSlipOffsetMs;
	float PushSlipOffset;
	int32   ChannelReadOffset;

	float Gain;
	float GainRampMs;
	bool  Muted;

	FStreamingTrackParams() { Reset(); }

	TArray<FStreamingChannelParams> ChannelParams;
	// TArray<FStreamChannelReceiver*> mChannelReceviers;

	void Reset()
	{
		Pan = 0.0f;
		SlipSpeed = 1.0f;
		Slipped = false;
		CurrentSlipOffsetMs = 0.0f;
		PushSlipOffset = 0.0f;
		ChannelReadOffset = 0;
		Gain = 1.0f;
		GainRampMs = 10.0f;
		Muted = false;
		ChannelParams.Reset();
		//ChannelReceivers.Empty();
	}
};

//USTRUCT()
//class FStandardStream : FHarmonixStream
//{
//public:
//
//	FStandardStream(FStreamReader* InStreamReader, float InStartMs, float InBufSecs);
//
//	FStandardStream();
//};