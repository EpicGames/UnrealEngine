// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixDsp/PannerDetails.h"
#include "HarmonixDsp/Streaming/StandardStream.h"

#include "TrackChannelInfo.generated.h"

#define UE_API HARMONIXDSP_API

USTRUCT(BlueprintType)
struct FStreamingChannelParamsArray
{
	GENERATED_BODY()

public:

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	TArray<FStreamingChannelParams> ChannelParams;

	FStreamingChannelParamsArray() {};
	FStreamingChannelParamsArray(const TArray<FStreamingChannelParams>& InChannelParams)
		: ChannelParams(InChannelParams)
	{
	}

	FStreamingChannelParams& operator[](int32 Index) { return ChannelParams[Index]; }
	const FStreamingChannelParams& operator[](int32 Index) const { return ChannelParams[Index]; }
};

USTRUCT(BlueprintType)
struct FTrackChannelInfo
{
	GENERATED_BODY()

public:

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = -1, ClampMin = -1))
	int32 RealTrackIndex = -1;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FName Name;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FName Routing;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	TArray<FStreamingChannelParams> Channels;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	TMap<FName, FStreamingChannelParamsArray> PresetChannels;

	UE_API FTrackChannelInfo();
	UE_API FTrackChannelInfo(int32 InRealIndex);
	UE_API FTrackChannelInfo(FTrackChannelInfo&& Other);
	UE_API FTrackChannelInfo(const FTrackChannelInfo& Other);

	FTrackChannelInfo& operator=(const FTrackChannelInfo& Other)
	{
		RealTrackIndex = Other.RealTrackIndex;
		Name = Other.Name;
		Routing = Other.Routing;
		Channels = Other.Channels;
		PresetChannels = Other.PresetChannels;
		return *this;
	}

	UE_API int32  GetNumChannels() const;
	UE_API int32  GetStreamChannelIndex(int32 ChannelIndex) const;
	UE_API bool GetStreamIndexesPan(int32 StreamIndex, FPannerDetails& OutPan) const;
	UE_API bool GetStreamIndexesGain(int32 StreamIndex, float& OutGain) const;
	UE_API bool GetStreamIndexesPresetGain(FName PresetName, int32 StreamIndex, float& OutGain, bool DefaultIfNotFound = true) const;
	UE_API bool SetStreamIndexesPan(int32 StreamIndex, float Pan);
	UE_API bool SetStreamIndexesPan(int32 StreamIndex, const FPannerDetails& InPan);
	UE_API bool SetStreamIndexesGain(int32 StreamIndex, float InGain);
	UE_API bool SetStreamIndexesPresetGain(FName presetName, int32 StreamIndex, float InGain);

private:

	UE_API const FStreamingChannelParams* FindStreamingChannelParams(int32 StreamIndex) const;
	UE_API FStreamingChannelParams* FindStreamingChannelParams(int32 StreamIndex);
};

#undef UE_API
