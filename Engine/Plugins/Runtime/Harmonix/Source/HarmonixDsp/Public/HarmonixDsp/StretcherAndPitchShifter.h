// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/PitchShifterName.h"
#include "HarmonixDsp/Containers/TypedParameter.h"
#include "Sound/SoundWave.h"

#include "StretcherAndPitchShifter.generated.h"

class IAudioDataRenderer;
class FGainMatrix;
class FSoundWaveProxy;

USTRUCT()
struct FTimeStretchConfig
{
	GENERATED_BODY()

public:

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool bMaintainTime = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	FPitchShifterName PitchShifter;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	TMap<FName, FTypedParameter> PitchShifterOptions;

	UPROPERTY(EditDefaultsOnly, Category = "Settings")
	bool bSyncTempo = false;

	UPROPERTY(EditDefaultsOnly, Category = "Settings", Meta = (UIMin = 1, UIMax = 240, ClampMin = 1, ClampMax = 240))
	float OriginalTempo = 120.0f;
};

class IStretcherAndPitchShifter
{
public:
	IStretcherAndPitchShifter(FName InFactoryName) : MyFactoryName(InFactoryName) {}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~IStretcherAndPitchShifter() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FName GetFactoryName() const { return MyFactoryName; };

	virtual void UpdateOptions(const TMap<FName, FTypedParameter>& Options) {}

	virtual int32  GetInputFramesNeeded(int32 NumOutFramesNeeded, float PitchShift, float SpeedShift) = 0;
	virtual bool HandlesSpeedCorrection() const { return false; }
	virtual void SetSampleRateAndReset(float InSampleRate) {}
	virtual int32  GetLatencySamples() const { return 0; }
	virtual bool HasCurrentSampleFrame() const { return false; }
	virtual int32  GetCurrentSampleFrame() const { return 0; }


	void SetSampleSourceReset(const TSharedRef<const FSoundWaveData> InSoundWaveData, TSharedPtr<IAudioDataRenderer, ESPMode::ThreadSafe> InAudioRenderer, const FTimeStretchConfig* ShifterSettings)
	{ 
		MySoundWaveData = InSoundWaveData;

		MyAudioRenderer = InAudioRenderer;

		SetupAndResetImpl(ShifterSettings);
	}

	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData> overload instead")
	void SetSampleSourceReset(TSharedPtr<FSoundWaveProxy> InSoundWave, TSharedPtr<IAudioDataRenderer, ESPMode::ThreadSafe> InAudioRenderer, const FTimeStretchConfig* ShifterSettings)
	{
		if (InSoundWave.IsValid())
		{
			SetSampleSourceReset(InSoundWave->GetSoundWaveDataRef(), InAudioRenderer, ShifterSettings);

			return;
		}

		MySoundWaveData.Reset();
		MyAudioRenderer = InAudioRenderer;

		SetupAndResetImpl(ShifterSettings);
	}

	virtual double Render(TAudioBuffer<float>& OutputData, double Pos, int32 MaxFrame, double ResampleInc, double PitchShift, double Speed, bool MaintainPitchWhenSpeedChanges, const FGainMatrix& Gain) = 0;

	virtual void StereoPitchShift(float InPitchShift, int32 NumSampsToProcess, float* InLeftData, float* InRightData, float* OutLeftData, float* OutRightData) = 0;

	virtual bool TakeInput(TAudioBuffer<float>& InBuffer) = 0;
	virtual bool InputSilence(int32 NumFrames) = 0;
	virtual void StereoPitchShift(float InPitchShift, int32 NumOutputFrame, float* OutLeftData, float* OutRightData) = 0;
	virtual void PitchShift(float Pitch, float Speed, TAudioBuffer<float>& OutBuffer) = 0;

	virtual size_t GetMemoryUsage() const = 0;

	virtual void Cleanup() = 0;
protected:
	TSharedPtr<const FSoundWaveData> MySoundWaveData;
	
	UE_DEPRECATED(5.8, "Please use MySoundWaveData instead")
	TSharedPtr<FSoundWaveProxy> MyAudioData;
	
	TSharedPtr<IAudioDataRenderer, ESPMode::ThreadSafe> MyAudioRenderer;

	virtual void SetupAndResetImpl(const FTimeStretchConfig* ShifterSettings) { }

private:

	const FName MyFactoryName;
};

