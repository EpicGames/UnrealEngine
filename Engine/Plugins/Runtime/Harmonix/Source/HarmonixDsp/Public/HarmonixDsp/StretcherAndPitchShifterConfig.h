// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StretcherAndPitchShifterConfig.generated.h"

// base class for all Stretcher and Pitch Shifter config settings
// allows configuration to appear in the "Pitch Shifter Settings"
// along with any other pitch shifter settings
UCLASS(MinimalAPI, Abstract, EditInlineNew, CollapseCategories, Config = Engine, defaultconfig)
class UStretcherAndPitchShifterConfig : public UObject
{
	GENERATED_BODY()
};
