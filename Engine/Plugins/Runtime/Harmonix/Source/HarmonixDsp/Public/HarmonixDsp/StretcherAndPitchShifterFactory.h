// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioStreamCacheMemoryHandle.h"
#include "Features/IModularFeatures.h"
#include "HAL/CriticalSection.h"

#define UE_API HARMONIXDSP_API

class IStretcherAndPitchShifter;
struct FTimeStretchConfig;
using IFeatureClient = void;
class FAudioStreamCacheMemoryHandle;

class IStretcherAndPitchShifterFactory : public IModularFeature
{
protected:

	UE_API IStretcherAndPitchShifterFactory();

public:

	UE_API virtual ~IStretcherAndPitchShifterFactory();

	/**
	 * Gets the name of this factory type. This will uniquely identity itself, so it can be found by the FindFactory call below
	 * @return Name of factory
	 */
	virtual const TArray<FName>& GetFactoryNames() const = 0;

	/**
	 * Update the pitch shifters formant and volume correction settings for this factory
	 * TODO: Need to rethink how we expose pitch shifter settings
	 */
	virtual void SetFormantVolumeCorrection(float DBPerHalfStepUp, float DBPerHalfStepDown, float DBMaxUp, float DBMaxDown) {};

	/**
	 * The factory has the ability to pool pitch shifters
	 * @return Instance of an available pitch shifter
	 */
	virtual TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe> GetFreePitchShifter(const FTimeStretchConfig& InConfig) = 0;

	/**
	 * The factory has the ability to pool pitch shifters.
	 * Make sure to call this method with a pitch shifter you retrieved from GetFreePitchShifter
	 */
	virtual void ReleasePitchShifter(TSharedPtr<IStretcherAndPitchShifter, ESPMode::ThreadSafe>) = 0;

	/**
	 * The factory has the ability to pool pitch shifters.
	 * This let's you know if GetFreePitchShifter will return a valid pitch shifter 
	 */
	virtual bool HasFreePitchShifters(const FTimeStretchConfig& InConfig) const = 0;

	/** 
	 * Should return number of bytes used by all of the shifters in the pool.
	 */
	virtual size_t GetMemoryUsedByPool() const = 0;

protected:

	/**
	 * Override this method to do custom setup when a client is added
	 * Clients are added statically and managed by the IStretcherAndPitchShifterFactory
	 */
	virtual void OnClientAdded(IFeatureClient* Client, float SampleRate) = 0;

	/**
	 * Override this method to do custom teardown when a client is removed
	 * Clients are added statically and managed by the IStretcherAndPitchShifterFactory
	 */
	virtual void OnClientRemoved(IFeatureClient* Client) = 0;

public:

	/**
	 * Get the name of all StretcherAndPitchShifter factories in the Modular Features registry.
	 * @return "StretcherAndPitchShifter Factory"
	 */
	static UE_API FName GetModularFeatureName();

	/**
	* Statically adds a client to the factory module
	* (if it hasn't already beed added)
	* lets each instance of the factory module know about the client
	* allowing them to do any necessary setup 
	*/
	static UE_API void AddClient(IFeatureClient* Client, float SampleRate);

	/**
	* Statically removes a client from the factory module
	* (if it hasn't already been removed)
	* lets each instance know about the removed client
	* allowing them to do any shut down if necessary
	*/
	static UE_API void RemoveClient(IFeatureClient* Client);

	/**
	* @return whether the in "Client" has been added as a client to this factory module
	*/
	static UE_API bool HasClient(IFeatureClient* Client);
	
	/**
	* @return the number of unique clients added to the factory module
	*/
	static UE_API int32 GetNumClients();

	/**
	 * Gets all registered factory instances.
	 * @return Array of all factories.
	 */
	static UE_API TArray<IStretcherAndPitchShifterFactory*> GetAllRegisteredFactories();


	/**
	 * Gets all registered factory names
	 * @return Array of all factory names
	 */
	static UE_API TArray<FName> GetAllRegisteredFactoryNames();

	/**
	 * Finds a factory with available pitch shifters
	 * @return IStretcherAndPitchShifterFactory* instance
	 */
	static UE_API IStretcherAndPitchShifterFactory* FindBestFactory(const FTimeStretchConfig& InConfig);

	/**
	 * Returns the instance of the factory with the registered name
	 * null if a factory has not been registerred with the given name
	 * @return IStretcherAndPitchShifterFactory* instance
	 */
	static UE_API IStretcherAndPitchShifterFactory* FindFactory(const FName InFactoryName);

private:

	static UE_API void ResetMemoryUsageBytes(uint64 MemoryUsageBytes);

	static UE_API FCriticalSection ClientLock;

	static UE_API TArray<IFeatureClient*> Clients;

	static UE_API size_t TotalMemoryUsed;

	static UE_API FAudioStreamCacheMemoryHandle MemoryHandle;
};

#undef UE_API
