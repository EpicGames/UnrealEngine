// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDsp/PitchShifterName.h"
#include "Harmonix/HarmonixDeveloperSettings.h"

#include "StretcherAndPitchShifterFactoryConfig.generated.h"

#define UE_API HARMONIXDSP_API

USTRUCT()
struct FPitchShifterNameRedirect
{
	GENERATED_BODY()

	FPitchShifterNameRedirect() 
	{}
	
	FPitchShifterNameRedirect(FName OldName, FName NewName)
		: OldName(OldName), NewName(NewName) 
	{}

	UPROPERTY(EditDefaultsOnly, Category = "Factory")
	FName OldName;

	UPROPERTY(EditDefaultsOnly, Category = "Factory")
	FName NewName;
};

UCLASS(MinimalAPI, config = Engine, defaultconfig, meta = (DisplayName = "Pitch Shifter Settings"))
class UStretcherAndPitchShifterFactoryConfig : public UHarmonixDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UStretcherAndPitchShifterFactoryConfig();

	UPROPERTY(config, EditDefaultsOnly, Category = "Factory")
	TArray<FName> FactoryPriority;

	UPROPERTY(config, EditDefaultsOnly, Category = "Factory")
	FPitchShifterName DefaultFactory;

	UE_API void AddFactoryNameRedirect(FName OldName, FName NewName);

	UE_API const FPitchShifterNameRedirect* FindFactoryNameRedirect(FName OldName) const;

public:

#if WITH_EDITORONLY_DATA

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif

	UE_API virtual void PostInitProperties() override;

private:

	void PruneDuplicatesAndAddMissingNames(TArray<FName>& InOutNames);

	// visible in editor, but not editable
	UPROPERTY(config, EditDefaultsOnly, Category = "Factory", Meta=(EditCondition="false"))
	TArray<FPitchShifterNameRedirect> FactoryNameRedirects;
};

#undef UE_API
