// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

template<typename DataType, int32 Stride>
class TFixedStridePtr
{
public:
	TFixedStridePtr() : MyPointer(nullptr) {}
	TFixedStridePtr(const TFixedStridePtr& Other) 
		: MyPointer(Other.MyPointer) 
	{}

	TFixedStridePtr& operator =(DataType* InPtr) 
	{ 
		MyPointer = InPtr; 
		return *this; 
	}

	TFixedStridePtr& operator =(const TFixedStridePtr& Other) 
	{ 
		MyPointer = Other.MyPointer; 
		return *this; 
	}

	operator DataType* () 
	{ 
		return MyPointer; 
	}

	DataType* operator +(int32 Offset) 
	{ 
		return MyPointer + (Offset * Stride); 
	}

	DataType* operator -(int32 Offset) 
	{ 
		return MyPointer - (Offset * Stride); 
	}

	TFixedStridePtr& operator +=(int32 Offset) 
	{ 
		MyPointer += (Offset * Stride); 
		return *this; 
	}

	TFixedStridePtr& operator -=(int32 Offset) 
	{ 
		MyPointer -= (Offset * Stride); 
		return *this; 
	}

	// prefix...
	TFixedStridePtr& operator ++() 
	{ 
		MyPointer += Stride; 
		return *this; 
	}

	// postfix...
	DataType* operator ++(int32) 
	{ 
		DataType* OrigPtr = MyPointer; 
		MyPointer += Stride; 
		return OrigPtr; 
	}

	// prefix...
	TFixedStridePtr& operator --() 
	{ 
		MyPointer -= Stride; 
		return *this; 
	}

	// postfix...
	DataType* operator --(int32) 
	{ 
		DataType* OrigPtr = MyPointer; 
		MyPointer -= Stride; 
		return OrigPtr; 
	}

	bool operator ==(const TFixedStridePtr& Other) 
	{ 
		return Other.MyPointer == MyPointer; 
	}

	bool operator ==(const DataType* Other) 
	{ 
		return Other == MyPointer; 
	}

	bool operator !=(const TFixedStridePtr& Other) 
	{ 
		return Other.MyPointer != MyPointer; 
	}

	bool operator !=(const DataType* Other) 
	{ 
		return Other != MyPointer; 
	}

	DataType& operator [](int32 Other) 
	{ 
		return *(MyPointer + (Other * Stride)); 
	}

	DataType& operator *() 
	{ 
		return *MyPointer; 
	}

private:
	DataType* MyPointer;
};

typedef TFixedStridePtr<float, 2> FInterleavedStereoFloatChannelPtr;
typedef TFixedStridePtr<float, 6> FInterleaved51FloatChannelPtr;
typedef TFixedStridePtr<float, 8> FInterleaved71FloatChannelPtr;

template <typename DataType>
class TDynamicStridePtr
{
public:
	TDynamicStridePtr()
		: Stride(0)
		, MyPointer(nullptr) 
	{}

	TDynamicStridePtr(DataType* InPointer, int32 InStride) 
		: Stride(InStride)
		, MyPointer(InPointer) 
	{}

	TDynamicStridePtr(int32 InStride) 
		: Stride(InStride)
		, MyPointer(nullptr) 
	{}
	
	TDynamicStridePtr(const TDynamicStridePtr& Other) 
		: Stride(Other.Stride)
		, MyPointer(Other.MyPointer) 
	{}

	void Init(const TDynamicStridePtr& Other) 
	{ 
		Stride = Other.Stride; 
		MyPointer = Other.MyPointer; 
	}

	TDynamicStridePtr& operator =(DataType* InPtr)
	{ 
		MyPointer = InPtr;
		return *this;
	}
	
	TDynamicStridePtr& operator =(const TDynamicStridePtr& Other) 
	{ 
		MyPointer = Other.MyPointer; 
		Stride = Other.Stride; 
		return *this; 
	}
	
	operator DataType* () 
	{ 
		return MyPointer; 
	}
	
	DataType* operator +(int32 Offset) 
	{ 
		return MyPointer + (Offset * Stride); 
	}
	
	DataType* operator -(int32 Offset) 
	{ 
		return MyPointer - (Offset * Stride); 
	}

	TDynamicStridePtr& operator +=(int32 Offset) 
	{ 
		MyPointer += (Offset * Stride); 
		return *this; 
	}
	
	TDynamicStridePtr& operator -=(int32 Offset) 
	{ 
		MyPointer -= (Offset * Stride); 
		return *this; 
	}

	// prefix...
	TDynamicStridePtr& operator ++() 
	{ 
		MyPointer += Stride; 
		return *this; 
	}
	
	// postfix...
	DataType* operator ++(int32) 
	{ 
		DataType* orig = MyPointer; 
		MyPointer += Stride; 
		return orig; 
	}

	// prefix...
	TDynamicStridePtr& operator --() 
	{ 
		MyPointer -= Stride; 
		return *this; 
	}
	
	// postfix...
	DataType* operator --(int32) 
	{ 
		DataType* orig = MyPointer; 
		MyPointer -= Stride; 
		return orig; 
	}

	bool operator ==(const TDynamicStridePtr& Other) const
	{ 
		return Other.MyPointer == MyPointer; 
	}
	
	bool operator ==(const DataType* Other) const
	{ 
		return Other == MyPointer; 
	}

	bool operator !=(const TDynamicStridePtr& Other) const
	{ 
		return Other.MyPointer != MyPointer; 
	}
	
	bool operator !=(const DataType* Other) const
	{ 
		return Other != MyPointer;
	}

	const DataType& operator [](int32 Offset) const
	{ 
		return *(MyPointer + (Offset * Stride)); 
	}

	DataType& operator [](int32 Offset)
	{ 
		return *(MyPointer + (Offset * Stride)); 
	}
	
	DataType& operator *() 
	{ 
		return *MyPointer; 
	}

private:
	int32     Stride;
	DataType* MyPointer;
};