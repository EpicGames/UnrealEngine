// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "TimeSyncOption.generated.h"

UENUM(BlueprintType)
enum class ETimeSyncOption : uint8
{
	// Time setting will always be as authored
	None = 0,
	// Time setting is interpreted as a multiple of quarter note(s) and kept in sync with the current tempo.
	TempoSync	UMETA(Json="TempoSync"),
	// Time setting is multiplied by the current music playback speed.
	SpeedScale	UMETA(Json="SpeedScale"),
	Num			UMETA(Hidden)
};
