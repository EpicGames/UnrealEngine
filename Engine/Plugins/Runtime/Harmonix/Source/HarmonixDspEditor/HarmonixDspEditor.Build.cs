// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixDspEditor : ModuleRules
{
	public HarmonixDspEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		//OptimizeCode = CodeOptimization.Never;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DeveloperSettings"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioEditor",
				"ContentBrowser",
				"CurveEditor",
				"Engine",
				"EditorWidgets",
				"EditorScriptingUtilities",
				"Harmonix",
				"HarmonixEditor",
				"HarmonixMidi",
				"HarmonixDsp",
				"UnrealEd",
				"Json",
				"AssetRegistry",
				"AssetDefinition",
				"AssetTools",
				"PropertyEditor",
				"DetailCustomizations",
				"Slate",
				"SlateCore",
				"Settings",
				"ToolMenus",
				"InputCore"
			}
		);
	}
}
