// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_FusionPatch.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "Algo/AnyOf.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorDialogLibrary.h"
#include "FusionPatchAssetFactory.h"
#include "FusionPatchImportOptions.h"
#include "Sound/SoundWaveProcedural.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "FusionPatchAssetFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_FusionPatch)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FFusionPatchExtension::RegisterMenus()
{
	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundWave::StaticClass());
	FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
	check(Section);
	Section->AddDynamicEntry("SoundWaveAssetConversion_CreateFusionPatch",
		FNewToolMenuSectionDelegate::CreateLambda([] (FToolMenuSection& InSection)
		{
			if (const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
			{
				// filter out USoundWaveProcedural assets
				if (!Algo::AnyOf(Context->SelectedAssets, [](const FAssetData& AssetData) { return AssetData.IsInstanceOf<USoundWaveProcedural>(); }))
				{
					const TAttribute<FText> Label = LOCTEXT("SoundWave_CreateFusionPatch", "Create Fusion Patch");
					const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_CreateFusionPatchToolTip", "Creates a Fusion Patch Asset from the selected sound waves,"
						"where each sound wave is used as a keyzone in the Fusion Patch");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.FusionPatch");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&FFusionPatchExtension::ExecuteCreateFusionPatch);

					InSection.AddMenuEntry("SoundWave_CreateFusionPatch", Label, ToolTip, Icon, UIAction);
				}
			}
		}));
}

void FFusionPatchExtension::ExecuteCreateFusionPatch(const FToolMenuContext& MenuContext)
{
	const FString DefaultSuffix = TEXT("_Patch");
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	UFusionPatchAssetFactory* FusionPatchFactory = NewObject<UFusionPatchAssetFactory>();

	TArray<TWeakObjectPtr<USoundWave>> StagedSoundWaves;
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		for (USoundWave* Wave : Context->LoadSelectedObjectsIf<USoundWave>([](const FAssetData& AssetData) { return !AssetData.IsInstanceOf<USoundWaveProcedural>(); }))
		{
			StagedSoundWaves.Add(Wave);
		}

		if (StagedSoundWaves.Num() > 0)
		{
			// Determine an appropriate name
			FString Name;
			FString PackagePath;
			AssetToolsModule.Get().CreateUniqueAssetName(StagedSoundWaves[0]->GetPackage()->GetName(), DefaultSuffix, PackagePath, Name);

			UFusionPatchCreateOptions::FArgs Args;
			Args.StagedSoundWaves = MoveTemp(StagedSoundWaves);
			Args.AssetName = Name;
			Args.Directory = FPackageName::GetLongPackagePath(PackagePath);
			bool WasOkayPressed = false;
			const UFusionPatchCreateOptions* CreateOptions = UFusionPatchCreateOptions::GetWithDialog(MoveTemp(Args), WasOkayPressed);
			if (WasOkayPressed)
			{
				FString UniqueName;
				AssetToolsModule.Get().CreateUniqueAssetName(CreateOptions->AssetName, TEXT(""), PackagePath, UniqueName);
				FusionPatchFactory->CreateOptions = CreateOptions;
				UObject* CreatedAsset = AssetToolsModule.Get().CreateAsset(UniqueName, CreateOptions->FusionPatchDir.Path, UFusionPatch::StaticClass(), FusionPatchFactory);

				TArray<UObject*> Assets = {CreatedAsset};
				FAssetToolsModule::GetModule().Get().SyncBrowserToAssets(Assets);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
