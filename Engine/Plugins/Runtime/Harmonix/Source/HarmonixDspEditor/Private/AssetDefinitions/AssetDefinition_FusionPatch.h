// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_FusionPatch.generated.h"

struct FToolMenuContext;

UCLASS()
class UAssetDefinition_FusionPatch : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "FusionPatchDefinition", "Fusion Sampler Patch"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(80, 120, 255); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFusionPatch::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Audio };
		return Categories;
	}
	virtual bool CanImport() const { return true; }

};

class FFusionPatchExtension
{
public:
	static void RegisterMenus();
	static void ExecuteCreateFusionPatch(const FToolMenuContext& MenuContext);
};