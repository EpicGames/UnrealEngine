// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdsrSettingsDetailCustomization.h"

#include "HarmonixDsp/Modulators/Adsr.h"
#include "DSP/AlignedBuffer.h"
#include "Editor.h"
#include "CurveEditor.h"
#include "RichCurveEditorModel.h"
#include "SCurveEditorPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "PropertyCustomizationHelpers.h"
#include "HarmonixDspEditorUtils.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "AdsrSettingsDetailsConfigCustomization"

class FAdsrCurveEditorModelRaw : public FRichCurveEditorModelRaw
{
public:

	FAdsrCurveEditorModelRaw(FRichCurve* InRichCurve, UObject* InOwner)
		: FRichCurveEditorModelRaw(InRichCurve, InOwner)
	{}

	virtual bool IsReadOnly() const override
	{
		return true;
	}

	virtual FLinearColor GetColor() const
	{
		return Color;
	}
	
};

void FAdsrSettingsDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
		[	
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
	[
	SNew(SUniformGridPanel)
		.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
		.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
		.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
		+SUniformGridPanel::Slot(0,0)
		[
			InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, IsEnabled))->CreatePropertyValueWidget()
		]
		+SUniformGridPanel::Slot(1,0)
		[
			InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, Target))->CreatePropertyValueWidget()
		]
	];
}

void FAdsrSettingsDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	MyPropertyHandle = InStructPropertyHandle;
	if (InStructPropertyHandle->IsValidHandle())
	{
		InStructPropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FAdsrSettingsDetailsCustomization::RefreshCurve));
		uint32 NumChildren = 0;
		InStructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
		}
	}

	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() == 0)
	{
		return;
	}

	CurveEditor = MakeShared<FCurveEditor>();
	FCurveEditorInitParams InitParams;
	CurveEditor->InitCurveEditor(InitParams);
	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");

	RefreshCurve();

	TUniquePtr<FCurveModel> CurveModel = MakeUnique<FAdsrCurveEditorModelRaw>(&RichCurve, Objects[0]);
	CurveModel->SetIsKeyDrawEnabled(false);
	FCurveModelID CurveModelID = CurveEditor->AddCurve(MoveTemp(CurveModel));
	CurveEditor->PinCurve(CurveModelID);
	
	StructBuilder.AddCustomRow(LOCTEXT("CurveEditor", "Curve Editor")).ValueContent()
	[
		SNew(SCurveEditorPanel, CurveEditor.ToSharedRef())
		.MinimumViewPanelHeight(200.0f)
	];
}

FAdsrSettings FAdsrSettingsDetailsCustomization::GetAdsrSettings()
{
	FAdsrSettings Settings;
	if (!MyPropertyHandle)
	{
		return Settings;
	}

	uint8 Target;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, Target))->GetValue(Target);
	
	bool IsEnabled;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, IsEnabled))->GetValue(IsEnabled);
	
	float AttackTime;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, AttackTime))->GetValue(AttackTime);
	
	float DecayTime;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, DecayTime))->GetValue(DecayTime);

	float SustainLevel;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, SustainLevel))->GetValue(SustainLevel);

	float ReleaseTime;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, ReleaseTime))->GetValue(ReleaseTime);

	float AttackCurve;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, AttackCurve))->GetValue(AttackCurve);

	float DecayCurve;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, DecayCurve))->GetValue(DecayCurve);

	float ReleaseCurve;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, ReleaseCurve))->GetValue(ReleaseCurve);
	
	float Depth;
	MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAdsrSettings, Depth))->GetValue(Depth);

	Settings.Target = (EAdsrTarget)Target;
	Settings.IsEnabled = IsEnabled;
	Settings.Depth = Depth;
	Settings.AttackTime = AttackTime;
	Settings.DecayTime =  DecayTime;
	Settings.SustainLevel = SustainLevel;
	Settings.ReleaseTime = ReleaseTime;
	Settings.AttackCurve = AttackCurve;
	Settings.DecayCurve = DecayCurve;
	Settings.ReleaseCurve = ReleaseCurve;

	return Settings;
}


void FAdsrSettingsDetailsCustomization::RefreshCurve()
{
	if (!CurveEditor.IsValid())
	{
		return;
	}
	
	// samples per second
	constexpr float CurveSampleRate = 100.0f;
	constexpr float AdsrSampleRate = 48000.0f;
	
	FAdsrSettings Settings = GetAdsrSettings();
	Settings.Target = EAdsrTarget::Volume;
	Settings.IsEnabled = true;

	Audio::FAlignedFloatBuffer AdsrEnvelope;
	
	constexpr float SustainPct = 1.0f / 3.0f;
	// Calculate the SustainTime DURATION as a percentage of the envelope duration 
	const float SustainTime = (Settings.AttackTime + Settings.DecayTime + Settings.ReleaseTime) * SustainPct;
	Harmonix::Dsp::Editor::GenerateAdsrEnvelope(Settings, SustainTime, AdsrSampleRate, AdsrEnvelope);
	
	// total time in seconds to render the ADSR
	const float EnvelopeDuration = AdsrEnvelope.Num() / AdsrSampleRate;
	
	const int32 CurveTotalSamples = FMath::TruncToInt32(CurveSampleRate * EnvelopeDuration);
	
	TArray<FRichCurveKey> NewKeys;

	// snap first sample to (0, 0)
	{
		FRichCurveKey Key;
		Key.Time = 0;
		Key.Value = 0;
		NewKeys.Add(Key);
	}
	
	for (int32 CurveSampleIdx = 1; CurveSampleIdx < CurveTotalSamples; ++CurveSampleIdx)
	{
		float CurveTime = CurveSampleIdx / CurveSampleRate;
		int32 AdsrSampleIdx = AdsrSampleRate * CurveTime;
		if (!AdsrEnvelope.IsValidIndex(AdsrSampleIdx))
		{
			break;
		}
		
		FRichCurveKey Key;
		Key.Time = CurveTime;
		Key.Value = AdsrEnvelope[AdsrSampleIdx];
		NewKeys.Add(Key);
	}

	// snap last sample to (duration, 0)
	{
		FRichCurveKey Key;
		Key.Time = CurveTotalSamples / CurveSampleRate;
		Key.Value = 0;
		NewKeys.Add(Key);
	}

	RichCurve.SetKeys(NewKeys);
	
	CurveEditor->ZoomToFitAll();
}

#undef LOCTEXT_NAMESPACE