// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "HarmonixDsp/Modulators/Settings/AdsrSettings.h"
#include "Curves/RichCurve.h"

class FCurveEditor;

class FAdsrSettingsDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FAdsrSettingsDetailsCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	void RefreshCurve();

	FAdsrSettings GetAdsrSettings();
	
	/** Handle to the struct being customized */
	TSharedPtr<IPropertyHandle> MyPropertyHandle;

	TSharedPtr<IPropertyUtilities> DetailBuilder;
	
	TSharedPtr<FCurveEditor> CurveEditor;

	FRichCurve RichCurve;
};