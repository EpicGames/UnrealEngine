// Copyright Epic Games, Inc. All Rights Reserved.
#include "FusionPatchDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"
#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"


class FKeyzoneDetailArrayBuilder : public FDetailArrayBuilder
{
public:
	static constexpr int32 MinDesiredSlotWidth = 80;

	FKeyzoneDetailArrayBuilder(TSharedRef<IPropertyHandle> InBaseProperty, const TArray<FName>& PropertyHeaderNames, bool DisplayHeaderNames = true)
		: FDetailArrayBuilder(InBaseProperty)
		, PropertyHeaderNames(PropertyHeaderNames)
		, DisplayHeaderNames(DisplayHeaderNames)
	{}

	virtual void RefreshChildren() override
	{
		OptionsPropertyHandles.Reset();
		TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
		check(PropertyHandle);

		TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray();
		check(ArrayHandle);

		uint32 Num;
		ArrayHandle->GetNumElements(Num);
		for (uint32 Idx = 0; Idx < Num; ++Idx)
		{
			OptionsPropertyHandles.Add(ArrayHandle->GetElement(Idx));
		}
		
		
		FDetailArrayBuilder::RefreshChildren();
	}

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		if (DisplayHeaderNames)
		{
			GenerateChildHeaderContent(ChildrenBuilder);
		}
		FDetailArrayBuilder::GenerateChildContent(ChildrenBuilder);
	}

	void GenerateChildHeaderContent(IDetailChildrenBuilder& ChildrenBuilder)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
		check(PropertyHandle);

		TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray();
		check(ArrayHandle);

		TSharedPtr<SUniformGridPanel> GridPanel;
		SAssignNew(GridPanel, SUniformGridPanel).MinDesiredSlotWidth(MinDesiredSlotWidth);

		uint32 NumElements;
		ArrayHandle->GetNumElements(NumElements);
		if (NumElements > 0)
		{
			TSharedRef<IPropertyHandle> Element = ArrayHandle->GetElement(0);

			int32 Column = 0;
			int32 Row = 0;
			for (const FName PropertyName : PropertyHeaderNames)
			{
				if (TSharedPtr<IPropertyHandle> ChildHandle = Element->GetChildHandle(PropertyName))
				{
					GridPanel->AddSlot(Column, Row)
						[
							ChildHandle->CreatePropertyNameWidget()
						];
				}
				else
				{
					GridPanel->AddSlot(Column, Row)
						[
							SNew(STextBlock)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.Text(FText::FromString(TEXT("Invalid")))
						];
				}
				Column += 1;
				Row = 0;
			}

			TSharedPtr<IPropertyHandle> SoundWaveHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, SoundWave));
			ChildrenBuilder.AddCustomRow(NSLOCTEXT("FusionPatch_Details", "KeyzonesHeader", "Keyzones"))
				.NameContent()
				[
					SoundWaveHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					GridPanel.ToSharedRef()
				];
		}
	}
	
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
		check(PropertyHandle);

		TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray();
		check(ArrayHandle);
		
		FUIAction CopyAction;
		FUIAction PasteAction;
		PropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);
		NodeRow
		.FilterString(PropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				PropertyHandle->CreatePropertyValueWidget()
			]
		]
		.CopyAction(CopyAction)
		.PasteAction(PasteAction);
		
	}
private:
	
	TArray<FName> PropertyHeaderNames;
	TArray<TSharedPtr<IPropertyHandle>> OptionsPropertyHandles;
	bool DisplayHeaderNames = true;
};

void FFusionPatchDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& FusionPatchDataCategory = DetailLayout.EditCategory(TEXT("Fusion Patch Data"));
	//IDetailCategoryBuilder& FusionPatchDataCategory = DetailLayout.EditCategory(TEXT("Fusion Patch Data"));
	IDetailCategoryBuilder& FilePathCategory = DetailLayout.EditCategory(TEXT("File Path"));

	//get a handle of Fusion Patch Data
	TSharedPtr<IPropertyHandle> FusionPatchDataHandle = DetailLayout.GetProperty("FusionPatchData");
	check(FusionPatchDataHandle);
	TSharedPtr<IPropertyHandle> SettingsHandle = FusionPatchDataHandle->GetChildHandle("Settings");
	check(SettingsHandle);

	//get a handle of the keyzones as array
	TSharedPtr<IPropertyHandle> KeyzonesPropertyHandle = FusionPatchDataHandle->GetChildHandle("Keyzones");
	TSharedPtr<IPropertyHandleArray> KeyzonesArrayHandle = KeyzonesPropertyHandle->AsArray();
	check(KeyzonesArrayHandle.IsValid());

	TArray<FName> PropertyHeaderNames = {
		GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MinNote),
		GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, RootNote),
		GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MaxNote),
		GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MinVelocity),
		GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MaxVelocity)
	};

	bool DisplayHeaderNames = false;
	TSharedPtr<FKeyzoneDetailArrayBuilder> ArrayBuilder = MakeShared<FKeyzoneDetailArrayBuilder>(KeyzonesPropertyHandle.ToSharedRef(), PropertyHeaderNames, DisplayHeaderNames);
	
	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda(
	[PropertyHeaderNames, DisplayHeaderNames](TSharedRef<IPropertyHandle> Element, int32 Index, IDetailChildrenBuilder& ChildrenBuilder)
	{
		TSharedPtr<IPropertyHandle> SoundWaveHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, SoundWave));

		SoundWaveHandle->MarkHiddenByCustomization();
		TSharedPtr<SUniformGridPanel> GridPanel;
		SAssignNew(GridPanel, SUniformGridPanel)
		.MinDesiredSlotWidth(FKeyzoneDetailArrayBuilder::MinDesiredSlotWidth);

		int32 Column = 0;
		for (const FName PropertyName : PropertyHeaderNames)
		{
			if (TSharedPtr<IPropertyHandle> ChildHandle = Element->GetChildHandle(PropertyName))
			{
				// display the header names along with the property values
				// whene the array builder isn't displaying the header names itself
				if (!DisplayHeaderNames)
				{
					ChildHandle->MarkHiddenByCustomization();
					GridPanel->AddSlot(Column, 0)
						[
							ChildHandle->CreatePropertyNameWidget()
						];
					GridPanel->AddSlot(Column, 1)
						[
							ChildHandle->CreatePropertyValueWidget()
						];
				}
				else // display only the property values
				{
					GridPanel->AddSlot(Column, 0)
						[
							ChildHandle->CreatePropertyValueWidget()
						];
				}
			}
			else
			{
				GridPanel->AddSlot(Column, 0)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::FromString(TEXT("Invalid")))
				];
			}

			Column += 1;
		}

		ChildrenBuilder.AddProperty(Element).CustomWidget(true)
			.NameContent()
			[
				SoundWaveHandle->CreatePropertyValueWidget()
			]
			.ValueContent()
			[
				GridPanel.ToSharedRef()
			];
	}));

	FusionPatchDataCategory.AddCustomBuilder(ArrayBuilder.ToSharedRef());
	//add the Settings properties back so they're visible
	FusionPatchDataCategory.AddProperty(SettingsHandle);

	//hide the non-customized versions of these properties
	DetailLayout.HideProperty(FusionPatchDataHandle);
}
