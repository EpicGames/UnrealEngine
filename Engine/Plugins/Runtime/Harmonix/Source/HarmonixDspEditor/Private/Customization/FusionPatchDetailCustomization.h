// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "DetailCategoryBuilder.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Layout/Visibility.h"

class SWidget;

//Fusion Patch detail customizatiaon class
class FFusionPatchDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FFusionPatchDetailCustomization);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};


