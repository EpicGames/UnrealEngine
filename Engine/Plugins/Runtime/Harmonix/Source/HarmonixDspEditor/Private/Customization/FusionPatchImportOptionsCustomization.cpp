// Copyright Epic Games, Inc. All Rights Reserved.
#include "FusionPatchImportOptionsCustomization.h"
#include "FusionPatchImportOptions.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"

#include "AssetViewUtils.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"

#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FusionPatchDetails"

class FFusionPatchAssetNameDetailCustomization : public IPropertyTypeCustomization
{
public:

	// identifier to make sure we're only customizing the AssetName property.
	struct FIdentifier : public IPropertyTypeIdentifier
	{
		FProperty const* const Property;

		FIdentifier(FProperty const* Property)
			: Property(Property)
		{}

		virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
		{
			return PropertyHandle.GetProperty() == Property && PropertyHandle.GetNumPerObjectValues() == 1;
		}
	};
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		HeaderRow.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		];

		HeaderRow.ValueContent()
		[
			// PropertyHandle->CreatePropertyValueWidget()
			// SNullWidget::NullWidget
			SNew(SEditableTextBox)			
			.Text_Lambda([PropertyHandle]()
			{
				FString OutString;
				PropertyHandle->GetValue(OutString);
				return FText::FromString(OutString);
			})
			.OnTextCommitted_Lambda([PropertyHandle]( const FText& Text, ETextCommit::Type Type)
			{
				PropertyHandle->SetValue(Text.ToString());
			})
			.ToolTipText_Lambda([]()
			{
				return LOCTEXT("FusionPatchAssetName_TT", "Asset Name");	
			})
			.OnVerifyTextChanged_Lambda([PropertyHandle](const FText& TextToVerify, FText& OutError) -> bool
			{
				TArray<UObject*> Outers;
				PropertyHandle->GetOuterObjects(Outers);
				
				if (Outers.IsEmpty())
				{
					OutError = LOCTEXT("FusionPatchAssetName_NoOuter", "Failed to find outer for naming");
					return false;
				}

				UFusionPatchCreateOptions* Options = Cast<UFusionPatchCreateOptions>(Outers[0]);
				if (!Options)
				{
					OutError = LOCTEXT("FusionPatchAssetName_NoOuter", "Failed to find outer for naming");
					return false;
				}

				// pre-emptively check for a number
				const FString AssetName = TextToVerify.ToString();
				if (AssetName.Len() > 0 && FChar::IsDigit(AssetName[0]))
				{
					OutError = LOCTEXT("FusionPatchAssetName_StartsWithDigit", "Patch name cannot start with a number");
					return false;
				}
				
				const FString PackagePath = Options->FusionPatchDir.Path;
                const FString PackageName = PackagePath / AssetName;
                const FString ObjectPath = FString::Printf(TEXT("%s.%s"), *PackageName, *(AssetName));

				return AssetViewUtils::IsValidObjectPathForCreate(ObjectPath, UFusionPatch::StaticClass(), OutError);
			})
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		// no children to customize
	}
	
};

void FFusionPatchCreateOptionsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	static const int32 MinDesiredSlotWidth = 80;
	TSharedPtr<IPropertyHandle> KeyzonesPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UFusionPatchCreateOptions, Keyzones));
	TSharedPtr<IPropertyHandle> SortOptionHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UFusionPatchCreateOptions, SortOption));
	check(KeyzonesPropertyHandle);
	TSharedPtr<IPropertyHandleArray> KeyzonesArrayHandle = KeyzonesPropertyHandle->AsArray();
	check(KeyzonesArrayHandle);
	
	TSharedPtr<IPropertyHandle> AssetNameHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UFusionPatchCreateOptions, AssetName));
	check(AssetNameHandle);
	DetailLayout.RegisterInstancedCustomPropertyTypeLayout(
		FStrProperty::StaticClass()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FFusionPatchAssetNameDetailCustomization>(); }),
		MakeShared<FFusionPatchAssetNameDetailCustomization::FIdentifier>(AssetNameHandle->GetProperty()));

	DetailLayout.HideProperty(KeyzonesPropertyHandle);
	IDetailCategoryBuilder& KeyzonesCategory = DetailLayout.EditCategory(TEXT("Keyzones"), FText::GetEmpty(), ECategoryPriority::Uncommon);
	KeyzonesCategory.InitiallyCollapsed(true);
	
	uint32 NumElements = 0;
	KeyzonesArrayHandle->GetNumElements(NumElements);
	if (NumElements > 0)
	{
		TSharedRef<IPropertyHandle> Element = KeyzonesArrayHandle->GetElement(0);
		TSharedPtr<IPropertyHandle> SoundWaveHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, SoundWave));
		TSharedPtr<IPropertyHandle> MinNoteHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MinNote));
		TSharedPtr<IPropertyHandle> MaxNoteHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MaxNote));
		TSharedPtr<IPropertyHandle> RootNoteHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, RootNote));
		TSharedPtr<IPropertyHandle> MinVelocityHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MinVelocity));
		TSharedPtr<IPropertyHandle> MaxVelocityHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MaxVelocity));

		KeyzonesCategory.AddCustomRow(NSLOCTEXT("FusionoPatch_Details", "KeyzonesHeader", "Keyzones"))
		.NameContent()
		[
			SoundWaveHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
		SNew(SUniformGridPanel)
			.MinDesiredSlotWidth(MinDesiredSlotWidth)
		   +SUniformGridPanel::Slot(0, 0)
		   [
				MinNoteHandle->CreatePropertyNameWidget()
		   ]
		   +SUniformGridPanel::Slot(1, 0)
		   [
			   RootNoteHandle->CreatePropertyNameWidget()
		   ]
		   +SUniformGridPanel::Slot(2, 0)
		   [
			   MaxNoteHandle->CreatePropertyNameWidget()
		   ]
		   +SUniformGridPanel::Slot(3, 0)
		   [
				MinVelocityHandle->CreatePropertyNameWidget()
		   ]
		   +SUniformGridPanel::Slot(4, 0)
		   [
		   		MaxVelocityHandle->CreatePropertyNameWidget()
		   ]
		];
	}
	TSharedPtr<FDetailArrayBuilder> ArrayBuilder = MakeShared<FDetailArrayBuilder>(KeyzonesPropertyHandle.ToSharedRef(), false, false, false);
	
	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda([](TSharedRef<IPropertyHandle> Element, int32 Index, IDetailChildrenBuilder& ChildrenBuilder)
	{
		TSharedPtr<IPropertyHandle> SoundWaveHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, SoundWave));
		TSharedPtr<IPropertyHandle> MinNoteHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MinNote));
		TSharedPtr<IPropertyHandle> MaxNoteHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MaxNote));
		TSharedPtr<IPropertyHandle> RootNoteHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, RootNote));
		TSharedPtr<IPropertyHandle> MinVelocityHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MinVelocity));
		TSharedPtr<IPropertyHandle> MaxVelocityHandle = Element->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKeyzoneSettings, MaxVelocity));

		UObject* SoundWave = nullptr;
		SoundWaveHandle->GetValue(SoundWave);
		if (!SoundWave)
		{
			return;
		}
				
		FText SoundWaveName = FText::FromString(SoundWave->GetPackage()->GetName());

		ChildrenBuilder.AddCustomRow(NSLOCTEXT("FusionPatch_Details", "KeyzoneProperty", "Keyzone"))
		.EditCondition(TAttribute<bool>::CreateLambda([]() { return false; }), nullptr)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(SoundWaveName)
		]
		.ValueContent()
		[
		 SNew(SUniformGridPanel)
			.MinDesiredSlotWidth(MinDesiredSlotWidth)
			+SUniformGridPanel::Slot(0, 0)
			[
				MinNoteHandle->CreatePropertyValueWidget()
			]
			+SUniformGridPanel::Slot(1, 0)
			[
				RootNoteHandle->CreatePropertyValueWidget()
			]
			+SUniformGridPanel::Slot(2, 0)
			[
				MaxNoteHandle->CreatePropertyValueWidget()
			]
			+SUniformGridPanel::Slot(3, 0)
			[
				MinVelocityHandle->CreatePropertyValueWidget()
			]
			+SUniformGridPanel::Slot(4, 0)
			[
				MaxVelocityHandle->CreatePropertyValueWidget()
			]
		];
	}));

	SortOptionHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([ArrayBuilder]()
	{
		ArrayBuilder->RefreshChildren();
	}));
	KeyzonesCategory.AddCustomBuilder(ArrayBuilder.ToSharedRef());

	DetailLayout.HideProperty(KeyzonesPropertyHandle);
}

#undef LOCTEXT_NAMESPACE // FusionPatchDetails