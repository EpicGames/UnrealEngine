// Copyright Epic Games, Inc. All Rights Reserved.

#include "FusionPatchSettingsDetailCustomization.h"
#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

void FFusionPatchSettingsDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget()
	];
}

void FFusionPatchSettingsDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIdx);
		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FFusionPatchSettings, Adsr))
		{
			CustomizeArrayProperty(ChildHandle.ToSharedRef(), StructBuilder, 2);
		}
		else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FFusionPatchSettings, Lfo))
		{
			CustomizeArrayProperty(ChildHandle.ToSharedRef(), StructBuilder, 3);
		}
		else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FFusionPatchSettings, Randomizer))
		{
			CustomizeArrayProperty(ChildHandle.ToSharedRef(), StructBuilder, 3);
		}
		else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FFusionPatchSettings, VelocityModulator))
		{
			CustomizeArrayProperty(ChildHandle.ToSharedRef(), StructBuilder, 3);
		}
		else
		{
			StructBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

void FFusionPatchSettingsDetailCustomization::CustomizeArrayProperty(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& StructBuilder, uint32 MaxPreviewNum)
{
	TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray();
	uint32 NumElements;
	ArrayHandle->GetNumElements(NumElements);
	for (uint32 ElemIdx = 0; ElemIdx < NumElements; ++ElemIdx)
	{
		TSharedRef<IPropertyHandle> Element = ArrayHandle->GetElement(ElemIdx);
		TSharedPtr<SUniformGridPanel> GridPanel;
		StructBuilder.AddProperty(Element).CustomWidget(true)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda([PropertyHandle, ElemIdx]()
			{
				return FText::FromString(FString::Printf(TEXT("%s: %d"),
					*PropertyHandle->GetPropertyDisplayName().ToString(), ElemIdx));
			})
		]
		.ValueContent()
		[
			CreateStructPropertiesPreviewValueWidget(Element, MaxPreviewNum)
		];
	}
}

TSharedRef<SWidget> FFusionPatchSettingsDetailCustomization::CreateStructPropertiesPreviewValueWidget(TSharedRef<IPropertyHandle> PropertyHandle, uint32 MaxPreviewNum)
{
	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);
	uint32 PreviewNum = FMath::Min(NumChildren, MaxPreviewNum);
	TSharedPtr<SUniformGridPanel> GridPanel = SNew(SUniformGridPanel).MinDesiredSlotWidth(100);
	for (uint32 ChildIdx = 0; ChildIdx < PreviewNum; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIdx);

		GridPanel->AddSlot(ChildIdx, 0)
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda([ChildHandle]()
			{
				FText ValueText;
				ChildHandle->GetValueAsFormattedText(ValueText);
				return FText::FromString(FString::Printf(TEXT("%s: %s"), *ChildHandle->GetPropertyDisplayName().ToString(), *ValueText.ToString()));
			})
		];
	}
	return GridPanel.ToSharedRef();
}
