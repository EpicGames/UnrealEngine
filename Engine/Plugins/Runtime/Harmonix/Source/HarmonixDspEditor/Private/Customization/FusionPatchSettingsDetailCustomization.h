// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class SWidget;

class FFusionPatchSettingsDetailCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FFusionPatchSettingsDetailCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	void CustomizeArrayProperty(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& StructBuilder, uint32 MaxPreviewNum);

	TSharedRef<SWidget> CreateStructPropertiesPreviewValueWidget(TSharedRef<IPropertyHandle> PropertyHandle, uint32 MaxPreviewNum);
};