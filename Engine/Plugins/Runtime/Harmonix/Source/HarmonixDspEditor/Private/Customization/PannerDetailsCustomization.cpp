// Copyright Epic Games, Inc. All Rights Reserved.

#include "PannerDetailsCustomization.h"

#include "HarmonixDsp/PannerDetails.h"

#include "Editor.h"
#include "SEnumCombo.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"

#define LOCTEXT_NAMESPACE "PannerDetailsConfigCustomization"


void FPannerDetailsCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	MyPropertyHandle = PropertyHandle;
	MyPropertyUtils = StructCustomizationUtils.GetPropertyUtilities();
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	UEnum* PannerModeEnum = StaticEnum<EPannerMode>();

	HeaderRow
	.NameContent()
		[	
			PropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(300.0f)
		.MaxDesiredWidth(300.0f)
		[
			SNew(SEnumComboBox, PannerModeEnum)
			.ContentPadding(FMargin(0.0f, 0.0f))
			.CurrentValue(this, &FPannerDetailsCustomization::OnGetPannerModeEnumValue)
			.OnEnumSelectionChanged(this, &FPannerDetailsCustomization::OnPannerModeEnumSelectionChanged)
		];
}

void FPannerDetailsCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	EPannerMode PannerMode = EPannerMode::Invalid;
	if (FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		PannerMode = PannerDetails->Mode;
	}

	// flaot position is used for Stereo and Surround
	switch (PannerMode)
	{
	case EPannerMode::LegacyStereo:
	case EPannerMode::Stereo:
	case EPannerMode::Surround:
	case EPannerMode::PolarSurround:
	{
		StructBuilder.AddCustomRow(LOCTEXT("Pan", "Pan"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Pan")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
			.Value_Lambda([this]() { return OnGetFloatValue(PanName); })
			.OnValueChanged_Lambda([this](float NewValue) { OnFloatValueChanged(NewValue, PanName); })
			.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType) { OnFloatValueCommitted(NewValue, CommitType, PanName); })
			.MinValue(-1.0f)
			.MaxValue(1.0f)
			.MinSliderValue(-1.0f)
			.MaxSliderValue(1.0f)
			.AllowSpin(true)
		];
	}
	}

	// edge proximity is used for Surround
	switch (PannerMode)
	{
	case EPannerMode::Surround:
	case EPannerMode::PolarSurround:
	{
		StructBuilder.AddCustomRow(LOCTEXT("EdgeProximity", "Edge Proximity"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Edge Proximity")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
			.Value_Lambda([this]() { return OnGetFloatValue(EdgeProximityName); })
			.OnValueChanged_Lambda([this](float NewValue) { OnFloatValueChanged(NewValue, EdgeProximityName); })
			.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType) { OnFloatValueCommitted(NewValue, CommitType, EdgeProximityName); })
			.MinValue(0.0f)
			.MaxValue(1.0f)
			.MinSliderValue(0.0f)
			.MaxSliderValue(1.0f)
			.AllowSpin(true)
		];
	}
	}

	// Only Display Enum for direct assignment
	if (PannerMode == EPannerMode::DirectAssignment)
	{
		UEnum* ChannelAssignmentEnum = StaticEnum<ESpeakerChannelAssignment>();

		StructBuilder.AddCustomRow(LOCTEXT("EdgeProximity", "Edge Proximity"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Edge Proximity")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(300.0f)
		.MaxDesiredWidth(300.0f)
		[
			SNew(SEnumComboBox, ChannelAssignmentEnum)
			.ContentPadding(FMargin(0.0f, 0.0f))
			.CurrentValue(this, &FPannerDetailsCustomization::OnGetChannelAssignmentEnumValue)
			.OnEnumSelectionChanged(this, &FPannerDetailsCustomization::OnChannelAssignmentEnumSelectionChanged)
		];
	}
}

void FPannerDetailsCustomization::OnPannerModeEnumSelectionChanged(int32 TypeIndex, ESelectInfo::Type SelectInfo)
{
	if (FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		if (TypeIndex < 0 || TypeIndex >= (int32)EPannerMode::Num)
		{
			return;
		}

		EPannerMode PannerMode = (EPannerMode)(TypeIndex);
		
		MyPropertyHandle->NotifyPreChange();
		PannerDetails->Mode = PannerMode;
		MyPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		MyPropertyHandle->NotifyFinishedChangingProperties();

		if (MyPropertyUtils)
		{
			MyPropertyUtils->ForceRefresh();
		}
	}
}

int32 FPannerDetailsCustomization::OnGetPannerModeEnumValue() const
{
	if (const FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		return (int32)PannerDetails->Mode;
	}

	return (int32)EPannerMode::Invalid;
}

void FPannerDetailsCustomization::OnChannelAssignmentEnumSelectionChanged(int32 TypeIndex, ESelectInfo::Type SelectInfo)
{
	if (FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		if (PannerDetails->Mode == EPannerMode::DirectAssignment)
		{
			if (TypeIndex < 0 || TypeIndex >= (int32)ESpeakerChannelAssignment::Num)
			{
				return;
			}

			MyPropertyHandle->NotifyPreChange();
			ESpeakerChannelAssignment ChannelAssignment = (ESpeakerChannelAssignment)TypeIndex;
			PannerDetails->Detail.ChannelAssignment = ChannelAssignment;
			MyPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			MyPropertyHandle->NotifyFinishedChangingProperties();
		}
	}
}

int32 FPannerDetailsCustomization::OnGetChannelAssignmentEnumValue() const
{
	if (const FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		if (PannerDetails->Mode == EPannerMode::DirectAssignment)
		{
			return (int32)PannerDetails->Detail.ChannelAssignment;
		}
	}

	return (int32)ESpeakerChannelAssignment::Invalid;
}

TOptional<float> FPannerDetailsCustomization::OnGetFloatValue(FName PropertyName) const
{
	TOptional<float> OutValue;
	if (const FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		if (PropertyName == PanName)
		{
			switch (PannerDetails->Mode)
			{
			case EPannerMode::LegacyStereo:
			case EPannerMode::Stereo:
			case EPannerMode::Surround:
			case EPannerMode::PolarSurround:
				OutValue = PannerDetails->Detail.Pan;
			}
		}
		else if (PropertyName == EdgeProximityName)
		{
			switch (PannerDetails->Mode)
			{
			case EPannerMode::Surround:
			case EPannerMode::PolarSurround:
				OutValue = PannerDetails->Detail.EdgeProximity;
			}
		}
	}

	return OutValue;
}

void FPannerDetailsCustomization::OnFloatValueChanged(float NewValue, FName PropertyName)
{
	if (FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		MyPropertyHandle->NotifyPreChange();
		if (PropertyName == PanName)
		{
			switch (PannerDetails->Mode)
			{
			case EPannerMode::LegacyStereo:
			case EPannerMode::Stereo:
			case EPannerMode::Surround:
			case EPannerMode::PolarSurround:
				PannerDetails->Detail.Pan = NewValue;
			}
		}
		else if (PropertyName == EdgeProximityName)
		{
			switch (PannerDetails->Mode)
			{
			case EPannerMode::Surround:
			case EPannerMode::PolarSurround:
				PannerDetails->Detail.EdgeProximity = NewValue;
			}
		}
		MyPropertyHandle->NotifyPostChange(EPropertyChangeType::Interactive);
		MyPropertyHandle->NotifyFinishedChangingProperties();
	}
}

void FPannerDetailsCustomization::OnFloatValueCommitted(float NewValue, ETextCommit::Type CommitType, FName PropertyName)
{
	if (FPannerDetails* PannerDetails = GetPannerDetailsPtr())
	{
		MyPropertyHandle->NotifyPreChange();
		if (PropertyName == PanName)
		{
			switch (PannerDetails->Mode)
			{
			case EPannerMode::LegacyStereo:
			case EPannerMode::Stereo:
			case EPannerMode::Surround:
			case EPannerMode::PolarSurround:
				PannerDetails->Detail.Pan = NewValue;
			}
		}
		else if (PropertyName == EdgeProximityName)
		{
			switch (PannerDetails->Mode)
			{
			case EPannerMode::Surround:
			case EPannerMode::PolarSurround:
				PannerDetails->Detail.EdgeProximity = NewValue;
			}
		}
		MyPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		MyPropertyHandle->NotifyFinishedChangingProperties();
	}
}

FPannerDetails* FPannerDetailsCustomization::GetPannerDetailsPtr()
{
	return const_cast<FPannerDetails*>(const_cast<const FPannerDetailsCustomization*>(this)->GetPannerDetailsPtr());
}

const FPannerDetails* FPannerDetailsCustomization::GetPannerDetailsPtr() const
{
	if (!MyPropertyHandle.IsValid() || !MyPropertyHandle->IsValidHandle())
	{
		return nullptr;
	}
	TArray<UObject*> Objects;
	MyPropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() == 0)
	{
		return nullptr;
	}

	return (FPannerDetails*)(MyPropertyHandle->GetValueBaseAddress((uint8*)(Objects[0])));
}

#undef LOCTEXT_NAMESPACE