// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyUtilities;
class IPropertyHandle;
struct FPannerDetails;

/**
* Owner: Jake Burga
*
* FPannerDetails have custom serialization code, so we need to expose the porperties to blueprints this way
*
*/
class FPannerDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FPannerDetailsCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	// enum dropdown callbacks
	void OnPannerModeEnumSelectionChanged(int32 TypeIndex, ESelectInfo::Type SelectInfo);
	int32 OnGetPannerModeEnumValue() const;

	void OnChannelAssignmentEnumSelectionChanged(int32 TypeIndex, ESelectInfo::Type SelectInfo);
	int32 OnGetChannelAssignmentEnumValue() const;

	// position (float) numeric entry box callbacks
	TOptional<float> OnGetFloatValue(FName PropertyName) const;
	void OnFloatValueChanged(float NewValue, FName PropertyName);
	void OnFloatValueCommitted(float NewValue, ETextCommit::Type CommitType, FName PropertyName);
	
	// get the underlying FPannerDetails as a ptr
	FPannerDetails* GetPannerDetailsPtr();
	const FPannerDetails* GetPannerDetailsPtr() const;

	/** Handle to the struct being customized */
	TSharedPtr<IPropertyHandle> MyPropertyHandle;
	TSharedPtr<IPropertyUtilities> MyPropertyUtils;

	const FName PanName = FName("Pan");
	const FName EdgeProximityName = FName("Edge Proximity");

	TSharedPtr<IPropertyUtilities> DetailBuilder;
};