// Copyright Epic Games, Inc. All Rights Reserved.

#include "PitchShifterConfigCustomization.h"

#include "HarmonixDsp/StretcherAndPitchShifterConfig.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactoryConfig.h"

#include "Editor.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "UObject/UObjectIterator.h"


#define LOCTEXT_NAMESPACE "PitchShifterConfigCustomization"

void FPitchShifterConfigCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// This 'EditCategory' call will ensure that the PitchShifterFactory Config appears above the PitchShifter Config
	DetailBuilder.EditCategory(UStretcherAndPitchShifterFactoryConfig::StaticClass()->GetFName(), FText::GetEmpty(), ECategoryPriority::Important);

	// Add all the UStretcherAndPitchShifterConfig CDOs to this settings details panel
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Pitch Shifter Configuration"));
	for (TObjectIterator<UStretcherAndPitchShifterConfig> SettingsIt(RF_NoFlags); SettingsIt; ++SettingsIt)
	{
		if (UStretcherAndPitchShifterConfig* Settings = *SettingsIt)
		{
			// Only Add the CDO of any UStretcherAndPitchShifterConfig objects.
			if (Settings->HasAnyFlags(RF_ClassDefaultObject) && !Settings->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
			{
				IDetailPropertyRow* Row = CategoryBuilder.AddExternalObjects({ Settings }, EPropertyLocation::Default, FAddPropertyParams().UniqueId(Settings->GetClass()->GetFName()));
				if (Row)
				{
					// need to customize the name, otherwise it'll just say "Object"
					Row->CustomWidget()
						.NameContent()
						[
							SNew(STextBlock).Text(Settings->GetClass()->GetDisplayNameText())
						];
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE