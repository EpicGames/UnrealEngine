// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/SToolTip.h"
#include "IPropertyUtilities.h"

/**
* Owner: Jake Burga
*
* Exposes a dropdown on FPitchShifterConfigCustomization properties in the detail panel
*
*/
class FPitchShifterConfigCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FPitchShifterConfigCustomization);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};