// Copyright Epic Games, Inc. All Rights Reserved.

#include "PitchShifterNameCustomization.h"

#include "HarmonixDsp/PitchShifterName.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactory.h"

#include "Editor.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyCustomizationHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"


#define LOCTEXT_NAMESPACE "PitchShifterNameCustomization"


void FPitchShifterNameCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PitchShifterNameValuePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPitchShifterName, Name));

	FPropertyComboBoxArgs ComboArgs(PitchShifterNameValuePropertyHandle,
		FOnGetPropertyComboBoxStrings::CreateSP(this, &FPitchShifterNameCustomization::OnGetStrings),
		FOnGetPropertyComboBoxValue::CreateSP(this, &FPitchShifterNameCustomization::OnGetValueString),
		FOnPropertyComboBoxValueSelected::CreateSP(this, &FPitchShifterNameCustomization::OnValueStringSelected)
	);
	ComboArgs.ShowSearchForItemCount = 1;

	HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), FText::GetEmpty())
		]
	.ValueContent()
		.MaxDesiredWidth(0.0f) // don't constrain the combo button width
		[
			PropertyCustomizationHelpers::MakePropertyComboBox(ComboArgs)
		];
}

void FPitchShifterNameCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{

}

void FPitchShifterNameCustomization::OnValueStringSelected(const FString& SelectedString)
{
	if (!PitchShifterNameValuePropertyHandle.IsValid() || !PitchShifterNameValuePropertyHandle->IsValidHandle())
		return;

	PitchShifterNameValuePropertyHandle->SetValue(FName(SelectedString));
}

void FPitchShifterNameCustomization::OnGetStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	for (FName Name : IStretcherAndPitchShifterFactory::GetAllRegisteredFactoryNames())
	{
		OutStrings.Add(MakeShared<FString>(Name.ToString()));
		OutRestrictedItems.Add(false);
	}
}

FString FPitchShifterNameCustomization::OnGetValueString() const
{
	if (!PitchShifterNameValuePropertyHandle.IsValid() || !PitchShifterNameValuePropertyHandle->IsValidHandle())
		return LOCTEXT("PitchShifterName_None", "INVALID").ToString();

	FName Name;
	const FPropertyAccess::Result RowResult = PitchShifterNameValuePropertyHandle->GetValue(Name);
	if (RowResult == FPropertyAccess::Success)
	{
		return Name.ToString();
	}
	else if (RowResult == FPropertyAccess::Fail)
	{
		return LOCTEXT("PitchShifterName_None", "INVALID").ToString();
	}
	else
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
}

#undef LOCTEXT_NAMESPACE