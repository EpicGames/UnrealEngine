// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/TypedParameterCustomization.h"

#include "Editor.h"
#include "SEnumCombo.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "TypedParameterCustomization"

void FTypedParameterCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TypedParameterHandle = PropertyHandle;
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	FTypedParameter* TypedParameter = (FTypedParameter*)(PropertyHandle->GetValueBaseAddress((uint8*)(Objects[0])));
	ensure(TypedParameter);

	EParameterType ParameterType = TypedParameter ? TypedParameter->GetType() : EParameterType::Invalid;
	UEnum* PropertyTypeEnum = StaticEnum<EParameterType>();

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), FText::GetEmpty())
	]
	.ValueContent()
		.MinDesiredWidth(300.0f)
		.MaxDesiredWidth(300.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SEnumComboBox, PropertyTypeEnum)
				.ContentPadding(FMargin(0.0f, 0.0f))
				.CurrentValue(this, &FTypedParameterCustomization::OnGetPropertyTypeEnumValue)
				.OnEnumSelectionChanged(this, &FTypedParameterCustomization::OnPropertyTypeEnumSelectionChanged)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(2.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ValueContainerWidget, SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					CreateTypedValueWidget(ParameterType)
				]
			]
		];
}

void FTypedParameterCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// do nothing
}

void FTypedParameterCustomization::OnPropertyTypeEnumSelectionChanged(int32 TypeIndex, ESelectInfo::Type SelectInfo)
{
	if (FTypedParameter* TypedParameter = GetTypedParameterPtr())
	{
		EParameterType Type = (EParameterType)TypeIndex;

		if (Type == TypedParameter->GetType())
		{
			return;
		}

		TypedParameterHandle->NotifyPreChange();
		TypedParameter->SetType(Type);
		TypedParameterHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		TypedParameterHandle->NotifyFinishedChangingProperties();
		if (ValueContainerWidget.IsValid())
		{
			ValueContainerWidget->ClearChildren();
			ValueContainerWidget->AddSlot()
				[
					CreateTypedValueWidget(Type)
				];
		}
	}
}

int32 FTypedParameterCustomization::OnGetPropertyTypeEnumValue() const
{
	if (const FTypedParameter* TypedParameter = GetTypedParameterPtr())
	{
		return (int32)TypedParameter->GetType();
	}

	return (int32)EParameterType::Invalid;
}

const FTypedParameter* FTypedParameterCustomization::GetTypedParameterPtr() const
{
	if (!TypedParameterHandle.IsValid() || !TypedParameterHandle->IsValidHandle())
	{
		return nullptr;
	}
	TArray<UObject*> Objects;
	TypedParameterHandle->GetOuterObjects(Objects);
	if (Objects.Num() == 0)
	{
		return nullptr;
	}

	return (FTypedParameter*)(TypedParameterHandle->GetValueBaseAddress((uint8*)(Objects[0])));
}


FTypedParameter* FTypedParameterCustomization::GetTypedParameterPtr()
{
	return const_cast<FTypedParameter*>(const_cast<const FTypedParameterCustomization*>(this)->GetTypedParameterPtr());
}

ECheckBoxState FTypedParameterCustomization::OnGetCheckedState() const
{
	if (const FTypedParameter* TypedParameter = GetTypedParameterPtr())
	{
		if (TypedParameter->IsType(EParameterType::Bool))
		{
			return TypedParameter->GetValue<bool>() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}
	return ECheckBoxState::Undetermined;
}

void FTypedParameterCustomization::OnCheckedStateChanged(ECheckBoxState State)
{
	if (FTypedParameter* TypedParameter = GetTypedParameterPtr())
	{
		TypedParameterHandle->NotifyPreChange();
		bool Value = State == ECheckBoxState::Checked ? true : false;
		TypedParameter->SetValue<bool>(Value);
		TypedParameterHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		TypedParameterHandle->NotifyFinishedChangingProperties();
	}
}

FText FTypedParameterCustomization::OnGetTextValue() const
{
	if (const FTypedParameter* TypedParameter = GetTypedParameterPtr())
	{
		if (TypedParameter->IsType(EParameterType::String))
		{
			return FText::FromString(TypedParameter->GetValue<FString>());
		}
		else if (TypedParameter->IsType(EParameterType::Name))
		{
			return FText::FromString(TypedParameter->GetValue<FName>().ToString());
		}
	}

	return FText::FromString(TEXT("INVALID DATA"));
}

void FTypedParameterCustomization::OnTextComitted(const FText& InText, ETextCommit::Type TextCommit)
{
	if (FTypedParameter* TypedParameter = GetTypedParameterPtr())
	{
		if (TypedParameter->IsType(EParameterType::String))
		{
			TypedParameterHandle->NotifyPreChange();
			TypedParameter->SetValue<FString>(InText.ToString());
			TypedParameterHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			TypedParameterHandle->NotifyFinishedChangingProperties();
		}
		else if (TypedParameter->IsType(EParameterType::Name))
		{
			TypedParameterHandle->NotifyPreChange();
			TypedParameter->SetValue<FName>(FName(InText.ToString()));
			TypedParameterHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			TypedParameterHandle->NotifyFinishedChangingProperties();
		}
	}
}

TSharedRef<SWidget> FTypedParameterCustomization::CreateTypedValueWidget(EParameterType Type)
{
	switch (Type)
	{
	case EParameterType::Bool:
		return SNew(SCheckBox)
			.IsChecked(this, &FTypedParameterCustomization::OnGetCheckedState)
			.OnCheckStateChanged(this, &FTypedParameterCustomization::OnCheckedStateChanged);
	case EParameterType::Double:
		return CreateNumericValueWidget<double>();
	case EParameterType::Float:
		return CreateNumericValueWidget<float>();
	case EParameterType::Int8:
		return CreateNumericValueWidget<int8>();
	case EParameterType::Int16:
		return CreateNumericValueWidget<int16>();
	case EParameterType::Int32:
		return CreateNumericValueWidget<int32>();
	case EParameterType::Int64:
		return CreateNumericValueWidget<int64>();
	case EParameterType::UInt8:
		return CreateNumericValueWidget<uint8>();
	case EParameterType::UInt16:
		return CreateNumericValueWidget<uint16>();
	case EParameterType::UInt32:
		return CreateNumericValueWidget<uint32>();
	case EParameterType::UInt64:
		return CreateNumericValueWidget<uint64>();
	case EParameterType::Name:
		return SNew(SEditableTextBox)
			.Text(this, &FTypedParameterCustomization::OnGetTextValue)
			.OnTextCommitted(this, &FTypedParameterCustomization::OnTextComitted);
	case EParameterType::String:
		return SNew(SEditableTextBox)
			.Text(this, &FTypedParameterCustomization::OnGetTextValue)
			.OnTextCommitted(this, &FTypedParameterCustomization::OnTextComitted);
	default:
		return CreateNumericValueWidget<uint32>();
	}
}

#undef LOCTEXT_NAMESPACE