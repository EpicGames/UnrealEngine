// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "PropertyHandle.h"
#include "IPropertyTypeCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "IPropertyUtilities.h"
#include "Misc/Variant.h"

#include "HarmonixDsp/Containers/TypedParameter.h"

class FTypedParameterCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FTypedParameterCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	// enum dropdown callbacks
	void OnPropertyTypeEnumSelectionChanged(int32 TypeIndex, ESelectInfo::Type SelectInfo);
	int32 OnGetPropertyTypeEnumValue() const;

	// creates the actual value widget based on the parameter type
	TSharedRef<class SWidget> CreateTypedValueWidget(EParameterType Type);

	// get the underlying FTypedParemeter as a ptr
	FTypedParameter* GetTypedParameterPtr();
	const FTypedParameter* GetTypedParameterPtr() const;

	// templated methods for numeric value widget callbacks
	template<typename T>
	TOptional<T> OnGetNumericValue() const
	{
		TOptional<T> OutValue;
		if (const FTypedParameter* TypedParameter = GetTypedParameterPtr())
		{
			if (TypedParameter->IsType<T>())
			{
				OutValue = TypedParameter->GetValue<T>();
			}
		}

		return OutValue;
	}

	template<typename T>
	void OnNumericValueChanged(T NewValue)
	{
		if (FTypedParameter* TypedParameter = GetTypedParameterPtr())
		{
			//TypedParameter->Value = FVariant(NewValue);
		}
	}

	template<typename T>
	void OnNumericValueCommitted(T NewValue, ETextCommit::Type CommitType)
	{
		if (FTypedParameter* TypedParameter = GetTypedParameterPtr())
		{
			TypedParameterHandle->NotifyPreChange();
			TypedParameter->SetValue<T>(NewValue);
			TypedParameterHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			TypedParameterHandle->NotifyFinishedChangingProperties();
		}
	}

	// templated method for creating a numeric value widget
	// using the templated callback methods abov
	template<typename T>
	TSharedRef<SWidget> CreateNumericValueWidget()
	{
		return SNew(SNumericEntryBox<T>)
			.Value(this, &FTypedParameterCustomization::OnGetNumericValue)
			.OnValueChanged(this, &FTypedParameterCustomization::OnNumericValueChanged)
			.OnValueCommitted(this, &FTypedParameterCustomization::OnNumericValueCommitted);
	}

	// check box widget callbacks
	ECheckBoxState OnGetCheckedState() const;
	void OnCheckedStateChanged(ECheckBoxState State);

	// Text Box widget callbacks for FName and FString value types
	FText OnGetTextValue() const;
	void OnTextComitted(const FText& InText, ETextCommit::Type TextCommit);

	// Ptr to the value container widget
	// used to swap out the type of value widget used
	// based on the type being editor
	TSharedPtr<class SHorizontalBox> ValueContainerWidget;

	/** Handle to the struct being customized */
	TSharedPtr<IPropertyHandle> TypedParameterHandle;
};