// Copyright Epic Games, Inc. All Rights Reserved.
#include "Dta/DtaParser.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Char.h"

DEFINE_LOG_CATEGORY(LogDtaParser)

#define LOCTEXT_NAMESPACE "DtaParser"

/**
*
* The Dta Grammar
*
*
* \\Pair: \(LHS RHS\)
* Pair: \(Symbol (Value+ | Object | (\(Object\))+)\)
* Object: Pair+
* Value: String | Number
* Symbol: [a-zA-Z_]
* String: \"[a-zA-Z_\.\/]\"
* Number: [0-9]+(\.[0-9]+)?
*
* Root: Object | (\(Object\))+
*
* <Root> is the starting point of reading a Dta
* - Which results in a single object or an array of objects
* We create
*
***/

class FParseNode
{
public:
	enum Type {
		Type_Object,
		Type_Array,
		Type_Pair,
		Type_Token,
		Type_Open,
		Type_Close,
		Type_None
	};

	// intentionally left unimplemented
	// not all subclasses need to implement
	virtual void ToJson(FString& OutJson) const { unimplemented(); };

	virtual Type GetType() const = 0;

	virtual ~FParseNode() {};

};

class FNode_Token : public FParseNode
{
public:
	FNode_Token(const FString& InToken) : Token(InToken) {};
	virtual Type GetType() const override { return Type_Token; }

	virtual void ToJson(FString& OutJson) const override
	{
		OutJson += Token;
	}

	const FString& GetTokenString() const { return Token; }

private:
	FString Token;
};

class FNode_Pair : public FParseNode
{
public:

	FNode_Pair(TSharedPtr<FNode_Token> InKey, TSharedPtr<FParseNode> InValue)
		: Key(InKey), Value(InValue) {}

	virtual Type GetType() const override { return Type_Pair; }

	virtual void ToJson(FString& OutJson) const override
	{
		OutJson += '"';
		Key->ToJson(OutJson);
		OutJson += '"';
		OutJson += ':';
		Value->ToJson(OutJson);
	}

	const FString& GetKeyString() const { return Key->GetTokenString(); }

	TSharedPtr<FParseNode> GetValue() { return Value; }
	
private:
	TSharedPtr<FNode_Token> Key;
	TSharedPtr<FParseNode> Value;
};

class FNode_Object : public FParseNode
{
public:
	FNode_Object() {}

	virtual Type GetType() const override { return Type_Object; }

	void Add(TSharedPtr<FNode_Pair> Pair) { Pairs.Add(Pair); }
	
	virtual void ToJson(FString& OutJson) const override
	{
		// quick check to see if *all* the keys in the map are identical
		// won't work if not all key names are the same
		const FString* LastKey = nullptr;
		bool HasIdenticalKeys = false;
		for (auto Pair : Pairs)
		{
			const FString& Key = Pair->GetKeyString();
			if (LastKey && Pair->GetKeyString().Equals(*LastKey))
			{
				HasIdenticalKeys = true;
			}
			else if (LastKey)
			{
				HasIdenticalKeys = false;
				break;
			}
			LastKey = &Key;
		}

		// if this object identical keys, turn it into an array of objects
		if (HasIdenticalKeys)
		{
			OutJson += '[';
			for (int idx = 0; idx < Pairs.Num(); ++idx)
			{
				auto Pair = Pairs[idx];
				OutJson += '{';
				Pair->ToJson(OutJson);
				OutJson += '}';
				if (idx + 1 < Pairs.Num())
					OutJson += ',';
			}
			OutJson += ']';
		}
		else // otherwise, keep it as an object with key-value pairs
		{
			TSet<FString> Keys;
			OutJson += '{';
			for (int idx = 0; idx < Pairs.Num(); ++idx)
			{
				auto Pair = Pairs[idx];
				if (Keys.Contains(Pair->GetKeyString()))
				{
					// ignore duplicate keys
					// (preferring the first values we see!)
					continue;
				}
				Keys.Add(Pair->GetKeyString());
				Pair->ToJson(OutJson);
				if (idx + 1 < Pairs.Num())
					OutJson += ',';
			}
			OutJson += '}';
		}
	}

private:
	TArray<TSharedPtr<FNode_Pair>> Pairs;
};

class FNode_Array : public FParseNode
{
public:
	FNode_Array() {};

	virtual Type GetType() const override { return Type_Array; }

	void Add(TSharedPtr<FParseNode> Node) { Values.Add(Node); }

	virtual void ToJson(FString& OutJson) const override
	{
		OutJson += '[';
		for (int idx = 0; idx < Values.Num(); ++idx)
		{
			auto Item = Values[idx];
			Item->ToJson(OutJson);
			if (idx + 1 < Values.Num())
				OutJson += ',';
		}
		OutJson += ']';
	}

private:

	TArray<TSharedPtr<FParseNode>> Values;
};



class FNode_Open : public FParseNode
{
public:
	FNode_Open() {};
	virtual Type GetType() const override { return Type_Open; }


};

class FNode_Close : public FParseNode
{
public:
	FNode_Close() {};
	virtual Type GetType() const override { return Type_Close; }
};


bool FDtaParser::DtaStringToJsonString(const FString& InDtaString, FString& OutJsonString, FString& OutErrorMessage)
{
	TArray<FString> Tokens;
	FParseContext Context;
	if (!Tokenize(InDtaString, Tokens, Context))
	{
		OutErrorMessage = FText::Format(LOCTEXT("DtaTokenizerError", "Error parsing DTA at {0} - {1}"),
			FText::FromString(FString::Printf(TEXT("(%d, %d): \'%c\'"), Context.LineIdx + 1, Context.CharIdx + 1, Context.CurrentChar)),
			FText::FromString(Context.GetErrorAsString())).ToString();
		return false;
	}
	
	FString ParseError;
	if (!ParseToJson(Tokens, OutJsonString, ParseError))
	{
		OutErrorMessage = FText::Format(LOCTEXT("DtaParserError", "Error parsing DTA: {0}"),
			FText::FromString(ParseError)).ToString();
		return false;
	}

	return true;
}

bool FDtaParser::ParseToJson(const TArray<FString>& Tokens, FString& OutJsonString, FString& OutErrorMessage)
{
	TArray<TSharedPtr<FParseNode>> Stack;
	for (int idx = 0; idx < Tokens.Num(); ++idx)
	{
		const FString& Token_0 = Tokens[idx];

		if (Token_0[0] == '(')
		{
			Stack.Add(MakeShared<FNode_Open>());
		}
		else if (Token_0[0] == ')')
		{
			TArray<TSharedPtr<FParseNode>> TempStack;
			
			while (!Stack.IsEmpty() && Stack.Last()->GetType() != FParseNode::Type_Open)
			{
				TempStack.Add(Stack.Pop());
			}

			// this should be guaranteed since we're keeping track of parens in the tokenizer
			if (!ensure(Stack.Num() > 0))
			{
				OutErrorMessage = LOCTEXT("DtaParserError_UnmatchedCloseParen", "Unmatched closing parenthesis").ToString();
				return false;
			}

			// pop off the open paren
			Stack.Pop();

			if (TempStack.Num() == 0)
			{
				OutErrorMessage = LOCTEXT("DtaParserError_EmptyParens", "Empty parentheses \"()\" are not valid DTA").ToString();
				return false;
			}

			// this is a list of pairs, so we need to turn it into a dictionary
			if (TempStack.Last()->GetType() == FParseNode::Type_Pair)
			{
				TSharedPtr<FNode_Object> NewDict = MakeShared<FNode_Object>();
				while (TempStack.Num())
				{
					TSharedPtr<FParseNode> Node = TempStack.Pop();
					TSharedPtr<FNode_Pair> Pair = StaticCastSharedPtr<FNode_Pair>(Node);
					check(Pair);
					NewDict->Add(Pair);
				}

				Stack.Add(NewDict);
			}
			else if (TempStack.Num() > 2)
			{
				TSharedPtr<FNode_Token> Key = StaticCastSharedPtr<FNode_Token>((TempStack.Pop()));
				check(Key);

				TSharedPtr<FParseNode> NewValue;
				if (TempStack.Last()->GetType() == FParseNode::Type_Pair)
				{
					TSharedPtr<FNode_Object> NewDict = MakeShared<FNode_Object>();
					while (TempStack.Num())
					{
						TSharedPtr<FParseNode> Node = TempStack.Pop();
						TSharedPtr<FNode_Pair> Pair = StaticCastSharedPtr<FNode_Pair>(Node);
						check(Pair);
						NewDict->Add(Pair);
					}

					NewValue = NewDict;
				}
				else
				{
					TSharedPtr<FNode_Array> NewArray = MakeShared<FNode_Array>();

					while (TempStack.Num())
					{
						TSharedPtr<FParseNode> Node = TempStack.Pop();
						NewArray->Add(Node);
					}

					NewValue = NewArray;
				}
				
				check(NewValue);
				TSharedPtr<FNode_Pair> NewPair = MakeShared<FNode_Pair>(Key, NewValue);
				Stack.Add(NewPair);
			}
			else if (TempStack.Num() == 2)
			{
				TSharedPtr<FNode_Token> Key = StaticCastSharedPtr<FNode_Token>(TempStack.Pop());
				check(Key);

				TSharedPtr<FParseNode> Value;
				if (TempStack.Last()->GetType() == FParseNode::Type_Pair)
				{
					TSharedPtr<FNode_Pair> Pair = StaticCastSharedPtr<FNode_Pair>(TempStack.Pop());
					TSharedPtr<FNode_Object> NewDict = MakeShared<FNode_Object>();
					NewDict->Add(Pair);
					Value = NewDict;
				}
				else
				{
					Value = TempStack.Pop();
				}

				TSharedPtr<FNode_Pair> NewPair = MakeShared<FNode_Pair>(Key, Value);
				Stack.Add(NewPair);
			}
			else
			{
				OutErrorMessage = LOCTEXT("DtaParserError_SingleToken", "Expected a key-value pair but found a single token in parentheses").ToString();
				return false;
			}
		}
		else
		{
			TSharedPtr<FNode_Token> NewToken = MakeShared<FNode_Token>(Token_0);
			Stack.Add(NewToken);
		}
	}

	if (Stack.Num() == 0)
	{
		OutJsonString = TEXT("{}");
		return true;
	}

	// Check for bare tokens at root level - these are always invalid
	// (e.g., a stray character before the first paren)
	// also..
	// Check if all remaining items are pairs (the common case for top-level DTA)
	bool bAllPairs = true;
	for (const auto& Node : Stack)
	{
		if (Node->GetType() == FParseNode::Type_Token)
		{
			OutErrorMessage = LOCTEXT("DtaParserError_BareToken", "Unexpected token outside of parentheses").ToString();
			return false;
		}

		if (Node->GetType() != FParseNode::Type_Pair)
		{
			bAllPairs = false;
		}
	}

	if (bAllPairs)
	{
		// Normal case: wrap top-level pairs into a root object
		TSharedPtr<FNode_Object> RootObject = MakeShared<FNode_Object>();
		while (Stack.Num())
		{
			TSharedPtr<FNode_Pair> Pair = StaticCastSharedPtr<FNode_Pair>(Stack.Pop());
			check(Pair);
			RootObject->Add(Pair);
		}
		RootObject->ToJson(OutJsonString);
	}
	else if (Stack.Num() == 1)
	{
		// Single non-pair item (e.g., a wrapped object or array)
		Stack[0]->ToJson(OutJsonString);
	}
	else
	{
		// Multiple non-pair items: wrap in a JSON array
		TSharedPtr<FNode_Array> RootArray = MakeShared<FNode_Array>();
		for (const auto& Node : Stack)
		{
			RootArray->Add(Node);
		}
		RootArray->ToJson(OutJsonString);
	}

	return true;
}

void FDtaParser::FParseContext::Reset()
{
	LineIdx = 0;
	CharIdx = 0;
	CurrentChar = '\0';
	PrevChar = '\0';
}

FString FDtaParser::FParseContext::GetErrorAsString() const
{
	switch (Error)
	{
	case ParenthesisMismatch:
		return LOCTEXT("DtaParserError_ParenMismatch", "Paranthesis Mismatch").ToString();
	case UnexpectedEndOfFile:
		return LOCTEXT("DtaParserError_UnexpectedEOF", "Unexpected End Of File").ToString();
	default:
		return LOCTEXT("DtaParserError_Unexpected", "Unexpected Error").ToString();
	}
}


bool FDtaParser::Tokenize(const FString& InDtaString, TArray<FString>& OutTokens, FParseContext& OutContext)
{
	OutTokens.Reset();
	OutContext.Reset();

	static const int kSlack = 10;
	FString Token;
	Token.Empty(kSlack);

	int Num = InDtaString.Len();
	bool IsInCommentSection = false;
	int OpenParenCount = 0;
	for (int Pos = 0; Pos < Num; ++Pos, ++OutContext.CharIdx)
	{
		OutContext.PrevChar = OutContext.CurrentChar;
		OutContext.CurrentChar = InDtaString[Pos];

		if (OutContext.CurrentChar == '\n')
		{
			++OutContext.LineIdx;
			OutContext.CharIdx = 0;
		}

		if (IsInCommentSection)
		{
			// break out of the comment section if we hit a newline.
			if (OutContext.CurrentChar == '\n')
			{
				IsInCommentSection = false;
			}
		}
		else if (OutContext.CurrentChar == ';')
		{
			// found a comment,
			// add any token we were building
			// and skip over characters until
			// we hit a newline
			IsInCommentSection = true;

			if (!Token.IsEmpty())
			{
				OutTokens.Add(Token);
			}

			Token.Empty(kSlack);
			continue;
		}
		else if (FChar::IsWhitespace(OutContext.CurrentChar))
		{
			if (!Token.IsEmpty())
			{
				OutTokens.Add(Token);
				Token.Empty(kSlack);
			}
			continue;
		}
		else if (OutContext.CurrentChar == '(' || OutContext.CurrentChar == ')')
		{
			if (!Token.IsEmpty())
			{
				OutTokens.Add(Token);
				Token.Empty(kSlack);
			}

			if (OutContext.CurrentChar == '(')
			{
				++OpenParenCount;
			}
			else
			{
				--OpenParenCount;
			}

			if (OpenParenCount < 0)
			{
				OutContext.Error = FParseContext::ParenthesisMismatch;
				return false;
			}
			// add the parenthesis as a token
			OutTokens.Add(FString::Chr(OutContext.CurrentChar));
		}
		else
		{
			Token += OutContext.CurrentChar;
		}
	}
	
	if (OpenParenCount > 0)
	{
		OutContext.Error = FParseContext::UnexpectedEndOfFile;
		return false;
	}

	return true;
};

#undef LOCTEXT_NAMESPACE