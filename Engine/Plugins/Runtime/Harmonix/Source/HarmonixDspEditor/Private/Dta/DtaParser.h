// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"


DECLARE_LOG_CATEGORY_EXTERN(LogDtaParser, Log, All);

#define UE_API HARMONIXDSPEDITOR_API

struct FDtaParser
{
	UE_API static bool DtaStringToJsonString(const FString& InDtaString, FString& OutJsonString, FString& OutErrorMessage);

private:

	struct FParseContext
	{
		enum EError
		{
			None,
			ParenthesisMismatch,
			UnexpectedEndOfFile
		};
		
		// line index in the dta string being read
		int LineIdx = 0;
		
		// character index within the line
		int CharIdx = 0;

		// the character being read
		TCHAR CurrentChar = '\0';

		// the previous "character" read 
		TCHAR PrevChar = '\0';

		EError Error = EError::None;
		
		void Reset();

		FString GetErrorAsString() const;
	};
	
	static bool Tokenize(const FString& InDtaString, TArray<FString>& OutTokens, FParseContext& OutContext);
	static bool ParseToJson(const TArray<FString>& Tokens, FString& OutString, FString& OutErrorMessage);
};

#undef UE_API
