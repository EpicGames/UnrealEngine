// Copyright Epic Games, Inc. All Rights Reserved.

#include "FusionPatchAssetFactory.h"
#include "FusionPatchImportOptions.h"
#include "EditorDialogLibrary.h"

#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "Dta/DtaParser.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include "JsonImporterHelper.h"
#include "FusionPatchJsonImporter.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "EditorFramework/AssetImportData.h"
#include "FileHelpers.h"
#include "UObject/Package.h"
#include "HAL/FileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FusionPatchAssetFactory)

#define LOCTEXT_NAMESPACE "FusionPatchAssetFactory"

DEFINE_LOG_CATEGORY(LogFusionPatchAssetFactory)

UFusionPatchAssetFactory::UFusionPatchAssetFactory()
{
	SupportedClass = UFusionPatch::StaticClass();
	Formats.Add(TEXT("fusion;Fusion Patch"));
	bText = true;
	bCreateNew = false;
	bEditorImport = true;
	ImportPriority = DefaultImportPriority + 20;
}

bool UFusionPatchAssetFactory::FactoryCanImport(const FString& Filename)
{
	return true;
}

bool UFusionPatchAssetFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	if (UFusionPatch* FusionPatch = Cast<UFusionPatch>(Obj))
	{
		// the PreferredReimportPath is the path of the new file, which can differ from the original SrcFilePath
		// if the file has the same name but a different extension, don't import it!
		if (!PreferredReimportPath.IsEmpty() && !IsSupportedFileExtension(PreferredReimportPath))
		{
			UE_LOGF(LogFusionPatchAssetFactory, Warning, "%ls: Failed to reimport with new file. Invalid extension: %ls", *FusionPatch->GetPathName(), *FPaths::GetExtension(PreferredReimportPath));
			return false;
		}

		if (FusionPatch->AssetImportData)
		{
			FusionPatch->AssetImportData->ExtractFilenames(OutFilenames);
		}
		return true;
	}
	return false;
}

void UFusionPatchAssetFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UFusionPatch* FusionPatch = Cast<UFusionPatch>(Obj);
	if (FusionPatch && ensure(NewReimportPaths.Num() == 1))
	{
		FusionPatch->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UFusionPatchAssetFactory::Reimport(UObject* Obj)
{
	UFusionPatch* FusionPatch = Cast<UFusionPatch>(Obj);
	if (!FusionPatch)
	{
		return EReimportResult::Failed;
	}

	if (!FusionPatch->AssetImportData)
	{
		return EReimportResult::Failed;
	}

	const FString Filename = FusionPatch->AssetImportData->GetFirstFilename();
	if (!Filename.Len() || IFileManager::Get().FileSize(*Filename) == INDEX_NONE)
	{
		return EReimportResult::Failed;
	}
	
	bool OutCanceled = false;
	if (UObject* ImportedObject = ImportObject(FusionPatch->GetClass(), FusionPatch->GetOuter(), *FusionPatch->GetName(), RF_Public | RF_Standalone, Filename, nullptr, OutCanceled))
	{
		// The fusion patch should have just been updated, right?
		ensure(ImportedObject == FusionPatch);

		if (FusionPatch->GetOuter())
		{
			FusionPatch->GetOuter()->MarkPackageDirty();
		}
		else
		{
			FusionPatch->MarkPackageDirty();
		}
		return EReimportResult::Succeeded;
	}
	
	if (OutCanceled)
	{
		UE_LOGF(LogFusionPatchAssetFactory, Warning, "import canceled");
		return EReimportResult::Cancelled;
	}

	UE_LOGF(LogFusionPatchAssetFactory, Warning, "import failed");
	return EReimportResult::Failed;
}

void UFusionPatchAssetFactory::CleanUp()
{
	ImportCounter = 0;
	ApplyOptionsToAllImport = EApplyAllOption::Unset;
	ReplaceExistingSamples = true;

	// prompt to save the imported objects collected during entire import
	TArray<UPackage*> PackagesToSave;
	Algo::Transform(ImportedObjects, PackagesToSave, [](const UObject* Object) { return Object->GetPackage(); });
    UEditorLoadingAndSavingUtils::SavePackagesWithDialog(PackagesToSave, true);
    ImportedObjects.Empty();

	UFactory::CleanUp();
}


bool UFusionPatchAssetFactory::GetReplaceExistingSamplesResponse(const FString& InName)
{
	const FText ReplaceExistingTitle = NSLOCTEXT("FusionPatchImporter", "ReplaceExistingSamplesTitle", "Replace Existing Samples");
	const FText ReplaceExistingMessage = FText::Format(NSLOCTEXT("FusionPatchImporter", "ReplaceExistingSamplesMsg", 
		"You are Reimporting a Fusion Patch with existing samples. Would you like to reimport existing Sound Wave Assets?" 
		"\n\nPatch Name: {0}"
		"\n\nYes. Reimport existing Samples. *If you made changes to any samples*, you will want to do this."
		"\n\nNo.  Don't reimport existing Samples. Just reimport the Fusion Patch settings"), 
		FText::FromString(InName));
	EAppReturnType::Type ReplaceExistingSamplesResponse = UEditorDialogLibrary::ShowMessage(ReplaceExistingTitle, ReplaceExistingMessage, EAppMsgType::YesNo, EAppReturnType::No, EAppMsgCategory::Info);
	

	switch (ReplaceExistingSamplesResponse)
	{
	case EAppReturnType::Yes:
		return true;
	case EAppReturnType::No:
		return false;
	default:
		ensureMsgf(false, TEXT("Unexpected response! default behavior is to NOT replace existing samples"));
		return false;
	}
}

bool UFusionPatchAssetFactory::GetApplyOptionsToAllImportResponse()
{
	const FText ReplaceExistingTitle = NSLOCTEXT("FusionPatchImporter", "ApplyOptionsToAllTitle", "Apply Options to All");
	const FText ReplaceExistingMessage = NSLOCTEXT("FusionPatchImporter", "ApplyOptionsToALlMsg", "Would you like to apply the selected options to all files being imported?");
	EAppReturnType::Type Response = UEditorDialogLibrary::ShowMessage(ReplaceExistingTitle, ReplaceExistingMessage, EAppMsgType::YesNo, EAppReturnType::No, EAppMsgCategory::Info);

	switch (Response)
	{
	case EAppReturnType::Yes:
		return true;
	case EAppReturnType::No:
		return false;
	default:
		ensureMsgf(false, TEXT("Unexpected response! default behavior is to NOT apply all"));
		return false;
	}
}


void UFusionPatchAssetFactory::UpdateFusionPatchImportNotificationItem(TSharedPtr<SNotificationItem> InItem, bool bImportSuccessful, FName InName)
{
	if (bImportSuccessful)
	{
		InItem->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
		InItem->SetText(FText::Format(NSLOCTEXT("FusionPatchImporter", "FusionPatchImportProgressNotification_Success", "Successfully imported Fusion Patch asset: {0}"), FText::FromString(InName.ToString())));
	}
	else {
		InItem->SetCompletionState(SNotificationItem::ECompletionState::CS_Fail);
		InItem->SetText(FText::Format(NSLOCTEXT("FusionPatchImporter", "FusionPatchImportProgressNotification_Failure", "Failed to import Fusion Patch: {0}"), FText::FromString(InName.ToString())));
	}
	InItem->SetExpireDuration(0.2f);
	InItem->ExpireAndFadeout();
}

UObject* UFusionPatchAssetFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	AdditionalImportedObjects.Empty();
	
	// get the existing fusion patch if we're reimporting
	UFusionPatch* FusionPatch = FindObject<UFusionPatch>(InParent, *InName.ToString());
	
	const FString LongPackagePath = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName());

	const UFusionPatchImportOptions* ImportOptions = GetDefault<UFusionPatchImportOptions>();

	// detect when we're importing another file so we can ask if we would like to apply the previously set settings to this and all other files
	// import counter gets reset after all files have been imported
	++ImportCounter;
	if (ApplyOptionsToAllImport == EApplyAllOption::Unset && ImportCounter > 1)
	{
		ApplyOptionsToAllImport = GetApplyOptionsToAllImportResponse() ? EApplyAllOption::Yes : EApplyAllOption::No;
	}
	
	if (ApplyOptionsToAllImport != EApplyAllOption::Yes)
	{
		UFusionPatchImportOptions::FArgs Args;
		Args.PatchName = InName;
		// If we're reimporting and the fusion patch has saved off the samples directory
		if (FusionPatch && !FusionPatch->SamplesImportDir.IsEmpty())
		{
			Args.Directory = FusionPatch->SamplesImportDir;
		}
		else
		{
			// Default samples directory to subdirectory in current directory: [CurrentDirectory] / [PatchName]
			Args.Directory = LongPackagePath / InName.ToString();
		}

		bool WasOkayPressed = false;
		ImportOptions = UFusionPatchImportOptions::GetWithDialog(MoveTemp(Args), WasOkayPressed);
		if (!WasOkayPressed)
		{
			// import cancelled by user
			return nullptr;
		}

		if (!ensure(ImportOptions))
		{
			return nullptr;
		}

		if (Warn->ReceivedUserCancel())
		{
			return nullptr;
		}

		// If the fusion patch already exists, ask whether we want to replace existing samples.
		// otherwise, always replace existing samples by default
		ReplaceExistingSamples = FusionPatch != nullptr ? GetReplaceExistingSamplesResponse(InName.ToString()) : true;
	}
	
	const FString SourceFile = GetCurrentFilename();
	FString JsonString;
	const FString DtaString = FString::ConstructFromPtrSize(Buffer, BufferEnd - Buffer);
	FString DtaErrorMessage;
	if (!FDtaParser::DtaStringToJsonString(DtaString, JsonString, DtaErrorMessage))
	{
		const FText ImportErrorMessage = FText::Format(
			LOCTEXT("ImportFailed_DtaToJson", "Failed to import asset:\n'{0}'.\nFailed to read .fusion file - data malformed: {1}\n"),
			FText::FromString(SourceFile), FText::FromString(DtaErrorMessage)
		);
		FMessageDialog::Open(EAppMsgType::Ok, ImportErrorMessage);
		UE_LOGF(LogFusionPatchAssetFactory, Error, "Failed to import .fusion asset: %ls - %ls", *SourceFile, *DtaErrorMessage);
		return nullptr;
	}
	
	FString ErrorMessage;
	TSharedPtr<FJsonObject> JsonObj = FJsonImporter::ParseJsonString(JsonString, ErrorMessage);
	bool bImportSuccessful = false;
	if (JsonObj.IsValid())
	{
		if (!FusionPatch)
		{
			FusionPatch = NewObject<UFusionPatch>(InParent, InName, Flags);
		}

		
		const FString SourcePath = FPaths::GetPath(SourceFile);
		
		//create a notification that displays the import progress at the lower right corner
		FNotificationInfo ImportNotificationInfo(NSLOCTEXT("FusionPatchImporter","FusionPatchImportProgressNotification_InProgress","Importing Fusion Asset(s)..."));
		ImportNotificationInfo.bFireAndForget = false;
		TSharedPtr<SNotificationItem> ImportNotificationItem;
		ImportNotificationItem = FSlateNotificationManager::Get().AddNotification(ImportNotificationInfo);


		// Pass import args to parser so it can import sub files
		FFusionPatchJsonImporter::FImportArgs ImportArgs(InName, SourcePath, LongPackagePath, ImportOptions->SamplesImportDir.Path, ReplaceExistingSamples);
		ImportArgs.SampleLoadingBehavior = ImportOptions->SampleLoadingBehavior;
		ImportArgs.SampleCompressionType = ImportOptions->SampleCompressionType;

		TArray<FString> ImportErrors;
		if (FFusionPatchJsonImporter::TryParseJson(JsonObj, FusionPatch, AdditionalImportedObjects, ImportArgs, ImportErrors))
		{
			UE_LOGF(LogFusionPatchAssetFactory, Log, "Successfully imported FusionPatch asset");

			if (ensure(FusionPatch->AssetImportData))
			{
				FusionPatch->AssetImportData->Update(SourceFile);
			}

			// save off the samples dest path for simplifying reimporting
			FusionPatch->SamplesImportDir = ImportArgs.SamplesDestPath;
			
			bImportSuccessful = true;
			UpdateFusionPatchImportNotificationItem(ImportNotificationItem, bImportSuccessful, InName);
			ImportedObjects.Append(AdditionalImportedObjects);
			ImportedObjects.Add(FusionPatch);
			return FusionPatch;
		}
		else
		{
			const FText ImportErrorMessage = FText::Format(
				LOCTEXT("ImportFailed_FusionPatchJsonImporter",
					"Failed to import asset:\n'{0}'.\nReasons:\n{1}"),
				FText::FromString(SourceFile),
				FText::FromString(FString::Printf(TEXT("%s"), *FString::Join(ImportErrors, TEXT("\n")))));
			FMessageDialog::Open(EAppMsgType::Ok, ImportErrorMessage);
			UE_LOGF(LogFusionPatchAssetFactory, Error, "Failed to import fusion patch: %ls", *ImportErrorMessage.ToString());
			
			UpdateFusionPatchImportNotificationItem(ImportNotificationItem, bImportSuccessful, InName);
		}

	}
	else
	{
		const FText ImportErrorMessage = FText::Format(LOCTEXT("ImportFailed_Json", "Failed to import asset:\n'{0}'.\nFailed to read Json:\n{1}"), FText::FromString(SourceFile), FText::FromString(ErrorMessage));
		FMessageDialog::Open(EAppMsgType::Ok, ImportErrorMessage);
		UE_LOGF(LogFusionPatchAssetFactory, Error, "Failed to read json: %ls", *ErrorMessage);
	}

	return nullptr;
}

UObject* UFusionPatchAssetFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UFusionPatch* NewAsset = NewObject<UFusionPatch>(InParent, InName, Flags);

	if (!NewAsset)
	{
		return nullptr;
	}

	if (CreateOptions)
	{
		NewAsset->UpdateKeyzones(CreateOptions->Keyzones);
		NewAsset->UpdateSettings(CreateOptions->FusionPatchSettings);
		CreateOptions = nullptr;
	}

	return NewAsset;
}

#undef LOCTEXT_NAMESPACE
