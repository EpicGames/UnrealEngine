// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "EditorReimportHandler.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

#include "FusionPatchAssetFactory.generated.h"

#define UE_API HARMONIXDSPEDITOR_API

class UFusionPatchImportOptions;
class UFusionPatchCreateOptions;

DECLARE_LOG_CATEGORY_EXTERN(LogFusionPatchAssetFactory, Log, All);

UCLASS(MinimalAPI)
class UFusionPatchAssetFactory : public UFactory, public FReimportHandler
{
	GENERATED_BODY()

public:

	UE_API UFusionPatchAssetFactory();

	UE_API virtual bool FactoryCanImport(const FString& Filename) override;
	UE_API virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	UE_API virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;;
	UE_API virtual EReimportResult::Type Reimport(UObject* Obj) override;
	UE_API virtual void CleanUp() override;

	UE_API virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn) override;
	UE_API virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;

	const UFusionPatchCreateOptions* CreateOptions = nullptr;
private:

	static UE_API bool GetReplaceExistingSamplesResponse(const FString& InName);
	static UE_API bool GetApplyOptionsToAllImportResponse();

	enum class EApplyAllOption : uint8
	{
		Unset,
		No,
		Yes
	};

	int32 ImportCounter = 0;
	EApplyAllOption ApplyOptionsToAllImport = EApplyAllOption::Unset;
	bool ReplaceExistingSamples = false;
	TArray<UObject*> ImportedObjects;

	UE_API void UpdateFusionPatchImportNotificationItem(TSharedPtr<SNotificationItem> InItem, bool bImportSuccessful, FName InName);
};

#undef UE_API
