// Copyright Epic Games, Inc. All Rights Reserved.

#include "FusionPatchImportOptions.h"
#include "EditorDialogLibrary.h"
#include "UObject/Package.h"
#include "Internationalization/Regex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FusionPatchImportOptions)

#if WITH_EDITOR

struct FMidiScale
{
public:
	
	FMidiScale(TArray<int8>& InNotes)
		: Notes(InNotes)
	{
	}
	
	FMidiScale(TArray<int8>&& InNotes)
	: Notes(MoveTemp(InNotes))
	{
	}

	int8 operator[](int32 Idx) const
	{
		int32 NoteIdx = Idx % Notes.Num();
		int32 OctaveIdx = Idx / Notes.Num();
		return Notes[NoteIdx] + (OctaveIdx * 12);
	}

	int32 NumNotes() const { return Notes.Num(); }

	static const FMidiScale MajorScale;
	
private:
	TArray<int8> Notes;
};

const FMidiScale FMidiScale::MajorScale(TArray<int8>({0, 2, 4, 5, 7, 9, 11}));

const UFusionPatchImportOptions* UFusionPatchImportOptions::GetWithDialog(FArgs&& Args, bool& OutWasOkayPressed)
{
	UFusionPatchImportOptions* Options = GetMutableDefault<UFusionPatchImportOptions>();
	Options->SamplesImportDir.Path = MoveTemp(Args.Directory);
	FFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("PatchName"), FText::FromName(Args.PatchName));
	const FText ImportDialogTitle = FText::Format(NSLOCTEXT("FusionPatchImportOptions", "FusionPatchImportOptionsTitle", "Fusion Patch Import Options: {PatchName}"), FormatArgs);
	OutWasOkayPressed = UEditorDialogLibrary::ShowObjectDetailsView(ImportDialogTitle, Options);
	return Options;
}

const UFusionPatchCreateOptions* UFusionPatchCreateOptions::GetWithDialog(FArgs&& Args, bool& OutWasOkayPressed)
{
	UFusionPatchCreateOptions* Options = GetMutableDefault<UFusionPatchCreateOptions>();
	Options->FusionPatchDir.Path = MoveTemp(Args.Directory);
	Options->AssetName = MoveTemp(Args.AssetName);
	Options->StagedSoundWaves = MoveTemp(Args.StagedSoundWaves);
	Options->UpdateKeyzonesWithSettings();
	const FText ImportDialogTitle = NSLOCTEXT("FusionPatchCreateOptions", "FusionPatchCreateOptionsTitle", "New Fusion Patch Options");
	FEditorDialogLibraryObjectDetailsViewOptions DialogOptions;
	DialogOptions.bAllowResizing = true;
	OutWasOkayPressed = UEditorDialogLibrary::ShowObjectDetailsView(ImportDialogTitle, Options, DialogOptions);
	Options->StagedSoundWaves.Empty();

	// apply some default settings that will work out of the box
	FAdsrSettings& VolumeAdsr = Options->FusionPatchSettings.Adsr[0];
	VolumeAdsr.Target = EAdsrTarget::Volume;
	VolumeAdsr.IsEnabled = true;
	VolumeAdsr.SustainLevel = 1.0f;
	VolumeAdsr.Depth = 1.0f;
	
	return Options;
}

void UFusionPatchCreateOptions::UpdateKeyzonesWithSettings()
{
	// autodetect parse option based on the sort option
	// default to none
	LockedNotesMask = ELockedNoteFlag::None;
	switch (SortOption)
	{
	case EFusionPatchKeyzoneSortOption::Lexical:
		SortLexical();
		break;
	case EFusionPatchKeyzoneSortOption::NoteNumber:
		SortNoteNumber();
		break;
	case EFusionPatchKeyzoneSortOption::NoteName:
		SortNoteName();
		break;
	case EFusionPatchKeyzoneSortOption::Index:
		SortIndex();
		break;
	}

	// apply scale to the root notes if we didn't parse notes from the note names
	if (LockedNotesMask == ELockedNoteFlag::None)
	{
		const FMidiScale* Scale = nullptr;
		switch (ScaleOption)
		{
		case EFusionPatchKeyzoneNoteScaleOption::MajorScale: Scale = &FMidiScale::MajorScale; break;
		}
		
		if (Scale)
		{
			for (int32 Idx = 0; Idx < Keyzones.Num(); ++Idx)
			{
				Keyzones[Idx].RootNote = FMath::Clamp(MinNote + (*Scale)[Idx], MinNote, MaxNote);
				Keyzones[Idx].MinNote = Keyzones[Idx].RootNote;
				Keyzones[Idx].MaxNote = Keyzones[Idx].RootNote;
			}
			LockedNotesMask = ELockedNoteFlag::Root;
		}
	}

	// apply the octave offset after applying the initial values
	for (FKeyzoneSettings& Keyzone : Keyzones)
	{
		Keyzone.RootNote = FMath::Clamp(Keyzone.RootNote + OctaveAdjust * 12, 0, 127);
		Keyzone.MinNote = FMath::Clamp(Keyzone.MinNote + OctaveAdjust * 12, 0, 127);
		Keyzone.MaxNote = FMath::Clamp(Keyzone.MaxNote + OctaveAdjust * 12, 0, 127);
	}

	// if the min and max notes haven't already been determined from the file names, lay them out
	if (ELockedNoteFlag::None ==  (LockedNotesMask & ELockedNoteFlag::MinMax))
	{
		switch (LayoutOption)
		{
		case EFusionPatchKeyzoneNoteLayoutOption::SingleNote:
			LayoutSingleNote();
			break;
		case EFusionPatchKeyzoneNoteLayoutOption::Distribute:
			LayoutDistribute();
			break;
		case EFusionPatchKeyzoneNoteLayoutOption::Layer:
			LayoutLayer();
			break;
		}
	}
	
	// set the root note only if we didn't parse it.
	if (ELockedNoteFlag::None == (LockedNotesMask & ELockedNoteFlag::Root))
	{
		for (FKeyzoneSettings& Keyzone : Keyzones)
		{
			switch (RootNoteOption)
			{
			case EFusionPatchKeyzoneRootNoteOption::Min:
				Keyzone.RootNote = Keyzone.MinNote;
				break;
			case EFusionPatchKeyzoneRootNoteOption::Max:
				Keyzone.RootNote = Keyzone.MaxNote;
				break;
			case EFusionPatchKeyzoneRootNoteOption::Centered:
				Keyzone.RootNote = (Keyzone.MaxNote + Keyzone.MinNote) / 2;
				break;
			}
		}
	}
}

void UFusionPatchCreateOptions::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	MaxNote = FMath::Clamp(MaxNote, 0, 127);
	MinNote = FMath::Min(MinNote, MaxNote);
	UpdateKeyzonesWithSettings();
	SaveConfig();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UFusionPatchCreateOptions::SortLexical()
{
	// assumes the sound waves are already in alphabetical order
	Keyzones.Reset();
	for (int32 Idx = 0; Idx < StagedSoundWaves.Num(); ++Idx)
	{
		TWeakObjectPtr<USoundWave> Wave = StagedSoundWaves[Idx];
		FKeyzoneSettings NewKeyzone;
		NewKeyzone.SoundWave = Wave.Get();
		NewKeyzone.RootNote = Idx;
		NewKeyzone.MinNote = Idx;
		NewKeyzone.MaxNote = Idx;
		Keyzones.Add(NewKeyzone);
	}
}

void UFusionPatchCreateOptions::SortNoteNumber()
{
	Keyzones.Reset();
	for (int32 Idx = 0; Idx < StagedSoundWaves.Num(); ++Idx)
	{
		TWeakObjectPtr<USoundWave> Wave = StagedSoundWaves[Idx];
		FKeyzoneSettings NewKeyzone;
		NewKeyzone.SoundWave = Wave.Get();

		FKeyzoneNoteParser::FParseResult ParseResult;
		if (FKeyzoneNoteParser::ParseMinRootMax(Wave->GetPackage()->GetName(), ParseResult))
		{
			LockedNotesMask = ELockedNoteFlag::MinRootMax;
			NewKeyzone.MinNote = ParseResult.MinNote;
			NewKeyzone.MaxNote = ParseResult.MaxNote;
			NewKeyzone.RootNote = ParseResult.RootNote;
		}
		else if (FKeyzoneNoteParser::ParseMinMax(Wave->GetPackage()->GetName(), ParseResult))
		{
			LockedNotesMask = ELockedNoteFlag::MinMax;
			NewKeyzone.MinNote = ParseResult.MinNote;
			NewKeyzone.MaxNote = ParseResult.MaxNote;
			NewKeyzone.RootNote = ParseResult.MinNote;
		}
		else if (FKeyzoneNoteParser::ParseRoot(Wave->GetPackage()->GetName(), ParseResult))
		{
			LockedNotesMask = ELockedNoteFlag::Root;
			NewKeyzone.RootNote = ParseResult.RootNote;
			NewKeyzone.MinNote = ParseResult.RootNote;
			NewKeyzone.MaxNote = ParseResult.RootNote;
		}
		else
		{
			NewKeyzone.RootNote = Idx;
			NewKeyzone.MinNote = Idx;
			NewKeyzone.MaxNote = Idx;
		}

		// sort the keyzones by root note
		const int32 Index = Algo::LowerBoundBy(Keyzones, NewKeyzone.RootNote, &FKeyzoneSettings::RootNote);
		Keyzones.Insert(NewKeyzone, Index);
	}
}

void UFusionPatchCreateOptions::SortNoteName()
{
	Keyzones.Reset();
	for (int32 Idx = 0; Idx < StagedSoundWaves.Num(); ++Idx)
	{
		TWeakObjectPtr<USoundWave> Wave = StagedSoundWaves[Idx];
		FKeyzoneSettings NewKeyzone;
		NewKeyzone.SoundWave = Wave.Get();
		
		FKeyzoneNoteParser::FParseResult ParseResult;
		if (FKeyzoneNoteParser::ParseNoteName(Wave->GetPackage()->GetName(), ParseResult))
		{
			LockedNotesMask = ELockedNoteFlag::Root;
			NewKeyzone.MinNote = ParseResult.RootNote;
			NewKeyzone.RootNote = ParseResult.RootNote;
			NewKeyzone.MaxNote = ParseResult.RootNote;
		}
		else
		{
			NewKeyzone.MinNote = Idx;
			NewKeyzone.RootNote = Idx;
			NewKeyzone.MaxNote = Idx;
		}

		const int32 Index = Algo::LowerBoundBy(Keyzones, NewKeyzone.RootNote, &FKeyzoneSettings::RootNote);
		Keyzones.Insert(NewKeyzone, Index);
	}
}

void UFusionPatchCreateOptions::SortIndex()
{
	Keyzones.Reset();
	for (int32 Idx = 0; Idx < StagedSoundWaves.Num(); ++Idx)
	{
		TWeakObjectPtr<USoundWave> Wave = StagedSoundWaves[Idx];
		FKeyzoneSettings NewKeyzone;
		NewKeyzone.SoundWave = Wave.Get();
		FKeyzoneNoteParser::FParseResult ParseResult;
		if (FKeyzoneNoteParser::ParseIndex(Wave->GetPackage()->GetName(), ParseResult))
		{
			NewKeyzone.MinNote = ParseResult.RootNote;
			NewKeyzone.RootNote = ParseResult.RootNote;
			NewKeyzone.MaxNote = ParseResult.RootNote;
		}
		else
		{
			NewKeyzone.MinNote = Idx;
			NewKeyzone.RootNote = Idx;
			NewKeyzone.MaxNote = Idx;
		}
		
		const int32 Index = Algo::LowerBoundBy(Keyzones, NewKeyzone.RootNote, &FKeyzoneSettings::RootNote);
		Keyzones.Insert(NewKeyzone, Index);
	}
}

void UFusionPatchCreateOptions::LayoutSingleNote()
{
	int32 NumKeyzones = Keyzones.Num();
	if (NumKeyzones == 0)
	{
		return;
	}

	if (LockedNotesMask == ELockedNoteFlag::None)
	{
		if (NumKeyzones == 1)
		{
			FKeyzoneSettings& Keyzone = Keyzones[0];
			switch (RootNoteOption)
			{
			case EFusionPatchKeyzoneRootNoteOption::Min:
				Keyzone.RootNote = MinNote;
				break;
			case EFusionPatchKeyzoneRootNoteOption::Max:
				Keyzone.RootNote = MaxNote;
				break;
			case EFusionPatchKeyzoneRootNoteOption::Centered:
				Keyzone.RootNote = (MaxNote - MinNote) / 2;
				break;
			}

			Keyzone.MinNote = Keyzone.RootNote;
			Keyzone.MaxNote = Keyzone.RootNote;
			return;
		}
		
		int32 StepOffset = 0;
		float StepSize = float(MaxNote - MinNote) / float(NumKeyzones);
		switch (RootNoteOption)
		{
		case EFusionPatchKeyzoneRootNoteOption::Min:
			StepOffset = 0;
			break;
		case EFusionPatchKeyzoneRootNoteOption::Max:
			StepOffset = 1;
			break;
		case EFusionPatchKeyzoneRootNoteOption::Centered:
			StepSize = float(MaxNote - MinNote) / float(NumKeyzones - 1);
			break;
		}
	
		for (int32 Idx = 0; Idx < NumKeyzones; ++Idx)
		{
			FKeyzoneSettings& Keyzone = Keyzones[Idx];
			Keyzone.RootNote = FMath::Clamp(MinNote + FMath::FloorToInt((Idx + StepOffset) * StepSize), MinNote, MaxNote);
			Keyzone.MinNote = Keyzone.RootNote;
			Keyzone.MaxNote = Keyzone.RootNote;
		}
	}
	else if (LockedNotesMask == ELockedNoteFlag::Root)
	{
		for (int32 Idx = 0; Idx < NumKeyzones; ++Idx)
		{
			FKeyzoneSettings& Keyzone = Keyzones[Idx];
			Keyzone.MinNote = Keyzone.RootNote;
			Keyzone.MaxNote = Keyzone.RootNote;
		}
	}
}

void UFusionPatchCreateOptions::LayoutDistribute()
{
	int32 NumKeyzones = Keyzones.Num();
	if (NumKeyzones == 0)
	{
		return;
	}
	if (NumKeyzones == 1)
	{
		FKeyzoneSettings& Keyzone = Keyzones[0];
		Keyzone.MinNote = FMath::Min(MinNote, Keyzone.RootNote);
		Keyzone.MaxNote = FMath::Max(MaxNote, Keyzone.RootNote);
		return;
	}
	
	// if we didn't parse anything, distribute evenly
	if (LockedNotesMask == ELockedNoteFlag::None)
	{
		float StepSize = float(MaxNote - MinNote) / float(NumKeyzones);
		for (int32 Idx = 0; Idx < NumKeyzones;  ++Idx)
		{
			FKeyzoneSettings& Keyzone = Keyzones[Idx];
			Keyzone.MinNote = FMath::Clamp(MinNote + FMath::FloorToInt(Idx * StepSize), MinNote, MaxNote);
			Keyzone.MaxNote = FMath::Clamp(MinNote + FMath::FloorToInt((Idx + 1) * StepSize) - 1, MinNote, MaxNote);
		}
		Keyzones[0].MinNote = MinNote;
		Keyzones.Last().MaxNote = MaxNote;
	}
	// if we parsed the root note only, distribute the min and max around the root note
	else if (LockedNotesMask == ELockedNoteFlag::Root)
	{
		switch (RootNoteOption)
		{
		case EFusionPatchKeyzoneRootNoteOption::Centered:
			Keyzones[0].MinNote = FMath::Min(MinNote, Keyzones[0].RootNote);
			Keyzones.Last().MaxNote = FMath::Max(MaxNote, Keyzones.Last().RootNote);
			
			for (int32 Idx = 0; Idx < NumKeyzones; ++Idx)
			{
				if (Idx < NumKeyzones - 1)
				{
					Keyzones[Idx].MaxNote = (Keyzones[Idx].RootNote + Keyzones[Idx+1].RootNote) / 2;
				}
				if (Idx > 0)
				{
					Keyzones[Idx].MinNote = (Keyzones[Idx].RootNote + Keyzones[Idx-1].RootNote) / 2;
				}
			}
			break;
		case EFusionPatchKeyzoneRootNoteOption::Min:
			for (int32 Idx = 0; Idx < NumKeyzones - 1; ++Idx)
			{
				Keyzones[Idx].MaxNote = Keyzones[Idx + 1].RootNote - 1;
				Keyzones[Idx].MinNote = Keyzones[Idx].RootNote;
			}
			Keyzones[0].MinNote = FMath::Min(MinNote, Keyzones[0].RootNote);
			Keyzones.Last().MaxNote = FMath::Max(MaxNote, Keyzones.Last().RootNote);
			break;
		case EFusionPatchKeyzoneRootNoteOption::Max:
			for (int32 Idx = NumKeyzones - 1; Idx > 0; --Idx)
			{
				Keyzones[Idx].MinNote = Keyzones[Idx - 1].RootNote + 1;
				Keyzones[Idx].MaxNote = Keyzones[Idx].RootNote;
			}
			Keyzones[0].MinNote = FMath::Min(MinNote, Keyzones[0].RootNote);
            Keyzones.Last().MaxNote = FMath::Max(MaxNote, Keyzones.Last().RootNote);
			break;
		}
	}
}

void UFusionPatchCreateOptions::LayoutLayer()
{
	// if just the root note was parsed, clamp the min and max notes
	if (LockedNotesMask == ELockedNoteFlag::Root)
	{
		for (FKeyzoneSettings& Keyzone : Keyzones)
		{
			Keyzone.MinNote = FMath::Min(MinNote, Keyzone.RootNote);
			Keyzone.MaxNote = FMath::Max(MaxNote, Keyzone.RootNote);
		}
	}
	// otherwise, apply min and max directly
	else if (ELockedNoteFlag::None == (LockedNotesMask & ELockedNoteFlag::MinMax))
	{
		for (FKeyzoneSettings& Keyzone : Keyzones)
		{
			Keyzone.MinNote = MinNote;
			Keyzone.MaxNote = MaxNote;
		}
	}
}
#endif

bool FKeyzoneNoteParser::ParseMinRootMax(const FString& Input, FParseResult& Output)
{
	Output.Reset();
	const FRegexPattern Pattern(TEXT("_(\\d+)_(\\d+)_(\\d+)"));

	FRegexMatcher Matcher(Pattern, Input);
	if (Matcher.FindNext())
	{
		Output.MinNote = FCString::Atoi(*Matcher.GetCaptureGroup(1));
		Output.RootNote = FCString::Atoi(*Matcher.GetCaptureGroup(2));
		Output.MaxNote = FCString::Atoi(*Matcher.GetCaptureGroup(3));
		return true;
	}
	return false;
}

bool FKeyzoneNoteParser::ParseMinMax(const FString& Input, FParseResult& Output)
{
	Output.Reset();
	const FRegexPattern Pattern(TEXT("_(\\d+)_(\\d+)"));

	FRegexMatcher Matcher(Pattern, Input);
	if (Matcher.FindNext())
	{
		Output.MinNote = FCString::Atoi(*Matcher.GetCaptureGroup(1));
		Output.MaxNote = FCString::Atoi(*Matcher.GetCaptureGroup(2));
		return true;
	}
	return false;
}

bool FKeyzoneNoteParser::ParseRoot(const FString& Input, FParseResult& Output)
{
	Output.Reset();
	const FRegexPattern Pattern(TEXT("_(\\d+)"));

	FRegexMatcher Matcher(Pattern, Input);
	if (Matcher.FindNext())
	{
		Output.RootNote = FCString::Atoi(*Matcher.GetCaptureGroup(1));
		return true;
	}
	return false;
}

bool FKeyzoneNoteParser::ParseNoteName(const FString& Input, FParseResult& Output)
{
	Output.Reset();
	const FRegexPattern Pattern(TEXT("_([A-Ga-g][b#]?-?\\d+)"));

	FRegexMatcher Matcher(Pattern, Input);
	if (Matcher.FindNext())
	{
		Output.RootNote = Harmonix::Midi::Constants::GetNoteNumberFromNoteName(TCHAR_TO_ANSI(*Matcher.GetCaptureGroup(1)));
		return true;
	}
	return false;
}

bool FKeyzoneNoteParser::ParseIndex(const FString& Input, FParseResult& Output)
{
	Output.Reset();
	const FRegexPattern Pattern(TEXT("_(\\d+)"));

	FRegexMatcher Matcher(Pattern, Input);
	if (Matcher.FindNext())
	{
		Output.RootNote = FCString::Atoi(*Matcher.GetCaptureGroup(1));
		return true;
	}
	return false;
}
