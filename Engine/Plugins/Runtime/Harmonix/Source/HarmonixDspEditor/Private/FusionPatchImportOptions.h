// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"

#include "Sound/SoundWave.h"
#include "Sound/SoundWaveLoadingBehavior.h"
#include "HarmonixDsp/FusionSampler/Settings/KeyzoneSettings.h"
#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"
#include "HarmonixMidi/MidiConstants.h"
#include "FusionPatchImportOptions.generated.h"

UCLASS(config = EditorPerProjectUserSettings)
class UFusionPatchImportOptions : public UObject
{
	GENERATED_BODY()

public:
	struct FArgs
	{
		FName PatchName;
		FString Directory;
	};

	static const UFusionPatchImportOptions* GetWithDialog(FArgs&& Args, bool& OutWasOkayPressed);

	/** The directory to save samples to */
	UPROPERTY(EditAnywhere, Category = "Import Options", Meta = (DisplayName = "Sound Waves Import Folder", ContentDir))
	FDirectoryPath SamplesImportDir;
	
	/** The loading behavior to apply to the imported samples */
	UPROPERTY(EditAnywhere, Category = "Import Options", Meta = (DisplayName = "Sound Wave Loading Behavior"))
	ESoundWaveLoadingBehavior SampleLoadingBehavior = ESoundWaveLoadingBehavior::RetainOnLoad;

	/** The compression type to apply to the imported samples */
	UPROPERTY(EditAnywhere, Category = "Import Options", Meta = (DisplayName = "Sound Wave Compression Type"))
	ESoundAssetCompressionType SampleCompressionType = ESoundAssetCompressionType::BinkAudio;
	
#if WITH_EDITOR
	/** UObject interface */
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		SaveConfig();
	}
#endif
};

UENUM()
enum class ELockedNoteFlag : uint8
{
	None = 0b000,
	Min	 = 0b001,
	Root = 0b010,
	Max  = 0b100,
	MinMax = Min | Max,
	MinRootMax = Min | Root | Max
};
ENUM_CLASS_FLAGS(ELockedNoteFlag)

UENUM()
enum class EFusionPatchKeyzoneSortOption : uint8
{
	Lexical		UMETA(DisplayName="Alphabetical", ToolTip="Alphabetical order."),
	Index		UMETA(DisplayName="Index Order", ToolTip="Sample name contains index, ordering the samples alphabetically then from lowest to highest index (snare_0, snare_1, tom_0)"),
	NoteNumber	UMETA(DisplayName="Note Number", ToolTip="Sample names contain note numbers with one of the acceptable formats: (name_[Min]_[Root]_[Max], name_[Min]_[Max], name_[Root])"),
	NoteName	UMETA(DisplayName="Note Name", ToolTip="Sample names contain note names: (eg. 'Eb1', 'G#3', 'B2')"),
};

UENUM()
enum class EFusionPatchKeyzoneNoteLayoutOption : uint8
{
	SingleNote	UMETA(DisplayName="Single Note", ToolTip="Assign each zone to a single note, 'root' note if parsed by note number or note name, otherwise each note from the selected 'min - max' range"),
	Distribute	UMETA(DisplayName="Distribute", ToolTip="Evenly distribute all zones, using the values parsed by the note number or note name, otherwise uses 'min-max' note range"),
	Layer		UMETA(DisplayName="Layer", ToolTip="Every zone will span 'min-max' note range."),
};

UENUM()
enum class EFusionPatchKeyzoneNoteScaleOption : uint8
{
	None		UMETA(DisplayName="No Scale (Equal Distance)", ToolTip="Distributes the notes evenly based on the selected layout"),
	MajorScale	UMETA(DisplayName="Major Scale", ToolTip="Distributes the notes across a major scale, starting with the Min Note")
};

UENUM()
enum class EFusionPatchKeyzoneRootNoteOption : uint8
{
	Min			UMETA(DisplayName="Align Min", ToolTip="Adjusts the range so that the Root note is the same as the Min note of the zone"),
	Max			UMETA(DisplayName="Align Max", ToolTip="Adjusts the range so that the Root note is the same as the Max note of the zone"),
	Centered	UMETA(DisplayName="Align Center", ToolTip="Adjusts the range so that the Root note of each keyzone is between its min and max note (Root = (Max + Min) / 2)")	
};

UCLASS(config = Editor)
class UFusionPatchCreateOptions : public UObject
{
	GENERATED_BODY()

public:
	struct FArgs
	{
		FString Directory;
		FString AssetName;
		TArray<TWeakObjectPtr<USoundWave>> StagedSoundWaves;
	};

	static const UFusionPatchCreateOptions* GetWithDialog(FArgs&& Args, bool& OutWasOkayPressed);

	// Which directory the FusionPatch should be created in
	UPROPERTY(config, EditAnywhere, Category = "Create Patch", Meta=(DisplayName = "Destination", ContentDir))
	FDirectoryPath FusionPatchDir;

	// The Name of the new fusion patch
	UPROPERTY(EditAnywhere, Category = "Create Patch", Meta=(DisplayName = ""))
	FString AssetName;
	
	// how to parse and sort the names of the samples, extract the note numbers (if applicable) and assigning root, min, max notes
	UPROPERTY(config, EditAnywhere, Category = "Create Patch", meta=(DisplayName="Assign Notes By"))
	EFusionPatchKeyzoneSortOption SortOption = EFusionPatchKeyzoneSortOption::NoteNumber;

	// how to assign the min and max notes for each keyzone based on ordering and note mapping
	UPROPERTY(config, EditAnywhere, Category = "Create Patch", meta=(DisplayName="Keyzone Layout", EditConditionHides, EditCondition="LockedNotesMask != ELockedNoteFlag::MinRootMax"))
	EFusionPatchKeyzoneNoteLayoutOption LayoutOption = EFusionPatchKeyzoneNoteLayoutOption::Distribute;

	// whether to assign the notes along a scale, or evenly spaced
	UPROPERTY(Config, EditAnywhere, Category = "Create Patch", meta=(DisplayName="Scale Mapping", EditConditionHides, EditCondition="SortOption == EFusionPatchKeyzoneSortOption::Lexical || SortOption == EFusionPatchKeyzoneSortOption::Index"))
	EFusionPatchKeyzoneNoteScaleOption ScaleOption = EFusionPatchKeyzoneNoteScaleOption::None;
	
	// how to assign the root note based on the min and max notes
	UPROPERTY(config, EditAnywhere, Category = "Create Patch", meta=(DisplayName="Note Alignment", EditConditionHides, EditCondition="LockedNotesMask != ELockedNoteFlag::MinRootMax"))
	EFusionPatchKeyzoneRootNoteOption RootNoteOption = EFusionPatchKeyzoneRootNoteOption::Min;

	UPROPERTY(config, EditAnywhere, Category = "Create Patch", meta=(ClampMin = 0, ClampMax = 127, UIMin = 0, UIMax = 127))
	int8 MinNote = 0;

	UPROPERTY(config, EditAnywhere, Category = "Create Patch", meta=(ClampMin = 0, ClampMax = 127, UIMin = 0, UIMax = 127))
	int8 MaxNote = 127;

	// Apply an additional octave, positive or negative, to the note values.
	// This can be used to adjust the note values parsed from the sample file names
	// or similarly adjust the octave when setting the notes by scale
	UPROPERTY(Config, EditAnywhere, Category = "Create Patch")
	int32 OctaveAdjust = 0;

	UPROPERTY(EditAnywhere, Category = "Create Patch")
	TArray<FKeyzoneSettings> Keyzones;

	UPROPERTY()
	FFusionPatchSettings FusionPatchSettings;

	TArray<TWeakObjectPtr<USoundWave>> StagedSoundWaves;

	UPROPERTY()
	ELockedNoteFlag LockedNotesMask = ELockedNoteFlag::None;

#if WITH_EDITOR
	
	/** UObject interface */
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	
	void SortLexical();

	void SortNoteNumber();

	void SortNoteName();

	void SortIndex();

	void LayoutSingleNote();

	void LayoutDistribute();

	void LayoutLayer();

	void UpdateKeyzonesWithSettings();
	
	
#endif
};

class FKeyzoneNoteParser
{
public:

	struct FParseResult
	{
		int32 MinNote = -1;
		int32 RootNote = -1;
		int32 MaxNote = -1;

		void Reset()
		{
			MinNote = -1;
			RootNote = -1;
			MaxNote = -1;
		}
	};

	static bool ParseMinRootMax(const FString& Input, FParseResult& Output);

	static bool ParseMinMax(const FString& Input, FParseResult& Output);
		
	static bool ParseRoot(const FString& Input, FParseResult& Output);

	static bool ParseNoteName(const FString& Input, FParseResult& Output);

	static bool ParseIndex(const FString& Input, FParseResult& Output);
};
