// Copyright Epic Games, Inc. All Rights Reserved.
#include "FusionPatchJsonImporter.h"
#include "SettingsJsonImporter.h"
#include "JsonImporterHelper.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"

#include "Misc/Paths.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"

#include "EditorAssetLibrary.h"
#include "EditorFramework/AssetImportData.h"
#include "Sound/SoundWave.h"

#define LOCTEXT_NAMESPACE "FusionPatchImporter"

DEFINE_LOG_CATEGORY(LogFusionPatchJsonImporter)

TArray<UObject*> FFusionPatchJsonImporter::ImportAudioSamples(const TArray<FString>& Files, const FString& DestinationPath, bool ReplaceExisting)
{
	UAutomatedAssetImportData* AutomatedAssetImportData = NewObject<UAutomatedAssetImportData>();
	AutomatedAssetImportData->bReplaceExisting = ReplaceExisting;
	AutomatedAssetImportData->Filenames = Files;
	AutomatedAssetImportData->DestinationPath = DestinationPath;
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	return AssetToolsModule.Get().ImportAssetsAutomated(AutomatedAssetImportData);
}

bool FFusionPatchJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, UFusionPatch* FusionPatch, TArray<UObject*>& OutAdditionalImportedObjects, const FImportArgs& ImportArgs, TArray<FString>& OutErrors)
{
	if (!ensure(FusionPatch))
		return false;

	if (!ensure(JsonObj.IsValid()))
		return false;

	int Version;
	if (!JsonObj->TryGetNumberField(TEXT("version"), Version))
	{
		UE_LOGF(LogHarmonixJsonImporter, Warning, "Failed to read version number importing Fusion Patch.");
		return false;
	}

	static const int kMinVersion = 1;
	if (Version < kMinVersion)
	{
		UE_LOGF(LogHarmonixJsonImporter, Warning, "Bad version number importing Fusion Patch! \"version\" = %d, must be >= %d)", Version, kMinVersion);
	};
	
	// import patch settings and keyzones first
	// make a copy of the settings and update it with the imported settings
	FFusionPatchSettings PatchSettingsImport = FusionPatch->GetSettings();
	for (TSharedPtr<FJsonValue> ArrayValue : IterField(JsonObj, "presets"))
	{
		TSharedPtr<FJsonObject> PresetObject = nullptr;
		if (!TryGetObject(ArrayValue, PresetObject))
		{
			UE_LOGF(LogFusionPatchJsonImporter, Warning, "Unable to parse Preset in json");
			continue;
		}

		for (const auto& Pair : PresetObject->Values)
		{
			TSharedPtr<FJsonValue> Value = Pair.Value;

			TSharedPtr<FJsonObject> PresetJson = nullptr;
			if (!TryGetObject(Value, PresetJson))
			{
				UE_LOGF(LogFusionPatchJsonImporter, Warning, "Unable to parse preset in json");
				continue;
			}
			
			if (!FSettingsJsonImporter::TryParseJson(PresetJson, PatchSettingsImport))
			{
				UE_LOGF(LogFusionPatchJsonImporter, Warning, "Unable to parse preset in json");
				continue;
			}
			// we only support one "preset" and that's the settings
			break;
		}
	}
	
	// collect audio sample files for import
	TArray<FString> AudioSampleFiles;
	bool AllFilesExist = true;
	TMap<int, int> KeyzoneIdxToPathIdx;

	// make a copy of the keyzones and update it with the imported settings
	TArray<FKeyzoneSettings> KeyzonesImported = FusionPatch->GetKeyzones();
	int NumKeyzones = 0;
	for (TSharedPtr<FJsonValue> ArrayValue : IterField(JsonObj, "keymap"))
	{
		int KeyzoneIdx = NumKeyzones;
		TSharedPtr<FJsonObject> KeyzoneObject = nullptr;
		if (!TryGetObject(ArrayValue, KeyzoneObject))
			continue;

		TSharedPtr<FJsonObject> KeyzoneValue = nullptr;
		if (!TryGetObjectField(KeyzoneObject, "keyzone", KeyzoneValue))
			continue;

		// Update array to contain keyzones up to this index
		KeyzonesImported.SetNum(KeyzoneIdx + 1);

		if (!KeyzoneIdxToPathIdx.Contains(KeyzoneIdx))
		{
			KeyzoneIdxToPathIdx.Add(KeyzoneIdx, -1);
		}

		FKeyzoneSettings& Settings = KeyzonesImported[KeyzoneIdx];
		NumKeyzones++;

		if (!FSettingsJsonImporter::TryParseJson(KeyzoneValue, Settings))
		{
			UE_LOGF(LogFusionPatchJsonImporter, Warning, "Unable to parse keyzone json");
			continue;
		}

		if (Settings.SamplePath.EndsWith(TEXT(".mogg")))
		{
			UE_LOGF(LogFusionPatchJsonImporter, Warning, "Found mogg keyzone (%ls). Will look for a .wav in the same folder.", *Settings.SamplePath);
			Settings.SamplePath = Settings.SamplePath.Replace(TEXT(".mogg"), TEXT(".wav"));
		}

		FString ImportPath = FPaths::Combine(*ImportArgs.SourcePath, *Settings.SamplePath);

		if (!FPaths::FileExists(ImportPath))
		{
			OutErrors.Add(FString::Printf(TEXT("Keyzone Idx = %d: Sample Path is invalid: %s"), KeyzoneIdx, *ImportPath));
			UE_LOGF(LogFusionPatchJsonImporter, Error, "Failed to import FusionPatch. Sample Path is invalid: %ls", *ImportPath);
			AllFilesExist = false;
			continue;
		}
		int32 PathIndex = AudioSampleFiles.AddUnique(ImportPath);
		KeyzoneIdxToPathIdx[KeyzoneIdx] = PathIndex;
	}
	
	// prune the list to match the size of the newly imported asset
	// KeyzoneIdx, at this point, will be equal to the number of _imported_ keyzones
	KeyzonesImported.SetNum(NumKeyzones);
	check(KeyzoneIdxToPathIdx.Num() == KeyzonesImported.Num());

	if (!AllFilesExist)
	{
		OutErrors.Add(FString::Printf(TEXT("Not all sample paths point to valid files on disk")));
		UE_LOGF(LogFusionPatchJsonImporter, Error, "Failed to import FusionPatch. Not all sample paths point to valid files on disk");
		return false;
	}

	TArray<UObject*> ImportedAssets = ImportAudioSamples(AudioSampleFiles, ImportArgs.SamplesDestPath, ImportArgs.ReplaceExistingSamples);
	OutAdditionalImportedObjects.Append(ImportedAssets);

	// update the sound wave loading behavior and compression type with the selection set in the import options
	for (UObject* Asset : ImportedAssets)
	{
		if (USoundWave* SoundWave = Cast<USoundWave>(Asset))
		{
			// call OverrideLoadingBehavior to actually apply the loading behavior
			SoundWave->OverrideLoadingBehavior(ImportArgs.SampleLoadingBehavior);

			// Assign the LoadingBehavior to the SoundWave so that it gets serialized with the new setting
			// (Override doesn't actually assign the new LoadingBehavior to the USoundWave, but rather to the underlying FSoundWaveDataPtr)
			SoundWave->LoadingBehavior = ImportArgs.SampleLoadingBehavior;

			// up until now, setting the loading behavior has had no effect on the sound wave...
			// calling SetSoundAssetCompressionType forces the asset to actually update its state (marked as Dirty)
			SoundWave->SetSoundAssetCompressionType(ImportArgs.SampleCompressionType, true);
		}
	}

	for (int KeyzoneIdx = 0; KeyzoneIdx < KeyzonesImported.Num(); ++KeyzoneIdx)
	{
		int32 PathIndex = KeyzoneIdxToPathIdx.Contains(KeyzoneIdx) ? KeyzoneIdxToPathIdx[KeyzoneIdx] : INDEX_NONE;

		if (PathIndex == INDEX_NONE)
		{
			FString SamplePath = KeyzonesImported[KeyzoneIdx].SamplePath;
			FString AssetName = ImportedAssets[KeyzoneIdx]->GetName();
			OutErrors.Add(FString::Printf(TEXT("Imported asset (Name: %s) failed to map asset (%s) to keyzone: %d"), *AssetName, *SamplePath, KeyzoneIdx));
			UE_LOGF(LogFusionPatchJsonImporter, Error, "Failed to import FusionPatch. Imported asset (Name: %ls) failed to map asset (%ls) to keyzone: %d", *AssetName, *SamplePath, KeyzoneIdx);
			continue;
		}

		if (!AudioSampleFiles.IsValidIndex(PathIndex))
		{
			FString SamplePath = KeyzonesImported[KeyzoneIdx].SamplePath;
			FString AssetName = AudioSampleFiles[KeyzoneIdx];
			OutErrors.Add(FString::Printf(TEXT("Imported asset (Name: %s) failed to map file (%s) to keyzone: %d"), *AssetName, *SamplePath, KeyzoneIdx));
			UE_LOGF(LogFusionPatchJsonImporter, Error, "Failed to import FusionPatch. Imported asset (Name: %ls) failed to map asset (%ls) to keyzone: %d", *AssetName, *SamplePath, KeyzoneIdx);
			continue;
		}

		const FString& AudioSampleFile = AudioSampleFiles[PathIndex];

		UObject** Asset = ImportedAssets.FindByPredicate([&AudioSampleFile](UObject* Asset)
			{
				USoundWave* SoundWave = Cast<USoundWave>(Asset);
				FString Filename = SoundWave->AssetImportData->GetFirstFilename();
				UE_LOGF(LogFusionPatchJsonImporter, Log, "Comparing filenames: %ls == %ls", *AudioSampleFile, *Filename);
				return FPaths::IsSamePath(Filename, AudioSampleFile);
			});

		USoundWave* SoundWave = Asset != nullptr ? Cast<USoundWave>(*Asset) : nullptr;

		if (!SoundWave)
		{
			// If the sound wave wasn't imported, check if it exists in the desired path 
			// (we didn't replace it on import, and/or it was imported previously by another asset)
			const FString AudioSampleName = FPaths::GetBaseFilename(AudioSampleFile);
			const FString AssetPath = ImportArgs.SamplesDestPath / AudioSampleName + "." + AudioSampleName;
			UObject* ExistingObject = UEditorAssetLibrary::LoadAsset(AssetPath);
			SoundWave = Cast<USoundWave>(ExistingObject);
		}

		if (SoundWave)
		{
			KeyzonesImported[KeyzoneIdx].SoundWave = SoundWave;
		}

		// The importer failed to map a sound wave to this keyzone!
		if (!KeyzonesImported[KeyzoneIdx].SoundWave) 
		{
			FString SamplePath = KeyzonesImported[KeyzoneIdx].SamplePath;
            FString AssetName = AudioSampleFiles[KeyzoneIdx];
			OutErrors.Add(FString::Printf(TEXT("Imported asset (Name: %s) failed to map file (%s) to keyzone: %d"), *AssetName, *SamplePath, KeyzoneIdx));
			UE_LOGF(LogFusionPatchJsonImporter, Error, "Failed to import FusionPatch. Imported asset (Name: %ls) failed to map asset (%ls) to keyzone: %d", *AssetName, *SamplePath, KeyzoneIdx);
		}
	}

	// no errors, update the FusionPatchData with the imported data
	if (OutErrors.Num() == 0)
	{
		FusionPatch->UpdateSettings(PatchSettingsImport);
		FusionPatch->UpdateKeyzones(KeyzonesImported);
	}
	
	// return whether we had any errors
	return OutErrors.Num() == 0;
}

#undef LOCTEXT_NAMESPACE