// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "JsonImporterHelper.h"
#include "Templates/SharedPointer.h"

#include "Sound/SoundWave.h"
#include "Sound/SoundWaveLoadingBehavior.h"

#include "Logging/LogMacros.h"

class UFusionPatch;
class FJsonObject;

struct FKeyzoneSettings;
struct FPanDetails;

DECLARE_LOG_CATEGORY_EXTERN(LogFusionPatchJsonImporter, Log, All);

class FFusionPatchJsonImporter : public FJsonImporter
{
public:

	struct FImportArgs
	{
		FImportArgs()
		{}

		FImportArgs(FName InName, 
					const FString& InSourcePath, 
					const FString& InDestPath, 
					const FString& InSamplesDestPath, 
					bool InReplaceExistingSamples = false, 
					ESoundWaveLoadingBehavior SampleLoadingBehavior = ESoundWaveLoadingBehavior::Inherited,
					ESoundAssetCompressionType SampleCompressionType = ESoundAssetCompressionType::BinkAudio)
			: Name(InName)
			, SourcePath(InSourcePath)
			, DestPath(InDestPath)
			, SamplesDestPath(InSamplesDestPath)
			, ReplaceExistingSamples(InReplaceExistingSamples)
			, SampleLoadingBehavior(SampleLoadingBehavior)
			, SampleCompressionType(SampleCompressionType)
		{}

		FName Name;
		FString SourcePath;
		FString DestPath;
		FString SamplesDestPath;
		bool ReplaceExistingSamples;
		ESoundWaveLoadingBehavior SampleLoadingBehavior;
		ESoundAssetCompressionType SampleCompressionType;

	};

	static TArray<UObject*> ImportAudioSamples(const TArray<FString>& Files, const FString& DestinationPath, bool ReplaceExisting);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, UFusionPatch* FusionPatch, TArray<UObject*>& OutAdditionalImportedObjects, const FImportArgs& ImportArgs, TArray<FString>& OutErrors);
};