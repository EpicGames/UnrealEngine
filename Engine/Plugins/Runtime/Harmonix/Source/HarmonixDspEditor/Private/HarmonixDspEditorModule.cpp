// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDspEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"

#include "IAssetTools.h"
#include "ISettingsModule.h"

#include "PropertyEditorModule.h"
#include "AssetDefinitions/AssetDefinition_FusionPatch.h"
#include "Customization/AdsrSettingsDetailCustomization.h"
#include "Customization/TypedParameterCustomization.h"
#include "Customization/PitchShifterNameCustomization.h"
#include "Customization/PitchShifterConfigCustomization.h"
#include "Customization/PannerDetailsCustomization.h"
#include "Customization/FusionPatchDetailCustomization.h"
#include "Customization/FusionPatchImportOptionsCustomization.h"
#include "Customization/FusionPatchSettingsDetailCustomization.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "HarmonixDspEditor"

DEFINE_LOG_CATEGORY(LogHarmonixDspEditor)

void FHarmonixDspEditorModule::StartupModule()
{
	// Register property customizations
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("AdsrSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAdsrSettingsDetailsCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TypedParameter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTypedParameterCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("PitchShifterName", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPitchShifterNameCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("PannerDetails", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPannerDetailsCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("FusionPatchSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFusionPatchSettingsDetailCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomClassLayout("StretcherAndPitchShifterFactoryConfig", FOnGetDetailCustomizationInstance::CreateStatic(&FPitchShifterConfigCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomClassLayout("FusionPatchCreateOptions", FOnGetDetailCustomizationInstance::CreateStatic(&FFusionPatchCreateOptionsCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomClassLayout(UFusionPatch::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FFusionPatchDetailCustomization::MakeInstance));

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FHarmonixDspEditorModule::RegisterMenus));
}

void FHarmonixDspEditorModule::ShutdownModule()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.UnregisterCustomClassLayout("FusionPatch");
	PropertyEditorModule.UnregisterCustomClassLayout("StretcherAndPitchShifterFactoryConfig");
	PropertyEditorModule.UnregisterCustomPropertyTypeLayout("FusionPatchSettings");
	PropertyEditorModule.UnregisterCustomPropertyTypeLayout("PannerDetails");
	PropertyEditorModule.UnregisterCustomPropertyTypeLayout("PitchShifterName");
	PropertyEditorModule.UnregisterCustomPropertyTypeLayout("TypedParameter");
	PropertyEditorModule.UnregisterCustomPropertyTypeLayout("AdsrSettings");
}

void FHarmonixDspEditorModule::RegisterMenus()
{
	FToolMenuOwnerScoped MenuOwner(this);
	FFusionPatchExtension::RegisterMenus();
}

IMPLEMENT_MODULE(FHarmonixDspEditorModule, HarmonixDspEditor);

#undef LOCTEXT_NAMESPACE
