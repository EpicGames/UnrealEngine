// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDspEditorUtils.h"
#include "HarmonixDsp/Modulators/Adsr.h"
#include "DSP/AlignedBuffer.h"

namespace Harmonix::Dsp::Editor
{
	void GenerateAdsrEnvelope(const FAdsrSettings& InAdsrSettings, float SustainTime, float SampleRate, Audio::FAlignedFloatBuffer& OutBuffer)
	{
		FAdsrSettings Settings;
		Settings.CopySettings(InAdsrSettings);
		Settings.Calculate();
		Modulators::FAdsr Adsr;
		Adsr.UseSettings(&Settings);
		Adsr.Prepare(SampleRate);

		// NOTE: Reading this code is a bit confusing
		// Settings.AttackTime, Settings.DecayTime and Settings.ReleaseTime are DURATIONS
		// whereas the calculations below are ABSOLUTE times
		
		// absolute time in seconds to trigger the ADSR Attack relative to start
		const float AttackTime = 0.0;

		// absolute time in seconds to trigger the ADSR Release relative to start
		const float ReleaseTime = Settings.AttackTime + SustainTime + Settings.DecayTime;
		
		// total time in seconds to render the ADSR
		const float RenderTime = Settings.AttackTime + SustainTime + Settings.DecayTime + Settings.ReleaseTime;

		// effective block rate
		const int32 SamplesPerAdvance = 4;

		// convert times to samples (frames)
		const int32 TotalSamples = FMath::TruncToInt32(SampleRate * RenderTime);
		const int32 AttackFrame = FMath::TruncToInt32(SampleRate * AttackTime);
		const int32 ReleaseFrame = FMath::TruncToInt32(SampleRate * ReleaseTime);

		int32 NumSamples = 0;
		
		OutBuffer.SetNumUninitialized(TotalSamples);
		while (NumSamples < TotalSamples)
		{
			if (NumSamples <= AttackFrame && AttackFrame < NumSamples + SamplesPerAdvance)
			{
				Adsr.Attack();
			}

			if (NumSamples <= ReleaseFrame && ReleaseFrame < NumSamples + SamplesPerAdvance)
			{
				Adsr.Release();
			}
			
			Adsr.Advance(SamplesPerAdvance);
			float Value = Adsr.GetValue();
			for (int32 Idx = 0; Idx < SamplesPerAdvance; ++Idx)
			{
				if (Idx + NumSamples >= TotalSamples)
				{
					break;
				}
				OutBuffer[Idx + NumSamples] = Value;
			}
			
			NumSamples += SamplesPerAdvance;
		}
	}
}