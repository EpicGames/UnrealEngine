// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonImporterHelper.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

DEFINE_LOG_CATEGORY(LogHarmonixJsonImporter)

const FString FJsonImporter::kEmptyString = FString();

bool FJsonImporter::CanImportJson(const FString& JsonString)
{
	FString ErrorMessage;
	TSharedPtr<FJsonObject> JsonObj = ParseJsonString(JsonString, ErrorMessage);
	if (JsonObj.IsValid())
	{
		return true;
	}
	return false;
}

TSharedPtr<FJsonObject> FJsonImporter::ParseJsonString(const FString& JsonString, FString& ErrorMessage)
{
	const TSharedRef<TJsonReader<>>& Reader = TJsonReaderFactory<>::Create(JsonString);

	TSharedPtr<FJsonObject> JsonObj;
	if (FJsonSerializer::Deserialize(Reader, JsonObj) && JsonObj.IsValid())
	{
		return JsonObj;
	}
	else
	{
		ErrorMessage = Reader->GetErrorMessage();
		return nullptr;
	}
}

FJsonImporter::TValueIterator FJsonImporter::IterValue(TSharedPtr<FJsonValue> Field)
{
	check(Field);
	return TValueIterator(Field);
}

FJsonImporter::TValueIterator FJsonImporter::IterField(TSharedPtr<FJsonObject> Object, const FString& FieldName)
{
	check(Object);
	return TValueIterator(Object, FieldName);
}

bool FJsonImporter::TryGetObjectField(TSharedPtr<FJsonObject> Object, const FString& FieldName, TSharedPtr<FJsonObject>& OutObject)
{
	check(Object);

	const TSharedPtr<FJsonObject>* ObjectPtr;
	if (Object->TryGetObjectField(FieldName, ObjectPtr))
	{
		OutObject = *ObjectPtr;
		return true;
	}

	return false;
}

bool FJsonImporter::TryGetBoolField(TSharedPtr<FJsonObject> Object, const FString& FieldName, bool& OutBool, bool Default)
{
	check(Object);

	int Value;

	if (Object->TryGetBoolField(FieldName, OutBool))
	{
		return true;
	}
	else if (Object->TryGetNumberField(FieldName, Value))
	{
		OutBool = Value == 1 ? true : false;
		return true;
	}
	else
	{
		OutBool = Default;
	}

	return false;
}

bool FJsonImporter::TryGetStringField(TSharedPtr<FJsonObject> Object, const FString& FieldName, FString& OutString, const FString& Default /*= kEmptyString*/)
{
	check(Object);

	if (Object->TryGetStringField(FieldName, OutString))
	{
		return true;
	}
	else
	{
		OutString = Default;
	}

	return false;
}

bool FJsonImporter::TryGetNameField(TSharedPtr<FJsonObject> Object, const FString& FieldName, FName& OutName, const FName& Default /*= NAME_None*/)
{
	check(Object);

	FString OutString;
	if (Object->TryGetStringField(FieldName, OutString))
	{
		OutName = FName(OutString);
		return true;
	}
	else
	{
		OutName = Default;
	}

	return false;
}

bool FJsonImporter::TryGetObject(TSharedPtr<FJsonValue> Field, TSharedPtr<FJsonObject>& OutObject)
{
	check(Field);

	const TSharedPtr<FJsonObject>* ObjectPtr;
	if (Field->TryGetObject(ObjectPtr))
	{
		OutObject = *ObjectPtr;
		return true;
	}

	return false;
}

bool FJsonImporter::TryGetBool(TSharedPtr<FJsonValue> Field, bool& OutBool, bool Default)
{
	check(Field);

	int Value;

	if (Field->TryGetBool(OutBool))
	{
		return true;
	}
	else if (Field->TryGetNumber(Value))
	{
		OutBool = Value == 1 ? true : false;
		return true;
	}
	else
	{
		OutBool = Default;
	}

	return false;
}

bool FJsonImporter::TryGetString(TSharedPtr<FJsonValue> Field, FString& OutString, const FString& Default /*= kEmptyString*/)
{
	check(Field);

	if (Field->TryGetString(OutString))
	{
		return true;
	}
	else
	{
		OutString = Default;
	}

	return false;
}


// *****************************************************************************************
//
//
// Helper for iterating over values
FJsonImporter::TValueIterator::TValueIterator(TSharedPtr<FJsonValue> Field)
{
	check(Field);

	if (!Field->TryGetArray(ArrayPtr))
	{
		ArrayPtr = nullptr;
		SingleValue = Field;
	}
}

FJsonImporter::TValueIterator::TValueIterator(TSharedPtr<FJsonObject> Object, const FString& FieldName)
{
	check(Object);

	if (!Object->TryGetArrayField(FieldName, ArrayPtr))
	{
		SingleValue = Object->TryGetField(FieldName);
		ArrayPtr = nullptr;
	}
}

const TSharedPtr<FJsonValue>& FJsonImporter::TValueIterator::operator*() const
{
	if (ArrayPtr == nullptr)
		return SingleValue;
	return (*ArrayPtr)[IterIndex];
}

const TSharedPtr<FJsonValue>* FJsonImporter::TValueIterator::operator->()
{
	if (ArrayPtr == nullptr)
		return &SingleValue;
	return &(*ArrayPtr)[IterIndex];
}

bool FJsonImporter::TValueIterator::operator==(const TValueIterator& Other) const
{
	return (IterIndex == Other.IterIndex) && (ArrayPtr == Other.ArrayPtr) && (SingleValue == Other.SingleValue);
}

bool FJsonImporter::TValueIterator::operator!=(const TValueIterator& Other) const
{
	return !(this->operator==(Other));
}

FJsonImporter::TValueIterator& FJsonImporter::TValueIterator::operator++() 
{ 
	++IterIndex; return *this; 
}

FJsonImporter::TValueIterator FJsonImporter::TValueIterator::operator++(int)
{
	TValueIterator Temp = *this;
	++(*this);
	return Temp;
}

FJsonImporter::TValueIterator FJsonImporter::TValueIterator::begin()
{
	TValueIterator Start = *this;
	Start.IterIndex = 0;
	return Start;
}

int FJsonImporter::TValueIterator::end()
{
	if (ArrayPtr)
		return ArrayPtr->Num();
	else if (SingleValue)
		return 1;
	return 0;
}