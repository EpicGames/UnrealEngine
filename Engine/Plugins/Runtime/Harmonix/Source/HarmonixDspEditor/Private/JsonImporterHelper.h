// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

#include "Containers/UnrealString.h"

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"

#include "UObject/Class.h"

class FJsonValue;
class FJsonObject;

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixJsonImporter, Log, All);

class FJsonImporter
{
public:

	class TValueIterator
	{
	public:
		TValueIterator(TSharedPtr<FJsonValue> Field);
		TValueIterator(TSharedPtr<FJsonObject> Object, const FString& FieldName);
		const TSharedPtr<FJsonValue>& operator*() const;
		const TSharedPtr<FJsonValue>* operator->() ;
		bool operator==(const TValueIterator& Other) const;
		bool operator!=(const TValueIterator& Other) const;
		bool operator==(int Index) const { return IterIndex == Index; }
		bool operator!=(int Index) const { return IterIndex != Index; }
		TValueIterator& operator++();
		TValueIterator operator++(int);
		TValueIterator begin();
		int end();
	private:
		int IterIndex = -1;
		TSharedPtr<FJsonValue> SingleValue;
		const TArray<TSharedPtr<FJsonValue>>* ArrayPtr = nullptr;
	};

	static const FString kEmptyString;

	static bool CanImportJson(const FString& JsonString);

	static TSharedPtr<FJsonObject> ParseJsonString(const FString& JsonString, FString& ErrorMessage);

	// helpful way to iterate a vaule or field if you're not sure if it's an array or a single value
	static TValueIterator IterValue(TSharedPtr<FJsonValue> Field);
	static TValueIterator IterField(TSharedPtr<FJsonObject> Object, const FString& FieldName);

	static bool TryGetObjectField(TSharedPtr<FJsonObject> Object, const FString& FieldName, TSharedPtr<FJsonObject>& OutObject);

	// shortcut to reading a number as a bool
	static bool TryGetBoolField(TSharedPtr<FJsonObject> Object, const FString& FieldName, bool& OutBool, bool Default = false);

	static bool TryGetStringField(TSharedPtr<FJsonObject> Object, const FString& FieldName, FString& OutString, const FString& Default = kEmptyString);

	static bool TryGetNameField(TSharedPtr<FJsonObject> Object, const FString& FildName, FName& OutName, const FName& Default = NAME_None);

	template<typename T>
	static bool TryGetNumberField(TSharedPtr<FJsonObject> Object, const FString& FieldName, T& OutNumber, T Default = static_cast<T>(0));

	template<typename T>
	static bool TryGetNumberField(TSharedPtr<FJsonObject> Object, const FString& FieldName, T& OutNumber, TFunction<T(T)> Conversion, T Default = static_cast<T>(0));

	template<typename T>
	static bool TryGetEnumField(TSharedPtr<FJsonObject> Object, const FString& FieldName, T& OutEnum, T Default);

	static bool TryGetObject(TSharedPtr<FJsonValue> Field, TSharedPtr<FJsonObject>& OutObject);

	// shortcut to reading a number as a bool
	static bool TryGetBool(TSharedPtr<FJsonValue> Field, bool& OutBool, bool Default = false);

	static bool TryGetString(TSharedPtr<FJsonValue> Field, FString& OutString, const FString& Default = kEmptyString);

	template<typename T>
	static bool TryGetNumber(TSharedPtr<FJsonValue> Field, T& OutNumber, T Default = static_cast<T>(0));

	template<typename T>
	static bool TryGetNumber(TSharedPtr<FJsonValue> Field, T& OutNumber, TFunction<T(T)> Conversion, T Default = static_cast<T>(0));

	template<typename T>
	static bool TryGetEnum(TSharedPtr<FJsonValue> Field, T& OutEnum, T Default);

	template<typename T> 
	static bool ParseEnum(const FString& JsonString, T& OutEnum);

	template<typename T>
	static bool EnumToJsonString(const T InEnum, FString& OutJsonString);
};

template<typename T>
bool FJsonImporter::TryGetNumberField(TSharedPtr<FJsonObject> Object, const FString& FieldName, T& OutNumber, T Default /*= static_cast<T>(0)*/)
{
	check(Object);
	if (Object->TryGetNumberField(FieldName, OutNumber))
	{
		return true;
	}
	else
	{
		OutNumber = Default;
	}

	return false;
}

template<typename T>
bool FJsonImporter::TryGetNumberField(TSharedPtr<FJsonObject> Object, const FString& FieldName, T& OutNumber, TFunction<T(T)> Conversion, T Default)
{
	check(Object);
	T BaseNumber;
	if (TryGetNumberField(Object, FieldName, BaseNumber, Default))
	{
		OutNumber = Conversion(BaseNumber);
		return true;
	}
	return false;
}

template<typename T>
bool FJsonImporter::TryGetEnumField(TSharedPtr<FJsonObject> Object, const FString& FieldName, T& OutEnum, T Default)
{
	check(Object);
	OutEnum = Default;
	FString JsonString;
	int Index;
	if (Object->TryGetStringField(FieldName, JsonString))
	{
		return ParseEnum<T>(JsonString, OutEnum);
	}
	else if (Object->TryGetNumberField(FieldName, Index))
	{
		// Parse the integer as an enum instead!
		OutEnum = static_cast<T>(Index);
		return true;
	}
	
	
	return false;
}

template<typename T>
bool FJsonImporter::TryGetNumber(TSharedPtr<FJsonValue> Field, T& OutNumber, T Default /*= static_cast<T>(0)*/)
{
	check(Field);
	if (Field->TryGetNumber(OutNumber))
	{
		return true;
	}
	else
	{
		OutNumber = Default;
	}

	return false;
}

template<typename T>
bool FJsonImporter::TryGetNumber(TSharedPtr<FJsonValue> Field, T& OutNumber, TFunction<T(T)> Conversion, T Default)
{
	check(Field);
	T BaseNumber;
	if (TryGetNumber(Field, OutNumber, Default))
	{
		OutNumber = Conversion(BaseNumber);
		return true;
	}
	return false;
}

template<typename T>
bool FJsonImporter::TryGetEnum(TSharedPtr<FJsonValue> Field, T& OutEnum, T Default)
{
	check(Field);
	OutEnum = Default;
	FString JsonString;
	int Index;
	if (Field->TryGetString(JsonString))
	{
		return ParseEnum(JsonString, OutEnum);
	}
	if (Field->TryGetNumber(Index))
	{
		OutEnum = static_cast<T>(Index);
		return true;
	}
	return false;
}

template<typename T>
bool FJsonImporter::ParseEnum(const FString& JsonString, T& OutEnum)
{
	if (JsonString.IsEmpty())
	{
		return false;
	}

	if (UEnum* Enum = StaticEnum<T>())
	{
		for (int32 idx = 0; idx < Enum->NumEnums(); ++idx)
		{
			FString EnumString;
			if (Enum->HasMetaData(TEXT("Json"), idx))
			{
				EnumString = Enum->GetMetaData(TEXT("Json"), idx);
			}
			else
			{
				EnumString = Enum->GetNameStringByIndex(idx);
			}
			
			if (!EnumString.IsEmpty() && EnumString.Equals(JsonString))
			{
				OutEnum = static_cast<T>(idx);
				return true;
			}
		}
	}
	return false;
}

template<typename T>
bool FJsonImporter::EnumToJsonString(const T InEnum, FString& OutJsonString)
{
	if (UEnum* Enum = StaticEnum<T>())
	{
		int idx = (int)InEnum;
		if (Enum->HasMetaData(TEXT("Json"), idx))
		{
			OutJsonString = Enum->GetMetaData(TEXT("Json"), idx);
			return true;
		}
	}
	return false;
}
