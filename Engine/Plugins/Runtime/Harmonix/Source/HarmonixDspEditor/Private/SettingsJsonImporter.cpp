// Copyright Epic Games, Inc. All Rights Reserved.

#include "SettingsJsonImporter.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/PitchShifterName.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "HarmonixDsp/FusionSampler/Settings/FusionPatchSettings.h"
#include "HarmonixDsp/StretcherAndPitchShifterFactoryConfig.h"

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> SettingsJson, FKeyzoneSettings& Settings)
{
	FKeyzoneSettings Defaults;
	TryGetNumberField(SettingsJson, "root_note", Settings.RootNote, Defaults.RootNote);
	TryGetNumberField(SettingsJson, "min_note", Settings.MinNote, Defaults.MinNote);
	TryGetNumberField(SettingsJson, "max_note", Settings.MaxNote, Defaults.MaxNote);
	TryGetBoolField(SettingsJson, "unpitched", Settings.bUnpitched, Defaults.bUnpitched);
	TryGetBoolField(SettingsJson, "velocity_to_volume", Settings.bVelocityToGain, Defaults.bVelocityToGain);

	TryParseJson(SettingsJson, Settings.TimeStretchConfig);

	const TSharedPtr<FJsonObject>* TimeStretchJson = nullptr;
	if (SettingsJson->TryGetObjectField(TEXT("timestretch_settings"), TimeStretchJson))
	{
		TryParseJson(*TimeStretchJson, Settings.TimeStretchConfig);
	}

	TryGetNumberField(SettingsJson, "random_weight", Settings.RandomWeight, Defaults.RandomWeight);
	TryGetNumberField(SettingsJson, "min_velocity", Settings.MinVelocity, Defaults.MinVelocity);
	TryGetNumberField(SettingsJson, "max_velocity", Settings.MaxVelocity, Defaults.MaxVelocity);
	float VolumeDb;
	TryGetNumberField(SettingsJson, "volume", VolumeDb, HarmonixDsp::kDbSilence);
	Settings.SetVolumeDb(VolumeDb);

	const TSharedPtr<FJsonObject>* PanJson = nullptr;
	if (SettingsJson->TryGetObjectField(TEXT("pan"), PanJson))
	{
		TryParseJson(*PanJson, Settings.Pan);
	}

	TryGetNumberField(SettingsJson, "fine_tune", Settings.FineTuneCents, Defaults.FineTuneCents);

	TryGetBoolField(SettingsJson, "is_note_off_zone", Settings.bIsNoteOffZone, Defaults.bIsNoteOffZone);
	TryGetNumberField(SettingsJson, "priority", Settings.Priority, Defaults.Priority);

	TryGetStringField(SettingsJson, "sample_path", Settings.SamplePath);

	TryGetNumberField(SettingsJson, "start_offset_frame", Settings.SampleStartOffset, Defaults.SampleStartOffset);
	TryGetNumberField(SettingsJson, "end_offset_frame", Settings.SampleEndOffset, Defaults.SampleEndOffset);

	TryGetBoolField(SettingsJson, "singleton", Settings.UseSingletonVoicePool);

	return true;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FTimeStretchConfig& TimeStretchConfig)
{
	FTimeStretchConfig Defaults;
	bool MaintainFormant = false;
	TryGetBoolField(JsonObj, "maintain_time", TimeStretchConfig.bMaintainTime, Defaults.bMaintainTime);
	TryGetBoolField(JsonObj, "maintain_formant", MaintainFormant);

	const UStretcherAndPitchShifterFactoryConfig* FactoryConfig = GetDefault<UStretcherAndPitchShifterFactoryConfig>();
	check(FactoryConfig);

	FName AlgorithmName = FactoryConfig->DefaultFactory.Name;
	TryGetNameField(JsonObj, "algorithm", AlgorithmName, AlgorithmName);

	if (AlgorithmName == "default")
	{
		TimeStretchConfig.PitchShifter = FactoryConfig->DefaultFactory;
	}
	else if (const FPitchShifterNameRedirect* Redirect = FactoryConfig->FindFactoryNameRedirect(AlgorithmName))
	{
		TimeStretchConfig.PitchShifter = Redirect->NewName;
	}
	else
	{
		TimeStretchConfig.PitchShifter = AlgorithmName;
	}

	TryGetBoolField(JsonObj, "sync_tempo", TimeStretchConfig.bSyncTempo, Defaults.bSyncTempo);
	TryGetNumberField(JsonObj, "orig_tempo", TimeStretchConfig.OriginalTempo, Defaults.OriginalTempo);

	int16 EnvelopeOrder = 0;
	JsonObj->TryGetNumberField(TEXT("envelope_order"), EnvelopeOrder);

	TimeStretchConfig.PitchShifterOptions.Add("EnvelopeOrder", EnvelopeOrder);
	TimeStretchConfig.PitchShifterOptions.Add("MaintainFormant", MaintainFormant);

	return true;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FPannerDetails& OutPanDetails)
{
	FPannerDetails Defaults;
	TryGetEnumField(JsonObj, "mode", OutPanDetails.Mode, Defaults.Mode);
	switch (OutPanDetails.Mode)
	{
	case EPannerMode::LegacyStereo:
	case EPannerMode::Stereo:
		TryGetNumberField(JsonObj, "position", OutPanDetails.Detail.Pan, Defaults.Detail.Pan);
		OutPanDetails.Detail.EdgeProximity = 0.0f;
		return true;
	case EPannerMode::Surround:
	case EPannerMode::PolarSurround:
		TryGetNumberField(JsonObj, "position", OutPanDetails.Detail.Pan, Defaults.Detail.Pan);
		TryGetNumberField(JsonObj, "edge_proximity", OutPanDetails.Detail.EdgeProximity, Defaults.Detail.EdgeProximity);
		return true;
	
	case EPannerMode::DirectAssignment:
		TryGetEnumField(JsonObj, "position", OutPanDetails.Detail.ChannelAssignment, ESpeakerChannelAssignment::LeftFront);
		return true;
	}
	return false;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FFusionPatchSettings& PatchSettings)
{
	FFusionPatchSettings Defaults;
	TryGetNumberField(JsonObj, "volume", PatchSettings.VolumeDb, HarmonixDsp::kDbSilence);
	
	TryParseJson(JsonObj, PatchSettings.PannerDetails);

	TryGetNumberField(JsonObj, "start_point", PatchSettings.StartPointOffsetMs, Defaults.StartPointOffsetMs);
	TryGetNumberField(JsonObj, "fine_tune", PatchSettings.FineTuneCents, Defaults.FineTuneCents);
	TryGetNumberField(JsonObj, "voice_limit", PatchSettings.MaxVoices, 8);
	TryGetEnumField(JsonObj, "layer_select_mode", PatchSettings.KeyzoneSelectMode, EKeyzoneSelectMode::Layers);

	// apply defaults just in case
	PatchSettings.DownPitchBendCents = -700.0f;
	PatchSettings.UpPitchBendCents = 700.0f;
	TSharedPtr<FJsonObject> PitchBendObj;
	if (TryGetObjectField(JsonObj, "pitch_bend", PitchBendObj))
	{
		const TArray<TSharedPtr<FJsonValue>>* RangeValues;
		if (PitchBendObj->TryGetArrayField(TEXT("range"), RangeValues))
		{
			if (ensure(RangeValues->Num() == 2))
			{
				TryGetNumber((*RangeValues)[0], PatchSettings.DownPitchBendCents, Defaults.DownPitchBendCents);
				TryGetNumber((*RangeValues)[1], PatchSettings.UpPitchBendCents, Defaults.UpPitchBendCents);
			}
		}
	}
	
	int32 AdsrIdx = -1;
	for (TSharedPtr<FJsonValue> ArrayValue : IterField(JsonObj, "adsrs"))
	{
		++AdsrIdx;
		if (AdsrIdx >= FFusionPatchSettings::kNumAdsrs)
			break;
	
		TSharedPtr<FJsonObject> AdsrObj;
		if (!TryGetObject(ArrayValue, AdsrObj))
			continue;

		TSharedPtr<FJsonObject> AdsrValue;
		if (!TryGetObjectField(AdsrObj, "adsr", AdsrValue))
			continue;

		FAdsrSettings& AdsrSettings = PatchSettings.Adsr[AdsrIdx];
		if (!TryParseJson(AdsrValue, AdsrSettings))
			continue;
	
		// old defaults
		if (AdsrSettings.Target == EAdsrTarget::Invalid)
		{
			AdsrSettings.Target = EAdsrTarget::Volume;
			AdsrSettings.Depth = 1.0f;
			AdsrSettings.IsEnabled = true;
		}
	
		AdsrSettings.Calculate();
	}

	{
	int LfoIdx = -1;
	for (TSharedPtr<FJsonValue> ArrayValue : IterField(JsonObj, "lfos"))
	{
		++LfoIdx;

		if (!ensure(LfoIdx < FFusionPatchSettings::kNumLfos))
			break;

		TSharedPtr<FJsonObject> LfoObj;
		if (!TryGetObject(ArrayValue, LfoObj))
			continue;

		TSharedPtr<FJsonObject> LfoValue;
		if (!TryGetObjectField(LfoObj, "lfo", LfoValue))
			continue;

		FLfoSettings& LfoSettings = PatchSettings.Lfo[LfoIdx];

		if (!TryParseJson(LfoValue, LfoSettings))
			continue;

		if (!LfoValue->HasField(TEXT("target")))
		{
			// old defaults
			if (LfoIdx == 0)
				LfoSettings.Target = ELfoTarget::Pan;
			else if (LfoIdx == 1)
				LfoSettings.Target = ELfoTarget::Pitch;
		}
	}
	}

	TSharedPtr<FJsonObject> FilterObj;
	if (TryGetObjectField(JsonObj, "filter", FilterObj))
	{
		TryParseJson(FilterObj, PatchSettings.Filter);
	}
	else
	{
		PatchSettings.Filter.IsEnabled = false;
	}

	{
	int RandomizerIdx = -1;
	for (TSharedPtr<FJsonValue> ArrayValue : IterField(JsonObj, "randomizers"))
	{
		++RandomizerIdx;
		if (!ensure(RandomizerIdx < FFusionPatchSettings::kNumRandomizers))
			break;

		TSharedPtr<FJsonObject> RandomizerObj;
		if (!TryGetObject(ArrayValue, RandomizerObj))
			continue;

		TSharedPtr<FJsonObject> RandomizerValue;
		if (!TryGetObjectField(RandomizerObj, "modulator", RandomizerValue))
			continue;

		TryParseJson(RandomizerValue, PatchSettings.Randomizer[RandomizerIdx]);
	}
	}

	{
	int ModIdx = -1;
	for (TSharedPtr<FJsonValue> ArrayValue : IterField(JsonObj, "velocity_mods"))
	{
		++ModIdx;
		if (!ensure(ModIdx < FFusionPatchSettings::kNumModulators))
			break;

		TSharedPtr<FJsonObject> VelocityModObj;
		if (!TryGetObject(ArrayValue, VelocityModObj))
			continue;

		TSharedPtr<FJsonObject> VelocityModValue;
		if (!TryGetObjectField(VelocityModObj, "modulator", VelocityModValue))
			continue;

		TryParseJson(VelocityModValue, PatchSettings.VelocityModulator[ModIdx]);
	}
	}

	{
	TSharedPtr<FJsonObject> PortamentoObj;
	if (TryGetObjectField(JsonObj, "portamento", PortamentoObj))
	{
		TryParseJson(PortamentoObj, PatchSettings.Portamento);
	}
	else
	{
		PatchSettings.Portamento.IsEnabled = false;
		PatchSettings.Portamento.Mode = EPortamentoMode::Legato;
		PatchSettings.Portamento.Seconds = 0.0f;
	}
	}

	return true;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FAdsrSettings& AdsrSettings)
{
	// simplify the conversion process :)
	static TFunction<float(float)> MsToSec = [](float Ms) { return Ms * 0.001f; };
	TryGetEnumField(JsonObj, "target", AdsrSettings.Target, EAdsrTarget::Invalid);
	TryGetNumberField(JsonObj, "depth", AdsrSettings.Depth, 0.0f);
	TryGetBoolField(JsonObj, "enabled", AdsrSettings.IsEnabled, false);
	TryGetNumberField(JsonObj, "attack", AdsrSettings.AttackTime, MsToSec, 1.0f);
	TryGetNumberField(JsonObj, "attack_curve", AdsrSettings.AttackCurve, 0.0f);
	TryGetNumberField(JsonObj, "decay", AdsrSettings.DecayTime, MsToSec, 1.0f);
	TryGetNumberField(JsonObj, "decay_curve", AdsrSettings.DecayCurve, 0.0f);
	TryGetNumberField(JsonObj, "sustain", AdsrSettings.SustainLevel, 1.0f);
	TryGetNumberField(JsonObj, "release", AdsrSettings.ReleaseTime, MsToSec, 1.0f);
	TryGetNumberField(JsonObj, "release_curve", AdsrSettings.ReleaseCurve, 0.0f);

	return true;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FLfoSettings& LfoSettings)
{
	TryGetEnumField(JsonObj, "target", LfoSettings.Target, ELfoTarget::None);
	TryGetNumberField(JsonObj, "frequency", LfoSettings.Freq);
	TryGetNumberField(JsonObj, "depth", LfoSettings.Depth);
	TryGetBoolField(JsonObj, "enabled", LfoSettings.IsEnabled);
	TryGetEnumField(JsonObj, "shape", LfoSettings.Shape, EWaveShape::None);
	TryGetBoolField(JsonObj, "retrigger", LfoSettings.ShouldRetrigger, false);
	TryGetNumberField(JsonObj, "initial_phase", LfoSettings.InitialPhase, 0.0f);
	TryGetBoolField(JsonObj, "beat_sync", LfoSettings.BeatSync, false);
	TryGetNumberField(JsonObj, "tempo", LfoSettings.TempoBPM, 120.0f);

	return true;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FBiquadFilterSettings& Filter)
{
	TryGetBoolField(JsonObj, "enabled", Filter.IsEnabled);
	TryGetEnumField(JsonObj, "type", Filter.Type, EBiquadFilterType::None);
	TryGetNumberField(JsonObj, "frequency", Filter.Freq);
	TryGetNumberField(JsonObj, "q", Filter.Q);
	TryGetNumberField(JsonObj, "gain_db", Filter.DesignedDBGain, 0.0f);

	return true;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FModulatorSettings& ModulatorSettings)
{
	TryGetEnumField(JsonObj, "target", ModulatorSettings.Target, EModulatorTarget::None);
	TryGetNumberField(JsonObj, "range", ModulatorSettings.Range, 0.0f);
	TryGetNumberField(JsonObj, "depth", ModulatorSettings.Depth, 0.0f);
	return true;
}

bool FSettingsJsonImporter::TryParseJson(TSharedPtr<FJsonObject> JsonObj, FPortamentoSettings& PortamentoSettings)
{
	TryGetBoolField(JsonObj, "enabled", PortamentoSettings.IsEnabled, false);
	TryGetEnumField(JsonObj, "mode", PortamentoSettings.Mode, EPortamentoMode::Legato);
	TryGetNumberField(JsonObj, "time", PortamentoSettings.Seconds, 0.0f);
	return true;
}
