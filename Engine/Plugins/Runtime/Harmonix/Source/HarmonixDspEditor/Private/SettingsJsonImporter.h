// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JsonImporterHelper.h"

class FJsonObject;

class FSettingsJsonImporter : public FJsonImporter
{
public:

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FKeyzoneSettings& OutSettings);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FPannerDetails& OutPanDetails);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FTimeStretchConfig& TimeStretchConfig);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FFusionPatchSettings& PatchSettings);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FAdsrSettings& AdsrSettings);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FLfoSettings& LfoSettings);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FBiquadFilterSettings& FilterSettings);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FModulatorSettings& ModulatorSettings);

	static bool TryParseJson(TSharedPtr<FJsonObject> JsonObj, struct FPortamentoSettings& PortamentoSettings);
};