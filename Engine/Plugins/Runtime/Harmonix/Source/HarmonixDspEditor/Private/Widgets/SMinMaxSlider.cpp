// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMinMaxSlider.h"

void SMinMaxSlider::Construct(const FArguments& InDeclaration)
{
	SSlider::Construct(SSlider::FArguments()
		.MinValue(InDeclaration._MinValue)
		.MaxValue(InDeclaration._MaxValue)
	);

	LowerHandleValue = InDeclaration._LowerHandleValue;
	UpperHandleValue = InDeclaration._UpperHandleValue;
	OnLowerHandleValueChanged = InDeclaration._OnLowerHandleValueChanged;
	OnUpperHandleValueChanged = InDeclaration._OnUpperHandleValueChanged;
	IndentHandleAttribute = InDeclaration._IndentHandle;
	SliderBarColorAttribute = InDeclaration._SliderBarColor;
	SliderLowerHandleColorAttribute = InDeclaration._SliderLowerHandleColor;
	SliderUpperHandleColorAttribute = InDeclaration._SliderUpperHandleColor;
}

float SMinMaxSlider::GetLowerNormalizedValue() const
{
	if (MaxValue == MinValue)
	{
		return 1.0f;
	}
	else
	{
		return (LowerHandleValue.Get() - MinValue) / (MaxValue - MinValue);
	}
}


float SMinMaxSlider::GetUpperNormalizedValue() const
{
	if (MaxValue == MinValue)
	{
		return 1.0f;
	}
	else
	{
		return (UpperHandleValue.Get() - MinValue) / (MaxValue - MinValue);
	}
}

void SMinMaxSlider::SetLowerValue(TAttribute<float> InValueAttribute)
{
	float NewLowerValue = InValueAttribute.Get() <= UpperHandleValue.Get() ? InValueAttribute.Get() : UpperHandleValue.Get();
	LowerHandleValue.Set(NewLowerValue);
}

void SMinMaxSlider::SetUpperValue(TAttribute<float> InValueAttribute)
{
	float NewUpperValue = InValueAttribute.Get() >= LowerHandleValue.Get() ? InValueAttribute.Get() : LowerHandleValue.Get();
	UpperHandleValue.Set(NewUpperValue);
}

void SMinMaxSlider::OnSliderValueChanged(float NewValue)
{
	if (CurrentHandle == ESliderHandle::LowerHandle)
	{
		OnLowerHandleValueChanged.ExecuteIfBound(NewValue);
	}
	if (CurrentHandle == ESliderHandle::UpperHandle)
	{
		OnUpperHandleValueChanged.ExecuteIfBound(NewValue);
	}
}

FVector2D SMinMaxSlider::CalculateLowerHandlePosition(const FGeometry& HandleGeometry, float Value) const
{
	const float ClampedValue = FMath::Clamp(GetLowerNormalizedValue(), 0.0f, 1.0f);

	float XPosition = ClampedValue * HandleGeometry.GetLocalSize().X;

	float YPosition = HandleGeometry.GetLocalSize().Y * 0.5f;

	return FVector2D(XPosition, YPosition);
}

FVector2D SMinMaxSlider::CalculateUpperHandlePosition(const FGeometry& HandleGeometry, float Value) const
{
	const float ClampedValue = FMath::Clamp(GetUpperNormalizedValue(), 0.0f, 1.0f);

	float XPosition = ClampedValue * HandleGeometry.GetLocalSize().X;

	float YPosition = HandleGeometry.GetLocalSize().Y * 0.5f;

	return FVector2D(XPosition, YPosition);
}

SMinMaxSlider::ESliderHandle SMinMaxSlider::DetermineClickedHandle(const FGeometry& HandleGeometry, const FVector2D& LocalMousePosition) const
{
	FVector2D LowerHandlePos = CalculateLowerHandlePosition(HandleGeometry, LowerHandleValue.Get());
	FVector2D UpperHandlePos = CalculateUpperHandlePosition(HandleGeometry, UpperHandleValue.Get());

	float LowerHandleDistance = FVector2D::DistSquared(LowerHandlePos, LocalMousePosition);
	float UpperHandleDistance = FVector2D::DistSquared(UpperHandlePos, LocalMousePosition);

	if (LowerHandleDistance < UpperHandleDistance)
	{
		return ESliderHandle::LowerHandle;
	}
	else {
		return ESliderHandle::UpperHandle;
	}
}

int32 SMinMaxSlider::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const float AllottedWidth = Orientation == Orient_Horizontal ? AllottedGeometry.GetLocalSize().X : AllottedGeometry.GetLocalSize().Y;
	const float AllottedHeight = Orientation == Orient_Horizontal ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X;

	float HandleRotation;
	FVector2f HandleTopRightPoint;
	FVector2f HandleTopLeftPoint;
	FVector2f SliderStartPoint;
	FVector2f SliderEndPoint;

	// calculate slider geometry as if it's a horizontal slider (we'll rotate it later if it's vertical)
	const FVector2f HandleSize = GetThumbImage()->ImageSize;
	const FVector2f HalfHandleSize = 0.5f * HandleSize;
	const float Indentation = IndentHandleAttribute.Get() ? HandleSize.X : 0.0f;

	// We clamp to make sure that the slider cannot go out of the slider Length.
	const float SliderPercent = FMath::Clamp(GetLowerNormalizedValue(), 0.0f, 1.0f);
	const float UpperSliderPercent = FMath::Clamp(GetUpperNormalizedValue(), 0.0f, 1.0f);

	const float SliderLength = AllottedWidth - (Indentation + HandleSize.X);
	const float SliderHandleOffset = SliderPercent * SliderLength;
	const float SliderUpperHandleOffset = UpperSliderPercent * SliderLength;
	const float SliderY = 0.5f * AllottedHeight;

	HandleRotation = 0.0f;
	HandleTopRightPoint = FVector2f(SliderUpperHandleOffset + (0.5f * Indentation), SliderY - HalfHandleSize.Y);
	HandleTopLeftPoint = FVector2f(SliderHandleOffset + (0.5f * Indentation), SliderY - HalfHandleSize.Y);

	SliderStartPoint = FVector2f(HalfHandleSize.X, SliderY);
	SliderEndPoint = FVector2f(AllottedWidth - HalfHandleSize.X, SliderY);

	FGeometry SliderGeometry = AllottedGeometry;

	// rotate the slider 90deg if it's vertical. The 0 side goes on the bottom, the 1 side on the top.
	if (Orientation == Orient_Vertical)
	{
		FSlateRenderTransform SlateRenderTransform = TransformCast<FSlateRenderTransform>(Concatenate(Inverse(FVector2f(AllottedWidth, 0)), FQuat2D(FMath::DegreesToRadians(-90.0f))));
		SliderGeometry = AllottedGeometry.MakeChild(
			FVector2f(AllottedWidth, AllottedHeight),
			FSlateLayoutTransform(),
			SlateRenderTransform, FVector2f::ZeroVector);
	}

	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	// draw slider bar
	auto BarTopLeft = FVector2f(SliderStartPoint.X, SliderStartPoint.Y - Style->BarThickness * 0.5f);
	auto BarSize = FVector2f(SliderEndPoint.X - SliderStartPoint.X, Style->BarThickness);
	auto BarImage = GetBarImage();
	auto ThumbImage = GetThumbImage();

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		SliderGeometry.ToPaintGeometry(BarSize, FSlateLayoutTransform(BarTopLeft)),
		BarImage,
		DrawEffects,
		BarImage->GetTint(InWidgetStyle) * SliderBarColorAttribute.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	++LayerId;

	// draw slider thumbs
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		SliderGeometry.ToPaintGeometry(GetThumbImage()->ImageSize, FSlateLayoutTransform(HandleTopLeftPoint)),
		ThumbImage,
		DrawEffects,
		ThumbImage->GetTint(InWidgetStyle) * SliderLowerHandleColorAttribute.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		SliderGeometry.ToPaintGeometry(GetThumbImage()->ImageSize, FSlateLayoutTransform(HandleTopRightPoint)),
		ThumbImage,
		DrawEffects,
		ThumbImage->GetTint(InWidgetStyle) * SliderUpperHandleColorAttribute.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	return LayerId;
}

float SMinMaxSlider::PositionToValue(const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& AbsolutePosition)
{
	float HandleValue = LowerHandleValue.Get();
	if (CurrentHandle == ESliderHandle::UpperHandle)
	{
		HandleValue = UpperHandleValue.Get();
	}

	const FVector2f LocalPosition = MyGeometry.AbsoluteToLocal(AbsolutePosition);

	float RelativeValue;
	float Denominator;
	// Only need X as we rotate the thumb image when rendering vertically
	const float Indentation = SSlider::GetThumbImage()->ImageSize.X * (IndentHandleAttribute.Get() ? 2.f : 1.f);
	const float HalfIndentation = 0.5f * Indentation;

	if (Orientation == Orient_Horizontal)
	{
		Denominator = MyGeometry.Size.X - Indentation;
		RelativeValue = (Denominator != 0.f) ? (LocalPosition.X - HalfIndentation) / Denominator : 0.f;
	}
	else
	{
		Denominator = MyGeometry.Size.Y - Indentation;
		// Inverse the calculation as top is 0 and bottom is 1
		RelativeValue = (Denominator != 0.f) ? ((MyGeometry.Size.Y - LocalPosition.Y) - HalfIndentation) / Denominator : 0.f;
	}

	RelativeValue = FMath::Clamp(RelativeValue, 0.0f, 1.0f) * (MaxValue - MinValue) + MinValue;
	if (bMouseUsesStep)
	{
		float direction = HandleValue - RelativeValue;
		float CurrentStepSize = StepSize.Get();
		if (direction > CurrentStepSize / 2.0f)
		{
			return FMath::Clamp(HandleValue - CurrentStepSize, MinValue, MaxValue);
		}
		else if (direction < CurrentStepSize / -2.0f)
		{
			return FMath::Clamp(HandleValue + CurrentStepSize, MinValue, MaxValue);
		}
		else
		{
			return HandleValue;
		}
	}
	return RelativeValue;
}

void SMinMaxSlider::CommitValue(float NewValue)
{
	const float OldValue = GetValue();

	if (NewValue != OldValue)
	{
		if (CurrentHandle == ESliderHandle::LowerHandle)
		{
			SetLowerValue(NewValue);

			Invalidate(EInvalidateWidgetReason::Paint);

			OnLowerHandleValueChanged.ExecuteIfBound(LowerHandleValue.Get());
		}

		if (CurrentHandle == ESliderHandle::UpperHandle)
		{
			SetUpperValue(NewValue);

			Invalidate(EInvalidateWidgetReason::Paint);

			OnUpperHandleValueChanged.ExecuteIfBound(UpperHandleValue.Get());
		}

	}
}

FReply SMinMaxSlider::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && !IsLocked())
	{
		FVector2D LocalMousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		CurrentHandle = DetermineClickedHandle(MyGeometry, LocalMousePosition);
		CachedCursor = GetCursor().Get(EMouseCursor::Default);

		CommitValue(PositionToValue(MyGeometry, MouseEvent.GetScreenSpacePosition()));

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SMinMaxSlider::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{

	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && HasMouseCaptureByUser(MouseEvent.GetUserIndex(), MouseEvent.GetPointerIndex()))
	{
		CurrentHandle = ESliderHandle::None;
		SetCursor(CachedCursor);

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();

}