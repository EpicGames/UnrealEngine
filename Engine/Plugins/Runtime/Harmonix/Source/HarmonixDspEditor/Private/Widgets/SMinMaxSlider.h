// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Input/SSlider.h"

DECLARE_DELEGATE_OneParam(FOnLowerValueChanged, float)
DECLARE_DELEGATE_OneParam(FOnUpperValueChanged, float)

/**
 * A custom range slider class inherited from SSlider for customizing FusionPatch's min and max note/velocity properties
 * 2 non-overlapped thumb handles controlling a lower value and a higher value
 */
class SMinMaxSlider : public SSlider
{
public:
	SLATE_BEGIN_ARGS(SMinMaxSlider)
		:_LowerHandleValue(0.0f)
		, _UpperHandleValue(1.0f)
		, _IndentHandle(true)
		, _SliderBarColor(FLinearColor::White)
		, _SliderLowerHandleColor(FLinearColor::White)
		, _SliderUpperHandleColor(FLinearColor::White)
		, _MinValue(0.0f)
		, _MaxValue(1.0f)
	{}
	    SLATE_ATTRIBUTE(float, LowerHandleValue)
		SLATE_ATTRIBUTE(float, UpperHandleValue)
		SLATE_ATTRIBUTE(bool, IndentHandle)
		SLATE_ATTRIBUTE(FSlateColor, SliderBarColor)
		SLATE_ATTRIBUTE(FSlateColor, SliderLowerHandleColor)
		SLATE_ATTRIBUTE(FSlateColor, SliderUpperHandleColor)
		SLATE_EVENT(FOnLowerValueChanged, OnLowerHandleValueChanged)
		SLATE_EVENT(FOnUpperValueChanged, OnUpperHandleValueChanged)
		SLATE_ARGUMENT(float, MinValue)
		SLATE_ARGUMENT(float, MaxValue)

    SLATE_END_ARGS()

	void Construct(const FArguments& InDeclaration);
	
    float GetLowerNormalizedValue() const;
	float GetUpperNormalizedValue() const;
	void SetLowerValue(TAttribute<float> InValueAttribute);
	void SetUpperValue(TAttribute<float> InValueAttribute);
	void OnSliderValueChanged(float NewValue);

protected:
	enum class ESliderHandle
	{
		None,
		LowerHandle,
		UpperHandle
	};

	FVector2D CalculateLowerHandlePosition(const FGeometry& HandleGeometry, float Value) const;
	FVector2D CalculateUpperHandlePosition(const FGeometry& HandleGeometry, float Value) const;
	ESliderHandle DetermineClickedHandle(const FGeometry& HandleGeometry, const FVector2D& LocalMousePosition) const;
	float PositionToValue(const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& AbsolutePosition);
	
	virtual void CommitValue(float NewValue) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
	TAttribute<float> LowerHandleValue;
	TAttribute<float> UpperHandleValue;
	FOnLowerValueChanged OnLowerHandleValueChanged;
	FOnUpperValueChanged OnUpperHandleValueChanged;
	TAttribute<bool> IndentHandleAttribute;
	TAttribute<FSlateColor> SliderBarColorAttribute;
	TAttribute<FSlateColor> SliderLowerHandleColorAttribute;
	TAttribute<FSlateColor> SliderUpperHandleColorAttribute;
	ESliderHandle CurrentHandle = ESliderHandle::None;

};