// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixDspEditor, Log, All);

class FHarmonixDspEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

private:
	void RegisterMenus();
};
