// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDsp/Modulators/Settings/AdsrSettings.h"
#include "DSP/AlignedBuffer.h"

namespace Harmonix::Dsp::Editor
{
	void HARMONIXDSPEDITOR_API GenerateAdsrEnvelope(const FAdsrSettings& InAdsrSettings, float SustainTime, float SampleRate, Audio::FAlignedFloatBuffer& OutBuffer);
}
