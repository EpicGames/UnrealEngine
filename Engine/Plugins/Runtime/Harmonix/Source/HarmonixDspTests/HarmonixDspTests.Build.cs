// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixDspTests : ModuleRules
{
	public HarmonixDspTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateIncludePaths.Add(System.IO.Path.Combine(GetModuleDirectory("HarmonixDspEditor"), "Private"));

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"UnrealEd",
				"HarmonixDsp",
				"HarmonixDspEditor",
				"HarmonixMidi",
				"Projects",
				"SignalProcessing",
				"Json"
			}
		);
	}
}
