// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Generate.h"
#include "HarmonixDsp/AudioAnalysis/FFTAnalyzer.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Dsp::AudioAnalysis::FFTAnalyzer::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FHarmonixFFTAnalyzerBasicTest,
		"Harmonix.Dsp.AudioAnalysis.Fft.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FHarmonixFFTAnalyzerBasicTest::RunTest(const FString&)
	{
		constexpr int32 NumFramesPerBlock = 256;
		constexpr float SampleRate = 48000;
		
		FFFTAnalyzer Analyzer{ SampleRate };
		FHarmonixFFTAnalyzerSettings Settings;
		Settings.MelScaleBinning = true;
		Analyzer.SetSettings(Settings);
		
		FHarmonixFFTAnalyzerResults Results;

		// process a few times so we can ensure we have some energy in the spectrum
		Audio::FAlignedFloatBuffer Buffer;
		Buffer.SetNumUninitialized(NumFramesPerBlock);
		constexpr float Frequency = 440;
		float Phase = 0;
		constexpr int32 NumBlocks = 200;
		for (int i = 0; i < NumBlocks; ++i)
		{
			// Fill the buffer with a sine
			HarmonixDsp::GenerateSine(Buffer.GetData(), NumFramesPerBlock, Frequency, SampleRate, Phase);
			Analyzer.Process(Buffer, Results);
		}
		
		// make sure we got some results
		UTEST_TRUE("Got some spectrum results", Results.Spectrum.Num() > 0);
		bool ABinWasGreaterThanZero = false;
		for (const float& SpectrumValue : Results.Spectrum)
		{
			if (SpectrumValue > 0.0f)
			{
				ABinWasGreaterThanZero = true;
				break;
			}
		}
		UTEST_TRUE("A bin read greater than zero energy", ABinWasGreaterThanZero);
		
		return true;
	}
}

#endif
