// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/AudioBuffer.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixDspTests::AudioBuffer
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestInitEmpty, 
		"Harmonix.Dsp.AudioBuffer.Init.Empty", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTestInitEmpty::RunTest(const FString&)
	{
		const TAudioBuffer<float> Buffer;
		UTEST_TRUE("Buffer is silent", Buffer.GetIsSilent());
		UTEST_EQUAL("Buffer has zero channels", Buffer.GetNumValidChannels(), 0);
		UTEST_EQUAL("Buffer has zero samples", Buffer.GetLengthInFrames(), 0);
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FTestChannelCopyIsEqual, 
	"Harmonix.Dsp.AudioBuffer.EqualWithTolerance.CopyIsEqual", 
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTestChannelCopyIsEqual::RunTest(const FString&)
	{
		constexpr int32 NumChannels = 1;
		constexpr int32 NumSamples = 1024;

		// fill one buffer with some value
		TAudioBuffer<float> Buffer{NumChannels, NumSamples, EAudioBufferCleanupMode::Delete};
		Buffer.Fill(0.5f);

		// copy to a new buffer
		TAudioBuffer<float> BufferCopy{NumChannels, NumSamples, EAudioBufferCleanupMode::Delete};
		BufferCopy.Copy(Buffer);

		// the copy channel should evaluate as equal to the original
		UTEST_TRUE("Copy is equal", Buffer.EqualWithTolerance(0, BufferCopy, 0, UE_SMALL_NUMBER));
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FTestChannelSameIsEqual, 
	"Harmonix.Dsp.AudioBuffer.EqualWithTolerance.SameIsEqual", 
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTestChannelSameIsEqual::RunTest(const FString&)
	{
		constexpr int32 NumChannels = 1;
		constexpr int32 NumSamples = 1024;

		// fill buffer with some value
		TAudioBuffer<float> Buffer{NumChannels, NumSamples, EAudioBufferCleanupMode::Delete};
		Buffer.Fill(0.9f);

		// the channel should evaluate as equal to itself
		UTEST_TRUE("Same is equal", Buffer.EqualWithTolerance(0, Buffer, 0, UE_SMALL_NUMBER));
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FTestChannelDifferentIsNotEqual, 
	"Harmonix.Dsp.AudioBuffer.EqualWithTolerance.DifferentIsNotEqual", 
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTestChannelDifferentIsNotEqual::RunTest(const FString&)
	{
		constexpr int32 NumChannels = 1;
		constexpr int32 NumSamples = 1024;

		// fill one buffer with some value
		TAudioBuffer<float> Buffer{NumChannels, NumSamples, EAudioBufferCleanupMode::Delete};
		Buffer.Fill(0.25f);

		// fill another buffer with a different value
		TAudioBuffer<float> Buffer2{NumChannels, NumSamples, EAudioBufferCleanupMode::Delete};
		Buffer.Fill(0.85f);

		// the two channels shouldn't evaluate to equal
		UTEST_FALSE("Buffers are not equal", Buffer.EqualWithTolerance(0, Buffer2, 0, UE_SMALL_NUMBER));
		
		return true;
	}

}

#endif
