// Copyright Epic Games, Inc. All Rights Reserved.
#include "Dta/DtaParser.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Dsp::Dta::Tests
{
	static bool IsValidJson(const FString& JsonString)
	{
		TSharedPtr<FJsonObject> JsonObj;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
		return FJsonSerializer::Deserialize(Reader, JsonObj) && JsonObj.IsValid();
	}

	//~ Tokenizer Error Tests

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_UnbalancedOpenParen,
		"Harmonix.Dsp.Dta.DtaParser.Tokenizer.UnbalancedOpenParen",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_UnbalancedOpenParen::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(key value"), JsonString, ErrorMessage);
		UTEST_FALSE("Should fail on unbalanced open paren", bResult);
		UTEST_TRUE("Error message should not be empty", !ErrorMessage.IsEmpty());
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_UnbalancedCloseParen,
		"Harmonix.Dsp.Dta.DtaParser.Tokenizer.UnbalancedCloseParen",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_UnbalancedCloseParen::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("key value)"), JsonString, ErrorMessage);
		UTEST_FALSE("Should fail on unbalanced close paren", bResult);
		UTEST_TRUE("Error message should not be empty", !ErrorMessage.IsEmpty());
		return true;
	}

	//~ Parser Error Tests

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_EmptyParens,
		"Harmonix.Dsp.Dta.DtaParser.Parser.EmptyParens",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_EmptyParens::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("()"), JsonString, ErrorMessage);
		UTEST_FALSE("Should fail on empty parens", bResult);
		UTEST_TRUE("Error mentions empty parentheses", ErrorMessage.Contains(TEXT("Empty parentheses")));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_SingleToken,
		"Harmonix.Dsp.Dta.DtaParser.Parser.SingleToken",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_SingleToken::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(foo)"), JsonString, ErrorMessage);
		UTEST_FALSE("Should fail on single token in parens", bResult);
		UTEST_TRUE("Error mentions single token", ErrorMessage.Contains(TEXT("single token")));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_BareTokenAtRoot,
		"Harmonix.Dsp.Dta.DtaParser.Parser.BareTokenAtRoot",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_BareTokenAtRoot::RunTest(const FString&)
	{
		// Regression test for the actual crashing file: stray 's' before first paren
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("s(version 1)"), JsonString, ErrorMessage);
		UTEST_FALSE("Should fail on bare token at root level", bResult);
		UTEST_TRUE("Error mentions unexpected token", ErrorMessage.Contains(TEXT("Unexpected token")));
		return true;
	}

	//~ Simple Parsing Tests

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_SimpleKeyValue,
		"Harmonix.Dsp.Dta.DtaParser.Parser.SimpleKeyValue",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_SimpleKeyValue::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(version 1)"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"version\":1}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_QuotedStringValue,
		"Harmonix.Dsp.Dta.DtaParser.Parser.QuotedStringValue",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_QuotedStringValue::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(mode \"legacy_stereo\")"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"mode\":\"legacy_stereo\"}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_NestedObject,
		"Harmonix.Dsp.Dta.DtaParser.Parser.NestedObject",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_NestedObject::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(pan (mode \"stereo\")(position 0))"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"pan\":{\"mode\":\"stereo\",\"position\":0}}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_ArrayValues,
		"Harmonix.Dsp.Dta.DtaParser.Parser.ArrayValues",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_ArrayValues::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(range -200 200)"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"range\":[-200,200]}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_Comments,
		"Harmonix.Dsp.Dta.DtaParser.Tokenizer.Comments",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_Comments::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("; this is a comment\n(key value)"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully, ignoring comment", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"key\":value}")));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_InlineComment,
		"Harmonix.Dsp.Dta.DtaParser.Tokenizer.InlineComment",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_InlineComment::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(key value) ; trailing comment"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully, ignoring inline comment", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"key\":value}")));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_DeepNesting,
		"Harmonix.Dsp.Dta.DtaParser.Parser.DeepNesting",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_DeepNesting::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(a (b (c 1)))"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"a\":{\"b\":{\"c\":1}}}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	//~ Structural / Regression Tests

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_WrappedObjectAtRoot,
		"Harmonix.Dsp.Dta.DtaParser.Parser.WrappedObjectAtRoot",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_WrappedObjectAtRoot::RunTest(const FString&)
	{
		// This is the crash regression test: a root-level wrapped object
		// previously caused a StaticCastSharedPtr to FNode_Pair on an FNode_Object
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("((a 1)(b 2))"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully (was crashing before fix)", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"a\":1,\"b\":2}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_AnonymousObjectArray,
		"Harmonix.Dsp.Dta.DtaParser.Parser.AnonymousObjectArray",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_AnonymousObjectArray::RunTest(const FString&)
	{
		// Tests the "band" pattern: key followed by multiple anonymous objects
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(
			TEXT("(items ((a 1)(b 2))((c 3)(d 4)))"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"items\":[{\"a\":1,\"b\":2},{\"c\":3,\"d\":4}]}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_IdenticalKeysBecomesArray,
		"Harmonix.Dsp.Dta.DtaParser.Parser.IdenticalKeysBecomesArray",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_IdenticalKeysBecomesArray::RunTest(const FString&)
	{
		// When all keys in an object are identical, it becomes an array of objects
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(
			TEXT("(parent (child (x 1))(child (x 2)))"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_TRUE("Should contain array syntax", JsonString.Contains(TEXT("[")));
		UTEST_TRUE("Should contain child key", JsonString.Contains(TEXT("\"child\"")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_PathStringValue,
		"Harmonix.Dsp.Dta.DtaParser.Parser.PathStringValue",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_PathStringValue::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(
			TEXT("(sample_path \"../samples/test.mogg\")"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"sample_path\":\"../samples/test.mogg\"}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_NegativeAndFloatValues,
		"Harmonix.Dsp.Dta.DtaParser.Parser.NegativeAndFloatValues",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_NegativeAndFloatValues::RunTest(const FString&)
	{
		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(TEXT("(volume -1.0000)"), JsonString, ErrorMessage);
		UTEST_TRUE("Should parse successfully", bResult);
		UTEST_EQUAL("JSON output", JsonString, FString(TEXT("{\"volume\":-1.0000}")));
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));
		return true;
	}

	//~ Integration Test with realistic fusion-style DTA

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDtaParser_FusionPatchStructure,
		"Harmonix.Dsp.Dta.DtaParser.Integration.FusionPatchStructure",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDtaParser_FusionPatchStructure::RunTest(const FString&)
	{
		// Simplified fusion patch with key patterns found in real .fusion files:
		// - top-level pairs, nested objects, identical keys (keyzone), anonymous objects (band),
		//   quoted strings, negative numbers, floats, comments, paths
		const FString DtaInput = TEXT(
			"; test fusion patch\n"
			"(version 1)\n"
			"(keymap\n"
			"   (keyzone\n"
			"      (sample_path \"../samples/test.mogg\")\n"
			"      (root_note 60)\n"
			"      (volume -1.0000)\n"
			"      (pan\n"
			"         (mode \"legacy_stereo\")\n"
			"         (position 0.0000)\n"
			"      )\n"
			"   )\n"
			"   (keyzone\n"
			"      (sample_path \"../samples/test2.mogg\")\n"
			"      (root_note 84)\n"
			"      (volume -1.0000)\n"
			"      (pan\n"
			"         (mode \"legacy_stereo\")\n"
			"         (position 0.0000)\n"
			"      )\n"
			"   )\n"
			")\n"
			"(presets\n"
			"   (Default\n"
			"      (volume -2.0000)\n"
			"      (pitch_bend\n"
			"         (range -200.0000 200.0000)\n"
			"      )\n"
			"      (vocoder_effect\n"
			"         (enabled 1)\n"
			"         (band\n"
			"            (\n"
			"               (gain_db 1.0000)\n"
			"               (solo 0)\n"
			"            )\n"
			"            (\n"
			"               (gain_db 1.0000)\n"
			"               (solo 0)\n"
			"            )\n"
			"         )\n"
			"      )\n"
			"   )\n"
			")\n"
		);

		FString JsonString;
		FString ErrorMessage;
		bool bResult = FDtaParser::DtaStringToJsonString(DtaInput, JsonString, ErrorMessage);
		UTEST_TRUE(FString::Printf(TEXT("Should parse successfully: %s"), *ErrorMessage), bResult);
		UTEST_TRUE("Output should not be empty", !JsonString.IsEmpty());
		UTEST_TRUE("JSON is valid", IsValidJson(JsonString));

		// Verify key structural elements are present in the JSON
		UTEST_TRUE("Contains version key", JsonString.Contains(TEXT("\"version\"")));
		UTEST_TRUE("Contains keymap key", JsonString.Contains(TEXT("\"keymap\"")));
		UTEST_TRUE("Contains presets key", JsonString.Contains(TEXT("\"presets\"")));
		UTEST_TRUE("Contains sample_path", JsonString.Contains(TEXT("\"sample_path\"")));
		UTEST_TRUE("Contains band key", JsonString.Contains(TEXT("\"band\"")));
		UTEST_TRUE("Contains gain_db", JsonString.Contains(TEXT("\"gain_db\"")));
		UTEST_TRUE("Contains quoted string value", JsonString.Contains(TEXT("\"legacy_stereo\"")));
		UTEST_TRUE("Contains negative float", JsonString.Contains(TEXT("-2.0000")));
		UTEST_TRUE("Contains array for range", JsonString.Contains(TEXT("[-200.0000,200.0000]")));

		return true;
	}
}

#endif
