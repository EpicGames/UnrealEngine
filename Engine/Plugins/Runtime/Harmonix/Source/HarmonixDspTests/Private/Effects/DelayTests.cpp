// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/Effects/Delay.h"
#include "Misc/AutomationTest.h"


#if 0 // WITH_DEV_AUTOMATION_TESTS
	
namespace Harmonix::Dsp::Effects::Delay::Tests
{
	using namespace Harmonix::Dsp::Effects;

	class FTestDelay : public FAutomationTestBase
	{
	public:
		FTestDelay(const FString& InName, const bool bInComplexTask) 
			: FAutomationTestBase(InName, bInComplexTask) {};

		FTestDelay(const FString& InName)
			: FAutomationTestBase(InName, false) {};

		// simple struct to test input Sample with output OutDelay
		struct FTestCase
		{
			float Sample;
			float OutDelay;
		};

		struct FParameters
		{
			float SampleRate = 4;
			int32 NumChannels = 1;
			float MaxDelayTime = 1000.0f;

			float DelayInSamples = 1.0f;
			float WetGain = 1.0f;
			float Feedback = 0.0f;

			TArray<FTestCase> Cases;

			FString ToString() const
			{
				return FString::Printf(TEXT("Delay In Samples = %.2f\n"
										    "Wet Gain = %.2f\n"
											"Feedback = %.2f\n"), 
											DelayInSamples, 
											WetGain, 
											Feedback);
			};
		};

	protected:

		bool RunTest_ProcessInternal(const FString& Parameters, const FParameters& Test)
		{
			FDelay Delay;
			
			Delay.Prepare(Test.SampleRate, Test.NumChannels, Test.MaxDelayTime);
			float DelayInSeconds = Test.DelayInSamples / Test.SampleRate;
			Delay.Clear();
			Delay.SetTimeSyncOption(ETimeSyncOption::SpeedScale);
			Delay.SetFeedbackGain(Test.Feedback);
			Delay.SetWetGain(Test.WetGain);
			Delay.SetDelaySeconds(DelayInSeconds);

			Delay.ApplyNewParams();

			TArray<FDelay::FDelayOutput> Outputs;
			Outputs.SetNumUninitialized(Test.Cases.Num());


			float* DelayLine = Delay.DelayLine.GetValidChannelData(0);
			for (int TestIdx = 0; TestIdx < Test.Cases.Num(); ++TestIdx)
			{
				const FTestCase& Case = Test.Cases[TestIdx];
				FDelay::FDelayOutput& Output = Outputs[TestIdx];
				uint32 DelayLinePos = TestIdx % Delay.Length;
				Delay.ProcessInternal(Case.Sample, DelayLinePos, DelayLine, Test.DelayInSamples, Test.Feedback, Output);
				DelayLine[DelayLinePos] = Output.Feedback;
			};

			UTEST_TRUE(*PrintTestCases(Test.ToString(), Test.Cases, Outputs), CheckAll(Test.Cases, Outputs));

			return true;
		};

		bool CheckAll(const TArray<FTestCase> Cases, const TArray<FDelay::FDelayOutput> Outputs)
		{
			if (Cases.Num() != Outputs.Num())
				return false;

			for (int Idx = 0; Idx < Cases.Num(); ++Idx)
			{
				if (Cases[Idx].OutDelay != Outputs[Idx].Delay)
					return false;
			}

			// all tests passed
			return true;
		};

		FString PrintTestCases(const FString& What, const TArray<FTestCase> Cases, const TArray<FDelay::FDelayOutput> Outputs)
		{

			TArray<FString> Expected;
			Expected.Reset(Cases.Num());
			TArray<FString> Actual;
			Actual.Reset(Outputs.Num());

			for (const FTestCase& Case : Cases)
			{
				Expected.Add(FString::SanitizeFloat(Case.OutDelay, 2));
			};

			for (const FDelay::FDelayOutput& Output : Outputs)
			{
				Actual.Add(FString::SanitizeFloat(Output.Delay, 2));
			};

			return FString::Printf(TEXT("Expected outputs to be equal:\n"
				"Case: %s\n"
				"Expected: {%s}\n"
				"Actual:   {%s}\n"),
				*What, *FString::Join(Expected, TEXT(", ")), *FString::Join(Actual, TEXT(", ")));

		};
	};

	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
		FTestDelay2,
		FTestDelay,
		"Harmonix.Dsp.Effects.Delay.ProcessInternal_DelayInSamples=2.0_Feeback=1.0",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

	bool FTestDelay2::RunTest(const FString& Parameters)
	{
		FParameters Test;
		Test.SampleRate = 4;
		Test.NumChannels = 1;
		Test.MaxDelayTime = 1000.0f;
		Test.DelayInSamples = 2.0f;
		Test.WetGain = 1.0f;
		Test.Feedback = 1.0f;
		Test.Cases = {
			{1.0f, 0.0f},
			{0.0f, 0.0f},
			{0.0f, 1.0f},
			{0.0f, 0.0f},
			{0.0f, 1.0f},
			{0.0f, 0.0f},
			{0.0f, 1.0f}
		};

		return RunTest_ProcessInternal(Parameters, Test);
	};

	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
		FTestDelay1,
		FTestDelay,
		"Harmonix.Dsp.Effects.Delay.ProcessInternal_DelayInSamples=3.0",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
	)
	bool FTestDelay1::RunTest(const FString& Parameters)
	{
		FParameters Test;
		Test.SampleRate = 4;
		Test.NumChannels = 1;
		Test.MaxDelayTime = 1000.0f;
		Test.DelayInSamples = 3.0f;
		Test.WetGain = 1.0f;
		Test.Feedback = 0.0f;
		Test.Cases = {
				{1.0f, 0.0f},
				{2.0f, 0.0f},
				{4.0f, 0.0f},
				{0.0f, 1.0f},
				{0.0f, 2.0f},
				{0.0f, 4.0f}
		};

		return RunTest_ProcessInternal(Parameters, Test);
	};

	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
		FTestDelay0,
		FTestDelay,
		"Harmonix.Dsp.Effects.Delay.ProcessInternal_DelayInSamples=1.5",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
	)
		bool FTestDelay0::RunTest(const FString& Parameters)
	{
		FParameters Test;
		Test.SampleRate = 4;
		Test.NumChannels = 1;
		Test.MaxDelayTime = 1000.0f;
		Test.DelayInSamples = 1.5f;
		Test.WetGain = 1.0f;
		Test.Feedback = 0.0f;
		Test.Cases = {
				{1.0f, 0.0f},
				{0.0f, 0.5f},
				{2.0f, 0.5f},
				{0.0f, 1.0f},
				{4.0f, 1.0f},
				{8.0f, 2.0f},
				{2.0f, 6.0f},
				{1.0f, 5.0f},
				{0.0f, 1.5f},
				{0.0f, 0.5f},
				{0.0f, 0.0f}
		};

		return RunTest_ProcessInternal(Parameters, Test);
	};
};
#endif
