// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Generate.h"
#include "HarmonixDsp/Effects/DjFilter.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Dsp::Effects::DjFilter::Tests
{
	BEGIN_DEFINE_SPEC(
		FHarmonixDspDjFilterSpec,
		"Harmonix.Dsp.Effects.DjFilter",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	FDjFilter Filter{ 0 };
	
	END_DEFINE_SPEC(FHarmonixDspDjFilterSpec)

	void FHarmonixDspDjFilterSpec::Define()
	{
		Describe("With defaults", [this]()
		{
			constexpr float SampleRate = 48000;
			constexpr int32 NumSamplesPerBlock = 128;
			
			BeforeEach([this, SampleRate]()
			{
				Filter.Reset(SampleRate);
			});
			
			It("when Amount is 0, no filter is applied", [this, SampleRate, NumSamplesPerBlock]()
			{
				if (!TestEqual("Default Amount is zero", Filter.Amount.Get(), 0.0f))
				{
					return;
				}

				Audio::FAlignedFloatBuffer InputBuffer;
				InputBuffer.SetNumUninitialized(NumSamplesPerBlock);
				Audio::FAlignedFloatBuffer OutputBuffer;
				OutputBuffer.SetNumZeroed(NumSamplesPerBlock);
				
				constexpr int32 NumBlocks = 40;
				
				for (int32 i = 0; i < NumBlocks; ++i)
				{
					HarmonixDsp::GenerateWhiteNoise(InputBuffer.GetData(), NumSamplesPerBlock);
					Filter.Process(InputBuffer, OutputBuffer);
					
					for (int32 SampleIdx = 0; SampleIdx < NumSamplesPerBlock; ++SampleIdx)
					{
						if (!TestEqual(
							FString::Printf(TEXT("Samples match in block %i, sample %i"), i, SampleIdx),
							OutputBuffer[i],
							InputBuffer[i]))
						{
							return;
						}
					}
				}
			});

			It("when Amount is -1, low-pass filter is applied", [this, SampleRate, NumSamplesPerBlock]()
			{
				Filter.Amount = -1;
				
				Audio::FAlignedFloatBuffer InputBuffer;
				InputBuffer.SetNumUninitialized(NumSamplesPerBlock);
				Audio::FAlignedFloatBuffer OutputBuffer;
				OutputBuffer.SetNumZeroed(NumSamplesPerBlock);
				Audio::FAlignedFloatBuffer ExpectedOutputBuffer;
				ExpectedOutputBuffer.SetNumZeroed(NumSamplesPerBlock);

				// make a filter we can use for comparison
				Audio::FStateVariableFilter FilterForComparison;
				FilterForComparison.Init(SampleRate, 1);
				FilterForComparison.SetFrequency(Filter.LowPassMinFrequency);
				FilterForComparison.SetQ(Filter.Resonance);
				FilterForComparison.SetFilterType(Audio::EFilter::LowPass);
				FilterForComparison.Update();
				
				constexpr int32 NumBlocks = 40;
							
				for (int32 i = 0; i < NumBlocks; ++i)
				{
					HarmonixDsp::GenerateWhiteNoise(InputBuffer.GetData(), NumSamplesPerBlock);
					
					Filter.Process(InputBuffer, OutputBuffer);

					FilterForComparison.ProcessAudio(InputBuffer.GetData(), NumSamplesPerBlock, ExpectedOutputBuffer.GetData());

					// skip the first block, which will be doing the dry->wet cross-fade
					if (i == 0)
					{
						continue;
					}
					
					for (int32 SampleIdx = 0; SampleIdx < NumSamplesPerBlock; ++SampleIdx)
					{
						if (!TestEqual(
							FString::Printf(TEXT("Samples match in block %i, sample %i"), i, SampleIdx),
							OutputBuffer[i],
							ExpectedOutputBuffer[i]))
						{
							return;
						}
					}
				}
			});

			It("when Amount is 1, high-pass filter is applied", [this, SampleRate, NumSamplesPerBlock]()
			{
				Filter.Amount = 1;
							
				Audio::FAlignedFloatBuffer InputBuffer;
				InputBuffer.SetNumUninitialized(NumSamplesPerBlock);
				Audio::FAlignedFloatBuffer OutputBuffer;
				OutputBuffer.SetNumZeroed(NumSamplesPerBlock);
				Audio::FAlignedFloatBuffer ExpectedOutputBuffer;
				ExpectedOutputBuffer.SetNumZeroed(NumSamplesPerBlock);

				// make a filter we can use for comparison
				Audio::FStateVariableFilter FilterForComparison;
				FilterForComparison.Init(SampleRate, 1);
				FilterForComparison.SetFrequency(Filter.HighPassMaxFrequency);
				FilterForComparison.SetQ(Filter.Resonance);
				FilterForComparison.SetFilterType(Audio::EFilter::HighPass);
				FilterForComparison.Update();
							
				constexpr int32 NumBlocks = 40;
										
				for (int32 i = 0; i < NumBlocks; ++i)
				{
					HarmonixDsp::GenerateWhiteNoise(InputBuffer.GetData(), NumSamplesPerBlock);
								
					Filter.Process(InputBuffer, OutputBuffer);

					FilterForComparison.ProcessAudio(InputBuffer.GetData(), NumSamplesPerBlock, ExpectedOutputBuffer.GetData());

					// skip the first block, which will be doing the dry->wet cross-fade
					if (i == 0)
					{
						continue;
					}
								
					for (int32 SampleIdx = 0; SampleIdx < NumSamplesPerBlock; ++SampleIdx)
					{
						if (!TestEqual(
							FString::Printf(TEXT("Samples match in block %i, sample %i"), i, SampleIdx),
							OutputBuffer[i],
							ExpectedOutputBuffer[i]))
						{
							return;
						}
					}
				}
			});
		});
	}

}

#endif
