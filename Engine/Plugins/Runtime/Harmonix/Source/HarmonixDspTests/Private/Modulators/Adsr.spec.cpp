// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestUtility.h"
#include "HarmonixDspEditorUtils.h"
#include "HarmonixDsp/Modulators/Adsr.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "Audio/SimpleWaveWriter.h"
#include "Audio/SimpleWaveReader.h"
#include "Interfaces/IPluginManager.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Dsp::Adsr
{
	using namespace Harmonix::Testing;

	FString GetTestAdsrContentPath()
	{
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("Harmonix"));
		check(Plugin);
		FString TestAdsrContentDir = Plugin->GetContentDir() / TEXT("Editor/Tests") / TEXT("ADSR");
		return TestAdsrContentDir;
	}

	FString GetAudioAdsrCapturePath()
	{
		return FPaths::AudioCaptureDir() / TEXT("ADSR");
	}

	bool WriteDataToWav(const FString& Filename, int32 SampleRate, int32 NumChannels, const Audio::FAlignedFloatBuffer& Audio)
	{
		// write output to file in the "Audio Capture Directory"
		TUniquePtr<FArchive> Stream{ IFileManager::Get().CreateFileWriter(*Filename, IO_WRITE) };
		// create the Wave Writer to write output to file
		const TUniquePtr<Audio::FSimpleWaveWriter> Writer = MakeUnique<Audio::FSimpleWaveWriter>(MoveTemp(Stream), SampleRate, NumChannels, true);
		Writer->Write(MakeArrayView(Audio.GetData(), Audio.Num()));
		return true;
	}

	bool ReadDataFromWav(const FString& Filename, int32& OutSampleRate, int32& OutNumChannels, Audio::FAlignedFloatBuffer& OutAudio)
	{
		if (!IFileManager::Get().FileExists(*Filename))
		{
			return false;
		}
		
		TUniquePtr<FArchive> Stream{ IFileManager::Get().CreateFileReader(*Filename, IO_READ) };
		const TUniquePtr<Audio::FSimpleWaveReader> Reader = MakeUnique<Audio::FSimpleWaveReader>(MoveTemp(Stream));
		if (!Reader->IsDataValid())
		{
			return false;
		}

		OutSampleRate = Reader->GetSampleRate();
		OutNumChannels = Reader->GetNumChannels();
		OutAudio.SetNumUninitialized(Reader->GetNumSamples());

		int64 NumSamplesRead;
		bool AtEnd = Reader->Read(MakeArrayView(OutAudio.GetData(), OutAudio.Num()), NumSamplesRead);
		return NumSamplesRead == OutAudio.Num() && AtEnd;
	}

	BEGIN_DEFINE_SPEC(
	FHarmonixDspAdsrSpec,
	"Harmonix.Metasound.Modulators.Adsr",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	struct FParams
	{
		FAdsrSettings AdsrSettings;
		float SustainTime = 0.0f;
		int32 SampleRate = 48000;
		int32 NumChannels = 1;
		bool bWriteOutputToFile = false;
		FString ExpectedOutputFilepath;
		FString ActualOutputFilepath;
	};

	bool TestAdsrWithParams(const FParams& Params)
	{
		Audio::FAlignedFloatBuffer ExpectedOutput;
		{
			int32 OutSampleRate;
			int32 OutNumChannels;
			bool bSuccess = ReadDataFromWav(Params.ExpectedOutputFilepath, OutSampleRate, OutNumChannels, ExpectedOutput);
			
			if (!bSuccess)
			{
				AddError(FString::Printf(TEXT("Unable to read file: %s"), *Params.ExpectedOutputFilepath), 1);
				return false;
			}

			if (OutSampleRate != Params.SampleRate)
			{
				AddError(FString::Printf(TEXT("Expected Output: %s - File SampleRate: %d, Test Params.NumChannels: %d"), *Params.ExpectedOutputFilepath, OutSampleRate, Params.SampleRate), 1);
				return false;
			}

			if (OutNumChannels != Params.NumChannels)
			{
				AddError(FString::Printf(TEXT("Expected Output: %s - File NumChannels: %d, Test Params.NumChannels:  %d"), *Params.ExpectedOutputFilepath, OutNumChannels, Params.NumChannels), 1);
				return false;
			}
		}
		
		Audio::FAlignedFloatBuffer Buffer;
		Editor::GenerateAdsrEnvelope(Params.AdsrSettings, Params.SustainTime, Params.SampleRate, Buffer);

		if (Buffer.Num() != ExpectedOutput.Num())
		{
			AddError(FString::Printf(TEXT("Expected Output: %s - Expected NumSamples to be %d, was %d"),  *Params.ExpectedOutputFilepath, ExpectedOutput.Num(), Buffer.Num()), 1);
			return false;
		}
		
		bool bAllEqual = true;
		for (int32 SampleIdx = 0; SampleIdx < ExpectedOutput.Num(); ++SampleIdx)
		{
			bAllEqual &= FMath::IsNearlyEqual(ExpectedOutput[SampleIdx], Buffer[SampleIdx], UE_KINDA_SMALL_NUMBER);
			if (!bAllEqual)
			{
				AddError(FString::Printf(TEXT("Expected Output: %s: First Error Sample: %d - Expected: %.10f, was: %.10f" ),
					*Params.ExpectedOutputFilepath, SampleIdx, ExpectedOutput[SampleIdx], Buffer[SampleIdx]));
				break;
			}
		}
		
		if (Params.bWriteOutputToFile)
		{
			WriteDataToWav(Params.ActualOutputFilepath, Params.SampleRate, Params.NumChannels, Buffer);
		}
		
		return bAllEqual;
	}
	
	END_DEFINE_SPEC(FHarmonixDspAdsrSpec)

	void FHarmonixDspAdsrSpec::Define()
	{
		Describe("Curve", [this]()
		{
			int32 NumTests = 10;
		
			for (int32 Idx = 0; Idx < NumTests + 1; ++Idx)
			{
				float Curve = FMath::GetMappedRangeValueClamped(FVector2D(0.0f, float(NumTests)), FVector2D(-1.0f, 1.0f), Idx);

				FParams Params;
				Params.SampleRate = 48000;
				Params.NumChannels = 1;
				Params.AdsrSettings.Target = EAdsrTarget::Volume;
				Params.AdsrSettings.IsEnabled = true;
				Params.AdsrSettings.AttackTime = 0.5f;
				Params.AdsrSettings.DecayTime = 0.5f;
				Params.AdsrSettings.SustainLevel = 0.5f;
				Params.AdsrSettings.ReleaseTime = 0.5f;
				Params.SustainTime = 0.5f;
		
				Params.AdsrSettings.Depth = 1.0f;
				Params.AdsrSettings.AttackCurve = Curve;
				Params.AdsrSettings.DecayCurve = Curve;
				Params.AdsrSettings.ReleaseCurve = Curve;

				FString BaseName = FString::Format(TEXT("Expected_Adsr_{0}.wav"), {Idx});
				Params.ExpectedOutputFilepath = GetTestAdsrContentPath() / BaseName;

				IConsoleVariable* WriteOutputToFileCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("harmonix.tests.WriteOutputToFile"));
				int32 CVarValue = WriteOutputToFileCVar ? WriteOutputToFileCVar->GetInt() : 0;

				// Conditionally write the output based on the cvar value
				// 1: always write output
				// 2: write output on error 
				Params.bWriteOutputToFile = (CVarValue == 1 || (CVarValue == 2 && HasAnyErrors()));
				Params.ActualOutputFilepath = GetAudioAdsrCapturePath() / FString::Format(TEXT("Out_Adsr_{0}.wav"), { Idx });

				// replace '.' with ',' since test ui parses '.' as a delimiter
				FString TestDescription = FString::Printf(TEXT("Curve=\"%0.2f\""), Curve).Replace(TEXT("."), TEXT(","));
				It(TestDescription, [Params, this]
				{
					TestAdsrWithParams(Params);
				});
			}
		});
	}
}

#endif