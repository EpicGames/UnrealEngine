// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Modulators/MorphingLfo.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Dsp::Modulators::MorphingLfo::Tests
{
	BEGIN_DEFINE_SPEC(
		FHarmonixDspMorphingLfoSpec,
		"Harmonix.Dsp.Modulators.MorphingLfo",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	// anything that needs to persist between runs
	FMorphingLFO LFO { 0 };

	void SetParams(ETimeSyncOption SyncType, float Frequency, float Shape, bool Invert)
	{
		LFO.SyncType = SyncType;
		LFO.Frequency = Frequency;
		LFO.Shape = Shape;
		LFO.Invert = Invert;
	}
	
	void CheckSingleValueOutput(
		ETimeSyncOption SyncType,
		float Frequency,
		float Shape,
		bool Invert,
		const FMorphingLFO::FMusicTimingInfo* MusicTimingInfo = nullptr)
	{
		SetParams(SyncType, Frequency, Shape, Invert);
		
		constexpr int32 NumFrames = 128;
		
		// calculate the phase
		float ExpectedPhase;
		
		switch (SyncType)
		{
		case ETimeSyncOption::None:
			{
				const float PhaseInc = Frequency / LFO.GetSampleRate();
				ExpectedPhase = FMath::Fmod(PhaseInc * NumFrames, 1.0f);
				break;
			}
		case ETimeSyncOption::TempoSync:
			{
				if (!TestNotNull("Music timing info exists", MusicTimingInfo))
				{
					return;
				}
				
				const float CyclesPerBeat = Frequency * (4.0f / MusicTimingInfo->TimeSignature.Denominator);
				const float CyclesPerBar = CyclesPerBeat * MusicTimingInfo->TimeSignature.Numerator;
				const float TotalPhase = (MusicTimingInfo->Timestamp.Bar - 1) * CyclesPerBar + (MusicTimingInfo->Timestamp.Beat - 1) * CyclesPerBeat;
				const float FrequencyHz = Frequency * (MusicTimingInfo->Tempo / 60.0f);
				const float PhaseInc = FrequencyHz / LFO.GetSampleRate();
				ExpectedPhase = FMath::Fmod(TotalPhase + PhaseInc * NumFrames, 1.0f);
				break;
			}
		case ETimeSyncOption::SpeedScale:
			{
            	if (!TestNotNull("Music timing info exists", MusicTimingInfo))
            	{
            		return;
            	}

				const float AdjustedFrequency = Frequency * MusicTimingInfo->Speed;
				const float PhaseInc = AdjustedFrequency / LFO.GetSampleRate();
				ExpectedPhase = FMath::Fmod(PhaseInc * NumFrames, 1.0f);
            	break;
            }
		case ETimeSyncOption::Num:
		default:
			TestTrue("Valid sync type", false);
			return;
		}

		// Calculate expected output
		float ExpectedOutput = FMorphingLFO::GetValue(Shape, ExpectedPhase);
		if (Invert)
		{
			ExpectedOutput = 1.0f - ExpectedOutput;
		}

		// Advance
		float ActualOutput = -1;
		LFO.Advance(NumFrames, ActualOutput, MusicTimingInfo);
		
		// check the output
		TestEqual("Output is as expected", ActualOutput, ExpectedOutput);
	}

	void CheckBufferOutput(
		ETimeSyncOption SyncType,
		float Frequency,
		float Shape,
		bool Invert,
		const FMorphingLFO::FMusicTimingInfo* MusicTimingInfo = nullptr)
	{
		SetParams(SyncType, Frequency, Shape, Invert);

		constexpr int32 NumFrames = 480;

		// Advance the LFO
		TArray<float> LFOBuffer;
		LFOBuffer.SetNumUninitialized(NumFrames);
		LFO.Advance(LFOBuffer.GetData(), NumFrames, MusicTimingInfo);

		// Calculate the expected buffer
		float Phase = 0;
		float PhaseInc;

		switch (SyncType)
		{
		case ETimeSyncOption::None:
			PhaseInc = Frequency / LFO.GetSampleRate();
			break;
		case ETimeSyncOption::TempoSync:
			{
				if (!TestNotNull("Music timing info exists", MusicTimingInfo))
				{
					return;
				}

				const float CyclesPerBeat = Frequency * (4.0f / MusicTimingInfo->TimeSignature.Denominator);
				const float CyclesPerBar = CyclesPerBeat * MusicTimingInfo->TimeSignature.Numerator;
				const float TotalPhase = (MusicTimingInfo->Timestamp.Bar - 1) * CyclesPerBar + (MusicTimingInfo->Timestamp.Beat - 1) * CyclesPerBeat;
				Phase = FMath::Fmod(TotalPhase, 1.0f);
				
				const float FrequencyHz = Frequency * (MusicTimingInfo->Tempo / 60.0f);
				PhaseInc = FrequencyHz / LFO.GetSampleRate();
				break;
			}
		case ETimeSyncOption::SpeedScale:
			{
				if (!TestNotNull("Music timing info exists", MusicTimingInfo))
				{
					return;
				}
				
				const float AdjustedFrequency = Frequency * MusicTimingInfo->Speed;
				PhaseInc = AdjustedFrequency / LFO.GetSampleRate();
				break;
			}
		case ETimeSyncOption::Num:
		default:
			TestTrue("Valid sync type", false);
			return;
		}
		
		for (int32 i = 0; i < NumFrames; ++i)
		{
			float ExpectedSample = FMorphingLFO::GetValue(Shape, Phase);

			if (Invert)
			{
				ExpectedSample = 1.0f - ExpectedSample;
			}
			
			if (!TestEqual(
				FString::Printf(TEXT("Samples match at idx %d and phase %f"), i, Phase),
				LFOBuffer[i],
				ExpectedSample))
			{
				return;
			}
			
			Phase = FMath::Fmod(Phase + PhaseInc, 1.0f);
		}
	}

	END_DEFINE_SPEC(FHarmonixDspMorphingLfoSpec)
	
	void FHarmonixDspMorphingLfoSpec::Define()
	{
		BeforeEach([this]()
		{
			constexpr float SampleRate = 48000;
			LFO.Reset(SampleRate);
		});
		
		Describe("Advance (single-value)", [this]()
		{
			It("SyncType = None", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 0.9f;
				constexpr float Shape = 1.3f;
				constexpr bool Invert = false;

				CheckSingleValueOutput(ETimeSyncOption::None, Frequency, Shape, Invert);
			});

			It("SyncType = TempoSync", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 1.5f;
				constexpr float Shape = 0.2f;
				constexpr bool Invert = true;

				FMorphingLFO::FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Timestamp.Bar = 3;
				MusicTimingInfo.Timestamp.Beat = 2.5;
				MusicTimingInfo.TimeSignature.Numerator = 4;
				MusicTimingInfo.TimeSignature.Denominator = 4;
				MusicTimingInfo.Tempo = 134.0f;

				CheckSingleValueOutput(ETimeSyncOption::TempoSync, Frequency, Shape, Invert, &MusicTimingInfo);
			});

			It("SyncType = SpeedScale", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 30.0f;
				constexpr float Shape = 2.0f;
				constexpr bool Invert = false;

				FMorphingLFO::FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Speed = 0.3f;

				CheckSingleValueOutput(ETimeSyncOption::SpeedScale, Frequency, Shape, Invert, &MusicTimingInfo);
			});
		});

		Describe("Advance (buffer)", [this]()
		{
			It("SyncType = None", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 0.9f;
				constexpr float Shape = 1.3f;
				constexpr bool Invert = false;

				CheckBufferOutput(ETimeSyncOption::None, Frequency, Shape, Invert);
			});

			It("SyncType = TempoSync", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 1.5f;
				constexpr float Shape = 0.2f;
				constexpr bool Invert = true;

				FMorphingLFO::FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Timestamp.Bar = 3;
				MusicTimingInfo.Timestamp.Beat = 2.5;
				MusicTimingInfo.TimeSignature.Numerator = 4;
				MusicTimingInfo.TimeSignature.Denominator = 4;
				MusicTimingInfo.Tempo = 134.0f;

				CheckBufferOutput(ETimeSyncOption::TempoSync, Frequency, Shape, Invert, &MusicTimingInfo);
			});

			It("SyncType = SpeedScale", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 30.0f;
				constexpr float Shape = 2.0f;
				constexpr bool Invert = false;

				FMorphingLFO::FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Speed = 0.3f;

				CheckBufferOutput(ETimeSyncOption::SpeedScale, Frequency, Shape, Invert, &MusicTimingInfo);
			});
		});
	}
}

#endif
