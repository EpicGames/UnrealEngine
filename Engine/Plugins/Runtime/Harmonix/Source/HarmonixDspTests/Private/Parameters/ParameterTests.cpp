// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Parameters/Parameter.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Dsp::Parameters::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FParameterTestBool, 
		"Harmonix.Dsp.Parameters.Bool", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FParameterTestBool::RunTest(const FString&)
	{
		FBoolParameter Param(false);
		UTEST_FALSE("Param default", Param.Default);
		UTEST_EQUAL("Param value is default", Param.Get(), Param.Default);
		UTEST_FALSE("Min is false", Param.Min);
		UTEST_TRUE("Max is true", Param.Max);
		UTEST_TRUE("Step is true (1)", Param.Step);
		Param.Set(true);
		UTEST_TRUE("Param set worked", Param.Get());
		Param.Set(false);
		UTEST_FALSE("Param set worked again", Param.Get());
		return true;
	}

	template<typename T>
	bool CheckInit(T Min, T Max, T Default, T Step, FAutomationTestBase& Test)
	{
		TParameter<T> Param{Min, Max, Default, Step};

		const T ExpectedMin = FMath::Min(Min, Max);
		const T ExpectedMax = FMath::Max(Min, Max);
		const T ExpectedDefault = FMath::Clamp(Default, ExpectedMin, ExpectedMax);
		
		if (!Test.TestEqual("Default set", Param.Default, ExpectedDefault))
		{
			return false;
		}

		if (!Test.TestEqual("Value is default", Param.Get(), ExpectedDefault))
		{
			return false;
		}

		if (!Test.TestEqual("Min set", Param.Min, ExpectedMin))
		{
			return false;
		}

		if (!Test.TestEqual("Max set", Param.Max, ExpectedMax))
		{
			return false;
		}

		if (!Test.TestEqual("Step set", Param.Step, Step))
		{
			return false;
		}
		
		return true;
	}

	template<typename T>
	bool CheckGetSet(T Min, T Max, T Default, T Step, FAutomationTestBase& Test)
	{
		TParameter<T> Param{Min, Max, Default, Step};

		const T ExpectedMin = FMath::Min(Min, Max);
		const T ExpectedMax = FMath::Max(Min, Max);
		const T ExpectedDefault = FMath::Clamp(Default, ExpectedMin, ExpectedMax);

		// set below range (if min is not the actual minimum value for the type), expect clamping
		{
			T Value = ExpectedMin - T(1);
			if (Value < ExpectedMin)
			{
				Param.Set(Value);
				if (!Test.TestEqual("Value clamped to min", Param.Get(), ExpectedMin))
				{
					return false;
				}
			}
		}

		// set above the range (if max is not the actual maximum value for the type), expect clamping
		{
			T Value = ExpectedMax + T(1);
			if (Value > ExpectedMax)
			{
				Param.Set(Value);
				if (!Test.TestEqual("Value clamped to max", Param.Get(), ExpectedMax))
				{
					return false;
				}
			}
		}

		// if step is zero, set within the range and expect to get that value back
		if (Step == 0)
		{
			T Value = ExpectedMin;
			constexpr T NumIterations = 5;
			const T Inc = (ExpectedMax - ExpectedMin) / NumIterations;
			while (Value <= ExpectedMax)
			{
				Param.Set(Value);
				if (!Test.TestEqual("Value set", Param.Get(), Value))
				{
					return false;
				}
				Value += Inc;
			}
		}
		// otherwise, check for snapping to the step value
		else
		{
			T Value = ExpectedMin;
			constexpr T NumIterations = 7;
			const T Inc = (ExpectedMax - ExpectedMin) / NumIterations;
			while (Value <= ExpectedMax)
			{
				Param.Set(Value);
				const T Expected = ExpectedMin + Step * FMath::Floor((Value - ExpectedMin) / Step + T(0.5));
				if (!Test.TestEqual("Value set", Param.Get(), Expected))
				{
					return false;
				}
				Value += Inc;
			}
		}

		return true;
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FParameterTestInt32, 
		"Harmonix.Dsp.Parameters.int32", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FParameterTestInt32::RunTest(const FString&)
	{
		using T = int32;
		{
			constexpr T Min = 0;
			constexpr T Max = 20;
			constexpr T Default = 1;
			constexpr T Step = 2;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		{
			constexpr T Min = 190;
			constexpr T Max = -55;
			constexpr T Default = 99;
			constexpr T Step = 1;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		{
			constexpr T Min = -5;
			constexpr T Max = 450;
			constexpr T Default = 8;
			constexpr T Step = 0;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		return true;
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FParameterTestFloat, 
		"Harmonix.Dsp.Parameters.float", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FParameterTestFloat::RunTest(const FString&)
	{
		using T = float;
		{
			constexpr T Min = 0;
			constexpr T Max = 20;
			constexpr T Default = 1;
			constexpr T Step = 2;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		{
			constexpr T Min = 190;
			constexpr T Max = -55;
			constexpr T Default = 99;
			constexpr T Step = 1;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		{
			constexpr T Min = -5;
			constexpr T Max = 450;
			constexpr T Default = 8;
			constexpr T Step = 0;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		return true;
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FParameterTestUint8, 
		"Harmonix.Dsp.Parameters.uint8", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FParameterTestUint8::RunTest(const FString&)
	{
		using T = uint8;
		{
			constexpr T Min = 0;
			constexpr T Max = 20;
			constexpr T Default = 1;
			constexpr T Step = 2;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		{
			constexpr T Min = 190;
			constexpr T Max = 10;
			constexpr T Default = 99;
			constexpr T Step = 1;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		{
			constexpr T Min = 5;
			constexpr T Max = 178;
			constexpr T Default = 8;
			constexpr T Step = 0;
			UTEST_TRUE("Init", CheckInit<T>(Min, Max, Default, Step, *this));
			UTEST_TRUE("GetSet", CheckGetSet<T>(Min, Max, Default, Step, *this));
		}
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FParameterTestOperatorsAssignment,
		"Harmonix.Dsp.Parameters.operators.assignment",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FParameterTestOperatorsAssignment::RunTest(const FString&)
	{
		using T = float;
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			UTEST_EQUAL("Param1 (5.0f) == 5.0f", Param1, 5.0f);

			Param1 = 10.0f;
			UTEST_EQUAL("Param1 = 10.0f ==> Param1 == 10.0f", Param1, 10.0f);
		}

		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f, 0.1f);
			UTEST_EQUAL("Param1 (5.0f) == 5.0f", Param1, 5.0f);

			Param1 = 10.09f;
			UTEST_EQUAL("Param1 (0, 20, 5, 0.1) = 10.09f ==> Param1 == 10.1f", Param1, 10.1f);
		}

		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f, 0.1f);
			UTEST_EQUAL("Param1 (5.0f) == 5.0f", Param1, 5.0f);

			Param1 = 10.01f;
			UTEST_EQUAL("Param1 (0, 20, 5, 0.1) = 10.01f ==> Param1 == 10.0f", Param1, 10.0f);
		}

		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f, 0.1f);
			UTEST_EQUAL("Param1 (5.0f) == 5.0f", Param1, 5.0f);

			Param1 = -10.01f;
			UTEST_EQUAL("Param1 (0, 20, 5, 0.1) = -10.0f ==> Param1 == 0.0f", Param1, 0.0f);
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FParameterTestOperatorsComparison,
		"Harmonix.Dsp.Parameters.operators.comparison",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FParameterTestOperatorsComparison::RunTest(const FString&)
	{
		using T = float;
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			UTEST_TRUE("Param1 (5.0f) == 5.0f", Param1 == 5.0f);
			UTEST_TRUE("5.0f == Param1 (5.0f)", 5.0f == Param1);
			UTEST_FALSE("Param1 (5.0f) != 5.0f", Param1 != 5.0f);
			UTEST_FALSE("5.0f != Param1 (5.0f)", 5.0f != Param1);

			UTEST_FALSE("Param1 (5.0f) == 10.0f", Param1 == 10.0f);
			UTEST_FALSE("10.0f == Param1 (5.0f)", 10.0f == Param1);
			UTEST_TRUE("Param1 (5.0f) != 10.0f", Param1 != 10.0f);
			UTEST_TRUE("10.0f != Param1 (5.0f)", 10.0f != Param1);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			UTEST_TRUE("Param1 (5.0f) < 10.0f", Param1 < 10.0f);
			UTEST_FALSE("Param1 (5.0f) > 10.0f", Param1 > 10.0f);
			UTEST_TRUE("Param1 (5.0f) <= 10.0f", Param1 <= 10.0f);
			UTEST_FALSE("Param1 (5.0f) >= 10.0f", Param1 >= 10.0f);

			UTEST_FALSE("10.0f < Param1 (5.0f)", 10.0f < Param1);
			UTEST_TRUE("10.0f > Param1 (5.0f)", 10.0f > Param1);
			UTEST_FALSE("10.0f <= Param1 (5.0f)", 10.0f <= Param1);
			UTEST_TRUE("10.0f >= Param1 (5.0f)", 10.0f >= Param1);
			
			UTEST_FALSE("Param1 (5.0f) < 5.0f", Param1 < 5.0f);
			UTEST_FALSE("Param1 (5.0f) > 5.0f", Param1 > 5.0f);
			UTEST_TRUE("Param1 (5.0f) <= 5.0f", Param1 <= 5.0f);
			UTEST_TRUE("Param1 (5.0f) >= 5.0f", Param1 >= 5.0f);

			UTEST_FALSE("Param1 (5.0f) < 0.0f", Param1 < 0.0f);
			UTEST_TRUE("Param1 (5.0f) > 0.0f", Param1 > 0.0f);
			UTEST_FALSE("Param1 (5.0f) <= 0.0f", Param1 <= 0.0f);
			UTEST_TRUE("Param1 (5.0f) >= 0.0f", Param1 >= 0.0f);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			TParameter<T> Param2(0.0f, 10.0f, 5.0f);
			UTEST_TRUE("Param1 (5.0f) == Param2 (5.0f)", Param1 == Param2);
			UTEST_FALSE("Param1 (5.0f) != Param2 (5.0f)", Param1 != Param2);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 10.0f);
			TParameter<T> Param2(0.0f, 10.0f, 5.0f);
			UTEST_FALSE("Param1 (10.0f) == Param2 (5.0f)", Param1 == Param2);
			UTEST_TRUE("Param1 (10.0f) != Param2 (5.0f)", Param1 != Param2);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 10.0f);
			TParameter<T> Param2(0.0f, 10.0f, 5.0f);
			UTEST_FALSE("Param1 (10.0f) < Param2 (5.0f)", Param1 < Param2);
			UTEST_TRUE("Param1 (10.0f) > Param2 (5.0f)", Param1 > Param2);
			UTEST_FALSE("Param1 (10.0f) <= Param2 (5.0f)", Param1 <= Param2);
			UTEST_TRUE("Param1 (10.0f) >= Param2 (5.0f)", Param1 >= Param2);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			TParameter<T> Param2(0.0f, 10.0f, 10.0f);
			UTEST_TRUE("Param1 (5.0f) < Param2 (10.0f)", Param1 < Param2);
			UTEST_FALSE("Param1 (5.0f) > Param2 (10.0f)", Param1 > Param2);
			UTEST_TRUE("Param1 (5.0f) <= Param2 (10.0f)", Param1 <= Param2);
			UTEST_FALSE("Param1 (5.0f) >= Param2 (10.0f)", Param1 >= Param2);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			TParameter<T> Param2(0.0f, 10.0f, 5.0f);
			UTEST_FALSE("Param1 (5.0f) < Param2 (5.0f)", Param1 < Param2);
			UTEST_FALSE("Param1 (5.0f) > Param2 (5.0f)", Param1 > Param2);
			UTEST_TRUE("Param1 (5.0f) <= Param2 (5.0f)", Param1 <= Param2);
			UTEST_TRUE("Param1 (5.0f) >= Param2 (5.0f)", Param1 >= Param2);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f);

			UTEST_TRUE("pre-assignment: Param1 == 0.0f", Param1 == 0.0f);
			Param1 = 5.0f;
			UTEST_TRUE("post-assignment: Param1 == 5.0f", Param1 == 5.0f);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			TParameter<T> Param2(0.0f, 10.0f);

			UTEST_TRUE("pre-assignment: Param1 == 5.0f", Param1 == 5.0f);
			UTEST_TRUE("pre-assignment: Param2 == 0.0f", Param2 == 0.0f);
			
			Param2 = Param1;

			UTEST_TRUE("post-assignment: Param1 == 5.0f", Param1 == 5.0f);
			UTEST_TRUE("post-assignment: Param2 = Param1; Param2 == 5.0f", Param2 == 5.0f);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);

			UTEST_TRUE("pre-assignment: Param1 == 5.0f", Param1 == 5.0f);
			Param1 = 30.0f;
			UTEST_TRUE("post-assignment, clamped: Param1 = 30.0f; Param1 == 20.0f", Param1 == 20.0f);
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FParameterTestOperatorsMath,
		"Harmonix.Dsp.Parameters.operators.math",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FParameterTestOperatorsMath::RunTest(const FString&)
	{
		using T = float;
		{
			TParameter<T> Param(0.0f, 20.0f);
			Param += 10.0f;
			UTEST_EQUAL("Param (-20.0f) += 10.0f ==> Param (10.0f)", Param, 10.0f);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f);
			TParameter<T> Param2(0.0f, 20.0f, 10.0f);
			Param1 += Param2;
			UTEST_EQUAL("Param1 (-20.0f) += Param2 (10.0f) ==> Param1 (10.0f)", Param1, 10.0f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f);
			Param += 30.0f;
			UTEST_EQUAL("Param (0.0f) += 30.0f ==> Param (20.0f)", Param, 20.0f);
		}
		{
			TParameter<T> Param(-20.0f, 20.0f, 0.0f);
			Param -= 10.0f;
			UTEST_EQUAL("Param (0.0f) -= 10.0f ==> Param(-10.0f)", Param, -10.0f);
		}
		{
			TParameter<T> Param1(-20.0f, 20.0f, 0.0f);
			TParameter<T> Param2(0.0f, 20.0f, 10.0f);
			Param1 -= Param2;
			UTEST_EQUAL("Param1 (0.0f) -= Param2 (10.0f) ==> Param1 (-10.0f)", Param1, -10.0f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			Param -= 10.0f;
			UTEST_EQUAL("Param (5.0f) -= 10.0f ==> Param (0.0f)", Param, 0.0f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			T Result = Param - 10.0f;
			UTEST_EQUAL("Param (5.0f) - 10.0f == -5.0f", Result, -5.0f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			T Result = Param + 20.0f;
			UTEST_EQUAL("Param (5.0f) + 20.0f == 25.0f", Result, 25.0f);
		}

		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			Param *= 2.0f;
			UTEST_EQUAL("Param (5.0f) *= 2.0f ==> Param (10.0f)", Param, 10.0f);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			TParameter<T> Param2(0.0f, 20.0f, 2.0f);
			Param1 *= Param2;
			UTEST_EQUAL("Param1 (5.0f) *= Param2 (2.0f) ==> Param1 (10.0f)", Param1, 10.0f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			Param *= -2.0f;
			UTEST_EQUAL("Param (5.0f) *= -2.0f ==> Param (0.0f)", Param, 0.0f);
		}
		{
			TParameter<T> Param(-20.0f, 20.0f, 5.0f);
			Param *= -2.0f;
			UTEST_EQUAL("Param (5.0f) *= -2.0f ==> Param (-10.0f)", Param, -10.0f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			Param /= 2.0f;
			UTEST_EQUAL("Param (5.0f) /= 2.0f ==> 2.5f", Param, 2.5f);
		}
		{
			TParameter<T> Param1(0.0f, 20.0f, 5.0f);
			TParameter<T> Param2(0.0f, 20.0f, 2.0f);
			Param1 /= Param2;
			UTEST_EQUAL("Param1 (5.0f) /= Param2 (2.0f) ==> 2.5f", Param1, 2.5f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			Param /= -2.0f;
			UTEST_EQUAL("Param (5.0f) /= -2.0f ==> 0.0f", Param, 0.0f);
		}
		{
			TParameter<T> Param(-20.0f, 20.0f, 5.0f);
			Param /= -2.0f;
			UTEST_EQUAL("Param (5.0f) /= -2.0f ==> -2.5f", Param, -2.5f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			T Result = Param * 10.0f;
			UTEST_EQUAL("Param (5.0f) * 10.0f == 50.0f", Result, 50.0f);
		}
		{
			TParameter<T> Param(0.0f, 20.0f, 5.0f);
			T Result = Param * -10.0f;
			UTEST_EQUAL("Param (5.0f) * -10.0f == -50.0f", Result, -50.0f);
		}
		return true;
	}
}

#endif
