// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestUtility.h"
#include "HarmonixDsp/Ramper.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Dsp::Ramper::Tests
{
	using namespace Harmonix::Testing;

	class FTestRamper : public FAutomationTestBase
	{
	public:
		FTestRamper(const FString& InName, const bool bInComplexTask)
			: FAutomationTestBase(InName, bInComplexTask) {};

		FTestRamper(const FString& InName)
			: FAutomationTestBase(InName, false) {};

		struct FExpected
		{
			float NormalIncrement = 1.0f;
			float Increment = 1.0f;
			bool ShouldReachTarget = false;
			TArray<float> Values;
		};

		struct FParameters
		{
			float InitialValue = 0.0f;
			float TargetValue = 0.0f;
			float NumRampCallsPerSecond = 1.0f;
			float RampTimeMs = 1000.0f;

			FExpected Expected;
		};

	protected:
		bool RunTest_LinearRamper(const FString&, const FParameters& Params)
		{
			TLinearRamper<float> LinearRamper;

			UTEST_EQUAL("Linear Ramper Current Value before Initializing:", LinearRamper.GetCurrent(), 0.0f);

			LinearRamper.SetRampTimeMs(Params.NumRampCallsPerSecond, Params.RampTimeMs);
			UTEST_EQUAL("Linear Ramper Normal Increment after initializing", LinearRamper.NormalIncrement, Params.Expected.NormalIncrement);
			
			
			LinearRamper.SnapTo(Params.InitialValue);
			UTEST_EQUAL("Linear Ramper Current Value after Initializing", LinearRamper.GetCurrent(), Params.InitialValue);

			LinearRamper.SetTarget(Params.TargetValue);
			UTEST_EQUAL("Linear Ramper Current Value after setting target (no snapping)", LinearRamper.GetCurrent(), Params.InitialValue);
			UTEST_EQUAL("Linear Ramper Current Target after setting target", LinearRamper.GetTarget(), Params.TargetValue);
			UTEST_EQUAL("Linear Ramper Increment setting target", LinearRamper.Increment, Params.Expected.Increment);

			TArray<float> Actual;

			Actual.Reset(Params.Expected.Values.Num());

			// add initial value
			Actual.Add(LinearRamper.GetCurrent());
			while (Actual.Num() < Params.Expected.Values.Num())
			{
				// add intermediate ramping values
				LinearRamper.Ramp();
				Actual.Add(LinearRamper.GetCurrent());
			};

			UTEST_TRUE(*FString::Printf(TEXT("Ramper values should be the same:\n"
									         "Expected: %s\n"
									         "Actual:   %s"), 
										 *Utility::ArrayToString(Params.Expected.Values), 
										 *Utility::ArrayToString(Actual)), 
					   Utility::CheckAll(Params.Expected.Values, Actual));

			if (Params.Expected.ShouldReachTarget)
			{
				UTEST_EQUAL("Linear ramper should have reached target value: ", LinearRamper.GetCurrent(), Params.TargetValue);
			}

			return true;
		};
	};

	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
		FTestRamper0,
		FTestRamper,
		"Harmonix.Dsp.Ramper.TestLinearRamper.RampValues_0 -> 100",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTestRamper0::RunTest(const FString& Parameters)
	{
		FParameters Test;
		Test.InitialValue = 0.0f;
		Test.TargetValue = 100.0f;
		Test.NumRampCallsPerSecond = 10.0f;
		Test.RampTimeMs = 1000.0f;

		
		Test.Expected.NormalIncrement = 0.1f;
		Test.Expected.Increment = 10.0f;
		Test.Expected.ShouldReachTarget = true;
		Test.Expected.Values = {
			0.0f,
			10.0f,
			20.0f,
			30.0f,
			40.0f,
			50.0f,
			60.0f,
			70.0f,
			80.0f,
			90.0f,
			100.0f
		};

		return RunTest_LinearRamper(Parameters, Test);
	}
};

#endif