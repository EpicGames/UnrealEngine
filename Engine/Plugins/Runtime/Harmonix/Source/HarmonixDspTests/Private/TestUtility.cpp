// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestUtility.h"

namespace Harmonix::Testing::Utility
{
	FString ArrayToString(const TArray<float>& Array, int MinFractionalDigits)
	{
		auto ValueToString = [&MinFractionalDigits](const float& Value) -> FString
		{
			return FString::SanitizeFloat(Value, MinFractionalDigits);
		};
		return ArrayToString<float>(Array, ValueToString);
	}

	FString ArrayToString(const TArray<double>& Array, int MinFractionalDigits)
	{
		auto ValueToString = [&MinFractionalDigits](const double& Value) -> FString
		{
			return FString::SanitizeFloat(Value, MinFractionalDigits);
		};
		return ArrayToString<double>(Array, ValueToString);
	}

	bool CheckAll(const TArray<float>& Array0, const TArray<float>& Array1, float Tolerance /* = UE_SMALL_NUMBER */)
	{
		auto Comparator = [&Tolerance](const float& First, const float& Second)
		{
			return FMath::IsNearlyEqual(First, Second, Tolerance);
		};
		return CheckAll<float, float>(Array0, Array1, Comparator);
	}

	bool CheckAll(const TArray<double>& Array0, const TArray<double>& Array1, double Tolerance /* = UE_DOUBLE_SMALL_NUMBER */)
	{
		auto Comparator = [&Tolerance](const double& First, const double& Second)
		{
			return FMath::IsNearlyEqual(First, Second, Tolerance);
		};

		return CheckAll<double, double>(Array0, Array1, Comparator);
	}
};