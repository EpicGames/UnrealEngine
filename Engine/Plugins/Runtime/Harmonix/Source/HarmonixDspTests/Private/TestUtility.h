// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Templates/Function.h"

namespace Harmonix::Testing::Utility
{
	template<typename TType>
	FString ArrayToString(const TArray<TType>& Array, 
		// default string formatter
		TFunction<FString(const TType&)> ValueToString = [](const TType& Value) { return FString::Format(TEXT("{0}"), { Value }); })
	{
		TArray<FString> ValueStrings;
		ValueStrings.Reset(Array.Num());

		for (const TType& Value : Array)
		{
			ValueStrings.Add(ValueToString(Value));
		}

		return FString::Printf(TEXT("{%s}"), *FString::Join(ValueStrings, TEXT(", ")));
	};

	template<typename T0, typename T1>
	bool CheckAll(const TArray<T0>& Array0, const TArray<T1>& Array1, 
		// default comparator. Assumes types have == operator overload
		TFunction<bool(const T0&, const T1&)> Comparator = [](const T0& First, const T1& Second) { return First == Second; })
	{
		if (Array0.Num() != Array1.Num())
			return false;

		for (int Idx = 0; Idx < Array0.Num(); ++Idx)
		{
			if (!Comparator(Array0[Idx], Array1[Idx]))
				return false;
		}

		return true;
	}

	FString ArrayToString(const TArray<float>& Array, int MinFractionalDigits);
	FString ArrayToString(const TArray<double>& Array, int MinFractionalDigits);
	bool CheckAll(const TArray<float>& Array0, const TArray<float>& Array1, float Tolerance = UE_SMALL_NUMBER);
	bool CheckAll(const TArray<double>& Array0, const TArray<double>& Array1, double Tolerance = UE_DOUBLE_SMALL_NUMBER);
};
