// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixEditor : ModuleRules
{
	public HarmonixEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		//OptimizeCode = CodeOptimization.Never;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"EditorWidgets",
				"Harmonix",
				"UnrealEd",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"Settings",
				"InputCore"
			}
		);
	}
}
