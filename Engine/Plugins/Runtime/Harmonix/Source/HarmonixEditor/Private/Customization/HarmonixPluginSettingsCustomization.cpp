// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixPluginSettingsCustomization.h"

#include "Harmonix/HarmonixPluginSettings.h"
#include "Harmonix/HarmonixDeveloperSettings.h"

#include "Editor.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "UObject/UObjectIterator.h"


#define LOCTEXT_NAMESPACE "HarmonixPluginSettingsCustomization"

void FHarmonixPluginSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// This 'EditCategory' call will ensure that the Harmonix Plugin Settings will appear above the developer settings
	DetailBuilder.EditCategory(UHarmonixPluginSettings::StaticClass()->GetFName(), FText::GetEmpty(), ECategoryPriority::Important);

	// Add all the UHarmonixDeveloperSettings CDOs to this settings details panel
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Harmonix Developer Settings"));
	TArray<UClass*> SettingsClasses;
	GetDerivedClasses(UHarmonixDeveloperSettings::StaticClass(), SettingsClasses);
	for (const UClass* SettingsClass : SettingsClasses)
	{
		if (SettingsClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
		{
			continue;
		}

		UHarmonixDeveloperSettings* Settings = SettingsClass->GetDefaultObject<UHarmonixDeveloperSettings>();
		IDetailPropertyRow* Row = CategoryBuilder.AddExternalObjects({ Settings }, EPropertyLocation::Default, FAddPropertyParams().UniqueId(Settings->GetClass()->GetFName()));
		if (Row)
		{
			// need to customize the name, otherwise it'll just say "Object"
			Row->CustomWidget()
				.NameContent()
				[
					SNew(STextBlock).Text(Settings->GetClass()->GetDisplayNameText())
				];
		}
	}
}

#undef LOCTEXT_NAMESPACE