// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/SToolTip.h"
#include "IPropertyUtilities.h"

class FHarmonixPluginSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FHarmonixPluginSettingsCustomization);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};