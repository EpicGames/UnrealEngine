// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"

#include "IAssetTools.h"
#include "ISettingsModule.h"

#include "PropertyEditorModule.h"
#include "Customization/HarmonixPluginSettingsCustomization.h"

#define LOCTEXT_NAMESPACE "HarmonixEditor"

DEFINE_LOG_CATEGORY(LogHarmonixEditor)

void FHarmonixEditorModule::StartupModule()
{
	// Register property customizations
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomClassLayout("HarmonixPluginSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FHarmonixPluginSettingsCustomization::MakeInstance));

}

void FHarmonixEditorModule::ShutdownModule()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.UnregisterCustomClassLayout("HarmonixPluginSettings");
}

IMPLEMENT_MODULE(FHarmonixEditorModule, HarmonixEditor);

#undef LOCTEXT_NAMESPACE
