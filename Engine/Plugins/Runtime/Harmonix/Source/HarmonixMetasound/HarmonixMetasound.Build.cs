// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixMetasound : ModuleRules
{
	public HarmonixMetasound(ReadOnlyTargetRules Target) : base(Target)
	{
		//OptimizeCode = CodeOptimization.Never;

		// This next flag is needed as there are Metasound node derivatives 
		// that are implemented in .cpp files only... no header files. 
		IWYUSupport = IWYUSupport.None;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Core",
				"CoreUObject",
				"MusicEnvironment",
				"TimeManagement",
				"MovieScene",
				"MetasoundStandardNodes",
				"HarmonixDsp",
				"HarmonixMidi",
				"Harmonix"
			});
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioExtensions",
				"MetasoundEngine",
				"MetasoundGraphCore",
				"MetasoundFrontend",
				"MetasoundGenerator",
				"SignalProcessing"
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("AssetRegistry");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

		PrivateDefinitions.AddRange(
			new string[]
			{
				"METASOUND_PLUGIN=Harmonix",
				"METASOUND_MODULE=HarmonixMetasound"
			}
		);
	}
}
