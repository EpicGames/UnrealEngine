// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Analysis/FFTAnalyzerResultVertexAnalyzer.h"

namespace HarmonixMetasound::Analysis
{
	const FName& FFFTAnalyzerResultVertexAnalyzer::GetAnalyzerName()
	{
		static const FName AnalyzerName = "Harmonix.FFTAnalyzerResult";
		return AnalyzerName;
	}

	const FName& FFFTAnalyzerResultVertexAnalyzer::GetDataType()
	{
		return Metasound::GetMetasoundDataTypeName<FHarmonixFFTAnalyzerResults>();
	}

	const Metasound::Frontend::FAnalyzerOutput& FFFTAnalyzerResultVertexAnalyzer::FOutputs::GetValue()
	{
		static Metasound::Frontend::FAnalyzerOutput Value = { "LastFFTAnalyzerResult", Metasound::GetMetasoundDataTypeName<FHarmonixFFTAnalyzerResults>() };
		return Value;
	}

	const TArray<Metasound::Frontend::FAnalyzerOutput>& FFFTAnalyzerResultVertexAnalyzer::FFactory::GetAnalyzerOutputs() const
	{
		static const TArray<Metasound::Frontend::FAnalyzerOutput> Outputs { FOutputs::GetValue() };
		return Outputs;
	}

	FFFTAnalyzerResultVertexAnalyzer::FFFTAnalyzerResultVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
		, LastAnalyzerResult(FHarmonixFFTAnalyzerResultsWriteRef::CreateNew())
	{
		FHarmonixFFTAnalyzerResultsReadRef ReadRef = Metasound::TDataReadReference{ LastAnalyzerResult };

		BindOutputData(FOutputs::GetValue().Name, InParams.OperatorSettings, ReadRef);
	}

	void FFFTAnalyzerResultVertexAnalyzer::Execute()
	{
		*LastAnalyzerResult = GetVertexData<FHarmonixFFTAnalyzerResults>();
		MarkOutputDirty();
	}
}
