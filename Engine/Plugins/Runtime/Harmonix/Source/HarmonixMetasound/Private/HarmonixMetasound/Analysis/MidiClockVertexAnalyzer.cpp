// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Analysis/MidiClockVertexAnalyzer.h"

#include "HarmonixMetasound/DataTypes/MidiClock.h"

#include "HarmonixMidi/BarMap.h"

namespace HarmonixMetasound::Analysis
{
	const Metasound::Frontend::FAnalyzerOutput FMidiClockVertexAnalyzer::FOutputs::Timestamp = { "Timestamp", Metasound::GetMetasoundDataTypeName<FMusicTimestamp>() };
	const Metasound::Frontend::FAnalyzerOutput FMidiClockVertexAnalyzer::FOutputs::Tempo = { "Tempo", Metasound::GetMetasoundDataTypeName<float>() };
	const Metasound::Frontend::FAnalyzerOutput FMidiClockVertexAnalyzer::FOutputs::TimeSignature = { "TimeSignature", Metasound::GetMetasoundDataTypeName<FTimeSignature>() };
	const Metasound::Frontend::FAnalyzerOutput FMidiClockVertexAnalyzer::FOutputs::Speed = { "Speed", Metasound::GetMetasoundDataTypeName<float>() };
	
	const Metasound::Frontend::FAnalyzerOutput& FMidiClockVertexAnalyzer::FOutputs::GetValue()
	{
		return Timestamp;
	}
	
	const TArray<Metasound::Frontend::FAnalyzerOutput>& FMidiClockVertexAnalyzer::FFactory::GetAnalyzerOutputs() const
	{
		static const TArray<Metasound::Frontend::FAnalyzerOutput> Outputs
		{
			FOutputs::Timestamp,
			FOutputs::Tempo,
			FOutputs::TimeSignature,
			FOutputs::Speed
		};
		return Outputs;
	}

	const FName& FMidiClockVertexAnalyzer::GetAnalyzerName()
	{
		static const FName Name = "Harmonix.MidiClock";
		return Name;
	}

	const FName& FMidiClockVertexAnalyzer::GetDataType()
	{
		return Metasound::GetMetasoundDataTypeName<FMidiClock>();
	}

	FMidiClockVertexAnalyzer::FMidiClockVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
		, Timestamp(FMusicTimestampWriteRef::CreateNew())
		, Tempo(Metasound::FFloatWriteRef::CreateNew())
		, TimeSignature(FTimeSignatureWriteRef::CreateNew())
		, Speed(Metasound::FFloatWriteRef::CreateNew())
	{
		BindOutputData<FMusicTimestamp>(FOutputs::Timestamp.Name, InParams.OperatorSettings, Timestamp);
		BindOutputData<float>(FOutputs::Tempo.Name, InParams.OperatorSettings, Tempo);
		BindOutputData<FTimeSignature>(FOutputs::TimeSignature.Name, InParams.OperatorSettings, TimeSignature);
		BindOutputData<float>(FOutputs::Speed.Name, InParams.OperatorSettings, Speed);
	}

	void FMidiClockVertexAnalyzer::Execute()
	{
		const FMidiClock& Clock = GetVertexData<FMidiClock>();
		*Timestamp = Clock.GetMusicTimestampAtBlockEnd();
		*Tempo = Clock.GetTempoAtEndOfBlock();
		const FTimeSignature* TimeSigPtr = Clock.GetSongMapEvaluator().GetTimeSignatureAtBar((*Timestamp).Bar);
		if (TimeSigPtr)
		{
			*TimeSignature = *TimeSigPtr;
		}
		else
		{
			*TimeSignature = FTimeSignature(4, 4);
		}
		*Speed = Clock.GetSpeedAtEndOfBlock();
		MarkOutputDirty();
	}
}
