// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Analysis/MidiSongPosVertexAnalyzer.h"

#include "HarmonixMetasound/DataTypes/MidiClock.h"

#include "HarmonixMidi/BarMap.h"

REGISTER_METASOUND_DATATYPE(HarmonixMetasound::Analysis::FMidiClockSongPosition, "MIDIClockSongPosition")

namespace HarmonixMetasound::Analysis
{

	const Metasound::Frontend::FAnalyzerOutput FMidiSongPosVertexAnalyzer::SongPosition = { "MidiClockSongPosition", Metasound::GetMetasoundDataTypeName<FMidiClockSongPosition>() };
	
	const TArray<Metasound::Frontend::FAnalyzerOutput>& FMidiSongPosVertexAnalyzer::FFactory::GetAnalyzerOutputs() const
	{
		static const TArray<Metasound::Frontend::FAnalyzerOutput> Outputs
		{
			SongPosition,
		};
		return Outputs;
	}

	const FName& FMidiSongPosVertexAnalyzer::GetAnalyzerName()
	{
		static const FName Name = "Harmonix.SmoothingMusicClock";
		return Name;
	}

	const FName& FMidiSongPosVertexAnalyzer::GetDataType()
	{
		return Metasound::GetMetasoundDataTypeName<FMidiClock>();
	}

	FMidiSongPosVertexAnalyzer::FMidiSongPosVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
		, LastMidiClockSongPos(FMidiClockSongPositionWriteRef::CreateNew())
		, SampleRate(InParams.OperatorSettings.GetSampleRate())
		, BlockSize(InParams.OperatorSettings.GetNumFramesPerBlock())
	{
		BindOutputData<FMidiClockSongPosition>(SongPosition.Name, InParams.OperatorSettings, LastMidiClockSongPos);
		History = UMidiClockUpdateSubsystem::GetOrCreateClockHistory(InParams.AnalyzerAddress);
		History->SampleRate = SampleRate;
	}

	FMidiSongPosVertexAnalyzer::~FMidiSongPosVertexAnalyzer()
	{
		History = nullptr;
	}

	void FMidiSongPosVertexAnalyzer::Execute()
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;

		const FMidiClock& Clock = GetVertexData<FMidiClock>();
		CurrentSongMapEvaluator = &Clock.GetSongMapEvaluator();

		TSharedPtr<const FSongMapChain> MapChain = History->GetLatestMapsForProducer();

		if (!MapChain->SongMaps || Clock.GetSongMapsChangedInBlock() ||
			(LastClock.IsValid() && LastClock != Clock.AsWeak()) ||
			Clock.GetFirstTickInLoop() != MapChain->FirstTickInLoop ||
			Clock.GetLoopLengthTicks() != MapChain->LoopLengthTicks)
		{
			// Copy the song maps out of the clock's evaluator.
			TSharedPtr<FSongMaps> Maps = MakeShared<FSongMaps>(*CurrentSongMapEvaluator);
			// Now update the maps in the history system...
			// (Note: This is the only "sync point" between the rendering thread (this thread)
			// and the game thread. Any game thread system that looks at the clock history AND 
			// needs to use the song maps will lock those maps while using the song maps. If 
			// we need to make this update lock free in the future there are a few paths we 
			// could follow.)
			History->UpdateMaps(Maps, Clock.GetFirstTickInLoop(), Clock.GetLoopLengthTicks());
		}

		LastClock = Clock.AsWeak();

		LastMidiClockSongPos->CurrentSpeed = Clock.GetSpeedAtEndOfBlock();
		LastMidiClockSongPos->CurrentTransportState = Clock.GetTransportStateAtStartOfBlock();

		float QuarterNotesPerSecond = Clock.GetTempoAtStartOfBlock() / 60.0f;
		float FramesPerQuarterNote = SampleRate / QuarterNotesPerSecond;
		CurrentFramesPerTick = FramesPerQuarterNote / (float)CurrentSongMapEvaluator->GetTicksPerQuarterNote();

		const FMidiClockEvents& Events = Clock.GetMidiClockEventsInBlock();

		int32 CurrentBlockFrameIndex = -1;
		int32 BlockFrameAdvance = 0;
		for (const FMidiClockEvent& Event : Events)
		{
			if (Event.BlockFrameIndex != CurrentBlockFrameIndex)
			{
				CurrentBlockFrameIndex = Event.BlockFrameIndex;
				BlockFrameAdvance = 0;
			}

			bool FirstEventAfterSeekOrLoop = LastPosWasSeekOrLoop;
			LastPosWasSeekOrLoop = false;

			if (TryProcessAsAdvance(Event, BlockFrameAdvance, FirstEventAfterSeekOrLoop)) continue;
			if (TryProcessAsTempoChange(Event, BlockFrameAdvance, FirstEventAfterSeekOrLoop)) continue;
			if (TryProcessAsTimeSignatureChange(Event, BlockFrameAdvance, FirstEventAfterSeekOrLoop)) continue;
			if (TryProcessAsSpeedChange(Event, BlockFrameAdvance, FirstEventAfterSeekOrLoop)) continue;
			if (TryProcessAsTransportChange(Event, BlockFrameAdvance, FirstEventAfterSeekOrLoop)) continue;
			if (TryProcessAsLoop(Event, BlockFrameAdvance, FirstEventAfterSeekOrLoop)) continue;
			// This is the last type we recognize, so if it isn't a seek we're in trouble...
			ensureMsgf(TryProcessAsSeek(Event, BlockFrameAdvance, FirstEventAfterSeekOrLoop), TEXT("Unrecognized Clock Advance Event Type!"));
		}
		SampleCount += BlockSize;
	}

	bool FMidiSongPosVertexAnalyzer::TryProcessAsAdvance(const FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop)
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;
		if (const FAdvance* AsAdvance = Event.TryGet<FAdvance>())
		{
			// We need to know the END of this block of ticks because if the next thing we see is a loop or a seek
			// we need to insert a song position representing where we got to BEFORE we looped/seeked. 
			LastAdvanceUpToTick = AsAdvance->FirstTickToProcess + AsAdvance->NumberOfTicksToProcess;

			if (AsAdvance->FirstTickToProcess == LastTickProcessed)
			{
				// there must have been an advance, seek, loop, tempo, time signature, etc. change already processed 
				// that caused us to post a MidiSongPos for this tick. So move on.
				BlockFrameAdvance += FMath::FloorToInt32((float)AsAdvance->NumberOfTicksToProcess * CurrentFramesPerTick);
				return true;
			}
			LastTickProcessed = AsAdvance->FirstTickToProcess;
			LastTempoMapTickProcessed = AsAdvance->TempoMapTick;
			LastMidiClockSongPos->UpToTick = LastTickProcessed;
			LastMidiClockSongPos->TempoMapTick = LastTempoMapTickProcessed;
			LastMidiClockSongPos->SampleCount = SampleCount + Event.BlockFrameIndex + BlockFrameAdvance;
			LastMidiClockSongPos->MarkerType = (FirstEventAfterSeekOrLoop) ? FMidiClockSongPosition::EMarkerType::FirstPositionAfterSeekLoop : FMidiClockSongPosition::EMarkerType::None;
			{
				FMidiClockSongPositionHistory::FScopedItemWriteRef Slot = History->Positions.GetNextAtomicWriteSlot();
				*Slot = *LastMidiClockSongPos.Get();
			}

			BlockFrameAdvance += FMath::FloorToInt32((float)AsAdvance->NumberOfTicksToProcess * CurrentFramesPerTick);
			return true;
		}
		return false;
	}

	bool FMidiSongPosVertexAnalyzer::TryProcessAsTempoChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop)
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;
		if (const FTempoChange* AsTempo = Event.TryGet<FTempoChange>())
		{
			float QuarterNotesPerSecond = AsTempo->Tempo / 60.0f;
			float FramesPerQuarterNote = SampleRate / QuarterNotesPerSecond;
			CurrentFramesPerTick = FramesPerQuarterNote / (float)CurrentSongMapEvaluator->GetTicksPerQuarterNote();

			if (AsTempo->Tick == LastTickProcessed)
			{
				// there must have been an advance, seek, loop, tempo, time signature, etc. change already processed 
				// that caused us to post a MidiSongPos for this tick. So move on.
				///check(LastMidiClockSongPos->SongPos.Tempo == AsTempo->Tempo);
				return true;
			}
			LastTickProcessed = AsTempo->Tick;
			LastTempoMapTickProcessed = AsTempo->TempoMapTick;
			LastMidiClockSongPos->UpToTick = LastTickProcessed;
			LastMidiClockSongPos->TempoMapTick = LastTempoMapTickProcessed;
			LastMidiClockSongPos->SampleCount = SampleCount + Event.BlockFrameIndex + BlockFrameAdvance;
			LastMidiClockSongPos->MarkerType = (FirstEventAfterSeekOrLoop) ? FMidiClockSongPosition::EMarkerType::FirstPositionAfterSeekLoop : FMidiClockSongPosition::EMarkerType::None;
			{
				FMidiClockSongPositionHistory::FScopedItemWriteRef Slot = History->Positions.GetNextAtomicWriteSlot();
				*Slot = *LastMidiClockSongPos.Get();
			}

			return true;
		}
		return false;
	}

	bool FMidiSongPosVertexAnalyzer::TryProcessAsTimeSignatureChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop)
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;
		if (const FTimeSignatureChange* AsTimeSig = Event.TryGet<FTimeSignatureChange>())
		{
			if (AsTimeSig->Tick == LastTickProcessed)
			{
				// there must have been an advance, seek, loop, tempo, time signature, etc. change already processed 
				// that caused us to post a MidiSongPos for this tick. So move on.
				//check(LastMidiClockSongPos->SongPos.TimeSigNumerator == AsTimeSig->TimeSignature.Numerator &&
				//	LastMidiClockSongPos->SongPos.TimeSigDenominator == AsTimeSig->TimeSignature.Denominator);
				return true;
			}
			LastTickProcessed = AsTimeSig->Tick;
			LastTempoMapTickProcessed = AsTimeSig->TempoMapTick;
			LastMidiClockSongPos->UpToTick = LastTickProcessed;
			LastMidiClockSongPos->TempoMapTick = LastTempoMapTickProcessed;
			LastMidiClockSongPos->SampleCount = SampleCount + Event.BlockFrameIndex + BlockFrameAdvance;
			LastMidiClockSongPos->MarkerType = (FirstEventAfterSeekOrLoop) ? FMidiClockSongPosition::EMarkerType::FirstPositionAfterSeekLoop : FMidiClockSongPosition::EMarkerType::None;

			{
				FMidiClockSongPositionHistory::FScopedItemWriteRef Slot = History->Positions.GetNextAtomicWriteSlot();
				*Slot = *LastMidiClockSongPos.Get();
			}

			return true;
		}
		return false;
	}

	bool FMidiSongPosVertexAnalyzer::TryProcessAsSpeedChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop)
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;
		if (const FSpeedChange* AsSpeedChange = Event.TryGet<FSpeedChange>())
		{
			LastMidiClockSongPos->CurrentSpeed = AsSpeedChange->Speed;
			if (LastTickProcessed != -1)
			{
				LastMidiClockSongPos->SampleCount = SampleCount + Event.BlockFrameIndex + BlockFrameAdvance;
				LastMidiClockSongPos->MarkerType = (FirstEventAfterSeekOrLoop) ? FMidiClockSongPosition::EMarkerType::FirstPositionAfterSeekLoop : FMidiClockSongPosition::EMarkerType::None;
				LastMidiClockSongPos->UpToTick = LastTickProcessed;
				LastMidiClockSongPos->TempoMapTick = LastTempoMapTickProcessed;
				{
					FMidiClockSongPositionHistory::FScopedItemWriteRef Slot = History->Positions.GetNextAtomicWriteSlot();
					*Slot = *LastMidiClockSongPos.Get();
				}
			}
			return true;
		}
		return false;
	}

	bool FMidiSongPosVertexAnalyzer::TryProcessAsTransportChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop)
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;
		if (const FTransportChange* AsTransport = Event.TryGet<FTransportChange>())
		{
			LastMidiClockSongPos->CurrentTransportState = AsTransport->TransportState;
			if (LastTickProcessed != -1)
			{
				LastMidiClockSongPos->SampleCount = SampleCount + Event.BlockFrameIndex + BlockFrameAdvance;
				LastMidiClockSongPos->MarkerType = (FirstEventAfterSeekOrLoop) ? FMidiClockSongPosition::EMarkerType::FirstPositionAfterSeekLoop : FMidiClockSongPosition::EMarkerType::None;
				LastMidiClockSongPos->UpToTick = LastTickProcessed;
				LastMidiClockSongPos->TempoMapTick = LastTempoMapTickProcessed;
				{
					FMidiClockSongPositionHistory::FScopedItemWriteRef Slot = History->Positions.GetNextAtomicWriteSlot();
					*Slot = *LastMidiClockSongPos.Get();
				}
			}
			return true;
		}
		return false;
	}

	bool FMidiSongPosVertexAnalyzer::TryProcessAsLoop(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop)
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;
		if (const FLoop* AsLoop = Event.TryGet<FLoop>())
		{
			LastPosWasSeekOrLoop = true;
			// We need to insert a song position for where we got to BEFORE this loop back...
			if (LastAdvanceUpToTick != -1)
			{
				LastTempoMapTickProcessed = AsLoop->TempoMapTick;
				LastMidiClockSongPos->UpToTick = LastAdvanceUpToTick;
				LastMidiClockSongPos->TempoMapTick = LastTempoMapTickProcessed;
				LastMidiClockSongPos->SampleCount = SampleCount + Event.BlockFrameIndex + BlockFrameAdvance;
				LastMidiClockSongPos->MarkerType = FMidiClockSongPosition::EMarkerType::LastPositionBeforeSeekLoop;
				LastAdvanceUpToTick = -1;
				{
					FMidiClockSongPositionHistory::FScopedItemWriteRef Slot = History->Positions.GetNextAtomicWriteSlot();
					*Slot = *LastMidiClockSongPos.Get();
				}
			}
			return true;
		}
		return false;
	}

	bool FMidiSongPosVertexAnalyzer::TryProcessAsSeek(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop)
	{
		using namespace HarmonixMetasound::MidiClockMessageTypes;
		if (const FSeek* AsSeek = Event.TryGet<FSeek>())
		{
			LastPosWasSeekOrLoop = true;
			// We need to insert a song position for where we got to BEFORE this seek...
			if (LastAdvanceUpToTick != -1)
			{
				LastTempoMapTickProcessed = AsSeek->TempoMapTick;
				LastMidiClockSongPos->UpToTick = LastAdvanceUpToTick;
				LastMidiClockSongPos->TempoMapTick = LastTempoMapTickProcessed;
				LastMidiClockSongPos->SampleCount = SampleCount + Event.BlockFrameIndex + BlockFrameAdvance;
				LastMidiClockSongPos->MarkerType = FMidiClockSongPosition::EMarkerType::LastPositionBeforeSeekLoop;
				LastAdvanceUpToTick = -1;

				{
					FMidiClockSongPositionHistory::FScopedItemWriteRef Slot = History->Positions.GetNextAtomicWriteSlot();
					*Slot = *LastMidiClockSongPos.Get();
				}
			}
			return true;
		}
		return false;
	}
}
