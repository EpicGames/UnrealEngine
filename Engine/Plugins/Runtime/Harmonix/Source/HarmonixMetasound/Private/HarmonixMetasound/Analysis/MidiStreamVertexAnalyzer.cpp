// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Analysis/MidiStreamVertexAnalyzer.h"

#include "HarmonixMetasound/DataTypes/MidiStream.h"

namespace HarmonixMetasound::Analysis
{
	const FName& FMidiStreamVertexAnalyzer::GetAnalyzerName()
	{
		static const FName AnalyzerName = "Harmonix.MidiStream";
		return AnalyzerName;
	}

	const FName& FMidiStreamVertexAnalyzer::GetDataType()
	{
		return Metasound::GetMetasoundDataTypeName<FMidiStream>();
	}

	const Metasound::Frontend::FAnalyzerOutput& FMidiStreamVertexAnalyzer::FOutputs::GetValue()
	{
		static Metasound::Frontend::FAnalyzerOutput Value = { "LastMIDIEvent", Metasound::GetMetasoundDataTypeName<FMidiEventInfo>() };
		return Value;
	}

	const TArray<Metasound::Frontend::FAnalyzerOutput>& FMidiStreamVertexAnalyzer::FFactory::GetAnalyzerOutputs() const
	{
		static const TArray<Metasound::Frontend::FAnalyzerOutput> Outputs { FOutputs::GetValue() };
		return Outputs;
	}

	FMidiStreamVertexAnalyzer::FMidiStreamVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
		, LastMidiEvent(FMidiEventInfoWriteRef::CreateNew())
	{
		BindOutputData(FOutputs::GetValue().Name, InParams.OperatorSettings, Metasound::TDataReadReference{ LastMidiEvent });
	}

	void FMidiStreamVertexAnalyzer::Execute()
	{
		const FMidiStream& MidiStreamIn = GetVertexData<FMidiStream>();

		for (const FMidiStreamEvent& Event : MidiStreamIn.GetEventsInBlock())
		{
			FMidiEventInfo EventOut;
			if (const TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> Clock = MidiStreamIn.GetClock())
			{
				EventOut.Timestamp = Clock->GetSongMapEvaluator().TickToMusicTimestamp(Event.CurrentMidiTick);
			}
			EventOut.TrackIndex = Event.TrackIndex;
			EventOut.MidiMessage = Event.MidiMessage;
			*LastMidiEvent = MoveTemp(EventOut);
			MarkOutputDirty();
		}
	}
}
