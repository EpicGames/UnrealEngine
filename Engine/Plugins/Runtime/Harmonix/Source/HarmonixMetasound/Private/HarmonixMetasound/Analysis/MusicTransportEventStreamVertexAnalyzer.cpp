// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Analysis/MusicTransportEventStreamVertexAnalyzer.h"

#include "HarmonixMetasound/DataTypes/MusicTransport.h"

namespace HarmonixMetasound::Analysis
{
	const FName& FMusicTransportEventStreamVertexAnalyzer::GetAnalyzerName()
	{
		return AnalyzerName;
	}

	const FName& FMusicTransportEventStreamVertexAnalyzer::GetDataType()
	{
		return Metasound::GetMetasoundDataTypeName<FMusicTransportEventStream>();
	}
	
	const Metasound::Frontend::FAnalyzerOutput& FMusicTransportEventStreamVertexAnalyzer::FOutputs::GetValue()
	{
		return TransportEvent;
	}

	const Metasound::Frontend::FAnalyzerOutput FMusicTransportEventStreamVertexAnalyzer::FOutputs::SeekDestination = { "SeekDestination", Metasound::GetMetasoundDataTypeName<FMusicSeekTarget>() };
	const Metasound::Frontend::FAnalyzerOutput FMusicTransportEventStreamVertexAnalyzer::FOutputs::TransportEvent = { "TransportEvent", Metasound::GetMetasoundDataTypeName<FMusicTransportEvent>() };

	const TArray<Metasound::Frontend::FAnalyzerOutput>& FMusicTransportEventStreamVertexAnalyzer::FFactory::GetAnalyzerOutputs() const
	{
		return AnalyzerOutputs;
	}

	const TArray<Metasound::Frontend::FAnalyzerOutput> FMusicTransportEventStreamVertexAnalyzer::FFactory::AnalyzerOutputs{ FOutputs::SeekDestination, FOutputs::TransportEvent };

	FMusicTransportEventStreamVertexAnalyzer::FMusicTransportEventStreamVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
		, SeekDestination(FMusicSeekTargetWriteRef::CreateNew())
		, LastMusicTransportEvent(FMusicTransportEventWriteRef::CreateNew())
		, FramesPerBlock(InParams.OperatorSettings.GetNumFramesPerBlock())
		, SampleRate(InParams.OperatorSettings.GetSampleRate())
	{
		BindOutputData(FOutputs::SeekDestination.Name, InParams.OperatorSettings, Metasound::TDataReadReference{ SeekDestination });
		BindOutputData(FOutputs::TransportEvent.Name, InParams.OperatorSettings, Metasound::TDataReadReference{ LastMusicTransportEvent });
	}

	void FMusicTransportEventStreamVertexAnalyzer::Execute()
	{
		const FMusicTransportEventStream& MusicTransportEventStream = GetVertexData<FMusicTransportEventStream>();
		for (auto&& Event : MusicTransportEventStream.GetTransportEventsInBlock())
		{
			*SeekDestination = MusicTransportEventStream.GetNextSeekDestination();
			LastMusicTransportEvent->Time = Metasound::FTime::FromSeconds(double(Event.SampleIndex + NumFrames) / SampleRate);
			LastMusicTransportEvent->Request = Event.Request;
			MarkOutputDirty();
		}
		NumFrames += FramesPerBlock;
	}

	const FName FMusicTransportEventStreamVertexAnalyzer::AnalyzerName = "Harmonix.MusicTransport";
}
