// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Analysis/PeakTamer.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PeakTamer)

namespace Harmonix::AudioReactivity
{
	namespace PeakTamerPrivate
	{
		// Smooth a value given a delta time, a smoothing time, and the last value.
		// This is pretty much a one-pole lowpass filter.
		float SmoothValue(const float X0, const float Y1, const float DeltaTime, const float SmoothTime)
		{
			const float A0 = FMath::Clamp(DeltaTime / SmoothTime, 0.0f, 1.0f);
			const float B1 = 1.0f - A0;
			return X0 * A0 + Y1 * B1;
		}
	}
	
	void FPeakTamer::Configure(const FHarmonixPeakTamerSettings& InSettings)
	{
		Settings = InSettings;
	}

	void FPeakTamer::Update(float InputValue, float DeltaTimeSeconds)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPeakTamer::Update);
		
		// Clean the input
		InputValue = FMath::Max(InputValue, 0.0f);
		DeltaTimeSeconds = FMath::Max(DeltaTimeSeconds, 0.0f);

		// Smooth the peak
		{
			const float SmoothTimeSeconds = InputValue > Peak ? Settings.PeakAttackTimeSeconds : Settings.PeakReleaseTimeSeconds;
			Peak = PeakTamerPrivate::SmoothValue(InputValue, Peak, DeltaTimeSeconds, SmoothTimeSeconds);
		}

		// Compress the input
		// (basically a naive brick wall limiter with a threshold of 1)
		{
			const float Max = FMath::Max(Peak, 1.0f);
			const float ScaledValue = InputValue / Max;

			// If enabled, smooth the value
			if (Settings.bEnableValueSmoothing)
			{
				const float SmoothTimeSeconds = ScaledValue > Value
				? Settings.ValueAttackTimeSeconds
				: Settings.ValueReleaseTimeSeconds;
				const float NewValue = PeakTamerPrivate::SmoothValue(ScaledValue, Value, DeltaTimeSeconds, SmoothTimeSeconds);
				Value = FMath::Min(NewValue, 1.0f);
			}
			else
			{
				Value = FMath::Min(ScaledValue, 1.0f);
			}
		}
	}

}

UHarmonixPeakTamer* UHarmonixPeakTamer::CreateHarmonixPeakTamer()
{
	return NewObject<UHarmonixPeakTamer>();
}

void UHarmonixPeakTamer::Configure(const FHarmonixPeakTamerSettings& Settings)
{
	PeakTamer.Configure(Settings);
}

void UHarmonixPeakTamer::Update(const float InputValue, const float DeltaTimeSeconds)
{
	PeakTamer.Update(InputValue, DeltaTimeSeconds);
}

float UHarmonixPeakTamer::GetPeak() const
{
	return PeakTamer.GetPeak();
}

float UHarmonixPeakTamer::GetValue() const
{
	return PeakTamer.GetValue();
}
