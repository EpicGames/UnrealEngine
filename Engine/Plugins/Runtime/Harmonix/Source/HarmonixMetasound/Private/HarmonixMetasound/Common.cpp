// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Common.h"

#include "MetasoundNodeRegistrationMacro.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound
{
	const FName HarmonixNodeNamespace = TEXT("HarmonixNodes");

	namespace MetasoundNodeCategories
	{
		const FText Harmonix = METASOUND_LOCTEXT("Metasound_HarmonixCategory", "Harmonix");
		const FText Modulation = METASOUND_LOCTEXT("Metasound_ModulationCategory", "Modulation");
		const FText Analysis = METASOUND_LOCTEXT("Metasound_AnalsisCategory", "Analysis");
	}
	
	namespace CommonPinNames
	{
		namespace Inputs
		{
			DEFINE_INPUT_METASOUND_PARAM(Enable,                        "Enable",                             "Enables processing.")
			DEFINE_INPUT_METASOUND_PARAM(MidiStream,                    "MIDI Stream",						  "MIDI event stream.")
			DEFINE_INPUT_METASOUND_PARAM(MidiChannelNumber,             "MIDI Channel",                       "MIDI channel to render (1-16, 0 for all).")
			DEFINE_INPUT_METASOUND_PARAM(MidiChannelFilterSpecifier,    "MIDI Channel Filter",                "MIDI channels to process. '*' for all, or a string like '1,3,4-8' to identify specific channels.")
			DEFINE_INPUT_METASOUND_PARAM(MidiTrackNumber,               "Track Number",                       "Track number (1 based).")
			DEFINE_INPUT_METASOUND_PARAM(MidiTrackIndexFilterSpecifier, "MIDI Track Filter",                  "MIDI tracks to process. '*' for all, or a string like '1,3,4-8' to identify specific tracks.")
			DEFINE_INPUT_METASOUND_PARAM(MinMidiNote,                   "Min Note #",                         "Minimum MIDI note number (0-127).")
			DEFINE_INPUT_METASOUND_PARAM(MaxMidiNote,                   "Max Note #",                         "Maximum MIDI note number (0-127).")
			DEFINE_INPUT_METASOUND_PARAM(MinMidiVelocity,               "Min Velocity",                       "Minimum Velocity (0-127).")
			DEFINE_INPUT_METASOUND_PARAM(MaxMidiVelocity,               "Max Velocity",                       "Maximum Velocity (0-127).")
			DEFINE_INPUT_METASOUND_PARAM(MidiFileAsset,                 "MIDI File",                          "Standard MIDI file.")
			DEFINE_INPUT_METASOUND_PARAM(Transport,                     "Transport",                          "Play, pause, continue, stop, etc.")
			DEFINE_INPUT_METASOUND_PARAM(MidiClock,                     "MIDI Clock",                         "MIDI timing information.")
			DEFINE_INPUT_METASOUND_PARAM(Tempo,                         "Tempo",                              "BPM. This is MIDI bpm... ALWAYS QUARTER NOTES PER MINUTE regardless of time signature!")
			DEFINE_INPUT_METASOUND_PARAM(TimeSigNumerator,	            "Time Sig. Numerator",                "Time Signature Numerator")
			DEFINE_INPUT_METASOUND_PARAM(TimeSigDenominator,            "Time Sig. Denominator",              "TimeSignature Denominator")
			DEFINE_INPUT_METASOUND_PARAM(Speed,                         "Speed",                              "Playback speed multiplier.")
			DEFINE_INPUT_METASOUND_PARAM(Loop,				            "Simple Loop",						  "Whether looping is enabled for this node. Loop length will be rounded to the nearest bar.")
			DEFINE_INPUT_METASOUND_PARAM(LoopLengthBars,                "Loop Length (Bars)",                 "The number of bars to loop for simple looping")
			DEFINE_INPUT_METASOUND_PARAM(TransportPrepare,              "Prepare",                            "Causes receiving nodes to prepare to play (pre-buffer, cache, etc.)")
			DEFINE_INPUT_METASOUND_PARAM(TransportPlay,                 "Play",                               "Causes a transport play request.")
			DEFINE_INPUT_METASOUND_PARAM(TransportPause,                "Pause",                              "Causes a transport pause request.")
			DEFINE_INPUT_METASOUND_PARAM(TransportContinue,             "Continue",                           "Causes a transport continue request.")
			DEFINE_INPUT_METASOUND_PARAM(TransportStop,                 "Stop",                               "Causes a transport stop request.")
			DEFINE_INPUT_METASOUND_PARAM(TransportKill,                 "Kill",                               "Causes a transport kill request.")
			DEFINE_INPUT_METASOUND_PARAM(TriggerSeek,                   "Trigger Seek",                       "Causes a transport seek request. Seek destination provided on a separate input.")
			DEFINE_INPUT_METASOUND_PARAM(SeekDestination,               "Seek Target",                        "Destination for the next seek when it is triggered.")
			DEFINE_INPUT_METASOUND_PARAM(PrerollBars,                   "Pre-roll Bars",                      "Number of bars to look back when seeking MIDI to find note-on messages that should hold over.")
			DEFINE_INPUT_METASOUND_PARAM(GridSizeUnits,                 "Base Grid Size",                     "Base size of the qrid squares in musical terms.")
			DEFINE_INPUT_METASOUND_PARAM(GridSizeMult,                  "Grid Size Multiplier",               "Grid size multiplier.")
			DEFINE_INPUT_METASOUND_PARAM(OffsetUnits,                   "Base Offset Size ",                  "Base size of the qrid offset.")
			DEFINE_INPUT_METASOUND_PARAM(OffsetMult,                    "Offset Multiplier",                  "Offset multiplier.")
			DEFINE_INPUT_METASOUND_PARAM(Bar,		   					"Bar",                                "Musical bar (or measure) (1 based!)")
			DEFINE_INPUT_METASOUND_PARAM(FloatBeat,			            "Beat",                               "Musical beat. 1 based and in units of the time signature denomninator! The value is rounded to the nearest 32nd note.")
			DEFINE_INPUT_METASOUND_PARAM(SynthPatch,                    "Patch",                              "Synthesizer patch asset.")
			DEFINE_INPUT_METASOUND_PARAM(Transposition,                 "Transposition",                      "Transposition in semitones.")
			DEFINE_INPUT_METASOUND_PARAM(ClockSpeedToPitch,             "MIDI Clock Speed Affects Pitch",     "When true the clock speed of the incoming MIDI stream will affect the pitch of the synth.")
			DEFINE_INPUT_METASOUND_PARAM(ClockSpeedToFrequency,         "MIDI Clock Speed Affects Frequency", "When true the clock speed of the incoming MIDI stream will affect the calculated frequency.")
			DEFINE_INPUT_METASOUND_PARAM(LFOSyncType,					"LFO Sync Type",					  "Specifies if and how the LFO should sync to a clock")
			DEFINE_INPUT_METASOUND_PARAM(LFOFrequency,				    "LFO Frequency",					  "The period of the LFO. If Sync Type is TempoSync, the unit is cycles per quarter note. Otherwise, the unit is Hz (cycles per second).")
			DEFINE_INPUT_METASOUND_PARAM(LFOInvert,					    "Invert LFO",						  "Toggle to invert the LFO")
			DEFINE_INPUT_METASOUND_PARAM(LFOShape,					    "LFO Shape",						  "The shape of the LFO. 0.0-1.0 = morph between square and triangle. 1.0-2.0 = morph between triangle and sawtooth")
			DEFINE_INPUT_METASOUND_PARAM(AudioMono,					    "Audio Mono",					      "The mono audio.")
			DEFINE_INPUT_METASOUND_PARAM(AudioLeft,					    "Audio Left",					      "The left channel audio.")
			DEFINE_INPUT_METASOUND_PARAM(AudioRight,					"Audio Right",					      "The right channel audio.")
			DEFINE_INPUT_METASOUND_PARAM(AudioCenter,                   "Audio Center",                       "5.1/7.1: The center channel audio.")
			DEFINE_INPUT_METASOUND_PARAM(AudioLFE,                      "Audio LFE",                          "5.1/7.1: The LFE channel audio.")
			DEFINE_INPUT_METASOUND_PARAM(AudioLeftSide,                 "Audio Left Side",                    "Quad/5.1: Left Surround, 7.1: Left Side.")
			DEFINE_INPUT_METASOUND_PARAM(AudioRightSide,                "Audio Right Side",                   "Quad/5.1: Right Surround, 7.1: Right Side.")
			DEFINE_INPUT_METASOUND_PARAM(AudioLeftBack,                 "Audio Left Back - Surround",         "7.1: Left Back.")
			DEFINE_INPUT_METASOUND_PARAM(AudioRightBack,                "Audio Right Back - Surround",        "7.1: Right Back.")
			DEFINE_INPUT_METASOUND_PARAM(Timestamp,                     "Music Timestamp",                    "The Bar and Beat of interest.")
			DEFINE_INPUT_METASOUND_PARAM(ShouldQuantizeTimestamp,       "Quantize",                           "Quantize the provide timestamp (Bar & Beat) to a musical subdivision.")
			DEFINE_INPUT_METASOUND_PARAM(QuantizationUnit,              "Quantization Unit",                  "The music subdivision that a music timestamp should be quantized to.")
		}
		namespace Outputs
		{
			DEFINE_OUTPUT_METASOUND_PARAM(Transport,          "Transport",                   "Play, pause, continue, stop, etc.")
			DEFINE_OUTPUT_METASOUND_PARAM(MidiClock,          "MIDI Clock",                  "MIDI timing information.")
			DEFINE_OUTPUT_METASOUND_PARAM(MidiFileAsset,      "MIDI File",                   "Standard MIDI file.")
			DEFINE_OUTPUT_METASOUND_PARAM(MidiStream,         "MIDI Stream",				 "MIDI event stream.")
			DEFINE_OUTPUT_METASOUND_PARAM(NoteOn,             "Note On",                     "MIDI note on.")
			DEFINE_OUTPUT_METASOUND_PARAM(NoteOff,            "Note Off",                    "MIDI note off.")
			DEFINE_OUTPUT_METASOUND_PARAM(MidiNoteNumber,     "MIDI Note #",                 "MIDI note number (0 - 127).")
			DEFINE_OUTPUT_METASOUND_PARAM(Frequency,          "Frequency",                   "MIDI note number converted to frequency.")
			DEFINE_OUTPUT_METASOUND_PARAM(MidiVelocity,       "Velocity",                    "MIDI velocity (0 - 127).")
			DEFINE_OUTPUT_METASOUND_PARAM(NormalizedVelocity, "Normalized Velocity",         "Velocity (0.0 - 1.0).")
			DEFINE_OUTPUT_METASOUND_PARAM(TransportPrepare,   "Prepare",                     "Causes receiving nodes to prepare to play (pre-buffer, cache, etc.)")
			DEFINE_OUTPUT_METASOUND_PARAM(TransportPlay,      "Play",                        "Transport play requested.")
			DEFINE_OUTPUT_METASOUND_PARAM(TransportPause,     "Pause",                       "Transport pause requested.")
			DEFINE_OUTPUT_METASOUND_PARAM(TransportContinue,  "Continue",                    "Transport continue requested.")
			DEFINE_OUTPUT_METASOUND_PARAM(TransportStop,      "Stop",                        "Transport stop requested.")
			DEFINE_OUTPUT_METASOUND_PARAM(TransportKill,      "Kill",                        "Transport kill requested.")
			DEFINE_OUTPUT_METASOUND_PARAM(TransportSeek,      "Seek",                        "Transport seek requested.")
			DEFINE_OUTPUT_METASOUND_PARAM(MusicTimestamp,     "Music Timestamp",             "A single structure representing a musical time (bar & beat).")
			DEFINE_OUTPUT_METASOUND_PARAM(SeekTarget,         "Seek Target",                 "A well formed musical seek target.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioMono,          "Audio Mono",					 "The mono audio.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioLeft,          "Audio Left",				     "The left channel audio.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioRight,         "Audio Right",                 "The right channel audio.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioCenter,        "Audio Center",                "5.1/7.1: The center channel audio output.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioLFE,           "Audio LFE",                   "5.1/7.1: The LFE channel audio output.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioLeftSide,      "Audio Left Side",             "Quad/5.1: Left Surround, 7.1: Left Side.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioRightSide,     "Audio Right Side",            "Quad/5.1: Right Surround, 7.1: Right Side.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioLeftBack,      "Audio Left Back - Surround",  "7.1: Left Back.")
			DEFINE_OUTPUT_METASOUND_PARAM(AudioRightBack,     "Audio Right Back - Surround", "7.1: Right Back.")
			DEFINE_OUTPUT_METASOUND_PARAM(MusicTimespanBar,   "Bar",                         "Bar (Measure). 1 based!")
			DEFINE_OUTPUT_METASOUND_PARAM(MusicTimespanBeat,  "Beat",                        "Beat. 1 based!")
			DEFINE_OUTPUT_METASOUND_PARAM(Tempo,              "Tempo",                       "BPM. This is MIDI bpm... ALWAYS QUARTER NOTES PER MINUTE regardless of time signature!")
			DEFINE_OUTPUT_METASOUND_PARAM(Speed,              "Speed",                       "Playback speed multiplier.")
			DEFINE_OUTPUT_METASOUND_PARAM(TimeSigNumerator,   "Time Sig. Numerator",         "Time Signature Numerator")
			DEFINE_OUTPUT_METASOUND_PARAM(TimeSigDenominator, "Time Sig. Denominator",       "TimeSignature Denominator")
			DEFINE_OUTPUT_METASOUND_PARAM(SecsPerQuarter,     "Secs Per Quarter Note",       "Seconds per quarter note.")
			DEFINE_OUTPUT_METASOUND_PARAM(SecsPerBeat,        "Secs Per Beat",               "Seconds per beat. Equals seconds per beat if time signature denominator is 4.")
		}
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
