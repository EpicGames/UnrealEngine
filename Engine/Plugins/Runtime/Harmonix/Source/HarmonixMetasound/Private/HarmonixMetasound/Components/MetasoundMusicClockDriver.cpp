// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundMusicClockDriver.h"
#include "MetasoundGeneratorHandle.h"
#include "Components/AudioComponent.h"
#include "HarmonixMetasound/Analysis/MidiSongPosVertexAnalyzer.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "MetasoundGenerator.h"
#include "Async/Async.h"
#include "Harmonix.h"
#include "MetasoundSource.h"

DEFINE_LOG_CATEGORY_STATIC(LogMetasoundMusicClockDriver, Log, All)

namespace MetasoundMusicClockDriver
{
	float Fudge = 1.00f;
	float kP = 0.18f;
	float HistoricSmoothedAudioRenderLagSeconds = 0.030f; // this used to be baked-in/hardcoded into the smoothing of the audio render time. 
	float SmoothedAudioRenderLagSeconds = 0.030f; // 30 ms
	float MaxErrorSecondsBeforeJump = 0.060f; // 60ms
	int32 HighWaterNumDataAvailable = 0;
	double SlowestCorrectionSpeed = 0.98f;
	double FastestCorrectionSpeed = 1.02f;


	FAutoConsoleVariableRef CVarFudge(
		TEXT("au.MusicClockComponent.TEST.Fudge"),
		Fudge,
		TEXT("Clock Fudge FOR TESTING."),
		ECVF_Cheat);

	FAutoConsoleVariableRef CVarkP(
		TEXT("au.MusicClockComponent.kP"),
		kP,
		TEXT("Clock kP."),
		ECVF_Default);

	FAutoConsoleVariableRef CVarAudioRenderLag(
		TEXT("au.MusicClockComponent.SmoothedAudioRenderLagSeconds"),
		SmoothedAudioRenderLagSeconds,
		TEXT("SmoothedAudioRenderLagSeconds."),
		ECVF_Cheat);

	FAutoConsoleVariableRef CVarMaxErrorSecondsBeforeJump(
		TEXT("au.MusicClockComponent.MaxErrorSecondsBeforeJump"),
		MaxErrorSecondsBeforeJump,
		TEXT("MaxErrorSecondsBeforeJump."),
		ECVF_Default);
}

TOptional<FVector> FMetasoundMusicClockDriver::TryGetAudioSourceLocation() const
{
	if (UAudioComponent* AudioComponent = AudioComponentToWatch.Get())
	{
		return AudioComponent->GetComponentLocation();
	}
	return {};
}

bool FMetasoundMusicClockDriver::CalculateSongPosWithOffset(float MsOffset, ECalibratedMusicTimebase Timebase, FMidiSongPos& OutResult) const
{
	using namespace HarmonixMetasound::Analysis;

	check(IsInGameThread());

	// if we have an owner, ask them directly
	if (!ClockHistory)
	{
		return false;
	}

	if (!CurrentMapChain || !CurrentMapChain->SongMaps)
	{
		return false;
	}

	const FPerTimebaseSmoothedClockState* ClockState = nullptr;

	switch (Timebase)
	{
	case ECalibratedMusicTimebase::AudioRenderTime:  ClockState = &AudioRenderState; break;
	case ECalibratedMusicTimebase::ExperiencedTime:  ClockState = &PlayerExperienceState; break;
	case ECalibratedMusicTimebase::VideoRenderTime:
	default:                                         ClockState = &VideoRenderState; break;
	}

	float AbsMs = ClockState->TempoMapMs + MsOffset;
	float TempoMapOffsetTick = CurrentMapChain->SongMaps->MsToTick(AbsMs);
	float RelativeTicks = TempoMapOffsetTick - ClockState->TempoMapTick;
	float SmoothedOffsetTick = ClockState->LocalTick + RelativeTicks;

	if (CurrentMapChain->LoopLengthTicks > 0)
	{
		while (SmoothedOffsetTick < 0.0f)
		{
			SmoothedOffsetTick += CurrentMapChain->LoopLengthTicks;
		}
		while (SmoothedOffsetTick >= CurrentMapChain->LoopLengthTicks)
		{
			SmoothedOffsetTick -= CurrentMapChain->LoopLengthTicks;
		}
	}

	// first 99% of the song pos...
	OutResult.SetByTick(SmoothedOffsetTick, *(CurrentMapChain->SongMaps));
	// but tempo needs to be set to authoritiesTempo...
	OutResult.Tempo = CurrentMapChain->SongMaps->GetTempoAtTick(FMath::FloorToInt32(TempoMapOffsetTick));

	return true;
}

FMidiSongPos FMetasoundMusicClockDriver::CalculateSongPosAtMsForLoopingOrMonotonicClock(float AbsoluteMs, float& PositionTick, bool& SeekDetected, bool& LoopDetected) const
{
	using namespace HarmonixMetasound::Analysis;
	FMidiSongPos OutSongPos;
	if (!ClockHistory)
	{
		PositionTick = 0.0f;
		return OutSongPos;
	}

	if (!CurrentMapChain || !CurrentMapChain->SongMaps)
	{
		PositionTick = 0.0f;
		return OutSongPos;
	}

	float NewPositionTick = 0.0f;
	if (CurrentMapChain->LoopLengthTicks > 0)
	{
		float DrivingTick = CurrentMapChain->SongMaps->MsToTick(AbsoluteMs);
		float TickPastLoop = (float)(CurrentMapChain->FirstTickInLoop + CurrentMapChain->LoopLengthTicks);
		if (DrivingTick >= TickPastLoop)
		{
			float WrappedTick = FMath::Fmod(DrivingTick - CurrentMapChain->FirstTickInLoop, (float)CurrentMapChain->LoopLengthTicks);
			LoopDetected = (PositionTick - WrappedTick) > (float)(CurrentMapChain->LoopLengthTicks - 240);
			if (LoopDetected)
			{
				UE_LOGF(LogMetasoundMusicClockDriver, VeryVerbose, "Detected loop when calculating song pos (%f -> %f -> %f -> %d )", PositionTick, WrappedTick, PositionTick - WrappedTick, CurrentMapChain->LoopLengthTicks);
			}
			OutSongPos.SetByTick(WrappedTick, *CurrentMapChain->SongMaps);
			OutSongPos.Tempo = CurrentMapChain->SongMaps->GetTempoAtTick(FMath::FloorToInt32(DrivingTick));

			if (!LoopDetected)
			{
				SeekDetected = CheckForSeek(PositionTick, WrappedTick, OutSongPos.Tempo, CurrentMapChain->SongMaps->GetTicksPerQuarterNote());
				if (SeekDetected)
				{
					UE_LOGF(LogMetasoundMusicClockDriver, VeryVerbose, "Detected seek when calculating song pos (%f -> %f)", PositionTick, WrappedTick);
				}
			}

			PositionTick = WrappedTick;
			return OutSongPos;
		}
		NewPositionTick = DrivingTick;
	}
	else
	{
		NewPositionTick = CurrentMapChain->SongMaps->MsToTick(AbsoluteMs);
	}

	OutSongPos.SetByTimeAndTick(AbsoluteMs, NewPositionTick, *(CurrentMapChain->SongMaps));
	SeekDetected = CheckForSeek(PositionTick, NewPositionTick, OutSongPos.Tempo, CurrentMapChain->SongMaps->GetTicksPerQuarterNote());
	if (SeekDetected)
	{
		UE_LOGF(LogMetasoundMusicClockDriver, VeryVerbose, "Detected seek when calculating song pos (%f -> %f)", PositionTick, NewPositionTick);
	}
	PositionTick = NewPositionTick;
	return OutSongPos;
}

FMidiSongPos FMetasoundMusicClockDriver::CalculateSongPosAtMsForOffsetClock(float PositionMs, float ClockTickOffsetFromDrivingClock, float& PositionTick, bool& SeekDetected) const
{
	using namespace HarmonixMetasound::Analysis;
	
	FMidiSongPos OutSongPos;
	
	if (!ClockHistory)
	{
		PositionTick = 0.0f;
		return OutSongPos;
	}

	if (!CurrentMapChain || !CurrentMapChain->SongMaps)
	{
		PositionTick = 0.0f;
		return OutSongPos;
	}

	float NewPositionTick = CurrentMapChain->SongMaps->MsToTick(PositionMs);

	OutSongPos.SetByTick(NewPositionTick, *CurrentMapChain->SongMaps);
	OutSongPos.Tempo = CurrentMapChain->SongMaps->GetTempoAtTick(FMath::FloorToInt32(NewPositionTick - ClockTickOffsetFromDrivingClock));

	SeekDetected = CheckForSeek(PositionTick, NewPositionTick, OutSongPos.Tempo, CurrentMapChain->SongMaps->GetTicksPerQuarterNote());
	if (SeekDetected)
	{
		UE_LOGF(LogMetasoundMusicClockDriver, VeryVerbose, "Detected seek when calculating song pos (%f -> %f)", PositionTick, NewPositionTick);
	}

	PositionTick = NewPositionTick;

	return OutSongPos;
}

bool FMetasoundMusicClockDriver::CheckForSeek(float FirstTick, float NextTick, float CurrentTempo, int32 TicksPerQuarter) const
{
	float QuartersPerSecond = CurrentTempo / 60.0f;
	float ExpectedDeltaQuarters = QuartersPerSecond * DeltaSecondsBetweenRefreshes;
	float ExpectedDeltaTicks = ExpectedDeltaQuarters * (float)TicksPerQuarter;
	return FMath::Abs(ExpectedDeltaTicks - (NextTick - FirstTick)) > (ExpectedDeltaTicks * 2.0f);
}

bool FMetasoundMusicClockDriver::RefreshCurrentSongPos()
{
	//	Only for use when on the game thread.
	if (ensureMsgf(
			IsInGameThread(),
			TEXT("%hs called from non-game thread.  This is not supported"),
			__FUNCTION__) == false)
	{
		return false;
	}

	if (AudioComponentToWatch.IsValid())
	{
		if (!CurrentGeneratorHandle)
		{
			// cursor connection is not pending
			AttemptToConnectToAudioComponentsMetasound(AudioComponentToWatch.Get(), OnAttachedDelegate, OnDetachedDelegate);
		}
	}

	if (bRunning)
	{
		if (ClockHistory)
		{
			// cursor is attached and has the current info
			RefreshCurrentSongPosFromHistory();
			return true;
		}
		else
		{
			// Cursor not attached so use wall clock
			if (!WasEverConnected || RunPastMusicEnd)
			{
				RefreshCurrentSongPosFromWallClock();
				return true;
			}
		}
	}

	return false;
}

void FMetasoundMusicClockDriver::OnStart()
{
	check(IsInGameThread());

	SongPosOffsetMs = 0.0f;
	RenderStartSampleCount = 0;
	RenderStartWallClockTimeSeconds = 0.0;
	FreeRunStartTimeSecs = GetWallClockTime();
	bRunning = true;
}

void FMetasoundMusicClockDriver::OnPause()
{
	check(IsInGameThread());
	bRunning = false;
}

void FMetasoundMusicClockDriver::OnContinue()
{
	check(IsInGameThread());
	if (!ClockHistory)
	{
		RefreshCurrentSongPosFromWallClock();
	}
	bRunning = true;
}

void FMetasoundMusicClockDriver::OnStop()
{
	check(IsInGameThread());
	bRunning = false;
}

void FMetasoundMusicClockDriver::Disconnect()
{
	check(IsInGameThread());
	SetClockHistory(nullptr);
	DetachAllCallbacks();
	AudioComponentToWatch.Reset();
	CurrentGeneratorHandle.Reset();
}

const ISongMapEvaluator* FMetasoundMusicClockDriver::GetCurrentSongMapEvaluator() const
{
	using namespace HarmonixMetasound::Analysis;
	check(IsInGameThread());
	if (ClockHistory && CurrentMapChain && CurrentMapChain->SongMaps)
	{
		return CurrentMapChain->SongMaps.Get();
	}
	return &DefaultMaps;
}

bool FMetasoundMusicClockDriver::LoopedThisFrame(ECalibratedMusicTimebase Timebase) const
{
	switch (Timebase)
	{
	case ECalibratedMusicTimebase::RawAudioRenderTime:
	case ECalibratedMusicTimebase::AudioRenderTime:    return AudioRenderLoopDetected;
	case ECalibratedMusicTimebase::ExperiencedTime:	   return PlayerExperiencedLoopDetected;
	case ECalibratedMusicTimebase::VideoRenderTime:
	default:                                           return VideoRenderLoopDetected;
	}
}

bool FMetasoundMusicClockDriver::SeekedThisFrame(ECalibratedMusicTimebase Timebase) const
{
	switch (Timebase)
	{
	case ECalibratedMusicTimebase::RawAudioRenderTime:
	case ECalibratedMusicTimebase::AudioRenderTime:    return AudioRenderSeekDetected;
	case ECalibratedMusicTimebase::ExperiencedTime:	   return PlayerExperiencedSeekDetected;
	case ECalibratedMusicTimebase::VideoRenderTime:
	default:                                           return VideoRenderSeekDetected;
	}
}

#if !UE_BUILD_SHIPPING
FString FMetasoundMusicClockDriver::GetDisplayName() const
{
#if WITH_EDITORONLY_DATA
	// that method is only good WITH_EDITORONLY_DATA
	if (UAudioComponent* Component = AudioComponentToWatch.Get())
	{
		if (UMetaSoundSource* Source = Cast<UMetaSoundSource>(Component->GetSound()))
		{
			return FString::Printf(TEXT("Metasound: %s"), *Source->GetDisplayName().ToString());
		}
	}
#endif
	return TEXT("Metasound");
}
#endif

bool FMetasoundMusicClockDriver::ConnectToAudioComponentsMetasound(
	UAudioComponent* InAudioComponent,
	FName MetasoundOuputPinName,
	UMetasoundGeneratorHandle::FOnAttached::FDelegate OnGeneratorAttachedCallback,
	UMetasoundGeneratorHandle::FOnDetached::FDelegate OnGeneratorDetachedCallback)
{
	OnAttachedDelegate = OnGeneratorAttachedCallback;
	OnDetachedDelegate = OnGeneratorDetachedCallback;
	
	AudioComponentToWatch = InAudioComponent;
	MetasoundOutputName = MetasoundOuputPinName;
	bool Connected = AttemptToConnectToAudioComponentsMetasound(InAudioComponent, OnGeneratorAttachedCallback, OnGeneratorDetachedCallback);

	if(GetState() == EMusicClockState::Running)
	{
		OnStart();
	}
	
	return Connected;
}

bool FMetasoundMusicClockDriver::AttemptToConnectToAudioComponentsMetasound(
	UAudioComponent* InAudioComponent,
	UMetasoundGeneratorHandle::FOnAttached::FDelegate OnGeneratorAttachedCallback,
	UMetasoundGeneratorHandle::FOnDetached::FDelegate OnGeneratorDetachedCallback)
{
	using namespace HarmonixMetasound::Analysis;

	check(IsInGameThread());
	if (!AudioComponentToWatch.IsValid() || MetasoundOutputName.IsNone())
	{
		return false;
	}

	// we have an audio component, but it's not set to play a Metasound (yet)
	if (!Cast<UMetaSoundSource>(AudioComponentToWatch->GetSound()))
	{
		return false;
	}

	DetachAllCallbacks();
	CurrentGeneratorHandle.Reset(UMetasoundGeneratorHandle::CreateMetaSoundGeneratorHandle(AudioComponentToWatch.Get()));
	if (!CurrentGeneratorHandle)
	{
		return false;
	}

	bool WatchingOutput = CurrentGeneratorHandle->WatchOutput(MetasoundOutputName, 
		FOnMetasoundOutputValueChangedNative::CreateLambda([](FName, const FMetaSoundOutput&) {}),
		FMidiSongPosVertexAnalyzer::GetAnalyzerName(),
		FMidiSongPosVertexAnalyzer::SongPosition.Name);
	if (WatchingOutput)
	{
		ensure(CurrentGeneratorHandle->TryCreateAnalyzerAddress(MetasoundOutputName, 
				FMidiSongPosVertexAnalyzer::GetAnalyzerName(), 
				FMidiSongPosVertexAnalyzer::SongPosition.Name, 
				MidiSongPosAnalyzerAddress));
	}

	GeneratorAttachedCallbackHandle = CurrentGeneratorHandle->OnGeneratorHandleAttached.AddLambda([this, OnGeneratorAttachedCallback]()
	{
		OnGeneratorAttached();
		OnGeneratorAttachedCallback.ExecuteIfBound();
	});
	GeneratorDetachedCallbackHandle = CurrentGeneratorHandle->OnGeneratorHandleDetached.AddLambda([this, OnGeneratorDetachedCallback]()
	{
		OnGeneratorDetached();
		OnGeneratorDetachedCallback.ExecuteIfBound();
	});
	GeneratorIOUpdatedCallbackHandle = CurrentGeneratorHandle->OnIOUpdatedWithChanges.AddLambda([this](const TArray<Metasound::FVertexInterfaceChange>& VertexInterfaceChanges){OnGeneratorIOUpdatedWithChanges(VertexInterfaceChanges);});
	UMetasoundGeneratorHandle::FOnSetGraph::FDelegate OnSetGraph;
	OnSetGraph.BindLambda([this](){OnGraphSet();});
	GraphChangedCallbackHandle = CurrentGeneratorHandle->AddGraphSetCallback(MoveTemp(OnSetGraph));
	// OnGeneratorAttached();
	return true;
}

void FMetasoundMusicClockDriver::DetachAllCallbacks()
{
	if (CurrentGeneratorHandle)
	{
		CurrentGeneratorHandle->OnGeneratorHandleAttached.Remove(GeneratorAttachedCallbackHandle);
		GeneratorAttachedCallbackHandle.Reset();
		CurrentGeneratorHandle->OnGeneratorHandleDetached.Remove(GeneratorDetachedCallbackHandle);
		GeneratorDetachedCallbackHandle.Reset();

		CurrentGeneratorHandle->OnIOUpdatedWithChanges.Remove(GeneratorIOUpdatedCallbackHandle);
		GeneratorIOUpdatedCallbackHandle.Reset();
		CurrentGeneratorHandle->RemoveGraphSetCallback(GraphChangedCallbackHandle);
		GraphChangedCallbackHandle.Reset();

	}
	SetClockHistory(nullptr);
}

void FMetasoundMusicClockDriver::OnGeneratorAttached()
{
	WasEverConnected = true;
	UnderlyingGenerator = CurrentGeneratorHandle->GetGenerator();
	SetClockHistory(UMidiClockUpdateSubsystem::GetOrCreateClockHistory(MidiSongPosAnalyzerAddress));
	SmoothedAudioRenderClockHistoryCursor = ClockHistory->CreateReadCursor();
	SmoothedPlayerExperienceClockHistoryCursor = ClockHistory->CreateReadCursor();
	SmoothedVideoRenderClockHistoryCursor = ClockHistory->CreateReadCursor();
}

void FMetasoundMusicClockDriver::OnGraphSet()
{
	SetClockHistory(UMidiClockUpdateSubsystem::GetOrCreateClockHistory(MidiSongPosAnalyzerAddress));
	SmoothedAudioRenderClockHistoryCursor = ClockHistory->CreateReadCursor();
	SmoothedPlayerExperienceClockHistoryCursor = ClockHistory->CreateReadCursor();
	SmoothedVideoRenderClockHistoryCursor = ClockHistory->CreateReadCursor();
}


void FMetasoundMusicClockDriver::OnGeneratorIOUpdatedWithChanges(const TArray<Metasound::FVertexInterfaceChange>& VertexInterfaceChanges)
{
	if (!MetasoundOutputName.IsNone())
	{
		if (VertexInterfaceChanges.Num() > 0)
		{
			SetClockHistory(UMidiClockUpdateSubsystem::GetOrCreateClockHistory(MidiSongPosAnalyzerAddress));
			SmoothedAudioRenderClockHistoryCursor = ClockHistory->CreateReadCursor();
			SmoothedPlayerExperienceClockHistoryCursor = ClockHistory->CreateReadCursor();
			SmoothedVideoRenderClockHistoryCursor = ClockHistory->CreateReadCursor();
		}
	}
	
}


void FMetasoundMusicClockDriver::OnGeneratorDetached()
{
	TWeakPtr<Metasound::FMetasoundGenerator> DetachingGenerator = CurrentGeneratorHandle->GetGenerator();
	if (DetachingGenerator != UnderlyingGenerator)
	{
		// yes... it is possible we will get a detach callback for a generator that is not ours! yikes.
		return;
	}

	using namespace HarmonixMetasound::Analysis;
	if (GetState() != EMusicClockState::Stopped)
	{
		if (ClockHistory && CurrentMapChain && CurrentMapChain->SongMaps)
		{
			DefaultMaps.Copy(*(CurrentMapChain->SongMaps), 0, LastTickSeen);
		}
		SongPosOffsetMs = GetCurrentSmoothedAudioRenderSongPos().SecondsIncludingCountIn * 1000.0f;
		FreeRunStartTimeSecs = GetWallClockTime();
	}
	SetClockHistory(nullptr);
	SmoothedAudioRenderClockHistoryCursor = HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor();
	SmoothedPlayerExperienceClockHistoryCursor = HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor();
	SmoothedVideoRenderClockHistoryCursor = HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor();
}

void FMetasoundMusicClockDriver::RefreshCurrentSongPosFromWallClock()
{
	double FreeRunTime = (GetWallClockTime() - FreeRunStartTimeSecs) * CurrentClockAdvanceRate;
	SetCurrentSongPosByTime(ECalibratedMusicTimebase::RawAudioRenderTime, ((float)FreeRunTime * 1000.0) + SongPosOffsetMs, DefaultMaps);
	SetCurrentSongPos(ECalibratedMusicTimebase::AudioRenderTime, GetCurrentRawAudioRenderSongPos());
	SetCurrentSongPosByTime(ECalibratedMusicTimebase::ExperiencedTime, GetCurrentSmoothedAudioRenderSongPos().SecondsIncludingCountIn * 1000.0f - FHarmonixModule::GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs(), DefaultMaps);
	SetCurrentSongPosByTime(ECalibratedMusicTimebase::VideoRenderTime, GetCurrentSmoothedAudioRenderSongPos().SecondsIncludingCountIn * 1000.0f - FHarmonixModule::GetMeasuredVideoToAudioRenderOffsetMs(), DefaultMaps);

	const FMidiSongPos& CurrentSmoothedAudioRenderSongPos = GetCurrentSmoothedAudioRenderSongPos();
	UpdateMusicPlaybackRate(CurrentSmoothedAudioRenderSongPos.Tempo, CurrentClockAdvanceRate, CurrentSmoothedAudioRenderSongPos.TimeSigNumerator, CurrentSmoothedAudioRenderSongPos.TimeSigDenominator);
}

FString FMetasoundMusicClockDriver::HistoryFailureTypeToString(FMetasoundMusicClockDriver::EHistoryFailureType Error)
{
	switch (Error)
	{
	case FMetasoundMusicClockDriver::EHistoryFailureType::None: return "None";
	case FMetasoundMusicClockDriver::EHistoryFailureType::NotEnoughDataInTheHistoryRing: return "NotEnoughDataInTheHistoryRing";
	case FMetasoundMusicClockDriver::EHistoryFailureType::NotEnoughHistory: return "NotEnoughHistory";
	case FMetasoundMusicClockDriver::EHistoryFailureType::LookingForTimeInTheFutureOfWhatHasEvenRendered: return "LookingForTimeInTheFutureOfWhatHasEvenRendered";
	case FMetasoundMusicClockDriver::EHistoryFailureType::CaughtUpToRenderPosition: return "CaughtUpToRenderPosition";
	}
	return "Unrecognized";
}

void FMetasoundMusicClockDriver::RefreshCurrentSongPosFromHistory()
{
	using namespace HarmonixMetasound::Analysis;

	check(IsInGameThread());

	if (!bRunning || !SmoothedAudioRenderClockHistoryCursor.DataAvailable())
	{
		return;
	}
	
	if (!SmoothedAudioRenderClockHistoryCursor.Queue)
	{
		return;
	}

	if (!CurrentMapChain || !CurrentMapChain->SongMaps || !CurrentMapChain->IsLatest())
	{
		CurrentMapChain = ClockHistory->GetLatestMapsForConsumer();
		if (!CurrentMapChain || !CurrentMapChain->SongMaps)
		{
			return;
		}
	}

	auto Entry = ClockHistory->Positions.GetEntry(ClockHistory->Positions.GetLastWriteIndex());
	SetCurrentSongPosByTick(ECalibratedMusicTimebase::RawAudioRenderTime, Entry->Item.UpToTick, *(CurrentMapChain->SongMaps));
	Metasound::FSampleCount LastRenderPosSampleCount = Entry->Item.SampleCount;
	float SpeedAtRawRenderTime = Entry->Item.CurrentSpeed;
	LastTickSeen = Entry->Item.UpToTick;
	bool bClockIsStopped = Entry->Item.CurrentTransportState != HarmonixMetasound::EMusicPlayerTransportState::Playing;

	if (RenderStartWallClockTimeSeconds == 0.0)
	{
		// We are just starting up. We need to create the initial "sync point"...
		// Wall clock <-> Render Samples...
		RenderStartSampleCount = LastRenderPosSampleCount;
		RenderStartWallClockTimeSeconds = GetWallClockTime() - (double)RenderStartSampleCount / (double)ClockHistory->SampleRate;
		RenderSmoothingLagSeconds = MetasoundMusicClockDriver::SmoothedAudioRenderLagSeconds;
		ErrorTracker.Reset();
		LastRefreshWallClockTimeSeconds = RenderStartWallClockTimeSeconds;
	}

	double CurrentWallClockSeconds = GetWallClockTime();
	DeltaSecondsBetweenRefreshes = CurrentWallClockSeconds - LastRefreshWallClockTimeSeconds;
	LastRefreshWallClockTimeSeconds = CurrentWallClockSeconds;

	double ExpectedRenderedSeconds = (CurrentWallClockSeconds - RenderStartWallClockTimeSeconds) * SyncSpeed * MetasoundMusicClockDriver::Fudge;
	double RenderedSeconds = (double)LastRenderPosSampleCount / (double)ClockHistory->SampleRate;
	double Error = RenderedSeconds - ExpectedRenderedSeconds;

	if (!bClockIsStopped)
	{
		ErrorTracker.Push(Error);

		if (FMath::Abs(ErrorTracker.Min()) > MetasoundMusicClockDriver::MaxErrorSecondsBeforeJump)
		{
			UE_LOGF(LogMetasoundMusicClockDriver, Verbose, "======== MASSIVE ERROR (%f) - SEEKING ===========", (float)Error);
			RenderStartSampleCount = LastRenderPosSampleCount;
			RenderStartWallClockTimeSeconds = GetWallClockTime() - (double)RenderStartSampleCount / (double)ClockHistory->SampleRate;
			ExpectedRenderedSeconds = RenderedSeconds;
			RenderSmoothingLagSeconds = MetasoundMusicClockDriver::SmoothedAudioRenderLagSeconds;
			ErrorTracker.Reset();
			Error = 0;
			SyncSpeed = 1.0;
		}

		// Use proportional part of error to adjust speed ever so slightly...
		if (ExpectedRenderedSeconds > 0.0)
		{
			SyncSpeed += MetasoundMusicClockDriver::kP * ErrorTracker.Min() / ExpectedRenderedSeconds;
		}
		SyncSpeed = FMath::Clamp(SyncSpeed, MetasoundMusicClockDriver::SlowestCorrectionSpeed, MetasoundMusicClockDriver::FastestCorrectionSpeed);
	}

	//UE_LOGF(LogMetasoundMusicClockDriver, Log, "Local Minimum Error: %f Average: %f Speed: %f", (float)ErrorTracker.Min(), (float)ErrorTracker.Average(), (float)SyncSpeed);

	Metasound::FSampleCount ExpectedRenderPosSampleCount = (Metasound::FSampleCount)(ExpectedRenderedSeconds * (double)ClockHistory->SampleRate);

	// first smoothed render time. This is closest to the actual render time. If we catch up to the render time
	// it means we are rendering in such large blocks that we need to push up our "look behind" number for smoothing.
	const FMidiSongPos& CurrentSmoothedAudioRenderSongPos = GetCurrentSmoothedAudioRenderSongPos();
	float SpeedAtSmoothRenderedTime = 1.0f;
	float SmoothedTick = GetCurrentSongMapEvaluator()->MsToTick(CurrentSmoothedAudioRenderSongPos.SecondsIncludingCountIn * 1000.0f);
	float SmoothedTempoMapTick = AudioRenderState.TempoMapTick;
	EHistoryFailureType Result = CalculateSmoothedTick(ExpectedRenderPosSampleCount, LastRenderPosSampleCount,
		SmoothedTick, SmoothedTempoMapTick, SpeedAtSmoothRenderedTime, SmoothedAudioRenderClockHistoryCursor, RenderSmoothingLagSeconds);
	if (Result != EHistoryFailureType::None)
	{
		if (LastRenderPosSampleCount >  (RenderSmoothingLagSeconds * ClockHistory->SampleRate * 2))
		{
			if (RenderSmoothingLagSeconds < 0.250f)
			{
				RenderSmoothingLagSeconds += 0.005f;
				UE_LOGF(LogMetasoundMusicClockDriver, Verbose, "(%lld) Smoothed Render Time too close to actual render time. Bumping up smoothing lag! (%f)", LastRenderPosSampleCount, RenderSmoothingLagSeconds);
			}
			else
			{
				UE_LOGF(LogMetasoundMusicClockDriver, Verbose, "(%lld) Smoothed Render Time too close to actual render time. ALREADY MAX SMOOTHING LAG! (%f)", LastRenderPosSampleCount, RenderSmoothingLagSeconds);
			}
		}
		else
		{
			UE_LOGF(LogMetasoundMusicClockDriver, Verbose, "(%lld) Smoothed Render Time too close to actual render time. WAITING!", LastRenderPosSampleCount);
		}
		return;
	}

	if (SmoothedTempoMapTick != SmoothedTick && CurrentMapChain->LoopLengthTicks <= 0)
	{
		// The clock is offset from its SongMaps (eg. It is the output of a 
		// clock offset node.)
		UpdateCurrentTicksForOffsetClock(SmoothedTick, SmoothedTempoMapTick);
	}
	else
	{
		// The clock is looping or monotonically increasing... so we deal with it this way...
		UpdateCurrentTicksForLoopingOrMonotonicClock(SmoothedTick, SmoothedTempoMapTick);
	}

	UpdateMusicPlaybackRate(CurrentSmoothedAudioRenderSongPos.Tempo, SpeedAtRawRenderTime, CurrentSmoothedAudioRenderSongPos.TimeSigNumerator, CurrentSmoothedAudioRenderSongPos.TimeSigDenominator);
}

void FMetasoundMusicClockDriver::SetClockHistory(const UMidiClockUpdateSubsystem::FClockHistoryPtr& NewHistory)
{
	if (NewHistory != ClockHistory)
	{
		ClockHistory = NewHistory;
		// We null this out because ultimately we get the CurrentMapChain from the ClockHistory so if it ever
		// changes, we need to get a new CurrentMapChain assigned.
		CurrentMapChain = nullptr;
	}
}

void FMetasoundMusicClockDriver::UpdateCurrentTicksForOffsetClock(float SmoothedTick, float SmoothedTempoMapTick)
{
	float SmoothedPositionMs = CurrentMapChain->SongMaps->TickToMs(SmoothedTick);
	// We are behind the actual render time because of the lag we introduce to have enough
	// history... SO... Push forward to get a time that is approx. where the renderer is.
	SmoothedPositionMs += RenderSmoothingLagSeconds * 1000.0f;

	// Calculate the song position AND get the "local tick" for the Audio Render timebase
	SetCurrentSongPos(ECalibratedMusicTimebase::AudioRenderTime, CalculateSongPosAtMsForOffsetClock(SmoothedPositionMs,
		SmoothedTick - SmoothedTempoMapTick,
		AudioRenderState.LocalTick,
		AudioRenderSeekDetected));
	float SmoothedTempoMapMs = CurrentMapChain->SongMaps->TickToMs(SmoothedTempoMapTick);
	AudioRenderState.TempoMapMs = SmoothedTempoMapMs + (RenderSmoothingLagSeconds * 1000.0f);
	AudioRenderState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(AudioRenderState.TempoMapMs);

	// Now the time the user should actually be "experiencing" (ie "hearing") can be calculated as an offset
	// from the smooth audio rendering time...
	float LookBehindLagMs = FHarmonixModule::Get().GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs();
	float Ms = SmoothedPositionMs;
	Ms -= LookBehindLagMs;
	// Calculate the song position AND get the "local tick" for the Player Experience timebase
	SetCurrentSongPos(ECalibratedMusicTimebase::ExperiencedTime, CalculateSongPosAtMsForOffsetClock(Ms,
		SmoothedTick - SmoothedTempoMapTick,
		PlayerExperienceState.LocalTick,
		PlayerExperiencedSeekDetected));
	PlayerExperienceState.TempoMapMs = AudioRenderState.TempoMapMs - LookBehindLagMs;
	PlayerExperienceState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(PlayerExperienceState.TempoMapMs);

	// Now the time the game should be rendering graphics for can be calculated as an offset
	// from the smooth audio rendering time...
	LookBehindLagMs = FHarmonixModule::Get().GetMeasuredVideoToAudioRenderOffsetMs();
	Ms = SmoothedPositionMs;
	Ms -= LookBehindLagMs;
	// Calculate the song position AND get the "local tick" for the Video Render timebase
	SetCurrentSongPos(ECalibratedMusicTimebase::VideoRenderTime, CalculateSongPosAtMsForOffsetClock(Ms,
		SmoothedTick - SmoothedTempoMapTick,
		VideoRenderState.LocalTick,
		VideoRenderSeekDetected));
	VideoRenderState.TempoMapMs = AudioRenderState.TempoMapMs - LookBehindLagMs;
	VideoRenderState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(VideoRenderState.TempoMapMs);
}

void FMetasoundMusicClockDriver::UpdateCurrentTicksForLoopingOrMonotonicClock(float SmoothedTick, float SmoothedTempoMapTick)
{
	AudioRenderState.TempoMapMs = CurrentMapChain->SongMaps->TickToMs(SmoothedTempoMapTick);
	// We are behind the actual render time because of the lag we introduce to have enough
	// history... SO... Push forward to get a time that is approx. where the renderer is.
	AudioRenderState.TempoMapMs += RenderSmoothingLagSeconds * 1000.0f;

	// Calculate the song position AND get the "local tick" for the Audio Render timebase
	AudioRenderState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(AudioRenderState.TempoMapMs);
	SetCurrentSongPos(ECalibratedMusicTimebase::AudioRenderTime, CalculateSongPosAtMsForLoopingOrMonotonicClock(AudioRenderState.TempoMapMs,
		AudioRenderState.LocalTick,
		AudioRenderSeekDetected,
		AudioRenderLoopDetected));

	// Now the time the user should actually be "experiencing" (ie "hearing") can be calculated as an offset
	// from the smooth audio rendering time...
	float LookBehindLagMs = FHarmonixModule::Get().GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs();
	PlayerExperienceState.TempoMapMs = AudioRenderState.TempoMapMs - LookBehindLagMs;
	PlayerExperienceState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(PlayerExperienceState.TempoMapMs);
	SetCurrentSongPos(ECalibratedMusicTimebase::ExperiencedTime, CalculateSongPosAtMsForLoopingOrMonotonicClock(PlayerExperienceState.TempoMapMs,
		PlayerExperienceState.LocalTick,
		PlayerExperiencedSeekDetected,
		PlayerExperiencedLoopDetected));

	// Now the time the game should be rendering graphics for can be calculated as an offset
	// from the smooth audio rendering time...
	LookBehindLagMs = FHarmonixModule::Get().GetMeasuredVideoToAudioRenderOffsetMs();
	VideoRenderState.TempoMapMs = AudioRenderState.TempoMapMs - LookBehindLagMs;
	VideoRenderState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(VideoRenderState.TempoMapMs);
	SetCurrentSongPos(ECalibratedMusicTimebase::VideoRenderTime, CalculateSongPosAtMsForLoopingOrMonotonicClock(VideoRenderState.TempoMapMs,
		VideoRenderState.LocalTick,
		VideoRenderSeekDetected,
		VideoRenderLoopDetected));
}

FMetasoundMusicClockDriver::EHistoryFailureType FMetasoundMusicClockDriver::CalculateSmoothedTick(Metasound::FSampleCount ExpectedRenderPosSampleCount, Metasound::FSampleCount LastRenderPosSampleCount,
	float& SmoothedLocalTick, float& SmoothedTempoMapTick, float& CurrentSpeed, HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor& ReadCursor, float LookBehindSeconds)
{
	using namespace HarmonixMetasound::Analysis;

	// A little bookkeeping for tracking...
	if (MetasoundMusicClockDriver::HighWaterNumDataAvailable < ReadCursor.NumDataAvailable())
	{
		MetasoundMusicClockDriver::HighWaterNumDataAvailable = ReadCursor.NumDataAvailable();
	}

	Metasound::FSampleCount LookingForSampleFrame = ExpectedRenderPosSampleCount - (LookBehindSeconds * ClockHistory->SampleRate);

	int32 NumHistoryAvailable = ReadCursor.NumDataAvailable();
	if (LookingForSampleFrame >= LastRenderPosSampleCount && NumHistoryAvailable > 1)
	{
		while (ReadCursor.NumDataAvailable() > 1)
		{
			ReadCursor.ConsumeNext();
		}
		NumHistoryAvailable = ReadCursor.NumDataAvailable();
	}

	if (NumHistoryAvailable == 0)
	{
		return EHistoryFailureType::NotEnoughDataInTheHistoryRing;
	}

	FMidiClockSongPositionHistory::FScopedItemPeekRef PeekNextRef = ReadCursor.PeekNext();

	if (NumHistoryAvailable == 1 || PeekNextRef->SampleCount > LookingForSampleFrame)
	{
		SmoothedLocalTick = (float)PeekNextRef->UpToTick;
		SmoothedTempoMapTick = (float)PeekNextRef->TempoMapTick;
		CurrentSpeed = PeekNextRef->CurrentSpeed;
		return EHistoryFailureType::None;
	}

	// OK... or same SHOULD be in the history...
	FMidiClockSongPositionHistory::FScopedItemPeekRef PeekOneAheadRef(ReadCursor.PeekAhead(1));
	while (PeekOneAheadRef && PeekOneAheadRef->SampleCount <= LookingForSampleFrame)
	{
		ReadCursor.PeekAhead(2, PeekOneAheadRef);
		ReadCursor.PeekAhead(1, PeekNextRef);
		ReadCursor.ConsumeNext();
	}

	// Maybe the sample BEFORE our sample is in the history, but the sample AFTER is not, so we can't lerp?
	if (!PeekOneAheadRef)
	{
		return EHistoryFailureType::CaughtUpToRenderPosition;
	}

	check(LookingForSampleFrame >= PeekNextRef->SampleCount && LookingForSampleFrame < PeekOneAheadRef->SampleCount);

	// We now have enough to lerp!
	float LerpAlpha = (float)(LookingForSampleFrame - PeekNextRef->SampleCount) / (float)(PeekOneAheadRef->SampleCount - PeekNextRef->SampleCount);
	SmoothedLocalTick = FMath::Lerp<float>((float)PeekNextRef->UpToTick, (float)PeekOneAheadRef->UpToTick, LerpAlpha);
	SmoothedTempoMapTick = FMath::Lerp<float>((float)PeekNextRef->TempoMapTick, (float)PeekOneAheadRef->TempoMapTick, LerpAlpha);
	CurrentSpeed = PeekNextRef->CurrentSpeed;

	return EHistoryFailureType::None;
}
