// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Harmonix/LocalMinimumMagnitudeTracker.h"
#include "HarmonixMetasound/Components/MusicClockDriverBase.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"
#include "MetasoundGeneratorHandle.h"
#include "UObject/StrongObjectPtr.h"

namespace HarmonixMetasound::Analysis
{
	struct FSongMapChain;
}

class FMetasoundMusicClockDriver : public FMusicClockDriverBase
{
public:
	FMetasoundMusicClockDriver(UObject* WorldContextObj, const FMetasoundMusicClockSettings& Settings)
		: FMusicClockDriverBase(WorldContextObj, Settings) {}

	virtual TOptional<FVector> TryGetAudioSourceLocation() const override;

	virtual bool CalculateSongPosWithOffset(float MsOffset, ECalibratedMusicTimebase Timebase, FMidiSongPos& OutResult) const override;

	virtual void Disconnect() override;
	virtual bool RefreshCurrentSongPos() override;
	virtual void OnStart() override;
	virtual void OnPause() override;
	virtual void OnContinue() override;
	virtual void OnStop() override;
	virtual const ISongMapEvaluator* GetCurrentSongMapEvaluator() const override;
	
	virtual bool LoopedThisFrame(ECalibratedMusicTimebase Timebase) const override;
	virtual bool SeekedThisFrame(ECalibratedMusicTimebase Timebase) const override;

#if !UE_BUILD_SHIPPING
	virtual FString GetDisplayName() const override;
#endif

	bool ConnectToAudioComponentsMetasound(
		UAudioComponent* InAudioComponent,
		FName MetasoundOuputPinName = "MIDI Clock",
		UMetasoundGeneratorHandle::FOnAttached::FDelegate OnGeneratorAttachedCallback = UMetasoundGeneratorHandle::FOnAttached::FDelegate::CreateLambda([](){}),
		UMetasoundGeneratorHandle::FOnDetached::FDelegate OnGeneratorDetachedCallback = UMetasoundGeneratorHandle::FOnDetached::FDelegate::CreateLambda([](){}));

	bool RunPastMusicEnd = false;

protected:
	void OnGeneratorAttached();
	void OnGeneratorDetached();
	void OnGraphSet();
	void OnGeneratorIOUpdatedWithChanges(const TArray<Metasound::FVertexInterfaceChange>& VertexInterfaceChanges);

private:
	FName MetasoundOutputName;
	// We can keep a week reference to this because our "owner" is a UClass and
	// also has a reference to it...
	TWeakObjectPtr<UAudioComponent> AudioComponentToWatch;
	// We need a strong object ptr to this next thing since we will be the only one holding a reference to it...
	TStrongObjectPtr<UMetasoundGeneratorHandle> CurrentGeneratorHandle;
	TWeakPtr<Metasound::FMetasoundGenerator> UnderlyingGenerator;

	Metasound::Frontend::FAnalyzerAddress MidiSongPosAnalyzerAddress;

	UMidiClockUpdateSubsystem::FClockHistoryPtr ClockHistory;
	HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor SmoothedAudioRenderClockHistoryCursor;
	HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor SmoothedPlayerExperienceClockHistoryCursor;
	HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor SmoothedVideoRenderClockHistoryCursor;

	TSharedPtr<const HarmonixMetasound::Analysis::FSongMapChain> CurrentMapChain;
	
	bool bRunning = false;
	double FreeRunStartTimeSecs = 0.0;
	bool WasEverConnected = false;
	float SongPosOffsetMs = 0.0f;
	int32 LastTickSeen = 0;
	float RenderSmoothingLagSeconds = 0.030f;

	double RenderStartWallClockTimeSeconds = 0.0;
	double LastRefreshWallClockTimeSeconds = 0.0;
	double DeltaSecondsBetweenRefreshes = 0.0;

	Metasound::FSampleCount RenderStartSampleCount {0};

	static const int kFramesOfErrorHistory = 10;
	FLocalMinimumMagnitudeTracker<double, kFramesOfErrorHistory> ErrorTracker;
	double SyncSpeed = 1.0;

	struct FPerTimebaseSmoothedClockState
	{
		float TempoMapMs = 0.0f;
		float TempoMapTick = 0.0f;
		float LocalTick = 0.0f;
	};

	FPerTimebaseSmoothedClockState AudioRenderState;
	FPerTimebaseSmoothedClockState PlayerExperienceState;
	FPerTimebaseSmoothedClockState VideoRenderState;

	UMetasoundGeneratorHandle::FOnAttached::FDelegate OnAttachedDelegate;
	UMetasoundGeneratorHandle::FOnDetached::FDelegate OnDetachedDelegate;
	
	FDelegateHandle GeneratorAttachedCallbackHandle;
	FDelegateHandle GeneratorDetachedCallbackHandle;
	FDelegateHandle GeneratorIOUpdatedCallbackHandle;
	FDelegateHandle GraphChangedCallbackHandle;
	
	bool AudioRenderSeekDetected = false;
	bool AudioRenderLoopDetected = false;
	bool PlayerExperiencedSeekDetected = false;
	bool PlayerExperiencedLoopDetected = false;
	bool VideoRenderSeekDetected = false;
	bool VideoRenderLoopDetected = false;

	FMidiSongPos CalculateSongPosAtMsForLoopingOrMonotonicClock(float AbsoluteMs, float& PositionTick, bool& SeekDetected, bool& LoopDetected) const;
	FMidiSongPos CalculateSongPosAtMsForOffsetClock(float PositionMs, float ClockTickOffsetFromDrivingClock, float& PositionTick, bool& SeekDetected) const;
	bool CheckForSeek(float FirstTick, float NextTick, float CurrentTempo, int32 TicksPerQuarter) const;

	void UpdateCurrentTicksForOffsetClock(float SmoothedTick, float SmoothedTempoMapTick);
	void UpdateCurrentTicksForLoopingOrMonotonicClock(float SmoothedTick, float SmoothedTempoMapTick);

	bool AttemptToConnectToAudioComponentsMetasound(
		UAudioComponent* InAudioComponent,
		UMetasoundGeneratorHandle::FOnAttached::FDelegate OnGeneratorAttachedCallback,
		UMetasoundGeneratorHandle::FOnDetached::FDelegate OnGeneratorDetachedCallback);
	void DetachAllCallbacks();
	void RefreshCurrentSongPosFromWallClock();
	void RefreshCurrentSongPosFromHistory();
	// always go through this to change ClockHistory
	void SetClockHistory(const UMidiClockUpdateSubsystem::FClockHistoryPtr&);

	enum class EHistoryFailureType : uint8
	{
		None,
		NotEnoughDataInTheHistoryRing,
		NotEnoughHistory,
		LookingForTimeInTheFutureOfWhatHasEvenRendered,
		CaughtUpToRenderPosition
	};

	EHistoryFailureType CalculateSmoothedTick(Metasound::FSampleCount ExpectedRenderPosSampleCount, Metasound::FSampleCount LastRenderPosSampleCount,
		float& SmoothedTick, float& SmoothedTempoMapTick, float& CurrentSpeed, HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor& ReadCursor,
		float LookBehindSeconds);

	static FString HistoryFailureTypeToString(FMetasoundMusicClockDriver::EHistoryFailureType Error);
};
