// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/Components/MetasoundOfflinePlayerComponent.h"
#include "Components/AudioComponent.h"
#include "Sound/SoundGenerator.h"
#include "MetasoundGenerator.h"
#include "MetasoundGeneratorHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundOfflinePlayerComponent)

UMetasoundOfflinePlayerComponent::UMetasoundOfflinePlayerComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
}

UMetasoundGeneratorHandle* UMetasoundOfflinePlayerComponent::CreateGeneratorBasedOnAudioComponent(UAudioComponent* AudioComponent, int32 InSampleRate, int32 InBlockSize)
{
	if (!AudioComponent || !AudioComponent->GetSound())
	{
		return nullptr;
	}

	MetasoundSource = Cast<UMetaSoundSource>(AudioComponent->GetSound());
	if (!MetasoundSource)
	{
		return nullptr;
	}

	SampleRate = InSampleRate;
	BlockSize = InBlockSize;

	MetasoundSource->InitResources();

	FSoundGeneratorInitParams InitParams;
	InitParams.AudioDeviceID = -1;
	InitParams.AudioMixerNumOutputFrames = BlockSize;
	InitParams.bIsPreviewSound = false;
	InitParams.InstanceID = uint64(this);
	InitParams.AudioComponentId = AudioComponent->GetAudioComponentID();
	InitParams.NumChannels = 1;
	InitParams.NumFramesPerCallback = BlockSize;
	InitParams.SampleRate = SampleRate;

	TArray<FAudioParameter> DefaultParams;

	Generator = MetasoundSource->CreateSoundGenerator(InitParams, TArray<FAudioParameter>());
	if (!Generator)
	{
		return nullptr;
	}
	GeneratorBlockSize = Generator->GetDesiredNumSamplesToRenderPerCallback();

	StartSecond = GetWorld()->GetRealTimeSeconds();
	RenderedSamples = 0;
	ScratchBuffer.SetNum(GeneratorBlockSize);

	Generator->OnBeginGenerate();

	UMetasoundGeneratorHandle* NewHandle = UMetasoundGeneratorHandle::CreateMetaSoundGeneratorHandle(AudioComponent);
	ensureMsgf(NewHandle, TEXT("Failed to create MetasoundGeneratorHandle for offline metasound rendering."));

	return NewHandle;
}

TSharedPtr<Metasound::FMetasoundGeneratorHandle> UMetasoundOfflinePlayerComponent::CreateSharedGeneratorBasedOnAudioComponent(UAudioComponent* AudioComponent, int32 InSampleRate, int32 InBlockSize)
{
	if (!AudioComponent || !AudioComponent->GetSound())
	{
		return nullptr;
	}

	MetasoundSource = Cast<UMetaSoundSource>(AudioComponent->GetSound());
	if (!MetasoundSource)
	{
		return nullptr;
	}

	SampleRate = InSampleRate;
	BlockSize = InBlockSize;

	MetasoundSource->InitResources();

	FSoundGeneratorInitParams InitParams;
	InitParams.AudioDeviceID = -1;
	InitParams.AudioMixerNumOutputFrames = BlockSize;
	InitParams.bIsPreviewSound = false;
	InitParams.InstanceID = uint64(this);
	InitParams.AudioComponentId = AudioComponent->GetAudioComponentID();
	InitParams.NumChannels = 1;
	InitParams.NumFramesPerCallback = BlockSize;
	InitParams.SampleRate = SampleRate;

	TArray<FAudioParameter> DefaultParams;

	
	Generator = MetasoundSource->CreateSoundGenerator(InitParams, TArray<FAudioParameter>());
	if (!Generator)
	{
		return nullptr;
	}
	GeneratorBlockSize = Generator->GetDesiredNumSamplesToRenderPerCallback();
	
	StartSecond = GetWorld()->GetRealTimeSeconds();
	RenderedSamples = 0;
	ScratchBuffer.SetNum(GeneratorBlockSize);

	Generator->OnBeginGenerate();

	TSharedPtr<Metasound::FMetasoundGeneratorHandle> NewHandle = Metasound::FMetasoundGeneratorHandle::Create(AudioComponent);
	ensureMsgf(NewHandle.IsValid(), TEXT("Failed to create MetasoundGeneratorHandle for offline metasound rendering."));

	return NewHandle;
}

void UMetasoundOfflinePlayerComponent::ReleaseGenerator()
{
	if (MetasoundSource && Generator)
	{
		MetasoundSource->OnEndGenerate(Generator);
	}
	MetasoundSource = nullptr;
	Generator = nullptr;
}

void UMetasoundOfflinePlayerComponent::BeginPlay()
{
	Super::BeginPlay();
}


void UMetasoundOfflinePlayerComponent::BeginDestroy()
{
	Super::BeginDestroy();
	ReleaseGenerator();
}

TSharedPtr<Metasound::FMetasoundGenerator> UMetasoundOfflinePlayerComponent::GetGenerator()
{
	return StaticCastSharedPtr<Metasound::FMetasoundGenerator, ISoundGenerator, ESPMode::ThreadSafe>(Generator);
}

void UMetasoundOfflinePlayerComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
	if (!Generator) 
	{
		return;
	}

	const UWorld* World = GetWorld();
	const double CurrentTime = World->GetRealTimeSeconds();

	const double SecondsRendered = (double)RenderedSamples / (double)SampleRate;
	const double SecondsOfRenderingNeeded = CurrentTime - SecondsRendered - StartSecond;
	int32 SamplesNeeded = (int32)(SecondsOfRenderingNeeded * (double)SampleRate);

	while (SamplesNeeded >= GeneratorBlockSize)
	{
		Generator->GetNextBuffer(ScratchBuffer.GetData(), ScratchBuffer.Num(), true);
		SamplesNeeded -= GeneratorBlockSize;
		RenderedSamples += GeneratorBlockSize;
	}
}
