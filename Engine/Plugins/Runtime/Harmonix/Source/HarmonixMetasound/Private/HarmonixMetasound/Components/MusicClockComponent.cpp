// Copyright Epic Games, Inc. All Rights Reserved.
#include "Components/AudioComponent.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "HarmonixMetasound/Components/MetasoundMusicClockDriver.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/Components/MusicTimer.h"
#include "HarmonixMetasound/Components/WallClockMusicClockDriver.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"
#include "HarmonixMidi/MusicTimeSpan.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicClockComponent)

DEFINE_LOG_CATEGORY(LogMusicClock)

UMusicClockComponent::UMusicClockComponent()
{
	MakeDefaultSongMap();
	PrimaryComponentTick.bCanEverTick = false;
	bAutoActivate = false;
	MusicTimerManager = CreateDefaultSubobject<UMusicTimerManager>(TEXT("MusicTimerManager_MusicClockComponent"));
}

TNotNull<UMusicClockComponent*> UMusicClockComponent::CreateMusicClockShared(UObject* WorldContextObject, bool bPersistAcrossLevelTransition)
{
	// Mirroring UGameplayStatics::CreateSound2D / FAudioDevice::CreateComponent:
	// - If we have an owning actor, register with them, use them as the Outer
	// - If not, register with the world, use the transient package as the Outer
	// - But if we want to persist through level transitions, don't register at all

	AActor* OwningActor = nullptr;
	UWorld* World = nullptr;
	if (!bPersistAcrossLevelTransition && WorldContextObject)
	{
		OwningActor = Cast<AActor>(WorldContextObject);
		if (!OwningActor)
		{
			OwningActor = WorldContextObject->GetTypedOuter<AActor>();
		}

		World = WorldContextObject->GetWorld();
	}

	UObject* OuterToUse = OwningActor ? OwningActor : GetTransientPackageAsObject();
	TNotNull<UMusicClockComponent*> NewClock = NewObject<UMusicClockComponent>(OuterToUse);

	if (NewClock->GetOwner())
	{
		NewClock->RegisterComponent();
	}
	else if (World)
	{
		NewClock->RegisterComponentWithWorld(World);
	}

	return NewClock;
}

UMusicClockComponent* UMusicClockComponent::CreateMetasoundDrivenMusicClock(UObject* WorldContextObject, UAudioComponent* InAudioComponent, FName MetasoundOutputPinName, bool Start, bool bPersistAcrossLevelTransition)
{
	TNotNull<UMusicClockComponent*> NewClock = CreateMusicClockShared(WorldContextObject, bPersistAcrossLevelTransition);
	NewClock->DriveMethod = EMusicClockDriveMethod::MetaSound;
	NewClock->MetasoundOutputName = MetasoundOutputPinName;
	NewClock->ConnectToMetasoundOnAudioComponent(InAudioComponent);
	if (Start)
	{
		NewClock->Start();
	}
	return NewClock;
}

UMusicClockComponent* UMusicClockComponent::CreateWallClockDrivenMusicClock(UObject* WorldContextObject, UMidiFile* InTempoMap, bool Start, bool bPersistAcrossLevelTransition)
{
	TNotNull<UMusicClockComponent*> NewClock = CreateMusicClockShared(WorldContextObject, bPersistAcrossLevelTransition);
	NewClock->ConnectToWallClockForMidi(InTempoMap);
	if (Start)
	{
		NewClock->Start();
	}
	return NewClock;
}

UMusicClockComponent* UMusicClockComponent::CreateMusicClockWithSettings(UObject* WorldContextObject, TInstancedStruct<FMusicClockSettingsBase> MusicClockSettings, bool Start, bool bPersistAcrossLevelTransition)
{
	TNotNull<UMusicClockComponent*> NewClock = CreateMusicClockShared(WorldContextObject, bPersistAcrossLevelTransition);

	if (const FMetasoundMusicClockSettings* MetasoundClockSettings = MusicClockSettings.GetPtr<FMetasoundMusicClockSettings>())
	{
		NewClock->DriveMethod = EMusicClockDriveMethod::MetaSound;
		NewClock->DefaultTempo = MetasoundClockSettings->DefaultTempo;
		NewClock->DefaultTimeSignatureNum = MetasoundClockSettings->DefaultTimeSigNumerator;
		NewClock->DefaultTimeSignatureDenom = MetasoundClockSettings->DefaultTimeSigDenominator;
		NewClock->MetasoundOutputName = MetasoundClockSettings->MetasoundOutputName;
		NewClock->ConnectToMetasoundOnAudioComponent(MetasoundClockSettings->AudioComponent);
	}
	else if (const FWallClockMusicClockSettings* WallClockSettings = MusicClockSettings.GetPtr<FWallClockMusicClockSettings>())
	{
		NewClock->DriveMethod = EMusicClockDriveMethod::WallClock;
		NewClock->DefaultTempo = WallClockSettings->DefaultTempo;
		NewClock->DefaultTimeSignatureNum = WallClockSettings->DefaultTimeSigNumerator;
		NewClock->DefaultTimeSignatureDenom = WallClockSettings->DefaultTimeSigDenominator;
		NewClock->ConnectToWallClockForMidi(WallClockSettings->TempoMap);
	}
	else
	{
		NewClock->ConnectToCustomClockWithSettings(MusicClockSettings);
	}
	
	if (Start)
	{
		NewClock->Start();
	}
	return NewClock;
}


bool UMusicClockComponent::ConnectToMetasoundOnAudioComponent(UAudioComponent* InAudioComponent)
{
	DriveMethod = EMusicClockDriveMethod::MetaSound;
	MetasoundsAudioComponent = InAudioComponent;
	return ConnectToMetasound();
}

void UMusicClockComponent::ConnectToWallClockForMidi(UMidiFile* InTempoMap)
{
	DriveMethod = EMusicClockDriveMethod::WallClock;
	TempoMap = InTempoMap;
	ConnectToWallClock();
}

void UMusicClockComponent::ConnectToCustomClockWithSettings(TInstancedStruct<FMusicClockSettingsBase> InMusicClockSettings)
{
	DriveMethod = EMusicClockDriveMethod::Custom;
	MusicClockSettings = InMusicClockSettings;
	ConnectToCustomClock();
}

void UMusicClockComponent::SetDefaultTempo(float TempoBpm)
{
	// Defaults only used to make the tempo map, once before ever running.
	ensure(GetState() == EMusicClockState::Stopped);
	DefaultTempo = TempoBpm;
}

void UMusicClockComponent::SetDefaultTimeSignatureNum(int Num)
{
	ensure(GetState() == EMusicClockState::Stopped);
	DefaultTimeSignatureNum = Num;
}

void UMusicClockComponent::SetDefaultTimeSignatureDenom(int Denom)
{
	ensure(GetState() == EMusicClockState::Stopped);
	DefaultTimeSignatureDenom = Denom;
}

void UMusicClockComponent::UpdateClock()
{
	if (ClockDriver)
	{
		ClockDriver->UpdateClockForGameFrame();
		MusicTimerManager->UpdateForGameFrame(ClockDriver->CurrentSongPos);

		BroadcastSongPosChanges(TimebaseForBarAndBeatEvents);
		BroadcastSeekLoopDetections(ECalibratedMusicTimebase::AudioRenderTime);
		BroadcastSeekLoopDetections(ECalibratedMusicTimebase::ExperiencedTime);
		BroadcastSeekLoopDetections(ECalibratedMusicTimebase::VideoRenderTime);
	}
}

void UMusicClockComponent::CreateClockDriver()
{
	if (DriveMethod == EMusicClockDriveMethod::WallClock)
	{
		ConnectToWallClock();
	}
	else if (DriveMethod == EMusicClockDriveMethod::MetaSound)
	{
		ConnectToMetasound();
	}
	else if (DriveMethod == EMusicClockDriveMethod::Custom)
	{
		ConnectToCustomClock();
	}

	if (!ClockDriver)
	{
		UE_LOGF(LogMusicClock, Error, "Failed to create clock driver as configured for component '%ls', falling back to wall clock", *GetNameSafe(this));
		ConnectToWallClock();
	}
}

bool UMusicClockComponent::ConnectToMetasound()
{
	check(DriveMethod == EMusicClockDriveMethod::MetaSound);
	if (!::IsValid(MetasoundsAudioComponent))
	{
		return false;
	}
	DisconnectFromClockDriver();
	FMetasoundMusicClockSettings MetasoundClockSettings;
	MetasoundClockSettings.AudioComponent = MetasoundsAudioComponent;
	MetasoundClockSettings.MetasoundOutputName = MetasoundOutputName;
	MetasoundClockSettings.DefaultTempo = DefaultTempo;
	MetasoundClockSettings.DefaultTimeSigNumerator = DefaultTimeSignatureNum;
	MetasoundClockSettings.DefaultTimeSigDenominator = DefaultTimeSignatureDenom;
	TSharedPtr<FMetasoundMusicClockDriver> MetasoundClockDriver = MakeShared<FMetasoundMusicClockDriver>(this, MetasoundClockSettings);
	bool Connected = MetasoundClockDriver->ConnectToAudioComponentsMetasound(
		MetasoundsAudioComponent,
		MetasoundOutputName,
		UMetasoundGeneratorHandle::FOnAttached::FDelegate::CreateLambda([this](){ MusicClockConnectedEvent.Broadcast(); }),
		UMetasoundGeneratorHandle::FOnDetached::FDelegate::CreateLambda([this](){ MusicClockDisconnectedEvent.Broadcast(); }));
	MetasoundClockDriver->RunPastMusicEnd = RunPastMusicEnd;
	ClockDriver = MoveTemp(MetasoundClockDriver);
	if (IsActive())
	{
		ClockDriver->Start();
	}
	return Connected;
}

void UMusicClockComponent::ConnectToWallClock()
{
	// we don't 'check' that the driver mode is wall clock here because if the driver mode is metasound and we can't 
	// connect for some reason we will fall back to this clock driver!
	DisconnectFromClockDriver();
	FWallClockMusicClockSettings WallClockSettings;
	WallClockSettings.TempoMap = TempoMap;
	WallClockSettings.DefaultTempo = DefaultTempo;
	WallClockSettings.DefaultTimeSigNumerator = DefaultTimeSignatureNum;
	WallClockSettings.DefaultTimeSigDenominator = DefaultTimeSignatureDenom;
	ClockDriver = MakeShared<FWallClockMusicClockDriver>(this, WallClockSettings);
	if (IsActive())
	{
		ClockDriver->Start();
	}
}

void UMusicClockComponent::ConnectToCustomClock()
{
	check(DriveMethod == EMusicClockDriveMethod::Custom);
	DisconnectFromClockDriver();
	const FMusicClockSettingsBase* Settings = MusicClockSettings.GetPtr<FMusicClockSettingsBase>();
	if (!Settings)
	{
		UE_LOGF(LogMusicClock, Error, "Invalid settings for custom clock driver on component '%ls'", *GetNameSafe(this));
		return;
	}

	ClockDriver = Settings->MakeInstance(this);
	if (!ClockDriver)
	{
		UE_LOGF(LogMusicClock, Error, "Failed to create custom clock driver on component '%ls'", *GetNameSafe(this));
		return;
	}

	if (IsActive())
	{
		ClockDriver->Start();
	}
}

void UMusicClockComponent::DisconnectFromClockDriver()
{
	if (ClockDriver)
	{
		ClockDriver->Disconnect();
		ClockDriver = nullptr;
	}
}

FMidiSongPos UMusicClockComponent::CalculateSongPosWithOffset(float MsOffset, ECalibratedMusicTimebase Timebase) const
{
	FMidiSongPos Result;
	if (ClockDriver)
	{
		if (ClockDriver->CalculateSongPosWithOffset(MsOffset, Timebase, Result))
		{
			return Result;
		}
	}

	// otherwise, use our song maps copy
	Result.SetByTime((GetCurrentSongPosInternal(Timebase).SecondsIncludingCountIn * 1000.0f) + MsOffset, DefaultMaps);
	return Result;
}

const FMidiSongPos& UMusicClockComponent::GetRawUnsmoothedAudioRenderPos() const
{
	static const FMidiSongPos PosZero;
	return ClockDriver ? ClockDriver->GetCurrentRawAudioRenderSongPos() : PosZero;
}

void UMusicClockComponent::Activate(bool bReset)
{
	// cache the value of ShouldActivate before calling the super
	bool bShouldActivate = bReset || ShouldActivate();
	Super::Activate(bReset);

	if (!bShouldActivate)
	{
		return;
	}

	if (bReset && GetState() != EMusicClockState::Stopped)
	{
		if (ClockDriver)
		{
			ClockDriver->Stop();
			DisconnectFromClockDriver();
		}
		PlayStateEvent.Broadcast(EMusicClockState::Stopped);
	}

	if (GetState() == EMusicClockState::Running)
	{
		return;
	}

	MakeDefaultSongMap();

	if (!MetasoundsAudioComponent && GetOwner())
	{
		MetasoundsAudioComponent = GetOwner()->FindComponentByClass<UAudioComponent>();
	}

	if (!ClockDriver)
	{
		CreateClockDriver();
	}

	if (ensure(ClockDriver))
	{
		ClockDriver->Start();
		LastBroadcastBeat = -1;
		LastBroadcastBar = -1;
		UMidiClockUpdateSubsystem::TrackMusicClockComponent(this);
		PlayStateEvent.Broadcast(EMusicClockState::Running);
	}
}

void UMusicClockComponent::Deactivate()
{
	bool bShouldDeactivate = IsActive();
	Super::Deactivate();

	if (!bShouldDeactivate)
	{
		return;
	}

	if (GetState() == EMusicClockState::Stopped)
	{
		return;
	}

	if (ClockDriver)
	{
		ClockDriver->Stop();
	}
	LastBroadcastBeat = -1;
	LastBroadcastBar  = -1;
	UMidiClockUpdateSubsystem::StopTrackingMusicClockComponent(this);
	DisconnectFromClockDriver();
	PlayStateEvent.Broadcast(EMusicClockState::Stopped);
}

void UMusicClockComponent::BeginPlay()
{
	Super::BeginPlay();
}

void UMusicClockComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	UMidiClockUpdateSubsystem::StopTrackingMusicClockComponent(this);
	DisconnectFromClockDriver();
}

void UMusicClockComponent::BeginDestroy()
{
	Super::BeginDestroy();
	UMidiClockUpdateSubsystem::StopTrackingMusicClockComponent(this);
	DisconnectFromClockDriver();
}

void UMusicClockComponent::SetTempoMapForWallClock(UMidiFile* InTempoMap)
{
	ensure(GetState() == EMusicClockState::Stopped);
	TempoMap = InTempoMap;
	if (ClockDriver)
	{
		UE_LOGF(LogMusicClock, Log, "Tempo Maps have changed! Clock requires restart");
	}
}

void UMusicClockComponent::SetRunPastMusicEnd(bool bRunPastMusicEnd)
{
	RunPastMusicEnd = bRunPastMusicEnd;
	if (TSharedPtr<FMetasoundMusicClockDriver> MetasoundClockDriver = StaticCastSharedPtr<FMetasoundMusicClockDriver>(ClockDriver))
	{
		MetasoundClockDriver->RunPastMusicEnd = bRunPastMusicEnd;
	}
}

bool UMusicClockComponent::GetRunPastMusicEnd() const
{
	if (TSharedPtr<FMetasoundMusicClockDriver> MetasoundClockDriver = StaticCastSharedPtr<FMetasoundMusicClockDriver>(ClockDriver))
	{
		return MetasoundClockDriver->RunPastMusicEnd;
	}
	return RunPastMusicEnd;
}

float UMusicClockComponent::GetCurrentTempo() const
{
	if (ClockDriver)
	{
		return ClockDriver->Tempo;
	}
	return 0.0f;
}

float UMusicClockComponent::GetCurrentBeatsPerMinute() const
{
	if (ClockDriver)
	{
		return ClockDriver->GetBeatsPerMinute();
	}
	return 0.0f;
}

void UMusicClockComponent::GetCurrentTimeSignature(int& OutNumerator, int& OutDenominator) const
{
	OutNumerator = 0;
	OutDenominator = 0;
	if (ClockDriver)
	{
		OutNumerator = ClockDriver->TimeSignatureNum;
		OutDenominator = ClockDriver->TimeSignatureDenom;
	}
}

FTimeSignature UMusicClockComponent::GetCurrentTimeSignatureStruct() const
{
	return ClockDriver ? FTimeSignature(ClockDriver->TimeSignatureNum, ClockDriver->TimeSignatureDenom) : FTimeSignature(0, 0);
}

float UMusicClockComponent::GetCurrentBarsPerSecond() const
{
	if (ClockDriver)
	{
		return ClockDriver->CurrentBarsPerSecond;
	}
	return 0;
}

float UMusicClockComponent::GetCurrentSecondsPerBar() const
{
	if (ClockDriver)
	{
		return (ClockDriver->CurrentBarsPerSecond != 0) ? (1.0f / ClockDriver->CurrentBarsPerSecond) : 0;
	}
	return 0;
}

float UMusicClockComponent::GetCurrentBeatsPerSecond() const
{
	if (ClockDriver)
	{
		return ClockDriver->CurrentBeatsPerSecond;
	}
	return 0;
}

float UMusicClockComponent::GetCurrentSecondsPerBeat() const
{
	if (ClockDriver)
	{
		return (ClockDriver->CurrentBeatsPerSecond != 0) ? (1.0f / ClockDriver->CurrentBeatsPerSecond) : 0;
	}
	return 0;
}

float UMusicClockComponent::GetCurrentClockAdvanceRate() const
{
	if (ClockDriver)
	{
		return ClockDriver->CurrentClockAdvanceRate;
	}
	return 0.0f;
}

void UMusicClockComponent::Start()
{
	Activate(false);
}

void UMusicClockComponent::Pause()
{
	if (GetState() != EMusicClockState::Running)
	{
		return;
	}
	ClockDriver->Pause();
	PlayStateEvent.Broadcast(EMusicClockState::Paused);
}

void UMusicClockComponent::Continue()
{
	if (GetState() != EMusicClockState::Paused)
	{
		return;
	}
	ClockDriver->Continue();
	PlayStateEvent.Broadcast(EMusicClockState::Running);
}

void UMusicClockComponent::Stop()
{
	Deactivate();
}

EMusicClockState UMusicClockComponent::GetState() const
{
	if (!ClockDriver)
	{
		return EMusicClockState::Stopped;
	}

	return ClockDriver->GetState();
}


float UMusicClockComponent::GetSecondsIncludingCountIn(ECalibratedMusicTimebase Timebase) const
{
	return GetSongPos(Timebase).SecondsIncludingCountIn;
}

float UMusicClockComponent::GetSecondsFromBarOne(ECalibratedMusicTimebase Timebase) const
{
	return GetSongPos(Timebase).SecondsFromBarOne;
}

float UMusicClockComponent::GetBarsIncludingCountIn(ECalibratedMusicTimebase Timebase) const
{
	return GetSongPos(Timebase).BarsIncludingCountIn;
}

float UMusicClockComponent::GetBeatsIncludingCountIn(ECalibratedMusicTimebase Timebase) const
{
	return GetSongPos(Timebase).BeatsIncludingCountIn;
}

float UMusicClockComponent::GetTicksFromBarOne(ECalibratedMusicTimebase Timebase) const
{
	const float Seconds = GetSecondsFromBarOne(Timebase);
	return GetSongMaps().MsToTick(Seconds * 1000.0f);
}

float UMusicClockComponent::GetTicksIncludingCountIn(ECalibratedMusicTimebase Timebase) const
{
	const float Seconds = GetSecondsIncludingCountIn(Timebase);
	return GetSongMaps().MsToTick(Seconds * 1000.0f);
}

FMusicTimestamp UMusicClockComponent::GetCurrentTimestamp(ECalibratedMusicTimebase Timebase) const
{
	return GetSongPos(Timebase).Timestamp;
}

FString UMusicClockComponent::GetCurrentSectionName(ECalibratedMusicTimebase Timebase) const
{
	const FMidiSongPos& SongPos = GetSongPos(Timebase);
	return SongPos.CurrentSongSection.Name;
}

int32 UMusicClockComponent::GetCurrentSectionIndex(ECalibratedMusicTimebase Timebase) const
{
	const FMidiSongPos& SongPos = GetSongPos(Timebase);
	return GetSongMaps().GetSectionIndexAtTick(SongPos.CurrentSongSection.StartTick);
}

float UMusicClockComponent::GetCurrentSectionStartMs(ECalibratedMusicTimebase Timebase) const
{
	const FMidiSongPos& SongPos = GetSongPos(Timebase);
	return GetSongMaps().TickToMs(SongPos.CurrentSongSection.StartTick);
}

float UMusicClockComponent::GetCurrentSectionLengthMs(ECalibratedMusicTimebase Timebase) const
{
	const FMidiSongPos& SongPos = GetSongPos(Timebase);
	return GetSongMaps().TickToMs(SongPos.CurrentSongSection.LengthTicks);
}

float UMusicClockComponent::GetDistanceFromCurrentBeat(ECalibratedMusicTimebase Timebase) const
{
	return FMath::Fractional(GetSongPos(Timebase).BeatsIncludingCountIn);
}

float UMusicClockComponent::GetDistanceToNextBeat(ECalibratedMusicTimebase Timebase) const
{
	return 1 - GetDistanceFromCurrentBeat(Timebase);
}

float UMusicClockComponent::GetDistanceToClosestBeat(ECalibratedMusicTimebase Timebase) const
{
	return FMath::Min(GetDistanceFromCurrentBeat(Timebase), GetDistanceToNextBeat(Timebase));
}

float UMusicClockComponent::GetDistanceFromCurrentBar(ECalibratedMusicTimebase Timebase) const
{
	return FMath::Fractional(GetSongPos(Timebase).BarsIncludingCountIn);
}

float UMusicClockComponent::GetDistanceToNextBar(ECalibratedMusicTimebase Timebase) const
{
	return 1 - GetDistanceFromCurrentBar(Timebase);
}

float UMusicClockComponent::GetDistanceToClosestBar(ECalibratedMusicTimebase Timebase) const
{
	return FMath::Min(GetDistanceFromCurrentBar(Timebase), GetDistanceToNextBar(Timebase));
}

float UMusicClockComponent::GetDeltaBar(ECalibratedMusicTimebase Timebase) const
{
	return ClockDriver ? ClockDriver->GetDeltaBarF(Timebase) : 0.0f;
}

float UMusicClockComponent::GetDeltaBeat(ECalibratedMusicTimebase Timebase) const
{
	return ClockDriver ? ClockDriver->GetDeltaBeatF(Timebase) : 0.0f;
}

const TArray<FSongSection>& UMusicClockComponent::GetSongSections() const
{
	return GetSongMaps().GetSections();
}

float UMusicClockComponent::GetCountInSeconds() const
{
	return GetSongMaps().GetCountInSeconds();
}

float UMusicClockComponent::TickToMs(float Tick) const
{
	return GetSongMaps().TickToMs(Tick);
}

float UMusicClockComponent::BeatToMs(float Beat) const
{
	return GetSongMaps().GetMsAtBeat(Beat);
}

float UMusicClockComponent::MsToBeat(float Ms) const
{
	// "BeatAt" is one indexed, so we have to subtract one.
	return GetSongMaps().GetFractionalBeatAtMs(Ms) - 1.0f;
}

float UMusicClockComponent::GetMsPerBeatAtMs(float Ms) const
{
	return GetSongMaps().GetMsPerBeatAtMs(Ms);
}

float UMusicClockComponent::GetNumBeatsInBarAtMs(float Ms) const
{
	return GetSongMaps().GetNumBeatsInPulseBarAtMs(Ms);
}

float UMusicClockComponent::GetBeatInBarAtMs(float Ms) const
{
	return GetSongMaps().GetBeatInPulseBarAtMs(Ms);
}

float UMusicClockComponent::BarToMs(float Bar) const
{
	if (const FTimeSignature* TimeSigAtBar = GetSongMaps().GetTimeSignatureAtBar(static_cast<int32>(Bar)))
	{
		return BeatToMs(static_cast<float>(TimeSigAtBar->Numerator) * Bar);
	}

	return 0.0f;
}

float UMusicClockComponent::GetMsPerBarAtMs(float Ms) const
{
	return GetSongMaps().GetMsPerBarAtMs(Ms);
}

FString UMusicClockComponent::GetSectionNameAtMs(float Ms) const
{
	return GetSongMaps().GetSectionNameAtMs(Ms);
}

float UMusicClockComponent::GetSectionLengthMsAtMs(float Ms) const
{
	return GetSongMaps().GetSectionLengthMsAtMs(Ms);
}

float UMusicClockComponent::GetSectionStartMsAtMs(float Ms) const
{
	return GetSongMaps().GetSectionStartMsAtMs(Ms);
}

float UMusicClockComponent::GetSectionEndMsAtMs(float Ms) const
{
	return GetSongMaps().GetSectionEndMsAtMs(Ms);
}

int32 UMusicClockComponent::GetNumSections() const
{
	return GetSongMaps().GetNumSections();
}

float UMusicClockComponent::GetSongLengthMs() const
{
	return GetSongMaps().GetSongLengthMs();
}

float UMusicClockComponent::GetSongLengthBeats() const
{
	return GetSongMaps().GetSongLengthBeats();
}

float UMusicClockComponent::GetSongLengthBars() const
{
	return GetSongMaps().GetSongLengthFractionalBars();
}

float UMusicClockComponent::GetSongRemainingMs(ECalibratedMusicTimebase Timebase) const
{
	const float SongLengthMs = GetSongMaps().GetSongLengthMs();
	return SongLengthMs <= 0.f ? 0.f : SongLengthMs - (GetSongPos(Timebase).SecondsIncludingCountIn * 1000.0f);
}

bool UMusicClockComponent::SeekedThisFrame(ECalibratedMusicTimebase Timebase) const
{
	if (ClockDriver)
	{
		return ClockDriver->SeekedThisFrame(Timebase);
	}
	return false;
}

bool UMusicClockComponent::LoopedThisFrame(ECalibratedMusicTimebase Timebase) const
{
	if (ClockDriver)
	{
		return ClockDriver->LoopedThisFrame(Timebase);
	}
	return false;
}

TOptional<FVector> UMusicClockComponent::TryGetAudioSourceLocation() const
{
	if (ClockDriver)
	{
		return ClockDriver->TryGetAudioSourceLocation();
	}
	return {};
}

const ISongMapEvaluator& UMusicClockComponent::GetSongMaps() const
{
	const ISongMapEvaluator* SongMaps = ClockDriver ? ClockDriver->GetCurrentSongMapEvaluator() : nullptr;
	return SongMaps ? *SongMaps : DefaultMaps;
}

#if !UE_BUILD_SHIPPING
FString UMusicClockComponent::GetDisplayName() const
{
	if (!DisplayName.IsEmpty())
	{
		return DisplayName;
	}
	if (ClockDriver)
	{
		return ClockDriver->GetDisplayName();
	}
	return FString::Printf(TEXT("Driverless: %s %s"), *GetNameSafe(this), *GetNameSafe(GetOwner()));
}

void UMusicClockComponent::SetDisplayName(FString Name)
{
	DisplayName = Name;
}
#endif

FMusicTimerHandle UMusicClockComponent::BP_AddTimer(const FMusicTimeInterval& TimeInterval, TOptional<FMusicTimestamp> StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FOnMusicalTimerExecute TimerDelegate)
{
	if (!StartTime.IsSet())
	{
		StartTime = Quantize(GetCurrentTimestamp(Timebase), TimeInterval.Interval, EMidiFileQuantizeDirection::Down);
	}

	return MusicTimerManager->BP_AddTimer(TimeInterval, StartTime.GetValue(), Timebase, bLooping, TimerDelegate);
}

FMusicTimerHandle UMusicClockComponent::AddTimerNative(const FMusicTimeInterval& TimeInterval, TOptional<FMusicTimestamp> StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FSimpleDelegate TimerDelegate)
{
	if (!StartTime.IsSet())
	{
		StartTime = Quantize(GetCurrentTimestamp(Timebase), TimeInterval.Interval, EMidiFileQuantizeDirection::Down);
	}

	return MusicTimerManager->AddTimerNative(TimeInterval, StartTime.GetValue(), Timebase, bLooping, TimerDelegate);
}

void UMusicClockComponent::RemoveTimer(const FMusicTimerHandle& Handle)
{
	MusicTimerManager->RemoveTimer(Handle);
}

void UMusicClockComponent::PauseTimer(const FMusicTimerHandle& Handle, bool bPause)
{
	MusicTimerManager->PauseTimer(Handle, bPause);
}

FMusicTimestamp UMusicClockComponent::Quantize(const FMusicTimestamp& Time, EMidiClockSubdivisionQuantization QuantizationInterval, EMidiFileQuantizeDirection InDirection) const
{
	const ISongMapEvaluator& MapEvaluator = GetSongMaps();
	float Tick = MapEvaluator.MusicTimestampToTick(Time);
	int32 QuantizedTick = MapEvaluator.QuantizeTickToNearestSubdivision(FMath::FloorToInt32(Tick), InDirection, QuantizationInterval);
	return MapEvaluator.TickToMusicTimestamp(QuantizedTick);
}

const FMidiSongPos& UMusicClockComponent::GetSongPos(ECalibratedMusicTimebase Timebase) const
{
	return GetCurrentSongPosInternal(Timebase);
}

const FMidiSongPos& UMusicClockComponent::GetPreviousSongPos(ECalibratedMusicTimebase Timebase) const
{
	return GetPreviousSongPosInternal(Timebase);
}

FMidiSongPos UMusicClockComponent::GetCurrentSmoothedAudioRenderSongPos() const
{
	return GetSongPos(ECalibratedMusicTimebase::AudioRenderTime);
}

FMidiSongPos UMusicClockComponent::GetPreviousSmoothedAudioRenderSongPos() const
{
	return GetPreviousSongPos(ECalibratedMusicTimebase::AudioRenderTime);
}

FMidiSongPos UMusicClockComponent::GetCurrentVideoRenderSongPos() const
{
	return GetSongPos(ECalibratedMusicTimebase::VideoRenderTime);
}

FMidiSongPos UMusicClockComponent::GetPreviousVideoRenderSongPos() const
{
	return GetPreviousSongPos(ECalibratedMusicTimebase::VideoRenderTime);
}

FMidiSongPos UMusicClockComponent::GetCurrentPlayerExperiencedSongPos() const
{
	return GetSongPos(ECalibratedMusicTimebase::ExperiencedTime);
}

FMidiSongPos UMusicClockComponent::GetPreviousPlayerExperiencedSongPos() const
{
	return GetPreviousSongPos(ECalibratedMusicTimebase::ExperiencedTime);
}

FMidiSongPos UMusicClockComponent::GetCurrentRawAudioRenderSongPos() const
{
	return GetSongPos(ECalibratedMusicTimebase::RawAudioRenderTime);
}

float UMusicClockComponent::MeasureSpanProgress(const FMusicalTimeSpan& Span, ECalibratedMusicTimebase Timebase) const
{
	return Span.CalcPositionInSpan(GetCurrentSongPosInternal(Timebase), GetSongMaps());
}

void UMusicClockComponent::BroadcastSongPosChanges(ECalibratedMusicTimebase Timebase)
{
	const FMidiSongPos& Basis = GetCurrentSongPosInternal(Timebase);

	// The basis for a clock can be invalid while the clock is still spinning up
	// (connecting to a metasound)
	// so check to make sure it's valid
	// otherwise we'll be broadcasting bar and beat events that don't make any sense
	// although this can mean the first Bar/Beat events will come in a fraction of a second late
	if (!Basis.IsValid())
	{
		return;
	}
	
	const int32 CurrBar = FMath::FloorToInt32(Basis.BarsIncludingCountIn);
	if (LastBroadcastBar != CurrBar)
	{
		BarEvent.Broadcast(Basis.Timestamp.Bar);
		LastBroadcastBar = CurrBar;
	}
	const int32 CurrBeat = FMath::FloorToInt32(Basis.BeatsIncludingCountIn);
	if (LastBroadcastBeat != CurrBeat)
	{
		BeatEvent.Broadcast(CurrBeat, FMath::FloorToInt32(Basis.Timestamp.Beat));
		LastBroadcastBeat = CurrBeat;
	}
	const FSongSection& SongSection = Basis.CurrentSongSection;
	if (LastBroadcastSongSection.StartTick != SongSection.StartTick || LastBroadcastSongSection.LengthTicks != SongSection.LengthTicks)
	{
		SectionEvent.Broadcast(SongSection.Name, SongSection.StartTick, SongSection.LengthTicks);
		LastBroadcastSongSection = FSongSection(SongSection.Name, SongSection.StartTick, SongSection.LengthTicks);
	}
}

const FMusicTimeDiscontinuityEvent* UMusicClockComponent::GetMusicTimeDiscontinuityEventInternal(ECalibratedMusicTimebase Timebase) const
{
	switch (Timebase)
	{
	case ECalibratedMusicTimebase::AudioRenderTime:
		return &AudioRenderMusicTimeDiscontinuityEvent;
	case ECalibratedMusicTimebase::ExperiencedTime:
		return &PlayerExperienceMusicTimeDiscontinuityEvent;
	case ECalibratedMusicTimebase::VideoRenderTime:
		return &VideoRenderMusicTimeDiscontinuityEvent;
	default:
		return nullptr;
	}
}

void UMusicClockComponent::BroadcastSeekLoopDetections(ECalibratedMusicTimebase Timebase) const
{
	if (!ClockDriver)
	{
		return;
	}
	
	if (!SeekedThisFrame(Timebase) && !LoopedThisFrame(Timebase))
	{
		return;
	}
	
	if (const FMusicTimeDiscontinuityEvent* Event = GetMusicTimeDiscontinuityEventInternal(Timebase))
	{
		const FMidiSongPos& PrevSongPos = GetPreviousSongPosInternal(Timebase);
		const FMidiSongPos& CurrentSongPos = GetCurrentSongPosInternal(Timebase);

		if (SeekedThisFrame(Timebase))
		{
			Event->Broadcast(EMusicTimeDiscontinuityType::Seek, PrevSongPos, CurrentSongPos);
		}
		if (LoopedThisFrame(Timebase))
		{
			Event->Broadcast(EMusicTimeDiscontinuityType::Loop, PrevSongPos, CurrentSongPos);
		}
	}
}

void UMusicClockComponent::MakeDefaultSongMap()
{
	DefaultMaps.EmptyAllMaps();
	DefaultMaps.Init(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt);
	DefaultMaps.GetTempoMap().AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(DefaultTempo), 0);
	DefaultMaps.GetBarMap().AddTimeSignatureAtBarIncludingCountIn(0, DefaultTimeSignatureNum, DefaultTimeSignatureDenom);
}

const FMidiSongPos& UMusicClockComponent::GetCurrentSongPosInternal(ECalibratedMusicTimebase Timebase) const
{
	if (ClockDriver)
	{
		return ClockDriver->GetCurrentSongPos(Timebase);
	}
	static const FMidiSongPos PosZero;
	return PosZero;
}

const FMidiSongPos& UMusicClockComponent::GetPreviousSongPosInternal(ECalibratedMusicTimebase Timebase) const
{
	if (ClockDriver)
	{
		return ClockDriver->GetPreviousSongPos(Timebase);
	}

	static const FMidiSongPos PosZero;
	return PosZero;
}

float UMusicClockComponent::GetPositionSeconds() const
{
	if (!ClockDriver)
	{
		return 0.0f;
	}

	const FMidiSongPos& SongPos = ClockDriver->GetCurrentSongPos(ECalibratedMusicTimebase::VideoRenderTime);
	return SongPos.SecondsIncludingCountIn;
}

FMusicalTime UMusicClockComponent::GetPositionMusicalTime() const
{
	FMusicalTime Result;
	if (!ClockDriver)
	{
		return Result;
	}
	
	const FMidiSongPos& SongPos = ClockDriver->GetCurrentSongPos(ECalibratedMusicTimebase::VideoRenderTime);
	if (SongPos.TimeSigDenominator < 1)
	{
		return Result;
	}

	Result.Bar = SongPos.Timestamp.Bar - 1;
	Result.TicksPerBeat = (MusicalTime::TicksPerQuarterNote * 4 / SongPos.TimeSigDenominator);
	Result.TicksPerBar = Result.TicksPerBeat * SongPos.TimeSigNumerator;
	Result.TickInBar = static_cast<int32>((SongPos.Timestamp.Beat - 1.0f) * static_cast<float>(Result.TicksPerBeat));
	return Result;
}

int32 UMusicClockComponent::GetPositionAbsoluteTick() const
{
	// Note: Since this is an override of a function in the music environment system we need to
	// be sure we are using its standard ticks per quarter note units.
	const ISongMapEvaluator& MapEvaluator = GetSongMaps();
	float CurrentTick = MapEvaluator.FractionalBarIncludingCountInToTick(GetCurrentSongPosInternal(ECalibratedMusicTimebase::VideoRenderTime).BarsIncludingCountIn);
	return FMath::FloorToInt32((CurrentTick / static_cast<float>(MapEvaluator.GetTicksPerQuarterNote())) * static_cast<float>(MusicalTime::TicksPerQuarterNote));
}


FMusicalTime UMusicClockComponent::GetPositionMusicalTime(const FMusicalTime& SourceSpaceOffset) const
{
	FMusicalTime Result;
	const ISongMapEvaluator& MapEvaluator = GetSongMaps();
	float CurrentTick = MapEvaluator.FractionalBarIncludingCountInToTick(GetCurrentSongPosInternal(ECalibratedMusicTimebase::VideoRenderTime).BarsIncludingCountIn);
	
	float OffsetTick = MapEvaluator.FractionalBarIncludingCountInToTick(static_cast<float>(SourceSpaceOffset.FractionalBar()));
	float AdjustedTick = CurrentTick - OffsetTick;

	const FTimeSignature* TimeSignature = MapEvaluator.GetTimeSignatureAtTick(FMath::FloorToInt32(AdjustedTick));
	FMusicTimestamp Timestamp = MapEvaluator.TickToMusicTimestamp(AdjustedTick);
	Result.Bar = Timestamp.Bar - 1;
	Result.TicksPerBeat = (MusicalTime::TicksPerQuarterNote * 4 / TimeSignature->Denominator);
	Result.TicksPerBar = Result.TicksPerBeat * TimeSignature->Numerator;
	Result.TickInBar = static_cast<int32>((Timestamp.Beat - 1.0f) * static_cast<float>(Result.TicksPerBeat));
	return Result;
}

int32 UMusicClockComponent::GetPositionAbsoluteTick(const FMusicalTime& SourceSpaceOffset) const
{
	// Note: Since this is an override of a function in the music environment system we need to
	// be sure we are using its standard ticks per quarter note units.
	const ISongMapEvaluator& MapEvaluator = GetSongMaps();
	float CurrentTick = MapEvaluator.FractionalBarIncludingCountInToTick(GetCurrentSongPosInternal(ECalibratedMusicTimebase::VideoRenderTime).BarsIncludingCountIn);
	float OffsetTick = MapEvaluator.FractionalBarIncludingCountInToTick(static_cast<float>(SourceSpaceOffset.FractionalBar()));
	float AdjustedTick = CurrentTick - OffsetTick;
	return FMath::FloorToInt32((AdjustedTick / static_cast<float>(MapEvaluator.GetTicksPerQuarterNote())) * static_cast<float>(MusicalTime::TicksPerQuarterNote));
}

FMusicalTime UMusicClockComponent::Quantize(const FMusicalTime& MusicalTime, int32 QuantizationInterval, UFrameBasedMusicMap::EQuantizeDirection Direction) const
{
	FMusicalTime Result;
	const ISongMapEvaluator& MapEvaluator = GetSongMaps();
	float Tick = MapEvaluator.FractionalBarIncludingCountInToTick(static_cast<float>(MusicalTime.FractionalBar()));
	int32 QuantizedTick = MapEvaluator.QuantizeTickToNearestSubdivision(FMath::FloorToInt32(Tick), (EMidiFileQuantizeDirection)Direction, (EMidiClockSubdivisionQuantization) FrameBasedMusicMap::QuantiazationIntervalToQuartz(QuantizationInterval));
	FMusicTimestamp Timestamp = MapEvaluator.TickToMusicTimestamp(QuantizedTick);
	const FTimeSignature* TimeSignature = MapEvaluator.GetTimeSignatureAtTick(QuantizedTick);
	check(TimeSignature);
	Result.Bar = Timestamp.Bar - 1;
	Result.TicksPerBeat = (MusicalTime::TicksPerQuarterNote * 4 / TimeSignature->Denominator);
	Result.TicksPerBar = Result.TicksPerBeat * TimeSignature->Numerator;
	Result.TickInBar = static_cast<int32>((Timestamp.Beat - 1.0f) * static_cast<float>(Result.TicksPerBeat));
	return Result;
}