// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Components/MusicClockDriverBase.h"
#include "Engine/World.h"

void FMusicClockDriverBase::ResetDefaultSongMaps(float DefaultTempo, int32 DefaultTimeSigNum, int32 DefaultTimeSigDenom)
{
	DefaultMaps.EmptyAllMaps();
	DefaultMaps.Init(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt);
	DefaultMaps.GetTempoMap().AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(DefaultTempo), 0);
	DefaultMaps.GetBarMap().AddTimeSignatureAtBarIncludingCountIn(0, DefaultTimeSigNum, DefaultTimeSigDenom);
}

void FMusicClockDriverBase::UpdateBeatsAndBarsPerSecond()
{
	// Midi quarter note to beat conversion.
	const float BPM = GetBeatsPerMinute();
	CurrentBeatsPerSecond = (BPM / 60.0) * CurrentClockAdvanceRate;
	CurrentBarsPerSecond = (TimeSignatureNum > 0) ? (CurrentBeatsPerSecond / TimeSignatureNum) : 0;
}

FMusicClockDriverBase::FMusicClockDriverBase(UObject* WorldContextObject, const FMusicClockSettingsBase& Settings)
{
	ResetDefaultSongMaps(Settings.DefaultTempo, Settings.DefaultTimeSigNumerator, Settings.DefaultTimeSigDenominator);
	ContextObj = TWeakObjectPtr<UObject>(WorldContextObject);
}

FMusicClockDriverBase::~FMusicClockDriverBase()
{
}

bool FMusicClockDriverBase::CalculateSongPosWithOffset(float MsOffset, ECalibratedMusicTimebase Timebase, FMidiSongPos& OutResult) const
{
	const ISongMapEvaluator* SongMapEvaluator = GetCurrentSongMapEvaluator();
	if (!SongMapEvaluator)
	{
		return false;
	}

	const FMidiSongPos& SongPos = GetCurrentSongPos(Timebase);
	if (!SongPos.IsValid())
	{
		return false;
	}

	OutResult.SetByTick((SongPos.SecondsIncludingCountIn * 1000.0f) + MsOffset, *SongMapEvaluator);
	return true;
}

double FMusicClockDriverBase::GetWallClockTime() const
{
	if (TStrongObjectPtr<UObject> Obj = ContextObj.Pin())
	{
		if (Obj->GetWorld())
		{
			return Obj->GetWorld()->GetRealTimeSeconds();
		}
	}

	if (GWorld)
	{
		return GWorld->GetRealTimeSeconds();
	}
	
	return 0.0;
}


const FMidiSongPos& FMusicClockDriverBase::GetCurrentSongPos(ECalibratedMusicTimebase Timebase) const
{
	return CurrentSongPos[GetArrayIndexForTimebase(Timebase)];
}

const FMidiSongPos& FMusicClockDriverBase::GetPreviousSongPos(ECalibratedMusicTimebase Timebase) const
{
	return PrevSongPos[GetArrayIndexForTimebase(Timebase)];
}

float FMusicClockDriverBase::GetDeltaBarF(ECalibratedMusicTimebase Timebase) const
{
	return DeltaBarF[GetArrayIndexForTimebase(Timebase)];
}

float FMusicClockDriverBase::GetDeltaBeatF(ECalibratedMusicTimebase Timebase) const
{
	return DeltaBeatF[GetArrayIndexForTimebase(Timebase)];
}

void FMusicClockDriverBase::Start()
{
	OnStart();

	for (ECalibratedMusicTimebase Timebase : TEnumRange<ECalibratedMusicTimebase>())
	{
		CurrentSongPos[(int32)Timebase].Reset();
	}

	MusicClockState = EMusicClockState::Running;
}

void FMusicClockDriverBase::Continue()
{
	if (MusicClockState != EMusicClockState::Paused)
	{
		return;
	}
	
	OnContinue();
	MusicClockState = EMusicClockState::Running;
}

void FMusicClockDriverBase::Pause()
{
	if (MusicClockState != EMusicClockState::Running)
	{
		return;
	}

	OnPause();
	MusicClockState = EMusicClockState::Paused;
}

void FMusicClockDriverBase::Stop()
{
	OnStop();
	MusicClockState = EMusicClockState::Stopped;

	for (ECalibratedMusicTimebase Timebase : TEnumRange<ECalibratedMusicTimebase>())
	{
		CurrentSongPos[(int32)Timebase].Reset();
	}
}

void FMusicClockDriverBase::UpdateMusicPlaybackRate(float InTempo, float InClockAdvanceRate, int32 InTimeSigNum, int32 InTimeSigDenom)
{
	bool bHasAnyTimingChanges = false;
	if (!FMath::IsNearlyEqual(Tempo, InTempo))
	{
		Tempo = InTempo;
		bHasAnyTimingChanges = true;
	}

	if (!FMath::IsNearlyEqual(CurrentClockAdvanceRate, InClockAdvanceRate))
	{
		CurrentClockAdvanceRate = InClockAdvanceRate;
		bHasAnyTimingChanges = true;
	}
	
	if (TimeSignatureNum != InTimeSigNum || TimeSignatureDenom != InTimeSigDenom)
	{
		TimeSignatureNum = InTimeSigNum;
		TimeSignatureDenom = InTimeSigDenom;
		bHasAnyTimingChanges = true;
	}

	// only update bar/beat durations if there have been any other timing changes
	if (bHasAnyTimingChanges)
	{
		UpdateBeatsAndBarsPerSecond();
	}
}

float FMusicClockDriverBase::GetBeatsPerMinute() const
{
	// if we have a tempo set, we should have a time signature.
	ensure(TimeSignatureDenom != 0 || Tempo == 0);
    // Tempo is MidiTempo, which is in quarter notes per minute.
	return Tempo * TimeSignatureDenom / 4.0f; 
}

void FMusicClockDriverBase::UpdateClockForGameFrame()
{
	//	Not for use outside the game thread.
	if (ensureMsgf(
		IsInGameThread(),
		TEXT("%hs called from non-game thread.  This is not supported!"), __FUNCTION__) == false)
	{
		return;
	}

	if (GFrameCounter == LastUpdateFrame)
	{
		return;
	}

	//	Run the actual clock update.
	if (GetState() == EMusicClockState::Running)
	{
		for (ECalibratedMusicTimebase Timebase : TEnumRange<ECalibratedMusicTimebase>())
		{
			PrevSongPos[(int32)Timebase] = CurrentSongPos[(int32)Timebase];
		}

		if (RefreshCurrentSongPos())
		{
			for (ECalibratedMusicTimebase Timebase : TEnumRange<ECalibratedMusicTimebase>())
			{
				DeltaBarF[(int32)Timebase] = CurrentSongPos[(int32)Timebase].BarsIncludingCountIn - PrevSongPos[(int32)Timebase].BarsIncludingCountIn;
				DeltaBeatF[(int32)Timebase] = CurrentSongPos[(int32)Timebase].BeatsIncludingCountIn - PrevSongPos[(int32)Timebase].BeatsIncludingCountIn;
			}

			LastUpdateFrame = GFrameCounter;
		}
	}
}

int32 FMusicClockDriverBase::GetArrayIndexForTimebase(ECalibratedMusicTimebase Timebase) const
{
	int32 TimebaseIndex = (int32)Timebase;
	if (!ensure(TimebaseIndex >= 0 && TimebaseIndex < (int32)ECalibratedMusicTimebase::Count))
	{
		// minor robustness, return something even if there's an illegal value.
		return (int32)ECalibratedMusicTimebase::VideoRenderTime;
	}

	return TimebaseIndex;
}

void FMusicClockDriverBase::SetCurrentSongPos(ECalibratedMusicTimebase Timebase, const FMidiSongPos& InSongPos)
{
	CurrentSongPos[GetArrayIndexForTimebase(Timebase)] = InSongPos;
}

void FMusicClockDriverBase::SetCurrentSongPosByTime(ECalibratedMusicTimebase Timebase, float InElapsedMs, const ISongMapEvaluator& Map)
{
	CurrentSongPos[GetArrayIndexForTimebase(Timebase)].SetByTime(InElapsedMs, Map);
}

void FMusicClockDriverBase::SetCurrentSongPosByTick(ECalibratedMusicTimebase Timebase, float InTick, const ISongMapEvaluator& Map)
{
	CurrentSongPos[GetArrayIndexForTimebase(Timebase)].SetByTick(InTick, Map);
}
