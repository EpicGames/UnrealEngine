// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Components/MusicClockTypes.h"
#include "HarmonixMetasound/Components/WallClockMusicClockDriver.h"
#include "HarmonixMetasound/Components/MetasoundMusicClockDriver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicClockTypes)

TSharedPtr<FMusicClockDriverBase> FWallClockMusicClockSettings::MakeInstance(UObject* WorldContextObj) const
{
	return MakeShared<FWallClockMusicClockDriver>(WorldContextObj, *this);
}

TSharedPtr<FMusicClockDriverBase> FMetasoundMusicClockSettings::MakeInstance(UObject* WorldContextObj) const
{
	return MakeShared<FMetasoundMusicClockDriver>(WorldContextObj, *this);
}
