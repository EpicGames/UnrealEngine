// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Components/MusicTempometerComponent.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicTempometerComponent)

UMusicTempometerComponent::UMusicTempometerComponent()
{
	SetTickGroup(TG_PrePhysics);

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	PrimaryComponentTick.SetPriorityIncludingPrerequisites(true);
}

void UMusicTempometerComponent::PostLoad()
{
	Super::PostLoad();

	// Fix up deprecated properties
	FString DeprecatedErroneousNameString = GetName();
	DeprecatedErroneousNameString.RemoveFromEnd(TEXT("_GEN_VARIABLE"));
	FName DeprecatedErroneousName = *DeprecatedErroneousNameString;

	if (UMusicTempometerComponent* DefaultTempometer = GetClass()->GetDefaultObject<UMusicTempometerComponent>())
	{
		// If the CDO's parameter names and these parameter names don't match, they've already been edited.
		// Don't overwrite with the deprecated names.
		if (MPCParameters.CurrentFrameParameterNames != DefaultTempometer->MPCParameters.CurrentFrameParameterNames)
		{
			return;
		}
	}

	if (SecondsIncludingCountInParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.SecondsIncludingCountInParameterName = SecondsIncludingCountInParameterName_DEPRECATED;
		SecondsIncludingCountInParameterName_DEPRECATED = DeprecatedErroneousName;
	}

	if (BarsIncludingCountInParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.BarsIncludingCountInParameterName = BarsIncludingCountInParameterName_DEPRECATED;
		BarsIncludingCountInParameterName_DEPRECATED = DeprecatedErroneousName;
	}
	
	if (BeatsIncludingCountInParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.BeatsIncludingCountInParameterName = BeatsIncludingCountInParameterName_DEPRECATED;
		BeatsIncludingCountInParameterName_DEPRECATED = DeprecatedErroneousName;
	}
	
	if (SecondsFromBarOneParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.SecondsFromBarOneParameterName = SecondsFromBarOneParameterName_DEPRECATED;
		SecondsFromBarOneParameterName_DEPRECATED = DeprecatedErroneousName;
	}
	
	if (TimestampBarParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.TimestampBarParameterName = TimestampBarParameterName_DEPRECATED;
		TimestampBarParameterName_DEPRECATED = DeprecatedErroneousName;
	}

	if (TimestampBeatInBarParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.TimestampBeatInBarParameterName = TimestampBeatInBarParameterName_DEPRECATED;
		TimestampBeatInBarParameterName_DEPRECATED = DeprecatedErroneousName;
	}

	if (BarProgressParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.BarProgressParameterName = BarProgressParameterName_DEPRECATED;
		BarProgressParameterName_DEPRECATED = DeprecatedErroneousName;
	}

	if (BeatProgressParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.BeatProgressParameterName = BeatProgressParameterName_DEPRECATED;
		BeatProgressParameterName_DEPRECATED = DeprecatedErroneousName;
	}

	if (TimeSignatureNumeratorParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.TimeSignatureNumeratorParameterName = TimeSignatureNumeratorParameterName_DEPRECATED;
		TimeSignatureNumeratorParameterName_DEPRECATED = DeprecatedErroneousName;
	}

	if (TimeSignatureDenominatorParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.TimeSignatureDenominatorParameterName = TimeSignatureDenominatorParameterName_DEPRECATED;
		TimeSignatureDenominatorParameterName_DEPRECATED = DeprecatedErroneousName;
	}

	if (TempoParameterName_DEPRECATED != DeprecatedErroneousName)
	{
		MPCParameters.CurrentFrameParameterNames.TempoParameterName = TempoParameterName_DEPRECATED;
		TempoParameterName_DEPRECATED = DeprecatedErroneousName;
	}
}

void UMusicTempometerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (MaterialParameterCollection)
	{
		UpdateCachedSongPosIfNeeded();
	}
	else
	{
		SetComponentTickEnabled(false);
	}
}

#if WITH_EDITOR
void UMusicTempometerComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMusicTempometerComponent, MaterialParameterCollection))
	{
		SetComponentTickEnabled(MaterialParameterCollection != nullptr);
	}
}
#endif // WITH_EDITOR

void UMusicTempometerComponent::UpdateCachedSongPos() const
{
	LastFrameCounter = GFrameCounter;

	PreviousFrameSongPos = SongPos;

	SongPos = MusicTempometerUtilities::UpdateMaterialParameterCollectionFromClock(GetMutableClockNoMutex(), PreviousFrameSongPos, MaterialParameterCollection, MPCParameters, MaterialParameterCollectionInstance);
}

void UMusicTempometerComponent::UpdateCachedSongPosIfNeeded() const
{
	FScopeLock lock(&SongPosUpdateMutex);
	if (GFrameCounter != LastFrameCounter)
	{
		UpdateCachedSongPos();
	}
}

UMusicClockComponent* UMusicTempometerComponent::FindClock(AActor* Actor) const
{
	UMusicClockComponent* FoundClock = nullptr;
	for (UActorComponent* Component : Actor->GetComponents())
	{
		FoundClock = Cast<UMusicClockComponent>(Component);
		if (FoundClock)
		{
			break;
		}
	}
	return FoundClock;
}

void UMusicTempometerComponent::SetOwnerClock() const
{
	MusicClock = FindClock(GetOwner());
}

