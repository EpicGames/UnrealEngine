// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/Components/MusicTimer.h"
#include "HarmonixMetasound/Components/MusicTimerHandle.h"
#include "FrameBasedMusicMap.h"

DEFINE_LOG_CATEGORY(LogMusicTimer);

FMusicTimer::FMusicTimer(uint64 InUniqueIdentifier, const FMusicTimeInterval& InTimeInterval, const FMusicTimestamp& InStartTime, ECalibratedMusicTimebase InTimebase, bool InbLooping, FSimpleDelegate InDelegate)
	: Timebase(InTimebase)
	, Delegate(InDelegate)
	, TimeInterval(InTimeInterval)
	, StartTime(InStartTime)
	, UniqueIdentifier(InUniqueIdentifier)
	, bLooping(InbLooping)
{
}

uint64 FMusicTimer::GetUniqueIdentifier() const
{
	return UniqueIdentifier;
}

void FMusicTimer::Reset()
{
	StartTime.Reset();
	PausedTime.Reset();
	UniqueIdentifier = 0;
	bLooping = false;
}

bool FMusicTimer::Update(const FMusicTimestamp& CurrentMusicalTimestamp, const FTimeSignature& TimeSignature)
{
	FMusicTimestamp EndTimestamp = StartTime;
	Harmonix::IncrementTimestampByInterval(EndTimestamp, TimeInterval, TimeSignature);

	FMusicTimestamp OffsetTimeStamp = EndTimestamp;
	Harmonix::IncrementTimestampByOffset(OffsetTimeStamp, TimeInterval, TimeSignature);

	if (CurrentMusicalTimestamp >= OffsetTimeStamp)
	{
		Delegate.ExecuteIfBound();
		if (IsLooping())
		{
			// Make sure the timer start position is old position + delay. Do not use current time as we may have overshot. 
			StartTime = EndTimestamp;
		}
		else
		{
			// We are done, remove timer.
			return true;
		}
	}

	return false;
}