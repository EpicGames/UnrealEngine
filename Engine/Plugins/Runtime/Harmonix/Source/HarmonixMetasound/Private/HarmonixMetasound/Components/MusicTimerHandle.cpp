// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Components/MusicTimerHandle.h"

FMusicTimerHandle::FMusicTimerHandle(uint64 InHandle)
	: Handle(InHandle)
{
}

bool FMusicTimerHandle::IsValid() const
{
	return Handle != 0;
}

void FMusicTimerHandle::Invalidate()
{
	Handle = 0;
}

bool FMusicTimerHandle::operator==(const FMusicTimerHandle& Other) const
{
	return Handle == Other.Handle;
}

bool FMusicTimerHandle::operator!=(const FMusicTimerHandle& Other) const
{
	return Handle != Other.Handle;
}

FString FMusicTimerHandle::ToString() const
{
	return FString::Printf(TEXT("%llu"), Handle);
}

uint64 FMusicTimerHandle::GetUniqueIdentifier() const
{
	return Handle;
}

uint32 GetTypeHash(const FMusicTimerHandle& InHandle)
{
	return GetTypeHash(InHandle.Handle);
}