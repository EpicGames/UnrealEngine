// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/Components/MusicTimerManager.h"
#include "HarmonixMetasound/Components/MusicTimerHandle.h"

DEFINE_LOG_CATEGORY(LogMusicTimerManager);

std::atomic<uint64> UMusicTimerManager::NextTimerId = 1;

FMusicTimerHandle UMusicTimerManager::BP_AddTimer(const FMusicTimeInterval& TimeInterval, FMusicTimestamp StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FOnMusicalTimerExecute TimerDelegate)
{
	auto Lambda = TDelegate<void(void)>::CreateLambda([TimerDelegate]()
	{
		TimerDelegate.ExecuteIfBound();
	});

	return AddTimerNative(TimeInterval, StartTime, Timebase, bLooping, Lambda);
}

FMusicTimerHandle UMusicTimerManager::AddTimerNative(const FMusicTimeInterval& TimeInterval, FMusicTimestamp StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FSimpleDelegate TimerDelegate)
{
	FMusicTimerHandle Handle(GetUniqueTimerId());
	PendingTimers.Add(MakeUnique<FMusicTimer>(Handle.GetUniqueIdentifier(), TimeInterval, StartTime, Timebase, bLooping, TimerDelegate));
	return Handle;
}

void UMusicTimerManager::UpdateForGameFrame(const TStaticArray<FMidiSongPos, (int32)ECalibratedMusicTimebase::Count>& CurrentSongPositions)
{
	if (GFrameCounter == LastUpdateFrame)
	{
		return;
	}

	ActiveTimers.Append(MoveTemp(PendingTimers));
	PendingTimers.Empty();

	for (auto Iter = ActiveTimers.CreateIterator(); Iter; ++Iter)
	{
		TUniquePtr<FMusicTimer>& CurrentTimer = *Iter;
		if (!CurrentTimer || !CurrentTimer->GetUniqueIdentifier())
		{
			Iter.RemoveCurrentSwap();
			continue;
		}

		if (CurrentTimer->IsPaused())
		{
			continue;
		}

		const FMidiSongPos& MidiSongPos = CurrentSongPositions[static_cast<int32>(CurrentTimer->GetTimebase())];
		const FMusicTimestamp& CurrentMusicalTimestamp = MidiSongPos.Timestamp;
		FTimeSignature TimeSignature(MidiSongPos.TimeSigNumerator, MidiSongPos.TimeSigDenominator);
		if(bool bDone = CurrentTimer->Update(CurrentMusicalTimestamp, TimeSignature))
		{
			Iter.RemoveCurrentSwap();
		}
	}

	LastUpdateFrameSongPositions = CurrentSongPositions;
	LastUpdateFrame = GFrameCounter;
}

void UMusicTimerManager::RemoveTimer(const FMusicTimerHandle& Handle)
{
	PendingTimers.RemoveAll([Handle](const TUniquePtr<FMusicTimer>& Timer) { return Timer && Timer->GetUniqueIdentifier() == Handle.GetUniqueIdentifier(); });

	// Do not remove these immediately as they may be currently being iterated on.
	// They will be removed as part of the update.
	for (TUniquePtr<FMusicTimer>& ActiveTimer : ActiveTimers)
	{
		if (ActiveTimer && ActiveTimer->GetUniqueIdentifier() == Handle.GetUniqueIdentifier())
		{
			ActiveTimer->Reset();
		}
	}
}

void UMusicTimerManager::PauseTimer(const FMusicTimerHandle& Handle, bool bPause)
{
	auto MatchesHandleLambda = [&Handle](const TUniquePtr<FMusicTimer>& Timer) -> bool
	{
		return Timer && Timer->GetUniqueIdentifier() == Handle.GetUniqueIdentifier();
	};

	TUniquePtr<FMusicTimer>* FoundTimer = PendingTimers.FindByPredicate(MatchesHandleLambda);
	if (!FoundTimer)
	{
		// Not in pending timers, lets see if its in the active timers.
		FoundTimer = ActiveTimers.FindByPredicate(MatchesHandleLambda);
	}

	FMusicTimer* Timer = FoundTimer ? (*FoundTimer).Get() : nullptr;
	if (!Timer || bPause == Timer->IsPaused())
	{
		return;
	}

	const FMusicTimestamp LastFrameMusicalTimestamp = LastUpdateFrameSongPositions[static_cast<int32>(Timer->GetTimebase())].Timestamp;
	if (bPause)
	{
		Timer->PausedTime = LastFrameMusicalTimestamp;
	}
	else
	{
		const FMidiSongPos& LastFrameMidiSongPos = LastUpdateFrameSongPositions[static_cast<int32>(Timer->GetTimebase())];
		FTimeSignature LastFrameTimeSignature(LastFrameMidiSongPos.TimeSigNumerator, LastFrameMidiSongPos.TimeSigDenominator);

		const FMusicTimestamp& PausedTime = Timer->PausedTime.GetValue();
		const int32 BeatsPerBar = LastFrameTimeSignature.Numerator;
		const float CurrentAbsoluteBeats = float(LastFrameMusicalTimestamp.Bar - 1) * BeatsPerBar + (LastFrameMusicalTimestamp.Beat - 1.0f);
		const float PausedbsoluteBeats = float(PausedTime.Bar - 1) * BeatsPerBar + (PausedTime.Beat - 1.0f);
		const float PausedDeltaBeats = CurrentAbsoluteBeats - PausedbsoluteBeats;
		Harmonix::IncrementTimestampByBeats(Timer->StartTime, PausedDeltaBeats, LastFrameTimeSignature);
		Timer->PausedTime.Reset();
	}
}

FMusicTimestamp UMusicTimerManager::Quantize(const UMusicClockComponent* MusicClock, const FMusicTimestamp& Time, EMidiClockSubdivisionQuantization QuantizationInterval, EMidiFileQuantizeDirection InDirection) const
{
	if(!MusicClock)
	{
		return FMusicTimestamp();
	}

	return MusicClock->Quantize(Time, QuantizationInterval, InDirection);
}
