// Copyright Epic Games, Inc. All Rights Reserved.
#include "WallClockMusicClockDriver.h"
#include "Harmonix.h"

TOptional<FVector> FWallClockMusicClockDriver::TryGetAudioSourceLocation() const
{
	return {};
}

void FWallClockMusicClockDriver::Disconnect()
{
	TempoMapMidi = nullptr;
}

bool FWallClockMusicClockDriver::RefreshCurrentSongPos()
{
	check(IsInGameThread());
	double RunTime = GetWallClockTime() - StartTimeSecs;

	const ISongMapEvaluator* Maps = GetCurrentSongMapEvaluator();
	check(Maps);

	// Set Experienced time here, as this code is running on the GameThread, near the same time as input is processed
	SetCurrentSongPosByTime(ECalibratedMusicTimebase::ExperiencedTime, (float)(RunTime * 1000.0), *Maps);
	SetCurrentSongPosByTime(ECalibratedMusicTimebase::RawAudioRenderTime, GetCurrentPlayerExperiencedSongPos().SecondsIncludingCountIn * 1000.0f + FHarmonixModule::GetMeasuredUserExperienceAndReactionToAudioRenderOffsetMs(), *Maps);
	SetCurrentSongPosByTime(ECalibratedMusicTimebase::VideoRenderTime, GetCurrentRawAudioRenderSongPos().SecondsIncludingCountIn * 1000.0f - FHarmonixModule::GetMeasuredVideoToAudioRenderOffsetMs(), *Maps);
	SetCurrentSongPos(ECalibratedMusicTimebase::AudioRenderTime, GetCurrentRawAudioRenderSongPos());

	const FMidiSongPos& CurrentSmoothedAudioRenderSongPos = GetCurrentSmoothedAudioRenderSongPos();
	UpdateMusicPlaybackRate(CurrentSmoothedAudioRenderSongPos.Tempo, CurrentClockAdvanceRate, CurrentSmoothedAudioRenderSongPos.TimeSigNumerator, CurrentSmoothedAudioRenderSongPos.TimeSigDenominator);

	return true;
}

void FWallClockMusicClockDriver::OnStart()
{
	check(IsInGameThread());
	StartTimeSecs = GetWallClockTime();
	PauseTimeSecs = 0.0;
}

void FWallClockMusicClockDriver::OnPause()
{
	check(IsInGameThread());
	PauseTimeSecs = GetWallClockTime();
}

void FWallClockMusicClockDriver::OnContinue()
{
	check(IsInGameThread());
	double CurrentTime = GetWallClockTime();
	StartTimeSecs += (CurrentTime - PauseTimeSecs);
	PauseTimeSecs = 0.0;
	RefreshCurrentSongPos();
}

const ISongMapEvaluator* FWallClockMusicClockDriver::GetCurrentSongMapEvaluator() const
{
	check(IsInGameThread());
	if (TempoMapMidi.IsValid())
	{
		return TempoMapMidi->GetSongMaps();
	}
	return &DefaultMaps;
}

#if !UE_BUILD_SHIPPING
FString FWallClockMusicClockDriver::GetDisplayName() const
{
	return FString::Printf(TEXT("Wall: %.3f bpm"), GetBeatsPerMinute());
}
#endif
