// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMetasound/Components/MusicClockDriverBase.h"
#include "HarmonixMidi/MidiFile.h"

class FWallClockMusicClockDriver : public FMusicClockDriverBase
{
public:
	FWallClockMusicClockDriver(UObject* WorldContextObj, const FWallClockMusicClockSettings& Settings)
		: FMusicClockDriverBase(WorldContextObj, Settings)
		, TempoMapMidi(Settings.TempoMap)
	{}

	virtual TOptional<FVector> TryGetAudioSourceLocation() const override;

	virtual void Disconnect() override;
	virtual bool RefreshCurrentSongPos() override;
	virtual void OnStart() override;
	virtual void OnPause() override;
	virtual void OnContinue() override;
	virtual void OnStop() override {}
	virtual const ISongMapEvaluator* GetCurrentSongMapEvaluator() const override;
	
	virtual bool LoopedThisFrame(ECalibratedMusicTimebase Timebase) const override { return false; }
	virtual bool SeekedThisFrame(ECalibratedMusicTimebase Timebase) const override { return false; }

#if !UE_BUILD_SHIPPING
	virtual FString GetDisplayName() const override;
#endif

private:
	TWeakObjectPtr<UMidiFile> TempoMapMidi;

	double StartTimeSecs = 0.0;
	double PauseTimeSecs = 0.0f;
};