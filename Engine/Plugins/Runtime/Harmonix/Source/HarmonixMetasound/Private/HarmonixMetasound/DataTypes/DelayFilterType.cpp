// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/DelayFilterType.h"

#define LOCTEXT_NAMESPACE "HarmonixMetasound"

namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(EDelayFilterType, FEnumDelayFilterType, "DelayFilterType")
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayFilterType::LowPass, 
			"EDelayFilterType_LowPass_DisplayName", 
			"Low-Pass", 
			"EDelayFilterType_LowPass_TT", 
			"Low-pass filter"),
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayFilterType::HighPass, 
			"EDelayFilterType_HighPass_DisplayName", 
			"High-Pass", 
			"EDelayFilterType_HighPass_TT", 
			"High-pass filter"),
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayFilterType::BandPass, 
			"EDelayFilterType_BandPass_DisplayName", 
			"Band-Pass", 
			"EDelayFilterType_BandPass_TT", 
			"Band-pass filter")
	DEFINE_METASOUND_ENUM_END()
}

#undef LOCTEXT_NAMESPACE
