// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/DelayStereoType.h"

#define LOCTEXT_NAMESPACE "HarmonixMetasound"

namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(EDelayStereoType, FEnumDelayStereoType, "DelayStereoType")
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayStereoType::Default, 
			"EDelayStereoType_Default_DisplayName", 
			"Default", 
			"EDelayStereoType_Default_TT", 
			"Default panning"),
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayStereoType::CustomSpread, 
			"EDelayStereoType_CustomSpread_DisplayName", 
			"CustomSpread", 
			"EDelayStereoType_CustomSpread_TT", 
			"Uses user-provided stereo spread for delayed signal"),
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayStereoType::PingPongForceLR, 
			"EDelayStereoType_PingPongForceLR_DisplayName", 
			"PingPongForceLR", 
			"EDelayStereoType_PingPongForceLR_TT", 
			"Delay alternates between corresponding left and right output channels"),
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayStereoType::PingPongSum, 
			"EDelayStereoType_PingPongSum_DisplayName", 
			"PingPongSum", 
			"EDelayStereoType_PingPongSum_TT", 
			"Delay bounces all channels around each output channel"),
		DEFINE_METASOUND_ENUM_ENTRY(
			EDelayStereoType::PingPongIndividual, 
			"EDelayStereoType_PingPongIndividual_DisplayName", 
			"PingPongIndividual", 
			"EDelayStereoType_PingPongIndividual_TT", 
			"Delay bounces each channel around each output channel individually"),
	DEFINE_METASOUND_ENUM_END()
}

#undef LOCTEXT_NAMESPACE
