// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/FFTAnalyzerResult.h"

#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOutput.h"

REGISTER_METASOUND_DATATYPE(FHarmonixFFTAnalyzerResults, "HarmonixFFTAnalyzerResults");