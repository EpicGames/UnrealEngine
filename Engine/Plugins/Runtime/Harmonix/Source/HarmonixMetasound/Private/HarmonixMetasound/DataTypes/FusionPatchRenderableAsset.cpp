// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/DataTypes/FusionPatchRenderableAsset.h"

DEFINE_AUDIORENDERABLE_ASSET(HarmonixMetasound, FFusionPatchAsset, FusionPatchAsset, UFusionPatch)