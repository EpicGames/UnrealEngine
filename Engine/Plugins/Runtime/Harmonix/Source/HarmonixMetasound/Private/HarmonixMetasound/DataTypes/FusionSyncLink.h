// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "MetasoundDataReferenceMacro.h"
#include "Tasks/Task.h"

DECLARE_LOG_CATEGORY_EXTERN(LogFusionAsync, Log, Log)

namespace HarmonixMetasound
{
	/** This class is a thin wrapper around a task handle that allows multiple
	 * Fusion Sampler Nodes to kick asynchronous rendering jobs that are "joined"
	 * later by a MultiThreadedFusionSynchronizerNode.
	 */
	class FFusionSyncLink
	{
	public:
		using FRenderFunction = TUniqueFunction<void(void)>;
		
		FFusionSyncLink()
		{}

		~FFusionSyncLink()
		{
			Task.Wait();
		}

		/** This function is called by the FusionSamplerNode when it is 
		 * enabled for multi-threaded rendering.
		 */
		void KickAsyncRender(FRenderFunction&& InRenderFunction)
		{
			Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(InRenderFunction), LowLevelTasks::ETaskPriority::High);
		}

		/** This function is called by the MultiThreadedFusionSynchronizerNode
		 * where it gathers the task handles from all of the samplers it is "synchronizing"
		 * so it can then optimally "join" on the completion of all of those tasks.
		 */
		UE::Tasks::FTask GetTask() const
		{
			return Task;
		}

		/** This function is called by the FusionSamplerNode "PostExecute" 
		 * so that if no down-stream MultiThreadedFusionSynchronizerNode does a 
		 * join we can detect that and at least (a) notify the user, and 
		 * (b) ensure the task is completed before the metasound tries to
		 * render another block.
		 */
		void PostExecute()
		{
			if (!Task.IsCompleted())
			{
				UE_LOGF(LogFusionAsync, Warning, "Fusion Async Render: Task created, but was never checked for completion. This probably means you are missing a Fusion Synchronizer Node!");
				Task.Wait();
			}
			Task = {};
		}

		/** This function is called by the FusionSamplerNode "Reset" 
         * We need to block on reset and make sure the task completes and clear it out
         */
        void Reset()
        {
			if (!Task.IsCompleted())
			{
				Task.Wait();
			}
			Task = {};
        }

	private:
		UE::Tasks::FTask Task;
	};

	// Declare aliases IN the namespace...
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FFusionSyncLink, FFusionSyncLinkTypeInfo, FFusionSyncLinkReadRef, FFusionSyncLinkWriteRef)

}

// Declare reference types OUT of the namespace...
DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(HarmonixMetasound::FFusionSyncLink, HARMONIXMETASOUND_API)

