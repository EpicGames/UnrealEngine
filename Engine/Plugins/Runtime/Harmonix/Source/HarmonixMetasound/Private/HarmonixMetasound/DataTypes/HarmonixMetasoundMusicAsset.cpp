// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/HarmonixMetasoundMusicAsset.h"
#include "HarmonixMetasound/Interfaces/HarmonixMusicInterfaces.h"
#include "MetasoundSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixMetasoundMusicAsset)

bool UHarmonixMetasoundMusicAsset::AssetIsMissingHarmoixMusicInterface(const FAssetData& AssetData)
{
#if WITH_EDITORONLY_DATA
	bool GotDocInfo = false;
	Metasound::Frontend::FMetaSoundClassInfo ClassInfo(AssetData);
	return !ClassInfo.InheritsInterface(HarmonixMetasound::MusicAssetInterface::FrontendVersion.Name);
#else
	return false;
#endif
}
