// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/HarmonixMusicAsset.h"
#include "HarmonixMetasound/DataTypes/HarmonixMusicHandle.h"
#include "HarmonixMidi/MidiFile.h"
#include "MetasoundSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixMusicAsset)

void UHarmonixMusicAsset::CreateFrameBasedMusicMap(UFrameBasedMusicMap* Map) const
{
	if (!MidiSongMap)
	{
		return;
	}

	check(MidiSongMap->GetSongMaps());
	MidiSongMap->GetSongMaps()->FillInFrameBasedMusicMap(Map);
}

FMarkerProviderResults UHarmonixMusicAsset::GatherMarkers(const UFrameBasedMusicMap* Map) const
{
	if (!MidiSongMap)
	{
		return FMarkerProviderResults();
	}
	return MidiSongMap->GatherMarkers(Map);
}

FMusicHandle UHarmonixMusicAsset::PrepareToPlay_Internal(UObject* PlaybackContext, UAudioComponent* OnComponent, float FromSeconds, bool IsAudition)
{
	UHarmonixMusicHandle* MusicHandle = UHarmonixMusicHandle::Instantiate(this, PlaybackContext, OnComponent, FromSeconds, IsAudition);
	return MusicHandle;
}

FMusicHandle UHarmonixMusicAsset::Play_Internal(UObject* PlaybackContext, UAudioComponent* OnComponent, float FromSeconds, bool IsAudition)
{
	FMusicHandle MusicHandle = PrepareToPlay(PlaybackContext, OnComponent, FromSeconds, IsAudition);
	if (MusicHandle)
	{
		MusicHandle->Play();
	}
	return MusicHandle;
}

