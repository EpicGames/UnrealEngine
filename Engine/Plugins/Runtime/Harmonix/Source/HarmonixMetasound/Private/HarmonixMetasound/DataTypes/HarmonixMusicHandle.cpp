// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/HarmonixMusicHandle.h"

#include "Components/AudioComponent.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/DataTypes/HarmonixMetasoundMusicAsset.h"
#include "HarmonixMetasound/Interfaces/HarmonixMusicInterfaces.h"
#include "MetasoundSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixMusicHandle)

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixMusicHandle, Log, All)

UHarmonixMusicHandle* UHarmonixMusicHandle::Instantiate(UHarmonixMusicAsset* Asset, UObject* PlaybackContext, UAudioComponent* OnComponent, float FromSeconds, bool bIsAudition)
{
	check(PlaybackContext);
	check(!OnComponent || PlaybackContext->GetWorld() == OnComponent->GetWorld());

	// Don't crash on a poorly authored asset...
	if (!ensureAlwaysMsgf(Asset->GetMetaSoundSource(), TEXT("Attempt to instantiate a HarmonixMusicHandle for a HarmonixMusicAsset that has no MetaSound failed.")))
	{
		return nullptr;
	}

	UWorld* World = PlaybackContext->GetWorld();

	UHarmonixMusicHandle* TheHandle = NewObject<UHarmonixMusicHandle>(World, NAME_None, RF_Transient);
	TheHandle->PlayingAsset = Asset;

	// We need an AudioComponent to play on. If we were not provided one, spawn one...
	if (!OnComponent)
	{
		TheHandle->AudioComponent = NewObject<UAudioComponent>(World, NAME_None, EObjectFlags::RF_Transient);
		if (!TheHandle->AudioComponent)
		{
			return nullptr;
		}
		TheHandle->AudioComponent->bAllowSpatialization = false;
	}
	else
	{
		TheHandle->AudioComponent = OnComponent;
		TheHandle->AudioComponent->Stop();
	}
	if (bIsAudition)
	{
		TheHandle->AudioComponent->SetUISound(true);
	}

	TheHandle->AudioComponent->Sound = Asset->GetMetaSoundSource();
	TheHandle->AudioComponent->Play();

	// Now, because we are a music handle, we need a music clock. Spawn one...	
	TheHandle->MusicClockComponent = NewObject<UMusicClockComponent>(World, NAME_None, RF_Transient);
	TheHandle->MusicClockComponent->MetasoundOutputName = HarmonixMetasound::MusicAssetInterface::MidiClockOut.Resolve();
	TheHandle->MusicClockComponent->ConnectToMetasoundOnAudioComponent(TheHandle->AudioComponent);
	TheHandle->MusicClockComponent->Start();

	// NOTE: For now we are going to ignore "FromSeconds" as...
	// a) We aren't supporting 'async preparation' of music assets, and therefore...
	// b) We can just wait for the Play call and use the 'FromSeconds' from there.

	// Since we aren't yet supporting async preparation of music assets for playback,
	// just mark ourselves as ready to play.
	TheHandle->CurrentTransportState = EMusicHanldeTransportState::ReadyToPlay;

	return TheHandle;
}

bool UHarmonixMusicHandle::IsValid() const
{
	if (PlayingAsset && AudioComponent)
	{
		if (!bHasStarted || (bHasStarted && AudioComponent->GetPlayState() != EAudioComponentPlayState::Stopped))
		{
			return true;
		}
	}
	return false;
}

bool UHarmonixMusicHandle::Play(float FromSeconds)
{
	using namespace HarmonixMetasound::MusicAssetInterface;

	if (!AudioComponent || !MusicClockComponent)
	{
		return false;
	}

	FAudioParameter SeekTargetSeconds(SeekTargetSecondsIn.Resolve(), FromSeconds);
	AudioComponent->Activate();
	AudioComponent->SetParameter(MoveTemp(SeekTargetSeconds));
	AudioComponent->SetTriggerParameter(SeekIn.Resolve());
	AudioComponent->SetTriggerParameter(PlayIn.Resolve());
	MusicClockComponent->Start();
	CurrentTransportState = EMusicHanldeTransportState::Playing;
	return true;
}

void UHarmonixMusicHandle::Tick(float DeltaSeconds)
{
	// MusicClock no longer ticks, this function is empty
}

void UHarmonixMusicHandle::Pause()
{
	using namespace HarmonixMetasound::MusicAssetInterface;
	AudioComponent->SetTriggerParameter(PauseIn.Resolve());
	CurrentTransportState = EMusicHanldeTransportState::Paused;
}

void UHarmonixMusicHandle::Continue()
{
	using namespace HarmonixMetasound::MusicAssetInterface;
	AudioComponent->SetTriggerParameter(ContinueIn.Resolve());
	CurrentTransportState = EMusicHanldeTransportState::Playing;
}

void UHarmonixMusicHandle::Stop_Internal()
{
	using namespace HarmonixMetasound::MusicAssetInterface;
	if (MusicClockComponent)
	{
		MusicClockComponent->Stop();
		MusicClockComponent = nullptr;
	}
	if (AudioComponent)
	{
		AudioComponent->SetTriggerParameter(StopIn.Resolve());
		AudioComponent->Deactivate();
		AudioComponent = nullptr;
	}
	PlayingAsset = nullptr;
	CurrentTransportState = EMusicHanldeTransportState::Stale;
}

void UHarmonixMusicHandle::Kill_Internal()
{
	using namespace HarmonixMetasound::MusicAssetInterface;
	if (MusicClockComponent)
	{
		MusicClockComponent->Stop();
		MusicClockComponent = nullptr;
	}
	if (AudioComponent)
	{
		AudioComponent->SetTriggerParameter(KillIn.Resolve());
		AudioComponent->Deactivate();
		AudioComponent = nullptr;
	}
	PlayingAsset = nullptr;
	CurrentTransportState = EMusicHanldeTransportState::Stale;
}

bool UHarmonixMusicHandle::IsUsingAsset(const UObject* Asset) const
{
	return IsValid() && PlayingAsset == Asset;
}

TScriptInterface<IMusicEnvironmentClockSource> UHarmonixMusicHandle::GetMusicClockSource()
{
	return MusicClockComponent;
}
