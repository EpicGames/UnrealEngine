// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/HarmonixWaveMusicAsset.h"

#include "HarmonixMetasound/DataTypes/MidiAsset.h"
#include "HarmonixMetasound/Interfaces/HarmonixMusicInterfaces.h"
#include "HarmonixMetasound/Nodes/MetronomeNode.h"
#include "HarmonixMetasound/Nodes/MidiPlayerNode.h"
#include "HarmonixMetasound/Nodes/MusicSeekTargetBuilder.h"
#include "HarmonixMetasound/Nodes/TriggerToTransportNode.h"
#include "HarmonixMetasound/Nodes/TransportToTriggerNode.h"
#include "HarmonixMetasound/Nodes/TransportWavePlayerControllerNode.h"
#include "HarmonixMetasound/Nodes/MidiPlayerNode.h"

#include "MetasoundBuilderSubsystem.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundSource.h"
#include "MetasoundStandardNodesNames.h"

#include <functional>

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixWaveMusicAsset)

namespace HarmonixWaveMusicAsset
{
	static const FLazyName BuilderName(TEXT("HmxWaveMusicAssetBuilder"));

	EMetaSoundOutputAudioFormat GetOutputFormatFromChannelCount(int32 NumChannels)
	{
		switch (NumChannels)
		{ 
			case 1: return EMetaSoundOutputAudioFormat::Mono;
			case 2: return EMetaSoundOutputAudioFormat::Stereo;
			case 4: return EMetaSoundOutputAudioFormat::Quad;
			case 6: return EMetaSoundOutputAudioFormat::FiveDotOne;
			case 8: return EMetaSoundOutputAudioFormat::SevenDotOne;
		}
		return EMetaSoundOutputAudioFormat::Mono;
	}

	UMetaSoundSourceBuilder* MakeBuilder(FMetaSoundBuilderNodeOutputHandle& OnPlayNodeOutput,
	                                     FMetaSoundBuilderNodeInputHandle& OnFinishedNodeInput,
	                                     TArray<FMetaSoundBuilderNodeInputHandle>& AudioOutNodeInputs,
										 int32 NumChannels)
	{
		UMetaSoundBuilderSubsystem* MetaSoundBuilderSubsystem = GEngine->GetEngineSubsystem<UMetaSoundBuilderSubsystem>();
		if (!MetaSoundBuilderSubsystem)
		{
			return nullptr;
		}
		EMetaSoundBuilderResult Result;
		EMetaSoundOutputAudioFormat OutFormat = GetOutputFormatFromChannelCount(NumChannels);
		UMetaSoundSourceBuilder* SourceBuilder = MetaSoundBuilderSubsystem->CreateSourceBuilder(BuilderName, OnPlayNodeOutput, OnFinishedNodeInput, AudioOutNodeInputs, Result, OutFormat, true);
		check(AudioOutNodeInputs.Num() == NumChannels);
		if (!ensureMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to create the metasound source builder for the WaveMusicAssetBuilder!")))
		{
			return nullptr;
		}
		MetaSoundBuilderSubsystem->RegisterSourceBuilder(BuilderName, SourceBuilder);
		return SourceBuilder;
	}

	void DestroyBuilder()
	{
		UMetaSoundBuilderSubsystem* MetaSoundBuilderSubsystem = GEngine->GetEngineSubsystem<UMetaSoundBuilderSubsystem>();
		if (MetaSoundBuilderSubsystem)
		{
			MetaSoundBuilderSubsystem->UnregisterSourceBuilder(BuilderName);
		}
	}

	FMetaSoundNodeHandle MakeMetasoundStandardNode(UMetaSoundSourceBuilder* Builder, const FName& Namespace, const TCHAR* Name, const TCHAR* Varient, EMetaSoundBuilderResult& Result)
	{
		Metasound::FNodeClassName ClassName = { Namespace, Name, Varient };
		return Builder->AddNodeByClassName(ClassName, Result);
	}

	FMetaSoundNodeHandle MakeAppropriateWavePlayerNode(UMetaSoundSourceBuilder* Builder, int32 NumChannels)
	{
		FMetaSoundNodeHandle RetVal;
		EMetaSoundOutputAudioFormat Format = GetOutputFormatFromChannelCount(NumChannels);
		const FName* Varient = nullptr;
		switch (Format)
		{
		case EMetaSoundOutputAudioFormat::Mono:
			Varient = &Metasound::EngineNodes::MonoVariant;
			break;
		case EMetaSoundOutputAudioFormat::Stereo:
			Varient = &Metasound::EngineNodes::StereoVariant;
			break;
		case EMetaSoundOutputAudioFormat::Quad:
			Varient = &Metasound::EngineNodes::QuadVariant;
			break;
		case EMetaSoundOutputAudioFormat::FiveDotOne:
			Varient = &Metasound::EngineNodes::FiveDotOneVariant;
			break;
		case EMetaSoundOutputAudioFormat::SevenDotOne:
			Varient = &Metasound::EngineNodes::SevenDotOneVariant;
			break;
		default:
			return RetVal;
		}

		EMetaSoundBuilderResult Result;
		RetVal = MakeMetasoundStandardNode(Builder, Metasound::EngineNodes::Namespace, TEXT("Wave Player"), *Varient->ToString(), Result);
		if (!ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to add Wave Player - %s node"), *Varient->ToString()))
		{
			check(!RetVal.IsSet());
		}
		return RetVal;
	}

	const TCHAR* kChannelNames[9][8] = 
	{
		/* NONE */   {nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr},
		/* Mono */   {TEXT("Out Mono"),nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr},
		/* Stereo */ {TEXT("Out Left"),TEXT("Out Right"),nullptr,nullptr,nullptr,nullptr,nullptr,nullptr},
		/* NONE */   {nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr},
		/* Quad */   {TEXT("Out Front Left"),TEXT("Out Front Right"),TEXT("Out Side Left"),TEXT("Out Side Right"),nullptr,nullptr,nullptr,nullptr},
		/* NONE */   {nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr},
		/* 5.1 */    {TEXT("Out Front Left"),TEXT("Out Front Right"),TEXT("Out Front Center"),TEXT("Out Low Frequency"),TEXT("Out Side Left"),TEXT("Out Side Right"),nullptr,nullptr},
		/* NONE */   {nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr,nullptr},
		/* 7.1 */    {TEXT("Out Front Left"),TEXT("Out Front Right"),TEXT("Out Front Center"),TEXT("Out Low Frequency"),TEXT("Out Side Left"),TEXT("Out Side Right"),TEXT("Out Back Left"),TEXT("Out Back Right")},
	};

	const TCHAR* GetChannelName(int32 ChannelIndex, int32 TotalChannels)
	{
		check(ChannelIndex >= 0 && ChannelIndex < 8);
		check(TotalChannels > 0 && TotalChannels < 9);
		return kChannelNames[TotalChannels][ChannelIndex];
	}

	bool ConnectAudioOutput(UMetaSoundSourceBuilder* Builder, FMetaSoundNodeHandle WavePlayer, TArray<FMetaSoundBuilderNodeInputHandle>& AudioOutNodeInputs)
	{
		EMetaSoundBuilderResult Result;
		for (int32 Ch = 0; Ch < AudioOutNodeInputs.Num(); ++Ch)
		{
			Builder->ConnectNodeToGraphOutput(WavePlayer, GetChannelName(Ch, AudioOutNodeInputs.Num()), AudioOutNodeInputs[Ch], Result);
			if (!ensureMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to conect node output WavePlayer:%s to Graph Output!"), GetChannelName(Ch, AudioOutNodeInputs.Num())))
			{
				return false;
			}
		}
		return true;
	}
}

float UHarmonixWaveMusicAsset::GetSongLengthSeconds() const
{
	return WaveFile->GetDuration();
}

#if WITH_EDITOR
void UHarmonixWaveMusicAsset::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	CachedMetasound = nullptr;
}
#endif

#if defined(ADD_HMX_NODE) || defined(ADD_MSS_NODE) || defined(SET_NODE_INPUT) || defined(CONNECT_NODES) || \
		defined(CONNECT_GRAPH_IN_TO_NODE) || defined(CONNECT_NODE_TO_GRAPH_OUT) || defined(CONNECT_NODE_TO_NAMED_GRAPH_OUT)
# error "UTILITY MACRO NAME COLLISION!"
#endif

#define ADD_HMX_NODE(x)                                                                                     \
	SourceBuilder->AddNodeByClassName(x::GetClassName(), Result, x::GetCurrentMajorVersion());              \
	if (!ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to add " #x " node"))) \
		return nullptr

#define ADD_MSS_NODE(n, x, y)                                                                                      \
	MakeMetasoundStandardNode(SourceBuilder, n, x, y, Result);                                                     \
	if (!ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to add " x " - " y " node"))) \
		return nullptr

#define SET_NODE_INPUT(Node, Input, Value)                                                                          \
	SourceBuilder->SetNodeInputDefault(Node, Input, Value, Result);                                                 \
	if (!ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to set Input '" #Input "'!"))) \
		return nullptr

#define CONNECT_NODES(OutNode, OutName, InNode, InName)                                       \
	SourceBuilder->ConnectNodes(OutNode, OutName, InNode, InName, Result);                    \
	if (!ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded,                       \
			TEXT("Unable to connect " #OutNode ":" #OutName " -> " #InNode ":" #InName "!"))) \
		return nullptr

#define CONNECT_GRAPH_IN_TO_NODE(GraphInName, Node, NodeInName)                                   \
	SourceBuilder->ConnectGraphInputToNode(GraphInName, Node, NodeInName, Result);                \
	if (!ensureMsgf(Result == EMetaSoundBuilderResult::Succeeded,                                 \
			TEXT("Unable to conect graph input " #GraphInName " to " #Node ":" #NodeInName "!"))) \
		return nullptr

#define CONNECT_NODE_TO_GRAPH_OUT(Node, NodeOutName, GraphOut)                                  \
	SourceBuilder->ConnectNodeToGraphOutput(Node, NodeOutName, GraphOut, Result);               \
	if (!ensureMsgf(Result == EMetaSoundBuilderResult::Succeeded,                               \
			TEXT("Unable to conect node output " #Node ":" #NodeOutName " to " #GraphOut "!"))) \
		return nullptr

#define CONNECT_NODE_TO_NAMED_GRAPH_OUT(Node, NodeOutName, GraphOutName)                            \
	SourceBuilder->ConnectNodeToGraphOutput(Node, NodeOutName, GraphOutName, Result);               \
	if (!ensureMsgf(Result == EMetaSoundBuilderResult::Succeeded,                                   \
			TEXT("Unable to conect node output " #Node ":" #NodeOutName " to " #GraphOutName "!"))) \
		return nullptr

UMetaSoundSource* UHarmonixWaveMusicAsset::GetMetaSoundSource()
{
	using namespace HarmonixMetasound::Nodes;
	using namespace HarmonixWaveMusicAsset;

	if (CachedMetasound)
	{ 
		return CachedMetasound;
	}

	if (!WaveFile)
	{
		return nullptr;
	}

	EMetaSoundBuilderResult Result;
	FMetaSoundBuilderNodeOutputHandle OnPlayNodeOutput;
	FMetaSoundBuilderNodeInputHandle OnFinishedNodeInput;
	TArray<FMetaSoundBuilderNodeInputHandle> AudioOutNodeInputs;

	UMetaSoundSourceBuilder* SourceBuilder = MakeBuilder(OnPlayNodeOutput, OnFinishedNodeInput, AudioOutNodeInputs, WaveFile->NumChannels);
	if (!SourceBuilder) return nullptr;

	ON_SCOPE_EXIT { DestroyBuilder(); };

	// First all of the nodes in the music interface...
	SourceBuilder->AddInterface(HarmonixMetasound::MusicAssetInterface::GetVersion().Name, Result);
	if (!ensureMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to add Harmonix Music Interface to the MetaSound!")))
	{
		return nullptr;
	}

	// Now the rest of the nodes in the graph...
	FMetaSoundNodeHandle TriggerToTransport    = ADD_HMX_NODE(TriggerToTransportNode);
	FMetaSoundNodeHandle TransportToTrigger    = ADD_HMX_NODE(TransportToTriggerNode);
	FMetaSoundNodeHandle SecondsToMsMultiplier = ADD_MSS_NODE(Metasound::StandardNodes::Namespace, TEXT("Multiply"), TEXT("Float"));
	FMetaSoundNodeHandle SeekTargetConverter   = ADD_HMX_NODE(TimeMsToSeekTarget);
	FMetaSoundNodeHandle AnyStop               = ADD_MSS_NODE(TEXT("TriggerAny"), TEXT("Trigger Any (2)"), TEXT(""));
	FMetaSoundNodeHandle WaveController        = ADD_HMX_NODE(TransportWavePlayerControllerNode);
	FMetaSoundNodeHandle MidiPlayer            = ADD_HMX_NODE(MidiPlayerNode);

	// Have to treat the wave player node differently since the 'varient' depends on the number of channels in the wave file...
	FMetaSoundNodeHandle WavePlayer            = MakeAppropriateWavePlayerNode(SourceBuilder, WaveFile->NumChannels);
	if (!WavePlayer.IsSet()) { return nullptr; }

	SET_NODE_INPUT(SecondsToMsMultiplier, TEXT("AdditionalOperands"), 1000.0f);
	SET_NODE_INPUT(WavePlayer, TEXT("Wave Asset"), (UObject*)WaveFile.Get());
	SET_NODE_INPUT(MidiPlayer, MidiPlayerNode::Inputs::MidiFileAssetName, (UObject*)MidiSongMap.Get());

	CONNECT_NODES(SecondsToMsMultiplier, TEXT("Out"), SeekTargetConverter, TimeMsToSeekTarget::Inputs::TimeMsName);
	CONNECT_NODES(SeekTargetConverter, TimeMsToSeekTarget::Outputs::SeekTargetName, TriggerToTransport, TriggerToTransportNode::Inputs::SeekDestinationName);
	CONNECT_NODES(TriggerToTransport, TriggerToTransportNode::Outputs::TransportName,TransportToTrigger, TransportToTriggerNode::Inputs::TransportName);
	CONNECT_NODES(TriggerToTransport, TriggerToTransportNode::Outputs::TransportName, WaveController, TransportWavePlayerControllerNode::Inputs::TransportName);
	CONNECT_NODES(TriggerToTransport, TriggerToTransportNode::Outputs::TransportName, MidiPlayer, MidiPlayerNode::Inputs::TransportName);
	CONNECT_NODES(TransportToTrigger, TransportToTriggerNode::Outputs::TransportStopName, AnyStop, TEXT("In 0"));
	CONNECT_NODES(TransportToTrigger, TransportToTriggerNode::Outputs::TransportKillName, AnyStop, TEXT("In 1"));
	CONNECT_NODES(WaveController, TransportWavePlayerControllerNode::Outputs::TransportPlayName, WavePlayer, TEXT("Play"));
	CONNECT_NODES(WaveController, TransportWavePlayerControllerNode::Outputs::TransportStopName, WavePlayer, TEXT("Stop"));
	CONNECT_NODES(WaveController, TransportWavePlayerControllerNode::Outputs::StartTimeName, WavePlayer, TEXT("Start Time"));
	CONNECT_NODES(MidiPlayer, MidiPlayerNode::Outputs::MidiClockName, WaveController, TransportWavePlayerControllerNode::Inputs::MidiClockName);

	CONNECT_GRAPH_IN_TO_NODE(HarmonixMetasound::MusicAssetInterface::PlayIn, TriggerToTransport, TriggerToTransportNode::Inputs::TransportPlayName);
	CONNECT_GRAPH_IN_TO_NODE(HarmonixMetasound::MusicAssetInterface::PauseIn, TriggerToTransport, TriggerToTransportNode::Inputs::TransportPauseName);
	CONNECT_GRAPH_IN_TO_NODE(HarmonixMetasound::MusicAssetInterface::ContinueIn, TriggerToTransport, TriggerToTransportNode::Inputs::TransportContinueName);
	CONNECT_GRAPH_IN_TO_NODE(HarmonixMetasound::MusicAssetInterface::StopIn, TriggerToTransport, TriggerToTransportNode::Inputs::TransportStopName);
	CONNECT_GRAPH_IN_TO_NODE(HarmonixMetasound::MusicAssetInterface::KillIn, TriggerToTransport, TriggerToTransportNode::Inputs::TransportKillName);
	CONNECT_GRAPH_IN_TO_NODE(HarmonixMetasound::MusicAssetInterface::SeekIn, TriggerToTransport, TriggerToTransportNode::Inputs::TriggerSeekName);
	CONNECT_GRAPH_IN_TO_NODE(HarmonixMetasound::MusicAssetInterface::SeekTargetSecondsIn, SecondsToMsMultiplier, TEXT("PrimaryOperand"));
	
	CONNECT_NODE_TO_GRAPH_OUT(AnyStop, TEXT("Out"), OnFinishedNodeInput);
	CONNECT_NODE_TO_NAMED_GRAPH_OUT(MidiPlayer, MidiPlayerNode::Outputs::MidiClockName, HarmonixMetasound::MusicAssetInterface::MidiClockOut);

	// Have to treat the wave player output differently so we can wire it up based on the number of the channels in the wave file...
	if (!ConnectAudioOutput(SourceBuilder, WavePlayer, AudioOutNodeInputs)) return nullptr;

	FString MetasoundName = FString::Format(TEXT("{0}_MetaSound"), {GetFName().ToString()});
	CachedMetasound = Cast<UMetaSoundSource>(SourceBuilder->BuildNewMetaSound(*MetasoundName).GetObject());
	check(CachedMetasound);

	return CachedMetasound;
}

#undef ADD_HMX_NODE
#undef ADD_MSS_NODE
#undef SET_NODE_INPUT
#undef CONNECT_NODES
#undef CONNECT_GRAPH_IN_TO_NODE
#undef CONNECT_NODE_TO_GRAPH_OUT
#undef CONNECT_NODE_TO_NAMED_GRAPH_OUT
