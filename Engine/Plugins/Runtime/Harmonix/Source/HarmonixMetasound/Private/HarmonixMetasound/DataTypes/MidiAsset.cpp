// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MidiAsset.h"

#include "MetasoundDataTypeRegistrationMacro.h"

REGISTER_METASOUND_DATATYPE(HarmonixMetasound::FMidiAsset, "MIDIAsset", Metasound::ELiteralType::UObjectProxy, UMidiFile);

namespace HarmonixMetasound
{
	using namespace Metasound;

	FMidiAsset::FMidiAsset(const TSharedPtr<Audio::IProxyData>& InInitData)
	{
		if (InInitData.IsValid())
		{
			if (InInitData->CheckTypeCast<FMidiFileProxy>())
			{
				// should we be getting handed a SharedPtr here?
				MidiFileProxy = MakeShared<FMidiFileProxy, ESPMode::ThreadSafe>(InInitData->GetAs<FMidiFileProxy>());
			}
		}
	}

	bool FMidiAsset::IsMidiValid() const
	{
		return MidiFileProxy.IsValid();
	}
}
