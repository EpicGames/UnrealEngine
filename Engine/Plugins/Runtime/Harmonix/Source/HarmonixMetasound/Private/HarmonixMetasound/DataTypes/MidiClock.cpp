// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/DataTypes/MidiClock.h"

#include "MetasoundDataTypeRegistrationMacro.h"
#include "Engine/Engine.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixMidiClock, Log, All);

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(EMidiClockSubdivisionQuantization, FEnumMidiClockSubdivisionQuantizationType, "SubdivisionQuantizationType")
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::None, "NoneDesc", "None", "NoneTT", "None"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::Bar, "BarDesc", "Bar", "BarTT", "Bar"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::Beat, "BeatDesc", "Beat", "BeatTT", "Beat"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::ThirtySecondNote, "ThirtySecondNoteDesc", "1/32", "ThirtySecondNoteTT", "1/32"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::SixteenthNote, "SixteenthNoteDesc", "1/16", "SixteenthNoteTT", "1/16"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::EighthNote, "EighthNoteDesc", "1/8", "EighthNoteTT", "1/8"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::QuarterNote, "QuarterNoteDesc", "1/4", "QuarterNoteTT", "1/4"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::HalfNote, "HalfNoteDesc", "Half", "HalfNoteTT", "Half"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::WholeNote, "WholeNoteDesc", "Whole", "WholeNoteTT", "Whole"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::DottedSixteenthNote, "DottedSixteenthNoteDesc", "(dotted) 1/16", "DottedSixteenthNoteTT", "(dotted) 1/16"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::DottedEighthNote, "DottedEighthNoteDesc", "(dotted) 1/8", "DottedEighthNoteTT", "(dotted) 1/8"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::DottedQuarterNote, "DottedQuarterNoteDesc", "(dotted) 1/4", "DottedQuarterNoteTT", "(dotted) 1/4"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::DottedHalfNote, "DottedHalfNoteDesc", "(dotted) Half", "DottedHalfNoteTT", "(dotted) Half"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::DottedWholeNote, "DottedWholeNoteDesc", "(dotted) Whole", "DottedWholeNoteTT", "(dotted) Whole"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::SixteenthNoteTriplet, "SixteenthNoteTripletDesc", "1/16 (triplet)", "SixteenthNoteTripletTT", "1/16 (triplet)"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::EighthNoteTriplet, "EighthNoteTripletDesc", "1/8 (triplet)", "EighthNoteTripletTT", "1/8 (triplet)"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::QuarterNoteTriplet, "QuarterNoteTripletDesc", "1/4 (triplet)", "QuarterNoteTripletTT", "1/4 (triplet)"),
		DEFINE_METASOUND_ENUM_ENTRY(EMidiClockSubdivisionQuantization::HalfNoteTriplet, "HalfNoteTripletDesc", "1/2 (triplet)", "HalfNoteTripletTT", "1/2 (triplet)"),
	DEFINE_METASOUND_ENUM_END()
}

REGISTER_METASOUND_DATATYPE(HarmonixMetasound::FMidiClock, "MIDIClock")

namespace HarmonixMetasound
{
	using namespace Metasound;

	FMidiClock::FMidiClock(const FOperatorSettings& InSettings)
		: SongMapEvaluator(MakeShared<FSongMapsWithAlternateTempoSource>(MakeShared<FSongMaps>(120.0f, 4, 4)))
		, CurrentTempoInfoPointIndex(0)
		, CurrentTimeSignaturePointIndex(0)
		, TickResidualWhenDriven(0.0f)
		, BlockSize(InSettings.GetNumFramesPerBlock())
		, CurrentBlockFrameIndex(0)
		, FirstTickProcessedThisBlock(-1)
		, LastProcessedMidiTick(-1)
		, NextMidiTickToProcess(0)
		, NextTempoMapTickToProcess(0)
		, SampleRate(InSettings.GetSampleRate())
		, SampleCount(0)
		, FramesUntilNextProcess(0)
		, TransportAtBlockStart(EMusicPlayerTransportState::Invalid)
		, TransportAtBlockEnd(EMusicPlayerTransportState::Invalid)
		, SpeedAtBlockStart(0.0f)
		, SpeedAtBlockEnd(0.0f)
		, CurrentLocalSpeed(-1.0f)
		, TempoAtBlockStart(0.0f)
		, TempoAtBlockEnd(0.0f)
		, TimeSignatureAtBlockStart(FTimeSignature(0,1))
		, TimeSignatureAtBlockEnd(FTimeSignature(0,1))
		, NumTransportChangeInBlock(0)
		, NumSpeedChangeInBlock(0)
		, NumTempoChangeInBlock(0)
		, NumTimeSignatureChangeInBlock(0)
		, NextTempoChangeTick(std::numeric_limits<int32>::max())
		, NextTimeSigChangeTick(std::numeric_limits<int32>::max())
		, NextTempoOrTimeSigChangeTick(std::numeric_limits<int32>::max())
		, FirstTickInLoop(-1)
		, LoopLengthTicks(0)
		, MidiDataChangedInBlock(false)
		, NeedsSeekToDrivingClock(false)
	{
	}

	FMidiClock::FMidiClock(const FMidiClock& Other)
		: TSharedFromThis<FMidiClock, ESPMode::NotThreadSafe>(Other)
		, SongMapEvaluator(MakeShared<FSongMapsWithAlternateTempoSource>(Other.SongMapEvaluator))
		, CurrentTempoInfoPointIndex(Other.CurrentTempoInfoPointIndex)
		, CurrentTimeSignaturePointIndex(Other.CurrentTimeSignaturePointIndex)
		, ExternalClockDriver(Other.ExternalClockDriver)
		, TickResidualWhenDriven(Other.TickResidualWhenDriven)
		, BlockSize(Other.BlockSize)
		, CurrentBlockFrameIndex(Other.CurrentBlockFrameIndex)
		, FirstTickProcessedThisBlock(Other.FirstTickProcessedThisBlock)
		, LastProcessedMidiTick(Other.LastProcessedMidiTick)
		, NextMidiTickToProcess(Other.NextMidiTickToProcess)
		, NextTempoMapTickToProcess(Other.NextTempoMapTickToProcess)
		, SampleRate(Other.SampleRate)
		, SampleCount(Other.SampleCount)
		, FramesUntilNextProcess(Other.FramesUntilNextProcess)
		, TransportAtBlockStart(Other.TransportAtBlockStart)
		, TransportAtBlockEnd(Other.TransportAtBlockEnd)
		, SpeedAtBlockStart(Other.SpeedAtBlockStart)
		, SpeedAtBlockEnd(Other.SpeedAtBlockEnd)
		, CurrentLocalSpeed(Other.CurrentLocalSpeed)
		, TempoAtBlockStart(Other.TempoAtBlockStart)
		, TempoAtBlockEnd(Other.TempoAtBlockEnd)
		, TimeSignatureAtBlockStart(Other.TimeSignatureAtBlockStart)
		, TimeSignatureAtBlockEnd(Other.TimeSignatureAtBlockEnd)
		, NumTransportChangeInBlock(Other.NumTransportChangeInBlock)
		, NumSpeedChangeInBlock(Other.NumSpeedChangeInBlock)
		, NumTempoChangeInBlock(Other.NumTempoChangeInBlock)
		, NumTimeSignatureChangeInBlock(Other.NumTimeSignatureChangeInBlock)
		, NextTempoChangeTick(Other.NextTempoChangeTick)
		, NextTimeSigChangeTick(Other.NextTimeSigChangeTick)
		, NextTempoOrTimeSigChangeTick(Other.NextTempoOrTimeSigChangeTick)
		, FirstTickInLoop(Other.FirstTickInLoop)
		, LoopLengthTicks(Other.LoopLengthTicks)
		, MidiDataChangedInBlock(Other.MidiDataChangedInBlock)
		, NeedsSeekToDrivingClock(Other.NeedsSeekToDrivingClock)
		, MidiClockEventsInBlock(Other.MidiClockEventsInBlock)
	{
	}

	FMidiClock::~FMidiClock()
	{
	}

	FMidiClock& FMidiClock::operator=(const FMidiClock& Other)
	{
		if (this != &Other)
		{
			SongMapEvaluator = Other.SongMapEvaluator;
			CurrentTempoInfoPointIndex = Other.CurrentTempoInfoPointIndex;
			CurrentTimeSignaturePointIndex = Other.CurrentTimeSignaturePointIndex;
			ExternalClockDriver = Other.ExternalClockDriver;
			TickResidualWhenDriven = Other.TickResidualWhenDriven;
			BlockSize = Other.BlockSize;
			CurrentBlockFrameIndex = Other.CurrentBlockFrameIndex;
			FirstTickProcessedThisBlock = Other.FirstTickProcessedThisBlock;
			LastProcessedMidiTick = Other.LastProcessedMidiTick;
			NextMidiTickToProcess = Other.NextMidiTickToProcess;
			NextTempoMapTickToProcess = Other.NextTempoMapTickToProcess;
			SampleRate = Other.SampleRate;
			SampleCount = Other.SampleCount;
			FramesUntilNextProcess = Other.FramesUntilNextProcess;
			TransportAtBlockStart = Other.TransportAtBlockStart;
			TransportAtBlockEnd = Other.TransportAtBlockEnd;
			SpeedAtBlockStart = Other.SpeedAtBlockStart;
			SpeedAtBlockEnd = Other.SpeedAtBlockEnd;
			CurrentLocalSpeed = Other.CurrentLocalSpeed;
			TempoAtBlockStart = Other.TempoAtBlockStart;
			TempoAtBlockEnd = Other.TempoAtBlockEnd;
			TimeSignatureAtBlockStart = Other.TimeSignatureAtBlockStart;
			TimeSignatureAtBlockEnd = Other.TimeSignatureAtBlockEnd;
			NumTransportChangeInBlock = Other.NumTransportChangeInBlock;
			NumSpeedChangeInBlock = Other.NumSpeedChangeInBlock;
			NumTempoChangeInBlock = Other.NumTempoChangeInBlock;
			NextTempoChangeTick = Other.NextTempoChangeTick;
			NextTimeSigChangeTick = Other.NextTimeSigChangeTick;
			NextTempoOrTimeSigChangeTick = Other.NextTempoOrTimeSigChangeTick;
			FirstTickInLoop = Other.FirstTickInLoop;
			LoopLengthTicks = Other.LoopLengthTicks;
			MidiDataChangedInBlock = Other.MidiDataChangedInBlock;
			NeedsSeekToDrivingClock = Other.NeedsSeekToDrivingClock;
			MidiClockEventsInBlock = Other.MidiClockEventsInBlock;
		}

		return *this;
	}

	void FMidiClock::AttachToSongMapEvaluator(TSharedPtr<ISongMapEvaluator> SongMaps, bool ResetToStart)
	{
		MidiDataChangedInBlock = true;
		if (!SongMaps)
		{
			SongMaps = MakeShared<FSongMaps>(TempoAtBlockEnd, TimeSignatureAtBlockEnd.Numerator, TimeSignatureAtBlockEnd.Denominator);
		}

		if (ExternalClockDriver)
		{
			// use the tempo from the external clock...
			RebuildSongMapEvaluator(ExternalClockDriver->SongMapEvaluator->GetSongMapsWithTempoMap(), SongMaps);
		}
		else
		{
			// All maps come from the same source...
			RebuildSongMapEvaluator(SongMaps, SongMaps);
		}

		if (ResetToStart)
		{
			SeekTo(CurrentBlockFrameIndex, 0, 0);
		}
		PostTempoOrTimeSignatureEventsIfNeeded();
	}

	void FMidiClock::SongMapsChanged()
	{
		MidiDataChangedInBlock = true;
		PostTempoOrTimeSignatureEventsIfNeeded();
	}

	void FMidiClock::DetachFromSongMaps()
	{
		AttachToSongMapEvaluator(nullptr, false);
	}

	void FMidiClock::Reset(const FOperatorSettings& InSettings)
	{
		SongMapEvaluator = MakeShared<FSongMapsWithAlternateTempoSource>(MakeShared<FSongMaps>(120.0f, 4, 4));
		CurrentTempoInfoPointIndex = 0;
		CurrentTimeSignaturePointIndex = 0;
		TickResidualWhenDriven = 0.0f;
		BlockSize = InSettings.GetNumFramesPerBlock();
		CurrentBlockFrameIndex = 0;
		FirstTickProcessedThisBlock = -1;
		LastProcessedMidiTick = -1;
		NextMidiTickToProcess = 0;
		NextTempoMapTickToProcess = 0;
		SampleRate = InSettings.GetSampleRate();
		SampleCount = 0;
		FramesUntilNextProcess = 0;
		TransportAtBlockStart = EMusicPlayerTransportState::Invalid;
		TransportAtBlockEnd = EMusicPlayerTransportState::Invalid;
		SpeedAtBlockStart = 0.0f;
		SpeedAtBlockEnd = 0.0f;
		CurrentLocalSpeed = -1.0f;
		TempoAtBlockStart = 0.0f;
		TempoAtBlockEnd = 0.0f;
		TimeSignatureAtBlockStart = FTimeSignature(0, 1);
		TimeSignatureAtBlockEnd = FTimeSignature(0, 1);
		NumTransportChangeInBlock = 0;
		NumSpeedChangeInBlock = 0;
		NumTempoChangeInBlock = 0;
		NumTimeSignatureChangeInBlock = 0;
		NextTempoChangeTick = std::numeric_limits<int32>::max();
		NextTimeSigChangeTick = std::numeric_limits<int32>::max();
		NextTempoOrTimeSigChangeTick = std::numeric_limits<int32>::max();
		FirstTickInLoop = -1;
		LoopLengthTicks = 0;
		MidiDataChangedInBlock = false;
		NeedsSeekToDrivingClock = false;
		MidiClockEventsInBlock.Reset();
	}

	void FMidiClock::SetDrivingClock(FConstSharedMidiClockPtr NewExternalClockDriver)
	{
		MidiDataChangedInBlock = true;
		ExternalClockDriver = NewExternalClockDriver;
		TickResidualWhenDriven = 0.0f;
		NeedsSeekToDrivingClock = true;
		RebuildSongMapEvaluator(ExternalClockDriver ? 
									ExternalClockDriver->SongMapEvaluator->GetSongMapsWithTempoMap() : 
									SongMapEvaluator->GetSongMapsWithOtherMaps(),
								SongMapEvaluator->GetSongMapsWithOtherMaps());
	}

	void FMidiClock::PrepareBlock()
	{
		FirstTickProcessedThisBlock = -1;

		NumTransportChangeInBlock = 0;
		TransportAtBlockStart = TransportAtBlockEnd;
		
		NumSpeedChangeInBlock = 0;
		SpeedAtBlockStart = SpeedAtBlockEnd;
		
		NumTempoChangeInBlock = 0;
		TempoAtBlockStart = TempoAtBlockEnd;

		NumTimeSignatureChangeInBlock = 0;
		TimeSignatureAtBlockStart = TimeSignatureAtBlockEnd;
		
		CurrentBlockFrameIndex = 0;

		MidiDataChangedInBlock = false;

		MidiClockEventsInBlock.Reset();

		if (ExternalClockDriver && ExternalClockDriver->MidiDataChangedInBlock)
		{
			RebuildSongMapEvaluator(ExternalClockDriver->SongMapEvaluator->GetSongMapsWithTempoMap(), SongMapEvaluator->GetSongMapsWithOtherMaps());
		}
	}

	void FMidiClock::SetTransportState(int32 BlockFrameIndex, EMusicPlayerTransportState TransportState)
	{
		AddTransportStateChangeToBlock(BlockFrameIndex, TransportState);
	}

	void FMidiClock::SetSpeed(int32 BlockFrameIndex, float Speed)
	{
		CurrentLocalSpeed = Speed;
		AddSpeedChangeToBlock(BlockFrameIndex, Speed, true);
	}

	void FMidiClock::SetTempo(int32 BlockFrameIndex, int32 Tick, float Bpm, int32 TempoMapTick)
	{
		AddTempoChangeToBlock(BlockFrameIndex, Tick, Bpm, TempoMapTick);
	}

	void HarmonixMetasound::FMidiClock::SetTimeSignature(int32 BlockFrameIndex, int32 Tick, const FTimeSignature& TimeSignature, int32 TempoMapTick)
	{
		AddTimeSignatureChangeToBlock(BlockFrameIndex, Tick, TimeSignature, TempoMapTick);
	}

	void FMidiClock::SeekTo(int32 BlockFrameIndex, const FMusicSeekTarget& InTarget)
	{
		CurrentBlockFrameIndex = BlockFrameIndex;

		int32 Tick = 0;
		switch (InTarget.Type)
		{
		case ESeekPointType::BarBeat:
			Tick = SongMapEvaluator->MusicTimestampToTick(InTarget.BarBeat);
			break;
		default:
		case ESeekPointType::Millisecond:
			Tick = SongMapEvaluator->MsToTick(InTarget.Ms);
			break;
		}

		SeekTo(BlockFrameIndex, Tick, NextTempoMapTickToProcess);
	}

	void FMidiClock::SeekTo(int32 BlockFrameIndex, int32 Tick, int32 TempoMapTick)
	{
		if (NextMidiTickToProcess != Tick)
		{
			AddSeekToBlock(BlockFrameIndex, Tick, TempoMapTick);
			SampleCount = FMath::Max<FSampleCount>(FSampleCount(SongMapEvaluator->TickToMs(NextMidiTickToProcess) / 1000.0f * SampleRate), 0);
		}
	}

	void FMidiClock::AddTransientLoop(int32 BlockFrameIndex, int32 NewFirstTickInLoop, int32 NewLoopLengthTicks)
	{
		AddLoopToBlock(BlockFrameIndex, NewFirstTickInLoop, NewLoopLengthTicks, NextTempoMapTickToProcess);
	}

	void FMidiClock::SetupPersistentLoop(int32 NewFirstTickInLoop, int32 NewLoopLengthTicks)
	{
		if (ensureAlwaysMsgf(NewLoopLengthTicks > (Harmonix::Midi::Constants::GTicksPerQuarterNoteInt / 4), TEXT("For performance reasons, Midi Clock loops must be at least a 1/16th note long!")))
		{
			FirstTickInLoop = NewFirstTickInLoop;
			LoopLengthTicks = NewLoopLengthTicks;
		}
	}

	void FMidiClock::ClearPersistentLoop()
	{
		FirstTickInLoop = -1;
		LoopLengthTicks = 0;
	}

	bool FMidiClock::HasPersistentLoop() const
	{
		return FirstTickInLoop != -1 && LoopLengthTicks > 0;
	}

	float FMidiClock::GetLoopStartMs() const
	{
		if (!HasPersistentLoop() || FirstTickInLoop <= 0)
		{
			return 0.0f;
		}

		return SongMapEvaluator->TickToMs(FirstTickInLoop);
	}

	float FMidiClock::GetLoopEndMs() const
	{
		if (!HasPersistentLoop())
		{
			return 0.0f;
		}

		return SongMapEvaluator->TickToMs(FirstTickInLoop + LoopLengthTicks);
	}

	float FMidiClock::GetLoopLengthMs() const
	{
		return GetLoopEndMs() - GetLoopStartMs();
	}

	void FMidiClock::Advance(const FMidiClock& DrivingClock, int32 StartFrame, int32 NumFrames)
	{
		int32 EndFrame = StartFrame + NumFrames;
		const TArray<FMidiClockEvent>& ClockEvents = DrivingClock.GetMidiClockEventsInBlock();
		int32 Index = Algo::LowerBoundBy(ClockEvents, StartFrame, &FMidiClockEvent::BlockFrameIndex);
		if (ClockEvents.IsValidIndex(0) && NeedsSeekToDrivingClock)
		{
			NeedsSeekToDrivingClock = false;
			int32 DrivingClocksTick = DrivingClock.GetNextTickToProcessAtBlockFrame(StartFrame);
			if (DrivingClocksTick >= 0)
			{
				int32 OurStartTick = DrivingClocksTick;
				if (!FMath::IsNearlyEqual(CurrentLocalSpeed, 1.0f, 0.0001))
				{
					float FractionalToTick = (float)OurStartTick * CurrentLocalSpeed;
					OurStartTick = FMath::FloorToInt32(FractionalToTick);
					TickResidualWhenDriven = FMath::Fractional(FractionalToTick);
				}
				OurStartTick = WrapTickIfLooping(OurStartTick);
				SeekTo(CurrentBlockFrameIndex, OurStartTick, DrivingClocksTick);
			}
		}

		while (ClockEvents.IsValidIndex(Index))
		{
			const FMidiClockEvent& Event = ClockEvents[Index];
			if (Event.BlockFrameIndex >= EndFrame)
			{
				return;
			}

			HandleClockEvent(DrivingClock, Event);
			++Index;
		}
	}

	void FMidiClock::Advance(int32 StartFrame, int32 NumFrames)
	{
		if (TransportAtBlockEnd != EMusicPlayerTransportState::Playing && TransportAtBlockEnd != EMusicPlayerTransportState::Continuing)
		{
			UE_LOGF(LogMIDI, Error, "FMidiClock: Attempt to advance clock that is not playing!");
			return;
		}

		int32 BlockFrameIndex = StartFrame;
		while (NumFrames > FramesUntilNextProcess)
		{
			BlockFrameIndex += FramesUntilNextProcess;
			NumFrames -= FramesUntilNextProcess;
			FramesUntilNextProcess = kMidiGranularity;
			FSampleCount TargetFrame = SampleCount + (FSampleCount)((float)kMidiGranularity * SpeedAtBlockEnd);
			float TargetMs = ((float)TargetFrame * 1000.0f) / SampleRate;
			bool bDidLoop = AdvanceToMs(BlockFrameIndex, TargetMs);
			// AdvanceToMs will have updated SampleCount, but it will have been 
			// quantized to the resulting target midi tick. That is appropriate in 
			// some cases, but in this advance type (by sample frames) we need/want
			// the un-quantized value UNLESS the advance caused a loop...
			if (!bDidLoop)
			{
				// blow away the "quantized" value with our true, un-quantized value here...
				SampleCount = TargetFrame;
			}
		}
		FramesUntilNextProcess -= NumFrames;
	}

	bool FMidiClock::AdvanceToTick(int32 BlockFrameIndex, int32 UpToTick, int32 TempoMapTick)
	{
		using namespace MidiClockMessageTypes;

		bool bDidLoop = false;

		int32 TickAfterLoop = FirstTickInLoop + LoopLengthTicks;
		if (HasPersistentLoop() && UpToTick > TickAfterLoop)
		{
			// first we might need advance to the loop end...
			if (NextMidiTickToProcess > FirstTickInLoop)
			{
				int32 TicksToAdvance = TickAfterLoop - NextMidiTickToProcess;
				AddAdvanceToBlock(BlockFrameIndex, NextMidiTickToProcess, TicksToAdvance, TempoMapTick);
				TempoMapTick += TicksToAdvance;
				AddLoopToBlock(BlockFrameIndex, FirstTickInLoop, LoopLengthTicks, TempoMapTick);
				check(NextMidiTickToProcess == FirstTickInLoop);
				bDidLoop = true;
			}

			int NumTicksLeftToProcess = UpToTick - TickAfterLoop;
			while (NumTicksLeftToProcess)
			{
				int32 TicksThisPass = FMath::Min(LoopLengthTicks, NumTicksLeftToProcess);
				NumTicksLeftToProcess -= TicksThisPass;
				AddAdvanceToBlock(BlockFrameIndex, NextMidiTickToProcess, TicksThisPass, TempoMapTick);
				TempoMapTick += TicksThisPass;
				if (NumTicksLeftToProcess > 0)
				{
					AddLoopToBlock(BlockFrameIndex, FirstTickInLoop, LoopLengthTicks, TempoMapTick);
					check(NextMidiTickToProcess == FirstTickInLoop);
					bDidLoop = true;
				}
			}
		}
		else
		{
			int32 NumTicks = UpToTick - NextMidiTickToProcess;
			AddAdvanceToBlock(BlockFrameIndex, NextMidiTickToProcess, NumTicks, TempoMapTick);
			if (HasPersistentLoop() && NextMidiTickToProcess >= (FirstTickInLoop + LoopLengthTicks))
			{
				TempoMapTick += NumTicks;
				AddLoopToBlock(BlockFrameIndex, FirstTickInLoop, LoopLengthTicks, TempoMapTick);
				check(NextMidiTickToProcess == FirstTickInLoop);
				bDidLoop = true;
			}
		}
		SampleCount = FMath::Max<FSampleCount>(FSampleCount(SongMapEvaluator->TickToMs(NextMidiTickToProcess) / 1000.0f * SampleRate), 0);
		return bDidLoop;
	}

	bool FMidiClock::AdvanceToMs(int32 BlockFrameIndex, float Ms)
	{
		if (!ensureMsgf(!ExternalClockDriver, TEXT("Can't Advance an FMidiClock by Ms when it is being driven by another clock!")))
		{
			return false;
		}

		using namespace MidiClockMessageTypes;

		bool bDidLoop = false;

		int32 ToFutureTick = FMath::RoundToInt32(SongMapEvaluator->MsToTick(Ms));

		// NOTE: We CAN'T just pass this calculated future tick to the AdvanceToTick function
		// and let it wrap around the loop!... Because tempo changes during the loop might result in 
		// a different wrapped tick. So we have to do this here where we will wrap in "ms space"
		// instead of "tick space" each time around the loop...

		// do we need to wrap around the loop?
		int32 TickAfterLoop = FirstTickInLoop + LoopLengthTicks;
		if (HasPersistentLoop() && ToFutureTick > TickAfterLoop)
		{
			// first we might need advance to the loop end...
			if (NextMidiTickToProcess < TickAfterLoop)
			{
				AddAdvanceToBlock(BlockFrameIndex, NextMidiTickToProcess, TickAfterLoop - NextMidiTickToProcess, NextMidiTickToProcess);
			}

			AddLoopToBlock(BlockFrameIndex, FirstTickInLoop, LoopLengthTicks, FirstTickInLoop);
			check(NextMidiTickToProcess == FirstTickInLoop);
			bDidLoop = true;

			float MsRemainingAfterProcessingToLoopEnd = Ms - GetLoopEndMs();

			while (MsRemainingAfterProcessingToLoopEnd > 0.0f)
			{
				ToFutureTick = SongMapEvaluator->MsToTick(GetLoopStartMs() + MsRemainingAfterProcessingToLoopEnd);
				if (ToFutureTick > TickAfterLoop)
				{
					AddAdvanceToBlock(BlockFrameIndex, NextMidiTickToProcess, TickAfterLoop - NextMidiTickToProcess, NextMidiTickToProcess);
					AddLoopToBlock(BlockFrameIndex, FirstTickInLoop, LoopLengthTicks, FirstTickInLoop);
					check(NextMidiTickToProcess == FirstTickInLoop);
					MsRemainingAfterProcessingToLoopEnd -= GetLoopLengthMs();
				}
				else
				{
					AddAdvanceToBlock(BlockFrameIndex, NextMidiTickToProcess, ToFutureTick - NextMidiTickToProcess, NextMidiTickToProcess);
					MsRemainingAfterProcessingToLoopEnd = 0.0f;
				}
			}

			// Because we did some looping, we need to update our SampleCount so it is "wrapped" appropriately...
			SampleCount = FMath::Max<FSampleCount>(FSampleCount(SongMapEvaluator->TickToMs(NextMidiTickToProcess) / 1000.0f * SampleRate), 0);
		}
		else
		{
			if (ToFutureTick > NextMidiTickToProcess)
			{
				AddAdvanceToBlock(BlockFrameIndex, NextMidiTickToProcess, ToFutureTick - NextMidiTickToProcess, NextMidiTickToProcess);
				if (HasPersistentLoop() && LastProcessedMidiTick == TickAfterLoop)
				{
					AddLoopToBlock(BlockFrameIndex, FirstTickInLoop, LoopLengthTicks, FirstTickInLoop);
					check(NextMidiTickToProcess == FirstTickInLoop);
					bDidLoop = true;
				}
				SampleCount = FMath::Max<FSampleCount>(FSampleCount(Ms / 1000.0f * SampleRate), 0);
			}
		}
		return bDidLoop;
	}

	float FMidiClock::GetSpeedAtBlockSampleFrame(int32 FrameIndex) const
	{
		if (NumSpeedChangeInBlock == 0 || FrameIndex == 0)
		{
			return SpeedAtBlockStart;
		}

		float SpeedCandidate = SpeedAtBlockStart;
		for (const FMidiClockEvent& ClockEvent : MidiClockEventsInBlock)
		{
			if (const MidiClockMessageTypes::FSpeedChange* AsSpeedChange = ClockEvent.TryGet<MidiClockMessageTypes::FSpeedChange>())
			{
				if (ClockEvent.BlockFrameIndex > FrameIndex)
				{
					break;
				}
				SpeedCandidate = AsSpeedChange->Speed;
			}
		}
		return SpeedCandidate;
	}

	float FMidiClock::GetTempoAtBlockSampleFrame(int32 FrameIndex) const
	{
		if (ExternalClockDriver)
		{
			return ExternalClockDriver->GetTempoAtBlockSampleFrame(FrameIndex);
		}

		if (NumTempoChangeInBlock == 0 || FrameIndex == 0)
		{
			return TempoAtBlockStart;
		}

		float TempoCandidate = TempoAtBlockStart;
		for (const FMidiClockEvent& ClockEvent : MidiClockEventsInBlock)
		{
			if (const MidiClockMessageTypes::FTempoChange* AsTempoChange = ClockEvent.TryGet<MidiClockMessageTypes::FTempoChange>())
			{
				if (ClockEvent.BlockFrameIndex > FrameIndex)
				{
					break;
				}
				TempoCandidate = AsTempoChange->Tempo;
			}
		}
		return TempoCandidate;
	}

	float FMidiClock::GetCurrentSongPosMs() const
	{
		// We have processed THRU MostRecentProcessedMidiTick,
		// so our Ms position is AFTER that tick, just before the 
		// next tick...
		return SongMapEvaluator->TickToMs(NextMidiTickToProcess);
	}

	FMusicTimestamp FMidiClock::GetMusicTimestampAtBlockEnd() const
	{
		return SongMapEvaluator->TickToMusicTimestamp(NextMidiTickToProcess);
	}

	FMusicTimestamp FMidiClock::GetMusicTimestampAtBlockOffset(const int32 Offset) const
	{
		return SongMapEvaluator->TickToMusicTimestamp(GetNextTickToProcessAtBlockFrame(Offset));
	}

	float FMidiClock::GetSongPosMsAtBlockOffset(int32 Offset) const
	{
		return SongMapEvaluator->TickToMs(GetNextTickToProcessAtBlockFrame(Offset));
	}

	int32 FMidiClock::WrapTickIfLooping(int32 Tick) const
	{
		if (HasPersistentLoop())
		{
			// only wrap the tick if it we're passed the loop end tick
			int32 LastTickInLoop = FirstTickInLoop + LoopLengthTicks - 1;
			if (Tick > LastTickInLoop)
			{
				return FirstTickInLoop + ((Tick - LastTickInLoop - 1) % LoopLengthTicks);
			}
		}
		return Tick;
	}

	int32 FMidiClock::GetNextTickToProcessAtBlockFrame(int32 BlockFrame) const
	{
		if (BlockFrame == 0)
		{
			return FirstTickProcessedThisBlock < 0 ? 0 : FirstTickProcessedThisBlock;
		}

		int32 FoundNextTick = FirstTickProcessedThisBlock;
		int32 AtBlockIndex = 0;

		using namespace MidiClockMessageTypes;
		// we're going to have to look through the clock events...
		for (const FMidiClockEvent& Event : MidiClockEventsInBlock)
		{
			if (Event.BlockFrameIndex > BlockFrame)
			{
				break;
			}

			if (const FSeek* AsSeek = Event.TryGet<FSeek>())
			{
				FoundNextTick = AsSeek->NewNextTick;
				AtBlockIndex = Event.BlockFrameIndex;
			}
			else if (const FAdvance* AsAdvance = Event.TryGet<FAdvance>())
			{
				if (Event.BlockFrameIndex == BlockFrame)
				{
					FoundNextTick = AsAdvance->FirstTickToProcess;
				}
				else
				{
					FoundNextTick = AsAdvance->FirstTickToProcess + AsAdvance->NumberOfTicksToProcess;
				}
				AtBlockIndex = Event.BlockFrameIndex;
			}
			else if (const FLoop* AsLoop = Event.TryGet<FLoop>())
			{
				FoundNextTick = AsLoop->FirstTickInLoop;
				AtBlockIndex = Event.BlockFrameIndex;
			}
			else if (const FTempoChange* AsTempoChange = Event.TryGet<FTempoChange>())
			{
				FoundNextTick = AsTempoChange->Tick;
				AtBlockIndex = Event.BlockFrameIndex;
			}
			else if (const FTimeSignatureChange* AsTimeSigChange = Event.TryGet<FTimeSignatureChange>())
			{
				FoundNextTick = AsTimeSigChange->Tick;
				AtBlockIndex = Event.BlockFrameIndex;
			}
		}

		return FoundNextTick < 0 ? 0 : FoundNextTick;
	}

	void FMidiClock::AddEvent(const FMidiClockEvent& InEvent, bool bRequireSequential)
	{
		if (bRequireSequential && !MidiClockEventsInBlock.IsEmpty())
		{
			check(MidiClockEventsInBlock.Last().BlockFrameIndex <= InEvent.BlockFrameIndex);
		}

		MidiClockEventsInBlock.Add(InEvent);
	}

	void FMidiClock::HandleClockEvent(const FMidiClock& DrivingClock, const FMidiClockEvent& Event)
	{
		using namespace MidiClockMessageTypes;

		if (const FLoop* AsLoop = Event.TryGet<FLoop>())
		{
			const int32 Tick = WrapTickIfLooping(AsLoop->FirstTickInLoop);
			SeekTo(Event.BlockFrameIndex, Tick, AsLoop->TempoMapTick);
			TickResidualWhenDriven = 0.0f;
		}
		else if (const FSeek* AsSeek = Event.TryGet<FSeek>())
		{
			const int32 Tick = WrapTickIfLooping(AsSeek->NewNextTick);
			SeekTo(Event.BlockFrameIndex, Tick, AsSeek->TempoMapTick);
			TickResidualWhenDriven = 0.0f;
		}
		else if (const FAdvance* AsAdvance = Event.TryGet<FAdvance>())
		{
			// Advance based on the delta ticks, and not based on the absolute tick
			int32 UpToTick;
			if (FMath::IsNearlyEqual(CurrentLocalSpeed, 1.0f, 0.0001))
			{
				UpToTick = NextMidiTickToProcess + AsAdvance->NumberOfTicksToProcess;
			}
			else
			{
				float FractionalToTick = (float)NextMidiTickToProcess + ((float)AsAdvance->NumberOfTicksToProcess * CurrentLocalSpeed) + TickResidualWhenDriven;
				UpToTick = FMath::FloorToInt32(FractionalToTick);
				TickResidualWhenDriven = FMath::Fractional(FractionalToTick);
			}
			// No need to wrap the tick here because AdvanceToTick will handle that.
			AdvanceToTick(Event.BlockFrameIndex, UpToTick, AsAdvance->TempoMapTick);
		}
		else if (const FTempoChange* AsTempoChange = Event.TryGet<FTempoChange>())
		{
			AddTempoChangeToBlock(Event.BlockFrameIndex, NextMidiTickToProcess, AsTempoChange->Tempo, AsTempoChange->TempoMapTick);
		}
		else if (const FSpeedChange* AsSpeedChange = Event.TryGet<FSpeedChange>())
		{
			AddSpeedChangeToBlock(Event.BlockFrameIndex, AsSpeedChange->Speed, false);
		}
		else if (const FTransportChange* AsTransportChange = Event.TryGet<FTransportChange>())
		{
			AddTransportStateChangeToBlock(Event.BlockFrameIndex, AsTransportChange->TransportState);
		}
	}

	void FMidiClock::PostTempoOrTimeSignatureEventsIfNeeded()
	{
		if (SongMapEvaluator->GetNumTempoChanges() == 0 || ExternalClockDriver)
		{
			CurrentTempoInfoPointIndex = -1;
			if (!ExternalClockDriver)
			{
				AddTempoChangeToBlock(CurrentBlockFrameIndex, NextMidiTickToProcess, 120.0f, NextMidiTickToProcess);
			}
			NextTempoChangeTick = std::numeric_limits<int32>::max();
		}
		else
		{
			CurrentTempoInfoPointIndex = SongMapEvaluator->GetTempoPointIndexForTick(NextMidiTickToProcess > 0 ? NextMidiTickToProcess : 0);
			check(CurrentTempoInfoPointIndex >= 0 && CurrentTempoInfoPointIndex < SongMapEvaluator->GetNumTempoChanges());
			AddTempoChangeToBlock(CurrentBlockFrameIndex, NextMidiTickToProcess, SongMapEvaluator->GetTempoInfoPoint(CurrentTempoInfoPointIndex)->GetBPM(), NextMidiTickToProcess);
			if ((CurrentTempoInfoPointIndex + 1) < SongMapEvaluator->GetNumTempoChanges())
			{
				NextTempoChangeTick = SongMapEvaluator->GetTempoChangePointTick(CurrentTempoInfoPointIndex + 1);
			}
			else
			{
				NextTempoChangeTick = std::numeric_limits<int32>::max();
			}
		}

		if (SongMapEvaluator->GetNumTimeSignatureChanges() == 0)
		{
			CurrentTimeSignaturePointIndex = -1;
			AddTimeSignatureChangeToBlock(CurrentBlockFrameIndex, NextMidiTickToProcess, FTimeSignature(4,4), NextTempoMapTickToProcess);
			NextTimeSigChangeTick = std::numeric_limits<int32>::max();
		}
		else
		{
			CurrentTimeSignaturePointIndex = SongMapEvaluator->GetTimeSignaturePointIndexForTick(NextMidiTickToProcess > 0 ? NextMidiTickToProcess : 0);
			check(CurrentTimeSignaturePointIndex >= 0 && CurrentTimeSignaturePointIndex < SongMapEvaluator->GetNumTimeSignatureChanges());
			AddTimeSignatureChangeToBlock(CurrentBlockFrameIndex, NextMidiTickToProcess, SongMapEvaluator->GetTimeSignaturePoint(CurrentTimeSignaturePointIndex)->TimeSignature, NextTempoMapTickToProcess);
			if ((CurrentTimeSignaturePointIndex + 1) < SongMapEvaluator->GetNumTimeSignatureChanges())
			{
				NextTimeSigChangeTick = SongMapEvaluator->GetTimeSignatureChangePointTick(CurrentTimeSignaturePointIndex + 1);
			}
			else
			{
				NextTimeSigChangeTick = std::numeric_limits<int32>::max();
			}
			NextTempoOrTimeSigChangeTick = FMath::Min(NextTimeSigChangeTick, NextTempoChangeTick);
		}

		if (CurrentLocalSpeed < 0.0f)
		{
			SetSpeed(CurrentBlockFrameIndex, 1.0f);
		}
	}

	void FMidiClock::AddTransportStateChangeToBlock(int32 BlockFrameIndex, EMusicPlayerTransportState NewTransportState)
	{
		check(BlockFrameIndex >= CurrentBlockFrameIndex);

		CurrentBlockFrameIndex = BlockFrameIndex;

		if (TransportAtBlockEnd == NewTransportState)
		{
			// no need to add the transport message... it is already the current transport. 
			return;
		}

		if (NewTransportState == EMusicPlayerTransportState::Playing || NewTransportState == EMusicPlayerTransportState::Prepared)
		{
			PostTempoOrTimeSignatureEventsIfNeeded();
			FramesUntilNextProcess = 0;
		}

		AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FTransportChange(NewTransportState)));
		NumTransportChangeInBlock++;

		if (BlockFrameIndex == 0)
		{
			TransportAtBlockStart = NewTransportState;
		}

		TransportAtBlockEnd = NewTransportState;
	}

	void FMidiClock::AddTimeSignatureChangeToBlock(int32 BlockFrameIndex, int32 Tick, const FTimeSignature& TimeSignature, int32 TempoMapTick)
	{
		check(BlockFrameIndex >= CurrentBlockFrameIndex);

		check(Tick >= NextMidiTickToProcess);

		if (FirstTickProcessedThisBlock == -1)
		{
			FirstTickProcessedThisBlock = Tick;
		}

		CurrentBlockFrameIndex = BlockFrameIndex;
		NextMidiTickToProcess = Tick;
		NextTempoMapTickToProcess = TempoMapTick;

		if (TimeSignatureAtBlockEnd == TimeSignature)
		{
			// no need to add the time signature... it is already the current time signature. 
			return;
		}

		if (MidiClockMessageTypes::FTimeSignatureChange* PreviousTimeSigChangeOnSameTick = LookForEventOnMidiTick<MidiClockMessageTypes::FTimeSignatureChange>(BlockFrameIndex))
		{
			PreviousTimeSigChangeOnSameTick->TimeSignature = TimeSignature;
			PreviousTimeSigChangeOnSameTick->TempoMapTick = TempoMapTick;
		}
		else
		{
			AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FTimeSignatureChange(Tick, FTimeSignature(TimeSignature), TempoMapTick)));
			NumTimeSignatureChangeInBlock++;
		}

		if (BlockFrameIndex == 0)
		{
			TimeSignatureAtBlockStart = TimeSignature;
		}

		TimeSignatureAtBlockEnd = TimeSignature;
	}

	void FMidiClock::AddTempoChangeToBlock(int32 BlockFrameIndex, int32 Tick, float NewTempo, int32 TempoMapTick)
	{
		check(BlockFrameIndex >= CurrentBlockFrameIndex);

		check(Tick >= NextMidiTickToProcess);

		CurrentBlockFrameIndex = BlockFrameIndex;
		NextMidiTickToProcess = Tick;
		NextTempoMapTickToProcess = TempoMapTick;

		if (FirstTickProcessedThisBlock == -1)
		{
			FirstTickProcessedThisBlock = Tick;
		}

		if (TempoAtBlockEnd == NewTempo)
		{
			// no need to add the tempo... it is already the current tempo. 
			return;
		}

		if (MidiClockMessageTypes::FTempoChange* PreviousTempoChangeOnSameTick = LookForEventOnMidiTick<MidiClockMessageTypes::FTempoChange>(BlockFrameIndex))
		{
			PreviousTempoChangeOnSameTick->Tempo = NewTempo;
			PreviousTempoChangeOnSameTick->TempoMapTick = TempoMapTick;
		}
		else
		{
			AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FTempoChange(Tick, NewTempo, TempoMapTick)));
			NumTempoChangeInBlock++;
		}

		if (BlockFrameIndex == 0)
		{
			TempoAtBlockStart = NewTempo;
		}

		TempoAtBlockEnd = NewTempo;
	}

	void FMidiClock::AddSpeedChangeToBlock(int32 BlockFrameIndex, float NewSpeed, bool bIsNewLocalSpeed)
	{
		check(BlockFrameIndex >= CurrentBlockFrameIndex);

		CurrentBlockFrameIndex = BlockFrameIndex;

		if (ExternalClockDriver)
		{
			// actual speed message is the 
			if (bIsNewLocalSpeed)
			{
				NewSpeed *= ExternalClockDriver->GetSpeedAtBlockSampleFrame(BlockFrameIndex);
			}
			else
			{
				NewSpeed *= CurrentLocalSpeed;
			}
		}

		if (SpeedAtBlockEnd == NewSpeed)
		{
			// no need to add the speed... it is already the current speed. 
			return;
		}

		if (MidiClockMessageTypes::FSpeedChange* PreviousSpeedChangeOnSameTick = LookForEventOnBlockFrameIndex<MidiClockMessageTypes::FSpeedChange>(BlockFrameIndex))
		{
			PreviousSpeedChangeOnSameTick->Speed = NewSpeed;
		}
		else
		{
			AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FSpeedChange(NewSpeed)));
			NumSpeedChangeInBlock++;
		}

		if (BlockFrameIndex == 0)
		{
			SpeedAtBlockStart = NewSpeed;
		}

		SpeedAtBlockEnd = NewSpeed;
	}

	void FMidiClock::AddLoopToBlock(int32 BlockFrameIndex, int32 FirstTick, int32 LoopLength, int32 TempoMapTick)
	{
		check(BlockFrameIndex >= CurrentBlockFrameIndex);
		check(LastProcessedMidiTick == FirstTick + LoopLength - 1);

		CurrentBlockFrameIndex = BlockFrameIndex;

		if (NextMidiTickToProcess == FirstTick)
		{
			return;
		}

		if (MidiClockMessageTypes::FLoop* PreviousLoopOnSameTick = LookForEventOnBlockFrameIndex<MidiClockMessageTypes::FLoop>(BlockFrameIndex))
		{
			PreviousLoopOnSameTick->LengthInTicks = LoopLength;
			PreviousLoopOnSameTick->FirstTickInLoop = FirstTick;
			PreviousLoopOnSameTick->TempoMapTick = TempoMapTick;
		}
		else
		{
			AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FLoop(FirstTick, LoopLength, TempoMapTick)));
		}

		NextMidiTickToProcess = FirstTick;
		NextTempoMapTickToProcess = TempoMapTick;

		// The tempo and/or time signature may be different in the location where we are going... so... update...
		PostTempoOrTimeSignatureEventsIfNeeded();
	}

	void FMidiClock::AddSeekToBlock(int32 BlockFrameIndex, int32 ToTick, int32 TempoMapTick)
	{
		check(BlockFrameIndex >= CurrentBlockFrameIndex);

		CurrentBlockFrameIndex = BlockFrameIndex;

		if (NextMidiTickToProcess == ToTick)
		{
			return;
		}

		if (MidiClockMessageTypes::FSeek* PreviousSeekOnSameTick = LookForEventOnBlockFrameIndex<MidiClockMessageTypes::FSeek>(BlockFrameIndex))
		{
			check(PreviousSeekOnSameTick->LastTickProcessedBeforeSeek == LastProcessedMidiTick);
			PreviousSeekOnSameTick->NewNextTick = ToTick;
			PreviousSeekOnSameTick->TempoMapTick = TempoMapTick;
		}
		else
		{
			AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FSeek(LastProcessedMidiTick, ToTick, TempoMapTick)));
		}

		NextMidiTickToProcess = ToTick;
		NextTempoMapTickToProcess = TempoMapTick;

		// The tempo and/or time signature may be different in the location where we are going... so... update...
		PostTempoOrTimeSignatureEventsIfNeeded();
	}

	void FMidiClock::AddAdvanceToBlock(int32 BlockFrameIndex, int32 FirstTick, int32 NumTicks, int32 TempoMapTick)
	{
		check(BlockFrameIndex >= CurrentBlockFrameIndex);
		check(FirstTick == NextMidiTickToProcess);

		CurrentBlockFrameIndex = BlockFrameIndex;

		if (FirstTickProcessedThisBlock == -1)
		{
			FirstTickProcessedThisBlock = FirstTick;
		}

		// moving forward may cause us to move into a new tempo and/or time signature...
		while (FirstTick <= NextTempoOrTimeSigChangeTick && NextTempoOrTimeSigChangeTick < (FirstTick + NumTicks))
		{
			// process ticks UP TO the tempo or time signature change...
			int32 SpanNumTicks = NextTempoOrTimeSigChangeTick - FirstTick;
			if (SpanNumTicks > 0)
			{
				AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FAdvance(FirstTick, SpanNumTicks, TempoMapTick)));
				TempoMapTick += SpanNumTicks;
				NextMidiTickToProcess = FirstTick + SpanNumTicks;
				LastProcessedMidiTick = NextMidiTickToProcess - 1;
				FirstTick = NextMidiTickToProcess;
				NumTicks -= SpanNumTicks;
			}
			check(NextMidiTickToProcess == NextTempoOrTimeSigChangeTick);

			// time signature or tempo?
			if (NextMidiTickToProcess == NextTempoChangeTick && CurrentTempoInfoPointIndex != -1)
			{
				CurrentTempoInfoPointIndex++;
				const FTempoInfoPoint* TempoPoint = SongMapEvaluator->GetTempoInfoPoint(CurrentTempoInfoPointIndex);
				check(TempoPoint);
				if (!ExternalClockDriver)
				{
					AddTempoChangeToBlock(BlockFrameIndex, TempoPoint->StartTick, TempoPoint->GetBPM(), TempoMapTick);
				}
				int32 NextTempoInfoPointIndex = CurrentTempoInfoPointIndex + 1;
				if (NextTempoInfoPointIndex < SongMapEvaluator->GetNumTempoChanges())
				{
					const FTempoInfoPoint* NextTempoPoint = SongMapEvaluator->GetTempoInfoPoint(NextTempoInfoPointIndex);
					check(NextTempoPoint);
					NextTempoChangeTick = NextTempoPoint->StartTick;
				}
				else
				{
					NextTempoChangeTick = std::numeric_limits<int32>::max();
				}
			}
			// time signature or tempo?
			if (NextMidiTickToProcess == NextTimeSigChangeTick && CurrentTimeSignaturePointIndex != -1)
			{
				CurrentTimeSignaturePointIndex++;
				const FTimeSignaturePoint* TimeSignaturePoint = SongMapEvaluator->GetTimeSignaturePoint(CurrentTimeSignaturePointIndex);
				check(TimeSignaturePoint);
				AddTimeSignatureChangeToBlock(BlockFrameIndex, TimeSignaturePoint->StartTick, TimeSignaturePoint->TimeSignature, TempoMapTick);
				int32 NextTimeSignaturePointIndex = CurrentTimeSignaturePointIndex + 1;
				if (NextTimeSignaturePointIndex < SongMapEvaluator->GetNumTimeSignatureChanges())
				{
					const FTimeSignaturePoint* NextTimeSigPoint = SongMapEvaluator->GetTimeSignaturePoint(NextTimeSignaturePointIndex);
					check(NextTimeSigPoint);
					NextTimeSigChangeTick = NextTimeSigPoint->StartTick;
				}
				else
				{
					NextTimeSigChangeTick = std::numeric_limits<int32>::max();
				}
			}
			NextTempoOrTimeSigChangeTick = FMath::Min(NextTimeSigChangeTick, NextTempoChangeTick);
		}

		if (NumTicks > 0)
		{
			AddEvent(FMidiClockEvent(BlockFrameIndex, MidiClockMessageTypes::FAdvance(FirstTick, NumTicks, TempoMapTick)));
			NextMidiTickToProcess = FirstTick + NumTicks;
			TempoMapTick += NumTicks;
			LastProcessedMidiTick = NextMidiTickToProcess - 1;
		}

		NextTempoMapTickToProcess = TempoMapTick;
	}

	void FMidiClock::RebuildSongMapEvaluator(const TSharedPtr<const ISongMapEvaluator>& MapWithTempo, const TSharedPtr<const ISongMapEvaluator>& MapWithOtherMaps)
	{
		SongMapEvaluator = MakeShared<FSongMapsWithAlternateTempoSource>(MapWithTempo, MapWithOtherMaps);
		SongMapsChanged();
	}

}

#undef LOCTEXT_NAMESPACE
