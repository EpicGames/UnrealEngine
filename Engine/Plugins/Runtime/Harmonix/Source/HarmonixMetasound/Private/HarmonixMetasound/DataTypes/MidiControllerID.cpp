// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MidiControllerID.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiControllerID)

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace Metasound {
	DEFINE_METASOUND_ENUM_BEGIN(EStdMidiControllerID, FEnumStdMidiControllerID, "StdMIDIControllerID")
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::BankSelection, "EStdMidiControllerID_BankSelection_DisplayName", "0  Bank Selection", "EStdMidiControllerID_BankSelection_TT", "Allow switching between groups of 128 programs"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::ModWheel, "EStdMidiControllerID_ModWheel_DisplayName", "1  Modulation Wheel", "EStdMidiControllerID_ModWheel_TT", "Modulation Wheel (pitch, loudness, brighness etc.)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Breath, "EStdMidiControllerID_Breath_DisplayName", "2  Breath", "EStdMidiControllerID_Breath_TT", "Breath Controller"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined3, "EStdMidiControllerID_Undefined3_DisplayName", "3  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::FootController, "EStdMidiControllerID_FootController_DisplayName", "4  Foot Controller", "EStdMidiControllerID_FootController_TT", "Send a continuous stream of values based on how the pedal is used"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::PortamentoTime, "EStdMidiControllerID_PortamentoTime_DisplayName", "5  Portamento Time", "EStdMidiControllerID_PortamentoTime_TT", "Controls portamento rate to slide between 2 notes played subsequently"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::DataCoarse, "EStdMidiControllerID_DataCoarse_DisplayName", "6  Data Coarse", "EStdMidiControllerID_DataCoarse_TT", "Controls Value for NRPN or RPN parameters"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Volume, "EStdMidiControllerID_Volume_DisplayName", "7  Volume", "EStdMidiControllerID_Volume_TT", "Volume Control"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Balance, "EStdMidiControllerID_Balance_DisplayName", "8  Balance", "EStdMidiControllerID_Balance_TT", "Controls the left and right balance for Stereo (0 = hard left, 64 = center, 127 = hard right)"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined9, "EStdMidiControllerID_Undefined9_DisplayName", "9  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Pan, "EStdMidiControllerID_Pan_DisplayName", "10  Pan", "EStdMidiControllerID_Pan_TT", "Controls the left and right balance for Mono (0 = hard left, 64 = center, 127 = hard right)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Expression, "EStdMidiControllerID_Expression_DisplayName", "11  Expression", "EStdMidiControllerID_Expression_TT", "Expression is a percentage of volume (CC7)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::EffectControl1, "EStdMidiControllerID_EffectControl1_DisplayName", "12  Effect Controller 1", "EStdMidiControllerID_EffectControl1_TT", "Control a parameter of an effect"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::EffectControl2, "EStdMidiControllerID_EffectControl2_DisplayName", "13  Effect Controller 2", "EStdMidiControllerID_EffectControl2_TT", "Control a parameter of an effect"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined14, "EStdMidiControllerID_Undefined14_DisplayName", "14  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined15, "EStdMidiControllerID_Undefined15_DisplayName", "15  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::GeneralPurposeController1, "EStdMidiControllerID_GeneralPurposeController1_DisplayName", "16  General Purpose Controller 1"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::GeneralPurposeController2, "EStdMidiControllerID_GeneralPurposeController2_DisplayName", "17  General Purpose Controller 2"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::GeneralPurposeController3, "EStdMidiControllerID_GeneralPurposeController3_DisplayName", "18  General Purpose Controller 3"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::GeneralPurposeController4, "EStdMidiControllerID_GeneralPurposeController4_DisplayName", "19  General Purpose Controller 4"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined20, "EStdMidiControllerID_Undefined20_DisplayName", "20  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined21, "EStdMidiControllerID_Undefined21_DisplayName", "21  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined22, "EStdMidiControllerID_Undefined22_DisplayName", "22  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined23, "EStdMidiControllerID_Undefined23_DisplayName", "23  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined24, "EStdMidiControllerID_Undefined24_DisplayName", "24  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined25, "EStdMidiControllerID_Undefined25_DisplayName", "25  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined26, "EStdMidiControllerID_Undefined26_DisplayName", "26  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined27, "EStdMidiControllerID_Undefined27_DisplayName", "27  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined28, "EStdMidiControllerID_Undefined28_DisplayName", "28  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined29, "EStdMidiControllerID_Undefined29_DisplayName", "29  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined30, "EStdMidiControllerID_Undefined30_DisplayName", "30  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined31, "EStdMidiControllerID_Undefined31_DisplayName", "31  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl0, "EStdMidiControllerID_LSBControl0_DisplayName", "32  LSB Controller 0", "EStdMidiControllerID_LSBControl0_TT", "Controller 0 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl1, "EStdMidiControllerID_LSBControl1_DisplayName", "33  LSB Controller 1", "EStdMidiControllerID_LSBControl1_TT", "Controller 1 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl2, "EStdMidiControllerID_LSBControl2_DisplayName", "34  LSB Controller 2", "EStdMidiControllerID_LSBControl2_TT", "Controller 2 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl3, "EStdMidiControllerID_LSBControl3_DisplayName", "35  LSB Controller 3", "EStdMidiControllerID_LSBControl3_TT", "Controller 3 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl4, "EStdMidiControllerID_LSBControl4_DisplayName", "36  LSB Controller 4", "EStdMidiControllerID_LSBControl4_TT", "Controller 4 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl5, "EStdMidiControllerID_LSBControl5_DisplayName", "37  LSB Controller 5", "EStdMidiControllerID_LSBControl5_TT", "Controller 5 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl6, "EStdMidiControllerID_LSBControl6_DisplayName", "38  LSB Controller 6", "EStdMidiControllerID_LSBControl6_TT", "Controller 6 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl7, "EStdMidiControllerID_LSBControl7_DisplayName", "39  LSB Controller 7", "EStdMidiControllerID_LSBControl7_TT", "Controller 7 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl8, "EStdMidiControllerID_LSBControl8_DisplayName", "40  LSB Controller 8", "EStdMidiControllerID_LSBControl8_TT", "Controller 8 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl9, "EStdMidiControllerID_LSBControl9_DisplayName", "41  LSB Controller 9", "EStdMidiControllerID_LSBControl9_TT", "Controller 9 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl10, "EStdMidiControllerID_LSBControl10_DisplayName", "42  LSB Controller 10", "EStdMidiControllerID_LSBControl10_TT", "Controller 10 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl11, "EStdMidiControllerID_LSBControl11_DisplayName", "43  LSB Controller 11", "EStdMidiControllerID_LSBControl11_TT", "Controller 11 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl12, "EStdMidiControllerID_LSBControl12_DisplayName", "44  LSB Controller 12", "EStdMidiControllerID_LSBControl12_TT", "Controller 12 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl13, "EStdMidiControllerID_LSBControl13_DisplayName", "45  LSB Controller 13", "EStdMidiControllerID_LSBControl13_TT", "Controller 13 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl14, "EStdMidiControllerID_LSBControl14_DisplayName", "46  LSB Controller 14", "EStdMidiControllerID_LSBControl14_TT", "Controller 14 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl15, "EStdMidiControllerID_LSBControl15_DisplayName", "47  LSB Controller 15", "EStdMidiControllerID_LSBControl15_TT", "Controller 15 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl16, "EStdMidiControllerID_LSBControl16_DisplayName", "48  LSB Controller 16", "EStdMidiControllerID_LSBControl16_TT", "Controller 16 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl17, "EStdMidiControllerID_LSBControl17_DisplayName", "49  LSB Controller 17", "EStdMidiControllerID_LSBControl17_TT", "Controller 17 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl18, "EStdMidiControllerID_LSBControl18_DisplayName", "50  LSB Controller 18", "EStdMidiControllerID_LSBControl18_TT", "Controller 18 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl19, "EStdMidiControllerID_LSBControl19_DisplayName", "51  LSB Controller 19", "EStdMidiControllerID_LSBControl19_TT", "Controller 19 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl20, "EStdMidiControllerID_LSBControl20_DisplayName", "52  LSB Controller 20", "EStdMidiControllerID_LSBControl20_TT", "Controller 20 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl21, "EStdMidiControllerID_LSBControl21_DisplayName", "53  LSB Controller 21", "EStdMidiControllerID_LSBControl21_TT", "Controller 21 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl22, "EStdMidiControllerID_LSBControl22_DisplayName", "54  LSB Controller 22", "EStdMidiControllerID_LSBControl22_TT", "Controller 22 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl23, "EStdMidiControllerID_LSBControl23_DisplayName", "55  LSB Controller 23", "EStdMidiControllerID_LSBControl23_TT", "Controller 23 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl24, "EStdMidiControllerID_LSBControl24_DisplayName", "56  LSB Controller 24", "EStdMidiControllerID_LSBControl24_TT", "Controller 24 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl25, "EStdMidiControllerID_LSBControl25_DisplayName", "57  LSB Controller 25", "EStdMidiControllerID_LSBControl25_TT", "Controller 25 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl26, "EStdMidiControllerID_LSBControl26_DisplayName", "58  LSB Controller 26", "EStdMidiControllerID_LSBControl26_TT", "Controller 26 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl27, "EStdMidiControllerID_LSBControl27_DisplayName", "59  LSB Controller 27", "EStdMidiControllerID_LSBControl27_TT", "Controller 27 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl28, "EStdMidiControllerID_LSBControl28_DisplayName", "60  LSB Controller 28", "EStdMidiControllerID_LSBControl28_TT", "Controller 28 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl29, "EStdMidiControllerID_LSBControl29_DisplayName", "61  LSB Controller 29", "EStdMidiControllerID_LSBControl29_TT", "Controller 29 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl30, "EStdMidiControllerID_LSBControl30_DisplayName", "62  LSB Controller 30", "EStdMidiControllerID_LSBControl30_TT", "Controller 30 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LSBControl31, "EStdMidiControllerID_LSBControl31_DisplayName", "63  LSB Controller 31", "EStdMidiControllerID_LSBControl31_TT", "Controller 31 Least Significant Bit"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Hold, "EStdMidiControllerID_Hold_DisplayName", "64  Hold", "EStdMidiControllerID_Hold_TT", "On/Off switch that controls sustain (0 to 63 = Off, 64 to 127 = On)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::PortamentoSwitch, "EStdMidiControllerID_PortamentoSwitch_DisplayName", "65  Portamento Switch", "EStdMidiControllerID_PortamentoSwitch_TT", "Portamento On/Off switch (0 to 63 = Off, 64 to 127 = On)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Sustenuto, "EStdMidiControllerID_Sustenuto_DisplayName", "66  Sustenuto Pedal", "EStdMidiControllerID_Sustenuto_TT", "Sustain controller that ONLY holds notes that were “On” when the pedal was pressed (0 to 63 = Off, 64 to 127 = On)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoftPedal, "EStdMidiControllerID_SoftPedal_DisplayName", "67  Soft Pedal", "EStdMidiControllerID_SoftPedal_TT", "On/Off switch taht lowers the volume of notes played"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Legato, "EStdMidiControllerID_Legato_DisplayName", "68  Legato", "EStdMidiControllerID_Legato_TT", "Legato On/Off switch"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Hold2, "EStdMidiControllerID_Hold2_DisplayName", "69  Hold 2", "EStdMidiControllerID_Hold2_TT", "Another way to hold notes .Notes fade out according to their release parameter rather than when the pedal is released"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController1, "EStdMidiControllerID_SoundController1_DisplayName", "70  Sound Controller 1", "EStdMidiControllerID_SoundController1_TT", "Controls the way a sound is produced (Default = Sound Variation)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController2, "EStdMidiControllerID_SoundController2_DisplayName", "71  Sound Controller 2", "EStdMidiControllerID_SoundController2_TT", "Allows shaping the Voltage Controlled Filter (VCF) (Default = Resonance)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController3, "EStdMidiControllerID_SoundController3_DisplayName", "72  Sound Controller 3", "EStdMidiControllerID_SoundController3_TT", "Controls release time of the Voltage controlled Amplifier (VCA) (Default = Release Time)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController4, "EStdMidiControllerID_SoundController4_DisplayName", "73  Sound Controller 4", "EStdMidiControllerID_SoundController4_TT", "Controls the “Attack’ of a sound"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController5, "EStdMidiControllerID_SoundController5_DisplayName", "74  Sound Controller 5", "EStdMidiControllerID_SoundController5_TT", "Controls VCFs cutoff frequency of the filter"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController6, "EStdMidiControllerID_SoundController6_DisplayName", "75  Sound Controller 6", "EStdMidiControllerID_SoundController6_TT", "Generic Sound Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController7, "EStdMidiControllerID_SoundController7_DisplayName", "76  Sound Controller 7", "EStdMidiControllerID_SoundController7_TT", "Generic Sound Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController8, "EStdMidiControllerID_SoundController8_DisplayName", "77  Sound Controller 8", "EStdMidiControllerID_SoundController8_TT", "Generic Sound Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController9, "EStdMidiControllerID_SoundController9_DisplayName", "78  Sound Controller 9", "EStdMidiControllerID_SoundController9_TT", "Generic Sound Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::SoundController10, "EStdMidiControllerID_SoundController10_DisplayName", "79  Sound Controller 10", "EStdMidiControllerID_SoundController10_TT", "Generic Sound Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::GeneralPurposeController5, "EStdMidiControllerID_GeneralPurposeController5_DisplayName", "80  General Purpose Controller 5", "EStdMidiControllerID_GeneralPurposeController5_TT", "Generic Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::GeneralPurposeController6, "EStdMidiControllerID_GeneralPurposeController6_DisplayName", "81  General Purpose Controller 6", "EStdMidiControllerID_GeneralPurposeController6_TT", "Generic Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::GeneralPurposeController7, "EStdMidiControllerID_GeneralPurposeController7_DisplayName", "82  General Purpose Controller 7", "EStdMidiControllerID_GeneralPurposeController7_TT", "Generic Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::GeneralPurposeController8, "EStdMidiControllerID_GeneralPurposeController8_DisplayName", "83  General Purpose Controller 8", "EStdMidiControllerID_GeneralPurposeController8_TT", "Generic Controller"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::PortamentoControl, "EStdMidiControllerID_PortamentoControl_DisplayName", "84  Portamento Control", "EStdMidiControllerID_PortamentoControl_TT", "Controls the amount of Portamento"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined85, "EStdMidiControllerID_Undefined85_DisplayName", "85  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined86, "EStdMidiControllerID_Undefined86_DisplayName", "86  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined87, "EStdMidiControllerID_Undefined87_DisplayName", "87  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::HighResolutionVelocityPrefix, "EStdMidiControllerID_HighResolutionVelocityPrefix_DisplayName", "88  High Resolution Velocity Prefix", "EStdMidiControllerID_HighResolutionVelocityPrefix_TT", "High Resolution Velocity Prefix (greater precision)"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined89, "EStdMidiControllerID_Undefined89_DisplayName", "89  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined90, "EStdMidiControllerID_Undefined90_DisplayName", "90  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Effects1Depth, "EStdMidiControllerID_Effects1Depth_DisplayName", "91  Effects 1 Depth", "EStdMidiControllerID_Effects1Depth_TT", "Usually controls reverb send amount"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Effects2Depth, "EStdMidiControllerID_Effects2Depth_DisplayName", "92  Effects 2 Depth", "EStdMidiControllerID_Effects2Depth_TT", "Usually controls chorus amount"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Effects3Depth, "EStdMidiControllerID_Effects3Depth_DisplayName", "93  Effects 3 Depth", "EStdMidiControllerID_Effects3Depth_TT", "Usually controls tremelo amount"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Effects4Depth, "EStdMidiControllerID_Effects4Depth_DisplayName", "94  Effects 4 Depth", "EStdMidiControllerID_Effects4Depth_TT", "Usually controls detune amount"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Effects5Depth, "EStdMidiControllerID_Effects5Depth_DisplayName", "95  Effects 5 Depth", "EStdMidiControllerID_Effects5Depth_TT", "Usually controls phaser amount"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::DataIncrement, "EStdMidiControllerID_DataIncrement_DisplayName", "96  Data Increment", "EStdMidiControllerID_DataIncrement_TT", "Increment data for RPN and NRPN messages"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::DataDecrement, "EStdMidiControllerID_DataDecrement_DisplayName", "97  Data Decrement", "EStdMidiControllerID_DataDecrement_TT", "Decrement data for RPN and NRPN messages"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::NRPNFine, "EStdMidiControllerID_NRPNFine_DisplayName", "98  NRPN Fine", "EStdMidiControllerID_NRPNFine_TT", "Non-Registered Parameter Number LSB (NRPN)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::NRPNCoarse, "EStdMidiControllerID_NRPNCoarse_DisplayName", "99  NRPN Coarse", "EStdMidiControllerID_NRPNCoarse_TT", "Non-Registered Parameter Number MSB (NRPN)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::RPNFine, "EStdMidiControllerID_RPNFine_DisplayName", "100  RPN Fine", "EStdMidiControllerID_RPNFine_TT", "Registered Parameter Number LSB (RPN)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::RPNCoarse, "EStdMidiControllerID_RPNCoarse_DisplayName", "101  RPN Couse", "EStdMidiControllerID_RPNCoarse_TT", "Registered Parameter Number MSB (RPN)"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined102, "EStdMidiControllerID_Undefined102_DisplayName", "102  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined103, "EStdMidiControllerID_Undefined103_DisplayName", "103  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined104, "EStdMidiControllerID_Undefined104_DisplayName", "104  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined105, "EStdMidiControllerID_Undefined105_DisplayName", "105  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined106, "EStdMidiControllerID_Undefined106_DisplayName", "106  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined107, "EStdMidiControllerID_Undefined107_DisplayName", "107  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined108, "EStdMidiControllerID_Undefined108_DisplayName", "108  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined109, "EStdMidiControllerID_Undefined109_DisplayName", "109  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined110, "EStdMidiControllerID_Undefined110_DisplayName", "110  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined111, "EStdMidiControllerID_Undefined111_DisplayName", "111  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined112, "EStdMidiControllerID_Undefined112_DisplayName", "112  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined113, "EStdMidiControllerID_Undefined113_DisplayName", "113  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined114, "EStdMidiControllerID_Undefined114_DisplayName", "114  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined115, "EStdMidiControllerID_Undefined115_DisplayName", "115  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined116, "EStdMidiControllerID_Undefined116_DisplayName", "116  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined117, "EStdMidiControllerID_Undefined117_DisplayName", "117  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined118, "EStdMidiControllerID_Undefined118_DisplayName", "118  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EStdMidiControllerID::Undefined119, "EStdMidiControllerID_Undefined119_DisplayName", "119  Undefined"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::AllSoundOff, "EStdMidiControllerID_AllSoundOff_DisplayName", "120  All Sound Off", "EStdMidiControllerID_AllSoundOff_TT", "All sound off (disregard release time or sustain)"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::Reset, "EStdMidiControllerID_Reset_DisplayName", "121  Reset", "EStdMidiControllerID_Reset_TT", "Reset all controllers to default"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::LocalKeyboardSwitch, "EStdMidiControllerID_LocalKeyboardSwitch_DisplayName", "122  Local Keyboard Switch", "EStdMidiControllerID_LocalKeyboardSwitch_TT", "Turns internal connection of a MIDI keyboard/workstation, etc On or Off"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::AllNotesOff, "EStdMidiControllerID_AllNotesOff_DisplayName", "123  All Notes Off", "EStdMidiControllerID_AllNotesOff_TT", "All notes off (maintain release time and notes held by sustain"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::OmniModeOff, "EStdMidiControllerID_OmniModeOff_DisplayName", "124  Omni Mode Off", "EStdMidiControllerID_OmniModeOff_TT", "Omni Mode Off"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::OmniModeOn, "EStdMidiControllerID_OmniModeOn_DisplayName", "125  Omni Mode On", "EStdMidiControllerID_OmniModeOn_TT", "Omni Mode On"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::MonoMode, "EStdMidiControllerID_MonoMode_DisplayName", "126  Mono Mode", "EStdMidiControllerID_MonoMode_TT", "Monophonic Mode"),
		DEFINE_METASOUND_ENUM_ENTRY(EStdMidiControllerID::PolyMode, "EStdMidiControllerID_PolyMode_DisplayName", "127  Poly Mode", "EStdMidiControllerID_PolyMode_TT", "Polyphonic Mode"),
	DEFINE_METASOUND_ENUM_END()
}

#undef LOCTEXT_NAMESPACE



