// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MidiEventInfo.h"

#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiEventInfo)

REGISTER_METASOUND_DATATYPE(FMidiEventInfo, "MIDIEventInfo");

uint8 FMidiEventInfo::GetChannel() const
{
	return MidiMessage.IsStd() ? MidiMessage.GetStdChannel() : 0;
}

bool FMidiEventInfo::IsNote() const
{
	return IsNoteOn() || IsNoteOff();
}

bool FMidiEventInfo::IsNoteOn() const
{
	return MidiMessage.IsNoteOn();
}

bool FMidiEventInfo::IsNoteOff() const
{
	return MidiMessage.IsNoteOff();
}

uint8 FMidiEventInfo::GetNoteNumber() const
{
	return IsNote() ? MidiMessage.GetStdData1() : 0;
}

uint8 FMidiEventInfo::GetVelocity() const
{
	return IsNote() ? MidiMessage.GetStdData2() : 0;
}

bool UMidiEventInfoBlueprintLibrary::IsMidiEventInfo(const FMetaSoundOutput& Output)
{
	return Output.IsType<FMidiEventInfo>();
}

FMidiEventInfo UMidiEventInfoBlueprintLibrary::GetMidiEventInfo(const FMetaSoundOutput& Output, bool& Success)
{
	FMidiEventInfo Info;
	Success = Output.Get(Info);
	return Info;
}

int32 UMidiEventInfoBlueprintLibrary::GetChannel(const FMidiEventInfo& Event)
{
	return Event.GetChannel();
}

bool UMidiEventInfoBlueprintLibrary::IsNote(const FMidiEventInfo& Event)
{
	return Event.IsNote();
}

bool UMidiEventInfoBlueprintLibrary::IsNoteOn(const FMidiEventInfo& Event)
{
	return Event.IsNoteOn();
}

bool UMidiEventInfoBlueprintLibrary::IsNoteOff(const FMidiEventInfo& Event)
{
	return Event.IsNoteOff();
}

int32 UMidiEventInfoBlueprintLibrary::GetNoteNumber(const FMidiEventInfo& Event)
{
	return Event.GetNoteNumber();
}

int32 UMidiEventInfoBlueprintLibrary::GetVelocity(const FMidiEventInfo& Event)
{
	return Event.GetVelocity();
}
