// Copyright Epic Games, Inc. All Rights Reserved.


#include "HarmonixMetasound/DataTypes/MidiStepSequence.h"

#include "Harmonix/PropertyUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiStepSequence)

DEFINE_LOG_CATEGORY_STATIC(LogMIDIStepSequence, Log, All);

DEFINE_AUDIORENDERABLE_ASSET(HarmonixMetasound, FMidiStepSequenceAsset, MIDIStepSequenceAsset, UMidiStepSequence)

bool FStepSequenceTable::IsPageBlank(int32 PageIdx) const
{
	if (Pages.IsValidIndex(PageIdx))
	{
		const FStepSequencePage& PageData = Pages[PageIdx];
		for (const FStepSequenceRow& CurrentRow : PageData.Rows)
		{
			for (const FStepSequenceCell& CurrentCell : CurrentRow.Cells)
			{
				if (CurrentCell.bEnabled)
				{
					return false;
				}
			}
		}
	}

	return true;
}

int32 FStepSequenceTable::GetFirstValidPage(bool bPlayBlankPages) const
{
	if (bPlayBlankPages)
	{
		return 0;
	}

	for (int32 PageIdx = 0; PageIdx < Pages.Num(); ++PageIdx)
	{
		if (!IsPageBlank(PageIdx))
		{
			return PageIdx;
		}
	}

	return 0;
}


void FStepSequenceTable::GetUnusedPages(bool bPlayBlankPages, TBitArray<FDefaultBitArrayAllocator>& OutUnusedPages) const
{
	if (!bPlayBlankPages)
	{
		for (int32 PageIdx = 0; PageIdx < Pages.Num(); ++PageIdx)
		{
			// OutUnusedPages should be the same size as Pages, but safety check regardless
			if (OutUnusedPages.IsValidIndex(PageIdx))
			{
				OutUnusedPages[PageIdx] = IsPageBlank(PageIdx);
			}
		}
	}
	// If we play blank pages, all pages are used.
}

int32 FStepSequenceTable::CalculateNumValidPages(bool bPlayBlankPages) const
{
	TBitArray UnusedPages(false, Pages.Num());
	GetUnusedPages(bPlayBlankPages, UnusedPages);
	return UnusedPages.Num() - UnusedPages.CountSetBits();
}

int32 FStepSequenceTable::CalculateAutoPageIndex(int32 TotalPagesProgressed, bool bPlayBlankPages, bool bLoop) const
{
	// Calculate blank pages in order to skip them
	TBitArray UnusedPages(false, Pages.Num());
	GetUnusedPages(bPlayBlankPages, UnusedPages);
	// Do this calc manually instead of CalculateNumValidPages so we don't do the unused page calc twice
	int32 NumValidPages = UnusedPages.Num() - UnusedPages.CountSetBits();

	if (NumValidPages == 0 || (!bLoop && TotalPagesProgressed >= NumValidPages))
	{
		// If we can't possibly autopage or we have reached the end of the loop, return INDEX_NONE to signal that
		return INDEX_NONE;
	}

	int32 TargetPage = TotalPagesProgressed % NumValidPages;

	// Rebase target page onto appropriate non-blank page (offset from page 0)
	int32 BlankPageProbeCount = TargetPage;
	for (int32 PageIdx = 0; PageIdx <= BlankPageProbeCount; ++PageIdx)
	{
		if (UnusedPages[PageIdx])
		{
			++BlankPageProbeCount;
			++TargetPage;
			if (!UnusedPages.IsValidIndex(TargetPage))
			{
				TargetPage = 0;
			}
		}
	}

	return TargetPage;
}

UMidiStepSequence::UMidiStepSequence(const FObjectInitializer& Initializer)
{
	if (StepTable.Notes.Num() == 0)
	{
		SetNumPages(0);
		SetNumPages(Pages);
		StepTable.Notes.SetNum(Rows);
	}
}

void UMidiStepSequence::PostReinitProperties()
{
	Super::PostReinitProperties();

	if (StepTable.Notes.Num() == 0)
	{
		SetNumPages(0);
		SetNumPages(Pages);
		StepTable.Notes.SetNum(Rows);
	}	
}

void UMidiStepSequence::SetNumPages(int32 Count)
{
	if (Count == Pages)
	{
		return;
	}

	Pages = FMath::Clamp(Count, kMinPages, kMaxPages);
	StepTable.Pages.SetNum(Pages);
	for (auto& Page : StepTable.Pages)
	{
		Page.Rows.SetNum(Rows);
		for (auto& Row : Page.Rows)
		{
			Row.Cells.SetNum(Columns);
		}
	}

	// This is a non-trivial change so we need to queue up a new copy of 
	// the renderable data
	UpdateRenderableForNonTrivialChange();
}

void UMidiStepSequence::SetNumRows(int32 Count)
{
	if (Count == Rows)
	{
		return;
	}

	Rows = FMath::Clamp(Count, kMinRows, kMaxRows);
	for (auto& Page : StepTable.Pages)
	{
		Page.Rows.SetNum(Rows);
		for (auto& Row : Page.Rows)
		{
			Row.Cells.SetNum(Columns);
		}
	}

	StepTable.Notes.SetNum(Rows);

	// This is a non-trivial change so we need to queue up a new copy of 
	// the renderable data
	UpdateRenderableForNonTrivialChange();
}

void UMidiStepSequence::DisableRowsAbove(int32 FirstDisabledRow)
{
	FStepSequenceTable* RenderableTable = nullptr;

	if (RenderableSequenceTable)
	{
		RenderableTable = *RenderableSequenceTable;
	}

	for (int32 PageIndex = 0; PageIndex < StepTable.Pages.Num(); ++PageIndex)
	{
		for (int32 RowIndex = 0; RowIndex < StepTable.Pages[PageIndex].Rows.Num(); ++RowIndex)
		{
			StepTable.Pages[PageIndex].Rows[RowIndex].bRowEnabled = RowIndex < FirstDisabledRow;

			// This is a trivial change, so we can update the renderable proxy immediately
			if (RenderableTable)
			{
				if (RenderableTable->Pages.IsValidIndex(PageIndex) && RenderableTable->Pages[PageIndex].Rows.IsValidIndex(RowIndex))
				{
					RenderableTable->Pages[PageIndex].Rows[RowIndex].bRowEnabled = RowIndex < FirstDisabledRow;
				}
			}
		}
	}
}

void UMidiStepSequence::SetNumColumns(int32 Count)
{
	if (Count == Columns)
	{
		return;
	}

	Columns = FMath::Clamp(Count, kMinColumns, kMaxColumns);
	for (auto& Page : StepTable.Pages)
	{
		for (auto& Row : Page.Rows)
		{
			Row.Cells.SetNum(Columns);
		}
	}

	// This is a non-trivial change so we need to queue up a new copy of 
	// the renderable data
	UpdateRenderableForNonTrivialChange();
}

bool UMidiStepSequence::SetRowNoteNumber(int32 RowIndex, int32 MidiNoteNumber)
{
	if (!StepTable.Notes.IsValidIndex(RowIndex))
	{
		return false;
	}

	StepTable.Notes[RowIndex].NoteNumber = MidiNoteNumber;

	// This is a trivial change, so we can update the renderable proxy immediately
	if (RenderableSequenceTable)
	{
		FStepSequenceTable* Table = *RenderableSequenceTable;
		Table->Notes[RowIndex].NoteNumber = MidiNoteNumber;
	}

	return true;
}

bool UMidiStepSequence::SetRowVelocity(int32 RowIndex, int32 Velocity)
{
	if (!StepTable.Notes.IsValidIndex(RowIndex))
	{
		return false;
	}

	StepTable.Notes[RowIndex].Velocity = Velocity;

	// This is a trivial change, so we can update the renderable proxy immediately
	if (RenderableSequenceTable)
	{
		FStepSequenceTable* Table = *RenderableSequenceTable;
		Table->Notes[RowIndex].Velocity = Velocity;
	}

	return true;
}

bool UMidiStepSequence::SetStepSkipIndex(int32 StepIndex)
{
	StepTable.StepSkipIndex = StepIndex;

	// This is a trivial change, so we can update the renderable proxy immediately
	if (RenderableSequenceTable)
	{
		FStepSequenceTable* Table = *RenderableSequenceTable;
		Table->StepSkipIndex = StepIndex;
	}

	return true;
}

FStepSequenceCell UMidiStepSequence::SetCell(int32 Row, int32 Column, bool State)
{
	return SetCellOnPage(0, Row, Column, State);
}

FStepSequenceCell UMidiStepSequence::SetCellOnPage(int32 Page, int32 Row, int32 Column, bool State)
{
	if (Page < 0 || Page >= Pages || Row < 0 || Row >= Rows || Column < 0 || Column >= Columns)
	{
		return FStepSequenceCell();
	}

	StepTable.Pages[Page].Rows[Row].Cells[Column].bEnabled = State;

	// This is a trivial change, so we can update the renderable proxy immediately
	if (RenderableSequenceTable)
	{
		FStepSequenceTable* Table = *RenderableSequenceTable;
		Table->Pages[Page].Rows[Row].Cells[Column].bEnabled = State;
	}

	return StepTable.Pages[Page].Rows[Row].Cells[Column];
}

FStepSequenceCell UMidiStepSequence::SetCellContinuation(int32 Row, int32 Column, bool bState)
{
	return SetCellContinuationOnPage(0, Row, Column, bState);
}

FStepSequenceCell UMidiStepSequence::SetCellContinuationOnPage(int32 Page, int32 Row, int32 Column, bool bState)
{
	if (Page < 0 || Page >= Pages || Row < 0 || Row >= Rows || Column < 0 || Column >= Columns)
	{
		return FStepSequenceCell();
	}
	
	StepTable.Pages[Page].Rows[Row].Cells[Column].bContinuation = bState;

	// This is a trivial change, so we can update the renderable proxy immediately
	if (RenderableSequenceTable)
	{
		FStepSequenceTable* Table = *RenderableSequenceTable;
		Table->Pages[Page].Rows[Row].Cells[Column].bContinuation = bState;
	}

	return StepTable.Pages[Page].Rows[Row].Cells[Column];
}

FStepSequenceCell UMidiStepSequence::GetCell(int32 Row, int32 Column)
{
	return GetCellOnPage(0, Row, Column);
}

FStepSequenceCell UMidiStepSequence::GetCellOnPage(int32 Page, int32 Row, int32 Column)
{
	if (Page < 0 || Page >= Pages || Row < 0 || Row >= Rows || Column < 0 || Column >= Columns)
	{
		return FStepSequenceCell();
	}

	return StepTable.Pages[Page].Rows[Row].Cells[Column];
}

FStepSequenceCell UMidiStepSequence::ToggleCell(int32 Row, int32 Column)
{
	return ToggleCellOnPage(0, Row, Column);
}

FStepSequenceCell UMidiStepSequence::ToggleCellOnPage(int32 Page, int32 Row, int32 Column)
{
	if (Page < 0 || Page >= Pages || Row < 0 || Row >= Rows || Column < 0 || Column >= Columns)
	{
		return FStepSequenceCell();
	}

	StepTable.Pages[Page].Rows[Row].Cells[Column].bEnabled = !StepTable.Pages[Page].Rows[Row].Cells[Column].bEnabled;

	// This is a trivial change, so we can update the renderable proxy immediately
	if (RenderableSequenceTable)
	{
		FStepSequenceTable* Table = *RenderableSequenceTable;
		Table->Pages[Page].Rows[Row].Cells[Column].bEnabled = StepTable.Pages[Page].Rows[Row].Cells[Column].bEnabled;
	}

	return StepTable.Pages[Page].Rows[Row].Cells[Column];
}

void UMidiStepSequence::SetStepTable(const FStepSequenceTable& NewStepTable)
{
	// Ensure the incoming step table fits our constraints
	// 1. Min & Max for pages, rows, and columns within our constraints
	// 2. Each page has the same number of rows
	// 3. Each row has the same number of columns
	// 4. The note array length matches the number of rows

	if (NewStepTable.Pages.Num() < kMinPages || NewStepTable.Pages.Num() > kMaxPages)
	{
		return;
	}
		
	int NumRows = NewStepTable.Pages[0].Rows.Num();

	if (NumRows < kMinRows || NumRows > kMaxRows)
	{
		return;
	}

	if (NewStepTable.Notes.Num() != NumRows)
	{
		return;
	}

	int NumColumns = NewStepTable.Pages[0].Rows[0].Cells.Num();

	if (NumColumns < kMinColumns || NumColumns > kMaxColumns)
	{
		return;
	}

	for (const FStepSequencePage& Page : NewStepTable.Pages)
	{
		if (Page.Rows.Num() != NumRows)
		{
			return;
		}

		for (const FStepSequenceRow& Row : Page.Rows)
		{
			if (Row.Cells.Num() != NumColumns)
			{
				return;
			}
		}
	}

	StepTable = NewStepTable;
	
	// Adjust the page, rows, and columns to match the new table
	Pages = StepTable.Pages.Num();
	Rows = NumRows;
	Columns = NumColumns;

	UpdateRenderableForNonTrivialChange();
}

#if WITH_EDITOR
void UMidiStepSequence::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	const FProperty* PropertyChanged = PropertyChangedChainEvent.Property;
	const EPropertyChangeType::Type PropertyChangeType = PropertyChangedChainEvent.ChangeType;

	FName ChangedPropertyName = PropertyChanged->GetFName();
	if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UMidiStepSequence, Columns))
	{
		for (auto& Page : StepTable.Pages)
		{
			for (auto& Row : Page.Rows)
			{
				Row.Cells.SetNum(Columns);
			}
		}
		UpdateRenderableForNonTrivialChange();
		return;
	}
	else if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UMidiStepSequence, Rows))
	{
		for (auto& Page : StepTable.Pages)
		{
			Page.Rows.SetNum(Rows);
			for (auto& Row : Page.Rows)
			{
				Row.Cells.SetNum(Columns);
			}
		}

		StepTable.Notes.SetNum(Rows);

		UpdateRenderableForNonTrivialChange();
		return;
	}
	else if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UMidiStepSequence, Pages))
	{
		StepTable.Pages.SetNum(Pages);

		for (auto& Page : StepTable.Pages)
		{
			Page.Rows.SetNum(Rows);
			for (auto& Row : Page.Rows)
			{
				Row.Cells.SetNum(Columns);
			}
		}

		UpdateRenderableForNonTrivialChange();
		return;
	}
	else if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UMidiStepSequence, CurrentPageNumber))
	{
		SetClampedCurrentPageNumber(CurrentPageNumber);
		return;
	}

	// for other changes, check against if we're changing the underlying FStepSequenceTable struct
	const FProperty* MemberChanged = PropertyChangedChainEvent.PropertyChain.GetHead()->GetValue();
	if (MemberChanged->GetFName() != GET_MEMBER_NAME_CHECKED(UMidiStepSequence, StepTable))
	{
		return;
	}

	// there is no renderable data, so no need to do any copying over on property changes
	if (!RenderableSequenceTable)
	{
		return;
	}

	// Determine what to do based on the property and the change type
	Harmonix::EPostEditAction PostEditAction = Harmonix::GetPropertyPostEditAction(PropertyChanged, PropertyChangeType);

	if (PostEditAction == Harmonix::EPostEditAction::UpdateTrivial)
	{
		// use the PropertyChangedChainEvent to copy the single property changed
		FStepSequenceTable* CopyToStruct = *RenderableSequenceTable;
		Harmonix::CopyStructProperty(CopyToStruct, &StepTable, PropertyChangedChainEvent);
	}
	else if (PostEditAction == Harmonix::EPostEditAction::UpdateNonTrivial)
	{
		UpdateRenderableForNonTrivialChange();
	}
	
}
#endif // WITH_EDITOR

TSharedPtr<Audio::IProxyData> UMidiStepSequence::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	if (!RenderableSequenceTable)
	{
		RenderableSequenceTable = MakeShared<FStepSequenceTableProxy::QueueType>(StepTable);
	}
	return MakeShared<FStepSequenceTableProxy>(RenderableSequenceTable);
}

#if WITH_EDITORONLY_DATA
const int32 UMidiStepSequence::GetClampedCurrentPageNumber()
{
	return FMath::Clamp(CurrentPageNumber, 1, Pages);
}

void UMidiStepSequence::SetClampedCurrentPageNumber(int32 InPageNumber)
{
	CurrentPageNumber = FMath::Clamp(InPageNumber, 1, Pages);
}
#endif

void UMidiStepSequence::UpdateRenderableForNonTrivialChange()
{
	// If no one has requested a renderable proxy we don't have to do anything
	if (!RenderableSequenceTable)
	{
		return;
	}
	RenderableSequenceTable->SetNewSettings(StepTable);
}

bool FStepSequenceNote::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Ar << NoteNumber;
	Ar << Velocity;

	bOutSuccess = true;
	return true;
}
