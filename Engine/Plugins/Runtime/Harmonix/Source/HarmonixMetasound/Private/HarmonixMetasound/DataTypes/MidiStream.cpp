// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MidiStream.h"

#include "MetasoundDataTypeRegistrationMacro.h"
#include "Algo/BinarySearch.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"

REGISTER_METASOUND_DATATYPE(HarmonixMetasound::FMidiStream, "MIDIStream")

namespace HarmonixMetasound
{
	DEFINE_LOG_CATEGORY(LogMidiStreamDataType);

	using namespace Metasound;

	FMidiStreamEvent::FMidiStreamEvent(const FMidiVoiceGeneratorBase* Owner, const FMidiMsg& Message)
	: FMidiStreamEvent(Owner != nullptr ? Owner->GetIdBits() : static_cast<uint32>(0), Message)
	{
	}

	FMidiStreamEvent::FMidiStreamEvent(const uint32 OwnerId, const FMidiMsg& Message)
	: MidiMessage(Message)
	, VoiceId(OwnerId, MidiMessage)
	{
	}

	void FMidiStream::SetClock(const FMidiClock& InClock)
	{
		Clock = InClock.AsWeak();
	}

	void FMidiStream::ResetClock()
	{
		Clock.Reset();
	}

	TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> FMidiStream::GetClock() const
	{
		return Clock.Pin();
	}


	void FMidiStream::PrepareBlock()
	{
		EventsInBlock.Empty(32);
	}

	void FMidiStream::AddMidiEvent(const FMidiStreamEvent& Event)
	{
		check (EventsInBlock.IsEmpty() || EventsInBlock.Last().BlockSampleFrameIndex <= Event.BlockSampleFrameIndex);
		EventsInBlock.Add(Event);
		TrackNote(Event);
	}

	void FMidiStream::InsertMidiEvent(const FMidiStreamEvent& Event)
	{
		int32 AtIndex = Algo::UpperBound(EventsInBlock, Event, [](const FMidiStreamEvent& NewEvent, const FMidiStreamEvent& ExistingEvent){ return NewEvent.BlockSampleFrameIndex < ExistingEvent.BlockSampleFrameIndex; });
		EventsInBlock.Insert(Event, AtIndex);
		TrackNote(Event);
	}

	void FMidiStream::AddNoteOffEventOrCancelPendingNoteOn(const FMidiStreamEvent& Event)
	{
		check(Event.MidiMessage.IsNoteOff());
		int32 NumRemoved = EventsInBlock.RemoveAll([&](const FMidiStreamEvent& EventInList)
			{
				return EventInList.MidiMessage.IsNoteOn() && EventInList.GetVoiceId() == Event.GetVoiceId();
			});
		if (NumRemoved == 0)
		{
			AddMidiEvent(Event);
		}
	}

	void FMidiStream::InsertNoteOffEventOrCancelPendingNoteOn(const FMidiStreamEvent& Event)
	{
		check(Event.MidiMessage.IsNoteOff());
		int32 NumRemoved = EventsInBlock.RemoveAll([&](const FMidiStreamEvent& EventInList)
			{
				return EventInList.MidiMessage.IsNoteOn() && EventInList.GetVoiceId() == Event.GetVoiceId();
			});
		if (NumRemoved == 0)
		{
			InsertMidiEvent(Event);
		}
	}

	const FString* FMidiStream::GetMidiTrackText(int32 TrackNumber, int32 TextIndex) const
	{
		if (!MidiFileSourceOfEvents || TrackNumber < 0 || TextIndex < 0)
		{
			return nullptr;
		}

		TSharedPtr<FMidiFileData> MidiFile = MidiFileSourceOfEvents->GetMidiFile();
		if (TrackNumber >= MidiFile->Tracks.Num())
		{
			return nullptr;
		}
		const FMidiTextRepository* Repository = MidiFile->Tracks[TrackNumber].GetTextRepository();
		if (TextIndex >= Repository->Num())
		{
			return nullptr;
		}
		return &(*Repository)[TextIndex];
	}

	void FMidiStream::Copy(const FMidiStream& From, FMidiStream& To, const FEventFilter& Filter, const FEventTransformer& Transformer)
	{
		// Copy the clock from the other stream
		if (const TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> FromClock = From.GetClock())
		{
			To.SetClock(*FromClock);
		}

		// Reset the current events
		To.EventsInBlock.Reset();
		
		// Copy the active notes from the other stream
		To.ActiveNotes = From.ActiveNotes;
		
		// Copy the events
		for (const FMidiStreamEvent& Event : From.GetEventsInBlock())
		{
			if (Filter(Event))
			{
				To.AddMidiEvent(Transformer(Event));
			}
		}
	}

	void FMidiStream::Merge(const FMidiStream& From, FMidiStream& To, const FEventFilter& Filter, const FEventTransformer& Transformer)
	{
		if (!ensure(&From != &To))
		{
			return;
		}
		
		const auto FromClock = From.GetClock();
		const auto ToClock = To.GetClock();

		// If the "to" clock is null, and the "from" clock isn't, overwrite the "to" clock
		if (FromClock.IsValid() && !ToClock.IsValid())
		{
			To.SetClock(*FromClock);
		}

		// Insert the events
		for (const FMidiStreamEvent& Event : From.GetEventsInBlock())
		{
			if (Filter(Event))
			{
				To.InsertMidiEvent(Transformer(Event));
			}
		}

		// Merge in the active notes from the other stream
		// Transform those notes so that the active note matches other events
		for (const FMidiStreamEvent& ActiveNote : From.ActiveNotes)
		{
			check(ActiveNote.MidiMessage.IsNoteOn());
			
			if (!To.ActiveNotes.ContainsByPredicate([&ActiveNote](const FMidiStreamEvent& TrackedEvent)
			{
				return ActiveNote.TrackIndex == TrackedEvent.TrackIndex && ActiveNote.VoiceId == TrackedEvent.GetVoiceId();
			}))
			{
				To.ActiveNotes.Add(Transformer(ActiveNote));
			}
		}
	}
	
	void FMidiStream::Merge(
		const FMidiStream& FromA,
		const FMidiStream& FromB,
		FMidiStream& To,
		const FEventFilter& Filter,
		const FEventTransformer& Transformer)
	{
		// Clear active notes. This unsticks notes when a midi stream is disconnected during live update.
		To.ActiveNotes.Empty();

		// Merge in stream A
		Merge(FromA, To, Filter, Transformer);

		// Merge in stream B and re-map the voice ids
		const auto RemapTransform = [&To, &Transformer](const FMidiStreamEvent& Event)
		{
			FMidiStreamEvent TransformedEvent = Transformer(Event);
			const uint32 GeneratorId = TransformedEvent.GetVoiceId().GetGeneratorId();
			TransformedEvent.VoiceId.ReassignGenerator(To.GeneratorMap.FindOrAdd(GeneratorId).GetIdBits());
			return TransformedEvent;
		};
		Merge(FromB, To, Filter, RemapTransform);
	}

	bool FMidiStream::NoteIsActive(const FMidiStreamEvent& Event) const
	{
		return ActiveNotes.ContainsByPredicate([&Event](const FMidiStreamEvent& ActiveNote)
		{
			return Event.TrackIndex == ActiveNote.TrackIndex && Event.GetVoiceId() == ActiveNote.GetVoiceId();
		});
	}

	void FMidiStream::TrackNote(const FMidiStreamEvent& Event)
	{
		if (!Event.MidiMessage.IsNoteMessage())
		{
			return;
		}

		if (Event.MidiMessage.IsNoteOn())
		{
			const int32 TrackIndex = Event.TrackIndex;
			const FMidiVoiceId VoiceId = Event.GetVoiceId();

			if (!ActiveNotes.ContainsByPredicate([TrackIndex, VoiceId](const HarmonixMetasound::FMidiStreamEvent& TrackedEvent)
			{
				return TrackIndex == TrackedEvent.TrackIndex && VoiceId == TrackedEvent.GetVoiceId();
			}))
			{
				ActiveNotes.Add(Event);
			}
		}
		else if (Event.MidiMessage.IsNoteOff())
		{
			const int32 TrackIndex = Event.TrackIndex;
			const FMidiVoiceId VoiceId = Event.GetVoiceId();
				
			ActiveNotes.RemoveAll([TrackIndex, VoiceId](const HarmonixMetasound::FMidiStreamEvent& TrackedEvent)
			{
				return TrackIndex == TrackedEvent.TrackIndex && VoiceId == TrackedEvent.GetVoiceId(); 
			});
		}
		// If this is an all notes off/kill, untrack everything
		// NOTE: This diverges from the MIDI spec, where "all notes off" has a track and channel associated with it.
		// If we end up supporting the track- and channel-specific "all notes off" we will need to add support here to avoid stuck notes.
		else
		{
			ActiveNotes.Reset();
		}
	}
}
