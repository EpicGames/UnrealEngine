// Copyright Epic Games, Inc. All Rights Reserved.


#include "HarmonixMetasound/DataTypes/MidiStutterSequence.h"

#include "Harmonix/PropertyUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiStutterSequence)

DEFINE_LOG_CATEGORY_STATIC(LogMIDIStutterSequence, Log, All);

DEFINE_AUDIORENDERABLE_ASSET(HarmonixMetasound, FStutterSequenceAsset, StutterSequenceAsset, UMidiStutterSequence)

#if WITH_EDITOR
void UMidiStutterSequence::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	const FProperty* PropertyChanged = PropertyChangedChainEvent.Property;
	const EPropertyChangeType::Type PropertyChangeType = PropertyChangedChainEvent.ChangeType;

	// check against if changing the underlying FStutterSequenceTable struct
	const FProperty* MemberChanged = PropertyChangedChainEvent.PropertyChain.GetHead()->GetValue();
	if (MemberChanged->GetFName() != GET_MEMBER_NAME_CHECKED(UMidiStutterSequence, StutterTable))
	{
		return;
	}

	// there is no renderable data, so no need to do any copying over on property changes
	if (!RenderableStutterTable)
	{
		return;
	}

	// Determine what to do based on the property and the change type
	Harmonix::EPostEditAction PostEditAction = Harmonix::GetPropertyPostEditAction(PropertyChanged, PropertyChangeType);

	if (PostEditAction == Harmonix::EPostEditAction::UpdateTrivial)
	{
		// use the PropertyChangedChainEvent to copy the single property changed
		FStutterSequenceTable* CopyToStruct = *RenderableStutterTable;
		Harmonix::CopyStructProperty(CopyToStruct, &StutterTable, PropertyChangedChainEvent);
	}
	else if (PostEditAction == Harmonix::EPostEditAction::UpdateNonTrivial)
	{
		UpdateRenderableForNonTrivialChange();
	}
	
}
#endif // WITH_EDITOR

TSharedPtr<Audio::IProxyData> UMidiStutterSequence::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	if (!RenderableStutterTable)
	{
		RenderableStutterTable = MakeShared<FStutterSequenceTableProxy::QueueType>(StutterTable);
	}
	return MakeShared<FStutterSequenceTableProxy>(RenderableStutterTable);
}

void UMidiStutterSequence::UpdateRenderableForNonTrivialChange()
{
	// If no one has requested a renderable proxy we don't have to do anything
	if (!RenderableStutterTable)
	{
		return;
	}
	RenderableStutterTable->SetNewSettings(StutterTable);
}
