// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/DataTypes/MusicLoopConfiguration.h"

#include "MetasoundParameterPack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicLoopConfiguration)

REGISTER_METASOUND_DATATYPE(FMusicLoopConfiguration, "MusicLoopConfiguration")

