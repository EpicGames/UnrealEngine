// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/DataTypes/MusicParameterBlueprintLibrary.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"

#include "MetasoundParameterPack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicParameterBlueprintLibrary)

#define IMPLEMENT_SET_GET_HAS(StructTypeName)                                                                                                                                                 \
	void UMusicParameterBlueprintLibrary::Set ## StructTypeName(UMetasoundParameterPack* Target, FName ParameterName, F ## StructTypeName& Value, ESetParamResult& Result, bool OnlyIfExists) \
	{                                                                                                                                                                                         \
		if (!Target)                                                                                                                                                                          \
		{                                                                                                                                                                                     \
			Result = ESetParamResult::Failed;                                                                                                                                                 \
			return;                                                                                                                                                                           \
		}                                                                                                                                                                                     \
		Result = Target->SetParameter<F ## StructTypeName>(ParameterName, Metasound::GetMetasoundDataTypeName<F ## StructTypeName>(), Value, OnlyIfExists);	                                  \
		return;                                                                                                                                                                               \
	}                                                                                                                                                                                         \
	F ## StructTypeName UMusicParameterBlueprintLibrary::Get ## StructTypeName(UMetasoundParameterPack* Target, FName ParameterName, ESetParamResult& Result)                                 \
	{                                                                                                                                                                                         \
		if (!Target)                                                                                                                                                                          \
		{                                                                                                                                                                                     \
			Result = ESetParamResult::Failed;                                                                                                                                                 \
			return F ## StructTypeName();                                                                                                                                                     \
		}                                                                                                                                                                                     \
		return Target->GetParameter<F ## StructTypeName>(ParameterName, Metasound::GetMetasoundDataTypeName<F ## StructTypeName>(), Result);                                                  \
	}                                                                                                                                                                                         \
	bool UMusicParameterBlueprintLibrary::Has ## StructTypeName(UMetasoundParameterPack* Target, FName ParameterName)                                                                         \
	{                                                                                                                                                                                         \
		if (!Target)                                                                                                                                                                          \
		{                                                                                                                                                                                     \
			return false;                                                                                                                                                                     \
		}                                                                                                                                                                                     \
		return Target->HasParameter<F ## StructTypeName>(ParameterName, Metasound::GetMetasoundDataTypeName<F ## StructTypeName>());                                                          \
	}

IMPLEMENT_SET_GET_HAS(MusicTimestamp)
IMPLEMENT_SET_GET_HAS(MusicLoopConfiguration)
IMPLEMENT_SET_GET_HAS(MusicSeekRequest)

#undef IMPLEMENT_SET_GET_HAS
