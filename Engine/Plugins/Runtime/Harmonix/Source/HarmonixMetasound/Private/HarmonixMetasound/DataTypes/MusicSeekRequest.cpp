// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/DataTypes/MusicSeekRequest.h"

#include "MetasoundParameterPack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicSeekRequest)

REGISTER_METASOUND_DATATYPE(FMusicSeekRequest, "MusicSeekRequest")

REGISTER_METASOUND_DATATYPE(FMusicSeekTarget, "MusicSeekTarget")
