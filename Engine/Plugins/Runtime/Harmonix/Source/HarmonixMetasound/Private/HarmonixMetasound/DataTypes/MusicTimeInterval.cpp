// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MusicTimeInterval.h"
#include "HarmonixMidi/MidiConstants.h"

FMusicTimeInterval::FMusicTimeInterval() = default;

FMusicTimeInterval::FMusicTimeInterval(EMidiClockSubdivisionQuantization InInterval, EMidiClockSubdivisionQuantization InOffset, int32 InIntervalMultiplier, int32 InOffsetMultiplier)
	: Interval(InInterval)
	, Offset(InOffset)
	, IntervalMultiplier(InIntervalMultiplier)
	, OffsetMultiplier(InOffsetMultiplier)
{

}

float FMusicTimeInterval::GetIntervalBeats(const FTimeSignature& TimeSignature) const
{
	return Harmonix::Midi::Constants::SubdivisionToBeats(Interval, TimeSignature) * IntervalMultiplier;
}

float FMusicTimeInterval::GetOffsetBeats(const FTimeSignature& TimeSignature) const
{
	const float IntervalBeats = GetIntervalBeats(TimeSignature);

	if (FMath::IsNearlyZero(IntervalBeats))
	{
		return 0.0f;
	}

	const float OffsetBeats = Harmonix::Midi::Constants::SubdivisionToBeats(Offset, TimeSignature) * OffsetMultiplier;
	return FMath::Fmod(OffsetBeats, IntervalBeats);
}

namespace Harmonix
{
	void IncrementTimestampByBeats(FMusicTimestamp& Timestamp, float Beats, const FTimeSignature& TimeSignature)
	{

        if(TimeSignature.Numerator == 0)
        {
            return;
        }

		// Add the beats
		Timestamp.Beat += Beats;

		// Add bars until the beat is within the bar
		// Beats and bars are 1-indexed, so, for example, in 4/4, Beat 4.999... is within the bar.
		while (Timestamp.Beat >= static_cast<float>(TimeSignature.Numerator + 1))
		{
			++Timestamp.Bar;
			Timestamp.Beat -= TimeSignature.Numerator;
		}
	}
	
	void IncrementTimestampByInterval(
		FMusicTimestamp& Timestamp,
		const FMusicTimeInterval& Interval,
		const FTimeSignature& TimeSignature)
	{
		// Get the pulse interval in beats
		const float IntervalBeats = Interval.GetIntervalBeats(TimeSignature);

		if (FMath::IsNearlyZero(IntervalBeats))
		{
			return;
		}

		IncrementTimestampByBeats(Timestamp, IntervalBeats, TimeSignature);
	}

	void IncrementTimestampByOffset(
		FMusicTimestamp& Timestamp,
		const FMusicTimeInterval& Interval,
		const FTimeSignature& TimeSignature)
	{
		// Get the offset in beats
		const float OffsetBeats = Interval.GetOffsetBeats(TimeSignature);

		if (FMath::IsNearlyZero(OffsetBeats))
		{
			return;
		}

		IncrementTimestampByBeats(Timestamp, OffsetBeats, TimeSignature);
	}
}
