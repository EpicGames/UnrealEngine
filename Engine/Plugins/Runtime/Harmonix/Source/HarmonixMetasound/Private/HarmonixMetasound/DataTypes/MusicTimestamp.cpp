// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"

#include "HarmonixMidi/BarMap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicTimestamp)

bool UMusicTimestampBlueprintLibrary::IsMusicTimestamp(const FMetaSoundOutput& Output)
{
	return Output.IsType<FMusicTimestamp>();
}

FMusicTimestamp UMusicTimestampBlueprintLibrary::GetMusicTimestamp(const FMetaSoundOutput& Output, bool& Success)
{
	FMusicTimestamp Timestamp;
	Success = Output.Get(Timestamp);
	return Timestamp;
}

REGISTER_METASOUND_DATATYPE(FMusicTimestamp, "MusicTimestamp")
