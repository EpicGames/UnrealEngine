// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"

#include "MetasoundDataTypeRegistrationMacro.h"

REGISTER_METASOUND_DATATYPE(HarmonixMetasound::FMusicTransportEvent, "MusicTransportEvent")
REGISTER_METASOUND_DATATYPE(HarmonixMetasound::FMusicTransportEventStream, "MusicTransport")

namespace HarmonixMetasound
{
	using namespace Metasound;

	FMusicTransportEventStream::FMusicTransportEventStream(const FOperatorSettings& InSettings)
	{
	}

	void FMusicTransportEventStream::AddTransportRequest(EMusicPlayerTransportRequest InRequest, int32 AtSampleIndex)
	{
		const FRequestEvent NewEvent({ AtSampleIndex, InRequest });
		TransportEventsThisBlock.Insert(NewEvent, Algo::UpperBoundBy(TransportEventsThisBlock, AtSampleIndex, [](const FRequestEvent& In){ return In.SampleIndex; }));

		// Update the last state if it's one we want to keep track of
		check(TransportEventsThisBlock.Num() > 0);

		switch (const EMusicPlayerTransportRequest LastRequest = TransportEventsThisBlock.Last().Request)
		{
		case EMusicPlayerTransportRequest::Prepare:
		case EMusicPlayerTransportRequest::Play:
		case EMusicPlayerTransportRequest::Pause:
		case EMusicPlayerTransportRequest::Continue:
		case EMusicPlayerTransportRequest::Stop:
		case EMusicPlayerTransportRequest::Kill:
			LastTransportStateRequest = LastRequest;
			break;
		case EMusicPlayerTransportRequest::None:
		case EMusicPlayerTransportRequest::Seek:
		case EMusicPlayerTransportRequest::Count:
		default:
			break;
		}
	}

	void FMusicTransportEventStream::AddSeekRequest(int32 AtSampleIndex, const FMusicSeekTarget& Target)
	{
		NextSeekDestination = Target;
		FRequestEvent NewEvent({ AtSampleIndex, EMusicPlayerTransportRequest::Seek });
		TransportEventsThisBlock.Insert(NewEvent, Algo::UpperBoundBy(TransportEventsThisBlock, AtSampleIndex, [](const FRequestEvent& In) { return In.SampleIndex; }));
	}

	void FMusicTransportEventStream::AdvanceBlock()
	{

	}

	void FMusicTransportEventStream::Reset()
	{
		TransportEventsThisBlock.Empty();
	}

	FMusicTransportControllable::FMusicTransportControllable(EMusicPlayerTransportState InitialState)
		: TransportState(InitialState)
	{
	}

	void FMusicTransportControllable::Init(const FMusicTransportEventStream& TransportEventStream, FTransportInitFn&& InitFn)
	{
		const EMusicPlayerTransportRequest LastTransportStateRequest = TransportEventStream.GetLastTransportStateRequest();

		if (LastTransportStateRequest == FMusicTransportEventStream::InitialTransportStateRequest)
		{
			return;
		}
		
		const EMusicPlayerTransportState DesiredState = GetDesiredState(LastTransportStateRequest);
		TransportState = InitFn(DesiredState);
	}

	EMusicPlayerTransportState FMusicTransportControllable::GetNextTransportState(EMusicPlayerTransportState DesiredState) const
	{
		switch (DesiredState)
     	{
		case EMusicPlayerTransportState::Invalid:
		case EMusicPlayerTransportState::Preparing:
		case EMusicPlayerTransportState::Prepared:
		case EMusicPlayerTransportState::Stopping:
		case EMusicPlayerTransportState::Killing:
			return EMusicPlayerTransportState::Prepared;
			
		case EMusicPlayerTransportState::Starting:
		case EMusicPlayerTransportState::Playing:
		case EMusicPlayerTransportState::Continuing:
			return EMusicPlayerTransportState::Playing;
			
     	case EMusicPlayerTransportState::Seeking:
     		return GetTransportState();
     
     	case EMusicPlayerTransportState::Pausing:
		case EMusicPlayerTransportState::Paused:
     		return EMusicPlayerTransportState::Paused;
     
     	default:
     		checkNoEntry();
     		return EMusicPlayerTransportState::Invalid;
     	}
	}


	void FMusicTransportControllable::ExecuteTransportSpans(FMusicTransportEventStreamReadRef& InTransportPin, int32 InBlockSize, TransportSpanProcessor& Callback)
	{
		TransportSpanPostProcessor PostProcessor = [](int32, int32, EMusicPlayerTransportState) {};
		ExecuteTransportSpans(InTransportPin, InBlockSize, Callback, PostProcessor);
	}

	void FMusicTransportControllable::ExecuteTransportSpans(FMusicTransportEventStreamReadRef& InTransportPin, int32 InBlockSize, TransportSpanProcessor& Callback, TransportSpanPostProcessor& PostProcessor)
	{
		const FMusicTransportEventStream::FEventList& TransportChangesThisBlock = InTransportPin->GetTransportEventsInBlock();;

		if (TransportChangesThisBlock.IsEmpty())
		{
			Callback(0, InBlockSize, TransportState);
			PostProcessor(0, InBlockSize, TransportState);
			return;
		}

		int32 SampleIndex = 0;
		
		if (TransportChangesThisBlock[0].SampleIndex > 0)
		{
			// we have some samples to process in the current transport state first!
			TransportState = Callback(SampleIndex, TransportChangesThisBlock[0].SampleIndex, TransportState);
			PostProcessor(SampleIndex, TransportChangesThisBlock[0].SampleIndex, TransportState);
			SampleIndex = TransportChangesThisBlock[0].SampleIndex;
		}

		for (int32 NextEventIndex = 0; SampleIndex < InBlockSize; ++NextEventIndex)
		{
			int32 SpanEnd = (NextEventIndex+1) < TransportChangesThisBlock.Num() ? 
				FMath::Min(TransportChangesThisBlock[NextEventIndex+1].SampleIndex, InBlockSize) :
				InBlockSize;

			const EMusicPlayerTransportState DesiredState = GetDesiredState(TransportChangesThisBlock[NextEventIndex].Request);

			if (DesiredState == EMusicPlayerTransportState::Seeking && IsEffectivelyStopped())
			{
				bReceivedSeekWhileStopped = true;
			}

			TransportState = Callback(SampleIndex, SpanEnd, DesiredState);
			PostProcessor(SampleIndex, SpanEnd, TransportState);
			if (TransportState == EMusicPlayerTransportState::Starting || TransportState == EMusicPlayerTransportState::Playing)
			{
				bReceivedSeekWhileStopped = false;
			}

			SampleIndex = SpanEnd;
		}
	}

	FString FMusicTransportControllable::StateToString(EMusicPlayerTransportState S)
	{
		switch (S)
		{
		case EMusicPlayerTransportState::Invalid:    return TEXT("Invalid");
		case EMusicPlayerTransportState::Preparing:  return TEXT("Preparing");
		case EMusicPlayerTransportState::Prepared:   return TEXT("Prepared");
		case EMusicPlayerTransportState::Starting:   return TEXT("Starting");
		case EMusicPlayerTransportState::Playing:    return TEXT("Playing");
		case EMusicPlayerTransportState::Seeking:    return TEXT("Seeking");
		case EMusicPlayerTransportState::Pausing:    return TEXT("Pausing");
		case EMusicPlayerTransportState::Paused:     return TEXT("Paused");
		case EMusicPlayerTransportState::Continuing: return TEXT("Continuing");
		case EMusicPlayerTransportState::Stopping:   return TEXT("Stopping");
		case EMusicPlayerTransportState::Killing:    return TEXT("Killing");
		}
		return TEXT("<unknown>");
	}

	EMusicPlayerTransportState FMusicTransportControllable::GetDesiredState(EMusicPlayerTransportRequest Request) const
	{
		switch (Request)
		{
		case EMusicPlayerTransportRequest::Prepare:
			if (TransportState == EMusicPlayerTransportState::Invalid)
			{
				return EMusicPlayerTransportState::Preparing;
			}
			break;
		case EMusicPlayerTransportRequest::Play:
			if (TransportState == EMusicPlayerTransportState::Prepared)
			{
				return EMusicPlayerTransportState::Starting;
			}
			if (TransportState == EMusicPlayerTransportState::Paused)
			{
				return EMusicPlayerTransportState::Continuing;
			}
			break;
		case EMusicPlayerTransportRequest::Seek:
			return EMusicPlayerTransportState::Seeking;
		case EMusicPlayerTransportRequest::Pause:
			if (TransportState == EMusicPlayerTransportState::Playing)
			{
				return EMusicPlayerTransportState::Pausing;
			}
			break;
		case EMusicPlayerTransportRequest::Continue:
			if (TransportState == EMusicPlayerTransportState::Paused)
			{
				return EMusicPlayerTransportState::Continuing;
			}
			break;
		case EMusicPlayerTransportRequest::Stop:
			if (TransportState == EMusicPlayerTransportState::Playing || TransportState == EMusicPlayerTransportState::Paused)
			{
				return EMusicPlayerTransportState::Stopping;
			}
			break;
		case EMusicPlayerTransportRequest::Kill:
			if (TransportState == EMusicPlayerTransportState::Playing || TransportState == EMusicPlayerTransportState::Paused)
			{
				return EMusicPlayerTransportState::Killing;
			}
			break;
		default:
			break;
		}

		return TransportState;
	}

	bool FMusicTransportControllable::IsEffectivelyStopped() const
	{
		// If anyone adds a state to EMusicPlayerTransportState this function needs to know about it!
		static_assert((uint32)EMusicPlayerTransportState::NumStates == 11);

		return TransportState == EMusicPlayerTransportState::Invalid ||
		       TransportState == EMusicPlayerTransportState::Preparing ||
		       TransportState == EMusicPlayerTransportState::Prepared ||
		       TransportState == EMusicPlayerTransportState::Stopping ||
		       TransportState == EMusicPlayerTransportState::Killing;
	}

	FString HARMONIXMETASOUND_API MusicPlayerTransportStateToString(EMusicPlayerTransportState State)
	{
		switch (State)
		{
		case EMusicPlayerTransportState::Invalid:    return TEXT("Invalid");
		case EMusicPlayerTransportState::Preparing:	 return TEXT("Preparing");
		case EMusicPlayerTransportState::Prepared:	 return TEXT("Prepared");
		case EMusicPlayerTransportState::Starting:	 return TEXT("Starting");
		case EMusicPlayerTransportState::Playing:	 return TEXT("Playing");
		case EMusicPlayerTransportState::Seeking:	 return TEXT("Seeking");
		case EMusicPlayerTransportState::Pausing:	 return TEXT("Pausing");
		case EMusicPlayerTransportState::Paused:	 return TEXT("Paused");
		case EMusicPlayerTransportState::Continuing: return TEXT("Continuing");
		case EMusicPlayerTransportState::Stopping:	 return TEXT("Stopping");
		case EMusicPlayerTransportState::Killing:	 return TEXT("Killing");
		}											
		return TEXT("<Unrecognized>");
	}
}
