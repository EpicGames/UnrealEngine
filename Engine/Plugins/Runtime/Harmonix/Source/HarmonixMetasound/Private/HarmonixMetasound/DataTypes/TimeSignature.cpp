// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/TimeSignature.h"

#include "MetasoundDataTypeRegistrationMacro.h"

#include "HarmonixMidi/BarMap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimeSignature)

bool UTimeSignatureBlueprintLibrary::IsTimeSignature(const FMetaSoundOutput& Output)
{
	return Output.IsType<FTimeSignature>();
}

FTimeSignature UTimeSignatureBlueprintLibrary::GetTimeSignature(const FMetaSoundOutput& Output, bool& Success)
{
	FTimeSignature TimeSignature;
	Success = Output.Get(TimeSignature);
	return TimeSignature;
}


REGISTER_METASOUND_DATATYPE(FTimeSignature, "TimeSignature")
