// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/TimeSyncOption.h"

#define LOCTEXT_NAMESPACE "HarmonixMetasound"

namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(ETimeSyncOption, FEnumTimeSyncOption, "TimeSyncOption")
		DEFINE_METASOUND_ENUM_ENTRY(
			ETimeSyncOption::None, 
			"ETimeSyncOption_None_DisplayName", 
			"None", 
			"ETimeSyncOption_None_TT", 
			"Will not sync to a music clock"),
		DEFINE_METASOUND_ENUM_ENTRY(
			ETimeSyncOption::TempoSync, 
			"ETimeSyncOption_TempoSync_DisplayName", 
			"TempoSync", 
			"ETimeSyncOption_TempoSync_TT", 
			"Time setting is interpreted as a multiple of quarter notes and kept in sync with the designated music clock"),
		DEFINE_METASOUND_ENUM_ENTRY(
			ETimeSyncOption::SpeedScale, 
			"ETimeSyncOption_SpeedScale_DisplayName", 
			"SpeedScale", 
			"ETimeSyncOption_SpeedScale_TT", 
			"Time setting is multiplied by the designated music clock's playback speed")
	DEFINE_METASOUND_ENUM_END()
}

#undef LOCTEXT_NAMESPACE
