// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMusicInterfaces.h"

#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "Metasound.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundSource.h"
#include "MetasoundTime.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSoundInterfaces"

namespace HarmonixMetasound
{

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "Harmonix.MusicAsset"
	namespace MusicAssetInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		const FMetasoundFrontendVersion FrontendVersion{ AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
		const FLazyName PlayIn(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Play"));
		const FLazyName PauseIn(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Pause"));
		const FLazyName ContinueIn(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Continue"));
		const FLazyName StopIn(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Stop"));
		const FLazyName KillIn(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Kill"));
		const FLazyName SeekIn(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Seek"));
		const FLazyName SeekTargetSecondsIn(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("SeekTargetSeconds"));
		const FLazyName MidiClockOut(AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("MIDI Clock"));

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: Audio::FParameterInterface{ FrontendVersion.Name, FrontendVersion.Number.ToInterfaceVersion() }
				{
					using namespace Metasound;

					Inputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfacePlay", "Play"),
							LOCTEXT("HarmonixMusicAssetInterfacePlay_Description", "Starts the music playing."),
							GetMetasoundDataTypeName<FTrigger>(),
							PlayIn.Resolve(),
							FText(),
							0
						});
					Inputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfacePause", "Pause"),
							LOCTEXT("HarmonixMusicAssetInterfacePause_Description", "Pauses the music."),
							GetMetasoundDataTypeName<FTrigger>(),
							PauseIn.Resolve(),
							FText(),
							1
						});
					Inputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfaceContinue", "Continue"),
							LOCTEXT("HarmonixMusicAssetInterfaceContinue_Description", "Continues music that was paused."),
							GetMetasoundDataTypeName<FTrigger>(),
							ContinueIn.Resolve(),
							FText(),
							2
						});
					Inputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfaceStop", "Stop"),
							LOCTEXT("HarmonixMusicAssetInterfaceStop_Description", "Stops the music playback."),
							GetMetasoundDataTypeName<FTrigger>(),
							StopIn.Resolve(),
							FText(),
							3
						});
					Inputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfaceKill", "Kill"),
							LOCTEXT("HarmonixMusicAssetInterfaceKill_Description", "Kills the music playback."),
							GetMetasoundDataTypeName<FTrigger>(),
							KillIn.Resolve(),
							FText(),
							4
						});
					Inputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfaceSeek", "Seek"),
							LOCTEXT("HarmonixMusicAssetInterfaceSeek_Description", "Triggers a seek."),
							GetMetasoundDataTypeName<FTrigger>(),
							SeekIn.Resolve(),
							FText(),
							5
						});
					Inputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfaceSeekTarget", "SeekTargetSeconds"),
							LOCTEXT("HarmonixMusicAssetInterfaceSeekTarget_Description", "The position the music should seek to."),
							GetMetasoundDataTypeName<float>(),
							SeekTargetSecondsIn.Resolve(),
							FText(),
							6
						});
					Outputs.Add(
						{
							LOCTEXT("HarmonixMusicAssetInterfaceMidiClockOut", "MIDI Clock"),
							LOCTEXT("HarmonixMusicAssetInterfaceMidiClockOut_Description", "A MIDI Clock output to drive musical time."),
							GetMetasoundDataTypeName<HarmonixMetasound::FMidiClock>(),
							MidiClockOut.Resolve()
						});
				}
			};

			static Audio::FParameterInterfacePtr InterfacePtr;
			if (InterfacePtr.IsValid() == false)
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	}
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

	void RegisterHarmonixMetasoundMusicInterfaces()
	{
		Audio::IAudioParameterInterfaceRegistry& AudioParamRegistry = Audio::IAudioParameterInterfaceRegistry::Get();
		AudioParamRegistry.RegisterInterface(MusicAssetInterface::CreateInterface());
	}
}

#undef LOCTEXT_NAMESPACE
