// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"

namespace HarmonixMetasound
{
	namespace MusicAssetInterface
	{
		const FMetasoundFrontendVersion& GetVersion();
		Audio::FParameterInterfacePtr CreateInterface();

		const extern FMetasoundFrontendVersion FrontendVersion;
		const extern FLazyName PlayIn;
		const extern FLazyName PauseIn;
		const extern FLazyName ContinueIn;
		const extern FLazyName StopIn;
		const extern FLazyName KillIn;
		const extern FLazyName SeekIn;
		const extern FLazyName SeekTargetSecondsIn;
		const extern FLazyName MidiClockOut;
	}

	void RegisterHarmonixMetasoundMusicInterfaces();
}
