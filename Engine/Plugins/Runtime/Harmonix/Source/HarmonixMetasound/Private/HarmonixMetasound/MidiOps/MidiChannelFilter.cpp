// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiChannelFilter.h"

namespace Harmonix::Midi::Ops
{
	void FMidiChannelFilter::Process(const HarmonixMetasound::FMidiStream& InStream, HarmonixMetasound::FMidiStream& OutStream)
	{
		const auto Filter = [this](const HarmonixMetasound::FMidiStreamEvent& Event)
		{
			if (Event.MidiMessage.IsStd())
			{
				const uint8 ChannelIdx = Event.MidiMessage.GetStdChannel();
				check(ChannelIdx < 16);
				return GetChannelEnabled(ChannelIdx + 1);
			}

			// Pass non-std messages
			return true;
		};

		// Copy events which pass the filter to the output
		HarmonixMetasound::FMidiStream::Copy(InStream, OutStream, Filter);

		// Unstick notes if necessary
		StuckNoteGuard.Process(InStream, OutStream, Filter);
	}

	void FMidiChannelFilter::SetChannelEnabled(const uint8 Channel, const bool Enabled)
	{
		const uint16 ChannelMask = MidiChannelToBitmask(Channel);

		if (Enabled)
		{
			EnabledChannels |= ChannelMask;
		}
		else
		{
			EnabledChannels &= ~ChannelMask;
		}
	}

	bool FMidiChannelFilter::GetChannelEnabled(const uint8 Channel) const
	{
		const uint16 ChannelMask = MidiChannelToBitmask(Channel);
		return (EnabledChannels & ChannelMask) == ChannelMask;
	}
}
