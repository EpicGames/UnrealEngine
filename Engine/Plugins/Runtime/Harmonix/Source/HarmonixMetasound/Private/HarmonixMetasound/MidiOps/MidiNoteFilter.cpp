// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiNoteFilter.h"

namespace Harmonix::Midi::Ops
{
	void FMidiNoteFilter::Process(const HarmonixMetasound::FMidiStream& InStream, HarmonixMetasound::FMidiStream& OutStream)
	{
		const auto Filter = [this](const HarmonixMetasound::FMidiStreamEvent& Event)
		{
			// Note ons must pass note number and velocity check
			if (Event.MidiMessage.IsNoteOn())
			{
				const uint8 NoteNumber = Event.MidiMessage.GetStdData1();
				const uint8 Velocity = Event.MidiMessage.GetStdData2();
				return NoteNumber >= MinNoteNumber
						&& NoteNumber <= MaxNoteNumber
						&& Velocity >= MinVelocity
						&& Velocity <= MaxVelocity;
			}

			// Note offs must pass note number check
			if (Event.MidiMessage.IsNoteOff())
			{
				const uint8 NoteNumber = Event.MidiMessage.GetStdData1();
				return NoteNumber >= MinNoteNumber && NoteNumber <= MaxNoteNumber;
			}

			// Handle all notes off/kill
			if (Event.MidiMessage.IsNoteMessage())
			{
				return true;
			}

			return IncludeOtherEvents.Get();
		};

		// Copy events which pass the filter to the output
		HarmonixMetasound::FMidiStream::Copy(InStream, OutStream, Filter);

		// Unstick notes if necessary
		StuckNoteGuard.Process(InStream, OutStream, Filter);
	}
}
