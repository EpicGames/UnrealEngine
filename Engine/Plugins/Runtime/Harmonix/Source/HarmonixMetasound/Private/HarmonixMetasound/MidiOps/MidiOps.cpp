// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiOps.h"

DEFINE_LOG_CATEGORY(LogHarmonixMidiOps)

namespace Harmonix::Midi::Ops
{
	uint16 MidiChannelToBitmask(const uint8 MidiChannel)
	{
		// special case: 0 is all channels
		if (MidiChannel == 0)
		{
			return AllChannelsOn;
		}

		// bad input: > 16 is no channels
		if (MidiChannel > 16)
		{
			UE_LOGF(LogHarmonixMidiOps, Warning, "Provided a MIDI channel > 16. You probably didn't mean to do that. Failing gracefully, probably.");
			return AllChannelsOff;
		}

		// shift to 0-15 range, then make the mask
		return 1 << (MidiChannel - 1);
	}

}
