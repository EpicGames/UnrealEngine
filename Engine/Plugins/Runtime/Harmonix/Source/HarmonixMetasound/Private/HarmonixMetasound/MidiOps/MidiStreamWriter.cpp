// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiStreamWriter.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMidi/MidiWriter.h"

namespace Harmonix::Midi::Ops
{
	using namespace HarmonixMetasound;
	FMidiStreamWriter::FMidiStreamWriter(TUniquePtr<FArchive>&& InArchive)
		: Archive(MoveTemp(InArchive))
	{}

	void FMidiStreamWriter::Process(const FMidiStream& InStream)
	{
		ensureMsgf(InStream.GetClock(), TEXT("Midi stream must have a midi clock for the MidiStreamWriter to process it"));
		if (TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> MidiClock = InStream.GetClock())
		{
			using namespace MidiClockMessageTypes;
			for (const FMidiClockEvent& ClockEvent : MidiClock->GetMidiClockEventsInBlock())
			{
				if (const FAdvance* AsAdvance = ClockEvent.TryGet<MidiClockMessageTypes::FAdvance>())
				{
					Process(InStream, AsAdvance->FirstTickToProcess, AsAdvance->LastTickToProcess());
				}
			}
		}
	}

	void FMidiStreamWriter::Process(const FMidiStream& InStream, int32 FirstTickToProcess, int32 LastTickToProcess)
	{
		bool AddedEvents = false;
		for (const FMidiStreamEvent& MidiEvent : InStream.GetEventsInBlock())
		{
			if (FirstTickToProcess <= MidiEvent.CurrentMidiTick && MidiEvent.CurrentMidiTick <= LastTickToProcess)
			{
				const int32 OffsetTick = MidiEvent.CurrentMidiTick - FirstTickToProcess;
				const int32 MidiTick = NextWriteTick + OffsetTick;
				FMidiTrack& MidiTrack = MidiTracks.FindOrAdd(MidiEvent.TrackIndex);
				if (MidiEvent.MidiMessage.IsText())
				{
					uint16 TextIndex = MidiEvent.MidiMessage.GetTextIndex();
					const FString* EventText = InStream.GetMidiTrackText(MidiEvent.TrackIndex, TextIndex);
					if (EventText == nullptr)
					{
						continue;
					}

					// make sure the text index is the same in the TextRepository as the event
					// this can potentially leave empty strings in the repository, which isn't optimal
					FMidiTextRepository& TextRepository = *MidiTrack.GetTextRepository();
					if (!TextRepository.IsValidIndex(TextIndex))
					{
						TextRepository.SetNum(TextIndex + 1);
						TextRepository[TextIndex] = *EventText;
					}

					// found a text entry mismatch - should we overrwrite or ignore?
					if (!ensure(TextRepository[TextIndex] == *EventText))
					{
						TextRepository[TextIndex] = *EventText;
					}

					// only add text events we have strings for
					MidiTrack.AddEvent(FMidiEvent(MidiTick, MidiEvent.MidiMessage));
					AddedEvents = true;	
				}
				else
				{
					MidiTrack.AddEvent(FMidiEvent(MidiTick, MidiEvent.MidiMessage));
					AddedEvents = true;						
				}

			}
		}
		NextWriteTick += LastTickToProcess - FirstTickToProcess + 1;

		if (AddedEvents)
		{
			Archive->Seek(0);
			FMidiWriter MidiWriter = FMidiWriter(*Archive);
			for (TPair<int32, const FMidiTrack&> Pair : MidiTracks)
			{
				Pair.Value.WriteStdMidi(MidiWriter);
			}
		}
	}
};