// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiTrackFilter.h"

namespace Harmonix::Midi::Ops
{
	void FMidiTrackFilter::Process(const HarmonixMetasound::FMidiStream& InStream, HarmonixMetasound::FMidiStream& OutStream)
	{
		const auto Filter = [this](const HarmonixMetasound::FMidiStreamEvent& Event)
		{
			ensure(Event.TrackIndex >= 0 && Event.TrackIndex <= UINT16_MAX);
			const uint16 TrackIndex = static_cast<uint16>(Event.TrackIndex);
			bool IncludeEvent = false;

			if (TrackIndex == 0)
			{
				IncludeEvent = IncludeConductorTrack;
			}
			else if (TrackIndex >= MinTrackIdx && TrackIndex <= MaxTrackIdx)
			{
				IncludeEvent = true;
			}

			return IncludeEvent;
		};
		
		// Copy events which pass the filter to the output
		HarmonixMetasound::FMidiStream::Copy(InStream, OutStream, Filter);

		// Unstick notes if necessary
		StuckNoteGuard.Process(InStream, OutStream, Filter);
	}

	void FMidiTrackFilter::SetTrackRange(const uint16 InMinTrackIdx, const uint16 InMaxTrackIdx, const bool InIncludeConductorTrack)
	{
		MinTrackIdx = FMath::Clamp(InMinTrackIdx, 0, UINT16_MAX);
		MaxTrackIdx = FMath::Clamp(InMaxTrackIdx, MinTrackIdx, UINT16_MAX);
		IncludeConductorTrack = InIncludeConductorTrack;
	}
}
