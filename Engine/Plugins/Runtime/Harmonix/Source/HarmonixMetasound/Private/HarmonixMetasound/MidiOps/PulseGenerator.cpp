// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/PulseGenerator.h"

namespace Harmonix::Midi::Ops
{
	void FPulseGenerator::Enable(const bool bEnable)
	{
		Enabled = bEnable;
	}

	void FPulseGenerator::SetInterval(const FMusicTimeInterval& NewInterval)
	{
		Interval = NewInterval;

		// Multiplier should be >= 1
		Interval.IntervalMultiplier = FMath::Max(Interval.IntervalMultiplier, 1);
	}

	void FPulseGenerator::SetDutyCycle(float InDutyCycle)
	{
		DutyCycle = FMath::Clamp(InDutyCycle, 0.0f, 1.0f);
	}

	void FPulseGenerator::Reset()
	{
		NextPulses.Reset();
	}

	void FPulseGenerator::Process(
		const HarmonixMetasound::FMidiClock& MidiClock,
		const TFunctionRef<void(const FPulseInfo&)>& TriggerPulseOn,
		const TFunctionRef<void(const FPulseInfo&)>& TriggerPulseOff)
	{
		bool bIntervalIsValid = Interval.Interval != EMidiClockSubdivisionQuantization::None;
		bool bTimeSigIsValid = CurrentTimeSignature.Numerator > 0 && CurrentTimeSignature.Denominator > 0;
		
		// ensure the pulse generator is lined up with the current clock phase
		if (NextPulses.IsEmpty())
		{
			CurrentTimeSignature = *MidiClock.GetSongMapEvaluator().GetTimeSignatureAtTick(MidiClock.GetLastProcessedMidiTick());
			bTimeSigIsValid = CurrentTimeSignature.Numerator > 0 && CurrentTimeSignature.Denominator > 0;
			
			// Find the next pulse and line up phase with the current bar
			const FMusicTimestamp ClockCurrentTimestamp = MidiClock.GetMusicTimestampAtBlockOffset(0);
			FPulse& PulseOn = NextPulses.Add_GetRef(FPulse::On(FMusicTimestamp()));
			PulseOn.Timestamp.Bar = ClockCurrentTimestamp.Bar;
			PulseOn.Timestamp.Beat = 1;
			IncrementTimestampByOffset(PulseOn.Timestamp, Interval, CurrentTimeSignature);
			while (bIntervalIsValid && bTimeSigIsValid && PulseOn.Timestamp < ClockCurrentTimestamp)
			{
				IncrementTimestampByInterval(PulseOn.Timestamp, Interval, CurrentTimeSignature);
			}
		}
		
		using namespace HarmonixMetasound;
		using namespace HarmonixMetasound::MidiClockMessageTypes;

		for (const FMidiClockEvent& ClockEvent : MidiClock.GetMidiClockEventsInBlock())
		{
			if (const FAdvance* AsAdvance = ClockEvent.TryGet<FAdvance>())
			{
				if (NextPulses.IsEmpty())
				{
					return;
				}

				// copy the value of the next pulse
				FPulse Pulse = NextPulses[0];
				int32 PulseTick = MidiClock.GetSongMapEvaluator().MusicTimestampToTick(Pulse.Timestamp);

				while (bIntervalIsValid && bTimeSigIsValid && AsAdvance->LastTickToProcess() >= PulseTick)
				{
					if (Pulse.Type == FPulse::EType::On)
					{
						FPulse NextPulseOff = FPulse::Off(Pulse.Timestamp);
						const float IntervalBeats = Interval.GetIntervalBeats(CurrentTimeSignature);
						const float DutyCycleClamped = FMath::Clamp(DutyCycle, 0.0f, 1.0f);
						IncrementTimestampByBeats(NextPulseOff.Timestamp, IntervalBeats * DutyCycleClamped, CurrentTimeSignature);

						int32 NextPulseOffTick = MidiClock.GetSongMapEvaluator().MusicTimestampToTick(NextPulseOff.Timestamp);

						// only send the Pulse if the PulseOff happens in the future
						if (NextPulseOffTick > PulseTick)
						{
							TriggerPulseOn({ ClockEvent.BlockFrameIndex, PulseTick });
							NextPulses.Add(NextPulseOff);
						}

						FPulse NextPulseOn = FPulse::On(Pulse.Timestamp);
						IncrementTimestampByInterval(NextPulseOn.Timestamp, Interval, CurrentTimeSignature);
						NextPulses.Add(NextPulseOn);
						
						NextPulses.RemoveAt(0);
					}
					else
					{
						TriggerPulseOff({ ClockEvent.BlockFrameIndex, PulseTick });
						NextPulses.RemoveAt(0);
					}

					if (NextPulses.IsEmpty())
					{
						break;
					}
					
					Pulse = NextPulses[0];
					PulseTick = MidiClock.GetSongMapEvaluator().MusicTimestampToTick(Pulse.Timestamp);
				}
			}
			else if (const FTimeSignatureChange* AsTimeSigChange = ClockEvent.TryGet<FTimeSignatureChange>())
			{
				CurrentTimeSignature = AsTimeSigChange->TimeSignature;
				bTimeSigIsValid = CurrentTimeSignature.Numerator > 0 && CurrentTimeSignature.Denominator > 0;
				
				// Time sig changes will come on the downbeat, and if we change time signature,
				// we want to reset the pulse, so the next pulse is now plus the offset
				for (FPulse& Pulse : NextPulses)
				{
					Pulse.Timestamp = MidiClock.GetSongMapEvaluator().TickToMusicTimestamp(AsTimeSigChange->Tick);
					IncrementTimestampByOffset(Pulse.Timestamp, Interval, CurrentTimeSignature);
				}
			}
			else if (const FLoop* AsLoop = ClockEvent.TryGet<FLoop>())
			{
				// We assume the pulse should be reset on loop, and the loop start should imply the phase of the pulse
				for (FPulse& Pulse : NextPulses)
				{
					Pulse.Timestamp = MidiClock.GetSongMapEvaluator().TickToMusicTimestamp(AsLoop->FirstTickInLoop);
					IncrementTimestampByOffset(Pulse.Timestamp, Interval, CurrentTimeSignature);
				}
			}
			else if (const FSeek* AsSeek = ClockEvent.TryGet<FSeek>())
			{
				for (FPulse& Pulse : NextPulses)
				{
					// When we seek, reset the pulse phase to the current bar
					const FMusicTimestamp ClockCurrentTimestamp = MidiClock.GetSongMapEvaluator().TickToMusicTimestamp(AsSeek->NewNextTick);
					Pulse.Timestamp.Bar = ClockCurrentTimestamp.Bar;
					Pulse.Timestamp.Beat = 1;
					IncrementTimestampByOffset(Pulse.Timestamp, Interval, CurrentTimeSignature);
					while (bIntervalIsValid && bTimeSigIsValid && Pulse.Timestamp < ClockCurrentTimestamp)
					{
						IncrementTimestampByInterval(Pulse.Timestamp, Interval, CurrentTimeSignature);
					}
				}
			}
		}
	}

	void FMidiPulseGenerator::Reset()
	{
		FPulseGenerator::Reset();

		LastNoteOn.Reset();
	}

	void FMidiPulseGenerator::Process(const HarmonixMetasound::FMidiClock& MidiClock, HarmonixMetasound::FMidiStream& OutStream)
	{
		OutStream.PrepareBlock();

		// kill any notes if the interval becomes invalid
		if (GetInterval().Interval == EMidiClockSubdivisionQuantization::None && LastNoteOn.IsSet())
		{
			check(LastNoteOn->MidiMessage.IsNoteOn());
			const int32 NoteOffSample = 0;

			// Trigger the note off one tick before the note on
			const int32 NoteOffTick = MidiClock.GetNextTickToProcessAtBlockFrame(0);

			FMidiMsg Msg{ FMidiMsg::CreateAllNotesOff() };
			HarmonixMetasound::FMidiStreamEvent Event{ &VoiceGenerator, Msg };
			Event.BlockSampleFrameIndex = NoteOffSample;
			Event.AuthoredMidiTick = NoteOffTick;
			Event.CurrentMidiTick = NoteOffTick;
			Event.TrackIndex = 1;
			OutStream.InsertMidiEvent(Event);

			LastNoteOn.Reset();	
		}

		FPulseGenerator::Process(MidiClock,
			[this, &OutStream](const FPulseInfo& Pulse)
			{
				PulseNoteOn(Pulse.BlockFrameIndex, Pulse.Tick, OutStream);
			},
			[this, &OutStream](const FPulseInfo& Pulse)
			{
				PulseNoteOff(Pulse.BlockFrameIndex, Pulse.Tick, OutStream);
			});
	}

	void FMidiPulseGenerator::PulseNoteOn(const int32 BlockFrameIndex, const int32 PulseTick, HarmonixMetasound::FMidiStream& OutStream)
	{
		if (Enabled)
		{
			FMidiMsg Msg{ FMidiMsg::CreateNoteOn(Channel - 1, NoteNumber, Velocity) };
			HarmonixMetasound::FMidiStreamEvent Event{ &VoiceGenerator, Msg };
			Event.BlockSampleFrameIndex = BlockFrameIndex;
			Event.AuthoredMidiTick = PulseTick;
			Event.CurrentMidiTick = PulseTick;
			Event.TrackIndex = Track;
			OutStream.InsertMidiEvent(Event);

			LastNoteOn.Emplace(MoveTemp(Event));
		}
	}

	void FMidiPulseGenerator::PulseNoteOff(const int32 BlockFrameIndex, const int32 PulseTick, HarmonixMetasound::FMidiStream& OutStream)
	{
		// Note off if there was a previous note on
		if (LastNoteOn.IsSet())
		{
			check(LastNoteOn->MidiMessage.IsNoteOn());
			// Trigger the note off one tick before the note on
			const int32 NoteOffTick = PulseTick;

			FMidiMsg Msg{ FMidiMsg::CreateNoteOff(LastNoteOn->MidiMessage.GetStdChannel(), LastNoteOn->MidiMessage.GetStdData1()) };
			HarmonixMetasound::FMidiStreamEvent Event{ &VoiceGenerator, Msg };
			Event.BlockSampleFrameIndex = BlockFrameIndex;
			Event.AuthoredMidiTick = PulseTick;
			Event.CurrentMidiTick = PulseTick;
			Event.TrackIndex = LastNoteOn->TrackIndex;
			OutStream.InsertMidiEvent(Event);

			LastNoteOn.Reset();
		}
	}

}
