// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/StuckNoteGuard.h"

namespace Harmonix::Midi::Ops
{
	void FStuckNoteGuard::Process(
		const HarmonixMetasound::FMidiStream& InStream,
		HarmonixMetasound::FMidiStream& OutStream,
		const FIncludeNotePredicate& Predicate)
	{
		TrackNotes(InStream, Predicate);
		
		// For all of the tracked notes, unstick ones that no longer pass the filter
		for (auto It = ActiveNotes.CreateIterator(); It; ++It)
		{
			if (!Predicate(*It))
			{
				// Send a note off
				check(It->MidiMessage.IsNoteOn()); // shouldn't happen, yell for help
				const uint8 Channel = It->MidiMessage.GetStdChannel();
				const uint8 NoteNumber = It->MidiMessage.GetStdData1();
				HarmonixMetasound::FMidiStreamEvent NoteOffEvent{ It->GetVoiceId().GetGeneratorId(), FMidiMsg::CreateNoteOff(Channel, NoteNumber) };
				OutStream.InsertMidiEvent(NoteOffEvent);

				// Stop tracking this note on this track
				It.RemoveCurrent();
			}
		}
	}

	void FStuckNoteGuard::UnstickNotes(const HarmonixMetasound::FMidiStream& StreamToCompare, const FUnstickNoteFn& UnstickNoteFn)
	{
		TrackNotes(StreamToCompare, [](const HarmonixMetasound::FMidiStreamEvent&) { return true; });
		
		for (auto It = ActiveNotes.CreateIterator(); It; ++It)
		{
			if (!StreamToCompare.NoteIsActive(*It))
			{
				UnstickNoteFn(*It);
				It.RemoveCurrent();
			}
		}
	}

	void FStuckNoteGuard::TrackNotes(const HarmonixMetasound::FMidiStream& InStream, const FIncludeNotePredicate& Predicate)
	{
		for (const HarmonixMetasound::FMidiStreamEvent& Event : InStream.GetEventsInBlock())
		{
			// If this is a note on which passes the filter, track it
			if (Event.MidiMessage.IsNoteOn() && Predicate(Event))
			{
				const int32 TrackIndex = Event.TrackIndex;
				const FMidiVoiceId VoiceId = Event.GetVoiceId();

				if (!ActiveNotes.ContainsByPredicate([TrackIndex, VoiceId](const HarmonixMetasound::FMidiStreamEvent& TrackedEvent)
				{
					return TrackIndex == TrackedEvent.TrackIndex && VoiceId == TrackedEvent.GetVoiceId();
				}))
				{
					ActiveNotes.Add(Event);
				}
			}
			// If this is a note off which passes the filter, untrack it
			else if (Event.MidiMessage.IsNoteOff() && Predicate(Event))
			{
				const int32 TrackIndex = Event.TrackIndex;
				const FMidiVoiceId VoiceId = Event.GetVoiceId();
				
				ActiveNotes.RemoveAll([TrackIndex, VoiceId](const HarmonixMetasound::FMidiStreamEvent& TrackedEvent)
				{
					return TrackIndex == TrackedEvent.TrackIndex && VoiceId == TrackedEvent.GetVoiceId(); 
				});
			}
			// If this is an all notes off/kill, untrack everything
			// NOTE: This diverges from the MIDI spec, where "all notes off" has a track and channel associated with it.
			// If we end up supporting the track- and channel-specific "all notes off" we will need to add support here to avoid stuck notes.
			else if (Event.MidiMessage.IsAllNotesOff() || Event.MidiMessage.IsAllNotesKill())
			{
				ActiveNotes.Reset();
			}
		}
	}
}
