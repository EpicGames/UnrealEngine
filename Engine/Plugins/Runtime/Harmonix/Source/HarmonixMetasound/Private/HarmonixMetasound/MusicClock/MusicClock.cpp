// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MusicClock/MusicClock.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogUMusicClock, Log, All)

UMusicClock::UMusicClock()
{
}

void UMusicClock::BeginDestroy()
{
	UMidiClockUpdateSubsystem::StopTrackingMusicClock(this);
	Super::BeginDestroy();
}

void UMusicClock::SetSource(const TScriptInterface<IMusicSource>& InSource)
{
	SourceObject = InSource.GetObject();
	SourceInterfaceCache = InSource ? Cast<IMusicSource>(InSource.GetObject()) : nullptr;

	// Reset state when rebinding
	CachedClockState = EMusicClockState::Stopped;
	CurrentPos.Reset();
	PrevPos.Reset();
	DeltaBarF = 0.f;
	DeltaBeatF = 0.f;
	LastBroadcastBar = -1;
	LastBroadcastBeat = -1;
	LastBroadcastSection = FSongSection();
	bLoopedThisFrame = false;
	bSeekedThisFrame = false;
}

TScriptInterface<IMusicSource> UMusicClock::GetSource() const
{
	if (UObject* Obj = SourceObject.Get())
	{
		return TScriptInterface<IMusicSource>(Obj);
	}
	return {};
}

IMusicSource* UMusicClock::GetValidSource() const
{
	return SourceObject.IsValid() ? SourceInterfaceCache : nullptr;
}

void UMusicClock::UpdateForGameFrame()
{
	if (!IsInGameThread())
	{
		return;
	}

	if (GFrameCounter == LastUpdateFrame)
	{
		return;
	}

	IMusicSource* SourcePtr = GetValidSource();
	if (!SourcePtr)
	{
		// Source was destroyed — transition to Stopped if we weren't already
		if (SourceInterfaceCache != nullptr)
		{
			SourceInterfaceCache = nullptr;
		}
		if (CachedClockState != EMusicClockState::Stopped)
		{
			CachedClockState = EMusicClockState::Stopped;
			CurrentPos.Reset();
			PrevPos.Reset();
			DeltaBarF = 0.f;
			DeltaBeatF = 0.f;
			Tempo = 0.f;
			CurrentBeatsPerSecondCached = 0.f;
			CurrentBarsPerSecondCached = 0.f;
			ClockAdvanceRate = 1.f;
			bLoopedThisFrame = false;
			bSeekedThisFrame = false;
			PlayStateEvent.Broadcast(EMusicClockState::Stopped);
		}
		LastUpdateFrame = GFrameCounter;
		return;
	}

	EMusicClockState PrevClockState = CachedClockState;
	Harmonix::ESourceState SourceState = SourcePtr->GetSourceState();

	// Derive the new clock state from the source
	EMusicClockState NewClockState;
	switch (SourceState)
	{
	case Harmonix::ESourceState::Running:   NewClockState = EMusicClockState::Running; break;
	case Harmonix::ESourceState::Paused:    NewClockState = EMusicClockState::Paused; break;
	case Harmonix::ESourceState::Preparing: NewClockState = EMusicClockState::Stopped; break;
	case Harmonix::ESourceState::Stopped:   NewClockState = EMusicClockState::Stopped; break;
	default:                                NewClockState = EMusicClockState::Stopped; break;
	}
	CachedClockState = NewClockState;

	if (SourceState == Harmonix::ESourceState::Running)
	{
		PrevPos = CurrentPos;
		CurrentPos = SourcePtr->GetCurrentSongPos();

		DeltaBarF = CurrentPos.BarsIncludingCountIn - PrevPos.BarsIncludingCountIn;
		DeltaBeatF = CurrentPos.BeatsIncludingCountIn - PrevPos.BeatsIncludingCountIn;

		bLoopedThisFrame = SourcePtr->LoopedThisFrame();
		bSeekedThisFrame = SourcePtr->SeekedThisFrame();

		UpdatePlaybackRate(CurrentPos, SourcePtr->GetCurrentClockAdvanceRate());
		BroadcastSongPosChanges();
	}
	else if (SourceState == Harmonix::ESourceState::Stopped)
	{
		// When stopped, reset position to 0
		if (PrevClockState != EMusicClockState::Stopped)
		{
			CurrentPos.Reset();
			PrevPos.Reset();
			DeltaBarF = 0.f;
			DeltaBeatF = 0.f;
			Tempo = 0.f;
			CurrentBeatsPerSecondCached = 0.f;
			CurrentBarsPerSecondCached = 0.f;
			ClockAdvanceRate = 1.f;
		}
		bLoopedThisFrame = false;
		bSeekedThisFrame = false;
	}
	else if (SourceState == Harmonix::ESourceState::Preparing)
	{
		// Preparing = intent to run, but no data yet. Reset position to zero
		// so we don't report stale data, but don't broadcast events.
		if (PrevClockState != EMusicClockState::Stopped)
		{
			CurrentPos.Reset();
			PrevPos.Reset();
			DeltaBarF = 0.f;
			DeltaBeatF = 0.f;
		}
		bLoopedThisFrame = false;
		bSeekedThisFrame = false;
	}
	else // Paused
	{
		// When paused, keep current position frozen. Zero out deltas.
		DeltaBarF = 0.f;
		DeltaBeatF = 0.f;
		bLoopedThisFrame = false;
		bSeekedThisFrame = false;
	}

	// Broadcast state changes
	if (NewClockState != PrevClockState)
	{
		PlayStateEvent.Broadcast(NewClockState);
	}

	LastUpdateFrame = GFrameCounter;
}

EMusicClockState UMusicClock::GetState() const
{
	return CachedClockState;
}

const FMidiSongPos& UMusicClock::GetCurrentSongPos() const
{
	return CurrentPos;
}

const FMidiSongPos& UMusicClock::GetPreviousSongPos() const
{
	return PrevPos;
}

float UMusicClock::GetSecondsIncludingCountIn() const
{
	return CurrentPos.SecondsIncludingCountIn;
}

float UMusicClock::GetSecondsFromBarOne() const
{
	return CurrentPos.SecondsFromBarOne;
}

float UMusicClock::GetBarsIncludingCountIn() const
{
	return CurrentPos.BarsIncludingCountIn;
}

float UMusicClock::GetBeatsIncludingCountIn() const
{
	return CurrentPos.BeatsIncludingCountIn;
}

FMusicTimestamp UMusicClock::GetCurrentTimestamp() const
{
	return CurrentPos.Timestamp;
}

float UMusicClock::GetCurrentTempo() const
{
	return Tempo;
}

float UMusicClock::GetCurrentBeatsPerMinute() const
{
	ensure(TimeSignatureDenom != 0 || Tempo == 0);
	return Tempo * TimeSignatureDenom / 4.0f;
}

float UMusicClock::GetCurrentBeatsPerSecond() const
{
	return CurrentBeatsPerSecondCached;
}

float UMusicClock::GetCurrentBarsPerSecond() const
{
	return CurrentBarsPerSecondCached;
}

float UMusicClock::GetCurrentClockAdvanceRate() const
{
	return ClockAdvanceRate;
}

void UMusicClock::GetCurrentTimeSignature(int& OutNumerator, int& OutDenominator) const
{
	OutNumerator = TimeSignatureNum;
	OutDenominator = TimeSignatureDenom;
}

FString UMusicClock::GetCurrentSectionName() const
{
	return CurrentPos.CurrentSongSection.Name;
}

int32 UMusicClock::GetCurrentSectionIndex() const
{
	const ISongMapEvaluator* Maps = GetSongMaps();
	if (Maps)
	{
		return Maps->GetSectionIndexAtTick(CurrentPos.CurrentSongSection.StartTick);
	}
	return -1;
}

float UMusicClock::GetCurrentSectionStartMs() const
{
	const ISongMapEvaluator* Maps = GetSongMaps();
	if (Maps)
	{
		return Maps->TickToMs(CurrentPos.CurrentSongSection.StartTick);
	}
	return 0.f;
}

float UMusicClock::GetCurrentSectionLengthMs() const
{
	const ISongMapEvaluator* Maps = GetSongMaps();
	if (Maps)
	{
		return Maps->TickToMs(CurrentPos.CurrentSongSection.LengthTicks);
	}
	return 0.f;
}

float UMusicClock::GetDistanceFromCurrentBeat() const
{
	return CurrentPos.BeatsIncludingCountIn - FMath::FloorToFloat(CurrentPos.BeatsIncludingCountIn);
}

float UMusicClock::GetDistanceToNextBeat() const
{
	return 1.0f - GetDistanceFromCurrentBeat();
}

float UMusicClock::GetDistanceToClosestBeat() const
{
	float Dist = GetDistanceFromCurrentBeat();
	return FMath::Min(Dist, 1.0f - Dist);
}

float UMusicClock::GetDistanceFromCurrentBar() const
{
	return CurrentPos.BarsIncludingCountIn - FMath::FloorToFloat(CurrentPos.BarsIncludingCountIn);
}

float UMusicClock::GetDistanceToNextBar() const
{
	return 1.0f - GetDistanceFromCurrentBar();
}

float UMusicClock::GetDistanceToClosestBar() const
{
	float Dist = GetDistanceFromCurrentBar();
	return FMath::Min(Dist, 1.0f - Dist);
}

float UMusicClock::GetDeltaBar() const
{
	return DeltaBarF;
}

float UMusicClock::GetDeltaBeat() const
{
	return DeltaBeatF;
}

bool UMusicClock::LoopedThisFrame() const
{
	return bLoopedThisFrame;
}

bool UMusicClock::SeekedThisFrame() const
{
	return bSeekedThisFrame;
}

bool UMusicClock::IsLooping() const
{
	IMusicSource* SourcePtr = GetValidSource();
	return SourcePtr ? SourcePtr->IsLooping() : false;
}

float UMusicClock::GetLoopStartMs() const
{
	IMusicSource* SourcePtr = GetValidSource();
	return SourcePtr ? SourcePtr->GetLoopStartMs() : 0.f;
}

float UMusicClock::GetLoopLengthMs() const
{
	IMusicSource* SourcePtr = GetValidSource();
	return SourcePtr ? SourcePtr->GetLoopLengthMs() : 0.f;
}

const ISongMapEvaluator* UMusicClock::GetSongMaps() const
{
	IMusicSource* SourcePtr = GetValidSource();
	if (SourcePtr)
	{
		return SourcePtr->GetCurrentSongMapEvaluator();
	}
	return nullptr;
}

float UMusicClock::GetSongLengthMs() const
{
	const ISongMapEvaluator* Maps = GetSongMaps();
	if (Maps)
	{
		return Maps->GetSongLengthMs();
	}
	return 0.f;
}

float UMusicClock::GetSongRemainingMs() const
{
	float LengthMs = GetSongLengthMs();
	if (LengthMs <= 0.f)
	{
		return 0.f;
	}
	return FMath::Max(0.f, LengthMs - (CurrentPos.SecondsIncludingCountIn * 1000.f));
}

TOptional<FVector> UMusicClock::TryGetAudioSourceLocation() const
{
	IMusicSource* SourcePtr = GetValidSource();
	if (SourcePtr)
	{
		return SourcePtr->TryGetAudioSourceLocation();
	}
	return {};
}

#if !UE_BUILD_SHIPPING
FString UMusicClock::GetDisplayName() const
{
	IMusicSource* SourcePtr = GetValidSource();
	if (SourcePtr)
	{
		return SourcePtr->GetDisplayName();
	}
	return TEXT("UMusicClock (no source)");
}
#endif

void UMusicClock::UpdatePlaybackRate(const FMidiSongPos& SongPos, float InClockAdvanceRate)
{
	bool bChanged = false;

	if (!FMath::IsNearlyEqual(Tempo, SongPos.Tempo))
	{
		Tempo = SongPos.Tempo;
		bChanged = true;
	}
	if (!FMath::IsNearlyEqual(ClockAdvanceRate, InClockAdvanceRate))
	{
		ClockAdvanceRate = InClockAdvanceRate;
		bChanged = true;
	}
	if (TimeSignatureNum != SongPos.TimeSigNumerator || TimeSignatureDenom != SongPos.TimeSigDenominator)
	{
		TimeSignatureNum = SongPos.TimeSigNumerator;
		TimeSignatureDenom = SongPos.TimeSigDenominator;
		bChanged = true;
	}

	if (bChanged)
	{
		float BPM = GetCurrentBeatsPerMinute();
		CurrentBeatsPerSecondCached = (BPM / 60.0f) * ClockAdvanceRate;
		CurrentBarsPerSecondCached = (TimeSignatureNum > 0) ? (CurrentBeatsPerSecondCached / TimeSignatureNum) : 0.f;
	}
}

void UMusicClock::BroadcastSongPosChanges()
{
	if (!CurrentPos.IsValid())
	{
		return;
	}

	const int32 CurrBar = FMath::FloorToInt32(CurrentPos.BarsIncludingCountIn);
	if (LastBroadcastBar != CurrBar)
	{
		BarEvent.Broadcast(CurrentPos.Timestamp.Bar);
		LastBroadcastBar = CurrBar;
	}

	const int32 CurrBeat = FMath::FloorToInt32(CurrentPos.BeatsIncludingCountIn);
	if (LastBroadcastBeat != CurrBeat)
	{
		BeatEvent.Broadcast(CurrBeat, FMath::FloorToInt32(CurrentPos.Timestamp.Beat));
		LastBroadcastBeat = CurrBeat;
	}

	const FSongSection& SongSection = CurrentPos.CurrentSongSection;
	if (LastBroadcastSection.StartTick != SongSection.StartTick || LastBroadcastSection.LengthTicks != SongSection.LengthTicks)
	{
		const ISongMapEvaluator* Maps = GetSongMaps();
		if (Maps)
		{
			float SectionStartMs = Maps->TickToMs(SongSection.StartTick);
			float SectionLengthMs = Maps->TickToMs(SongSection.LengthTicks);
			SectionEvent.Broadcast(SongSection.Name, SectionStartMs, SectionLengthMs);
		}
		LastBroadcastSection = FSongSection(SongSection.Name, SongSection.StartTick, SongSection.LengthTicks);
	}
}
