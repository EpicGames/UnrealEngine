// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MusicClock/TimeSource.h"

Harmonix::ESourceEvent Harmonix::ITimeSource::SourceEventFromStateTransition(ESourceState PrevState, ESourceState NewState)
{
	switch (PrevState)
	{
	case ESourceState::Stopped:
		switch (NewState)
		{
		case ESourceState::Running:
			return ESourceEvent::Start;
		default:
			return ESourceEvent::None;
		}
	case ESourceState::Running:
		switch (NewState)
		{
		case ESourceState::Running:
			return ESourceEvent::Advance;
		case ESourceState::Stopped:
			return ESourceEvent::Stop;
		case ESourceState::Paused:
			return ESourceEvent::Pause;
		default:
			return ESourceEvent::None;
		}
	case ESourceState::Paused:
		switch (NewState)
		{
		case ESourceState::Running:
			return ESourceEvent::Continue;
		case ESourceState::Stopped:
			return ESourceEvent::Stop;
		case ESourceState::Paused:
		default:
			return ESourceEvent::None;
		}
	default:
		return ESourceEvent::None;
	}
}