// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMetasound/MusicClock/WorldTimeSourceController.h"

#include "Engine/World.h"
#include "Misc/App.h"

Harmonix::FWorldTimeSourceController::FWorldTimeSourceController(UObject* InContextObj)
	: ContextObj(InContextObj)
{
}

FString Harmonix::FWorldTimeSourceController::GetDisplayName() const
{
	return TEXT("RealTimeSourceController");
}

TOptional<FVector> Harmonix::FWorldTimeSourceController::TryGetAudioSourceLocation() const
{
	return {};
}

double Harmonix::FWorldTimeSourceController::GetDeltaTime() const
{
	if (TStrongObjectPtr<UObject> Obj = ContextObj.Pin())
	{
		if (const UWorld* World = Obj->GetWorld())
		{
			FGameTime GameTime = World->GetTime();
			return GameTime.GetDeltaWorldTimeSeconds();
		}
	}

	return 0.0;
}


void Harmonix::FWorldTimeSourceController::Update()
{
	double DeltaTime = GetDeltaTime() * Speed;

	LatestSourceEvent = ESourceEvent::None;
	
	switch (TransportRequest)
	{
	default:
		break;
	case ETransportRequest::Stop:
		CurrentSourceState = ESourceState::Stopped;
		LatestSourceEvent = ESourceEvent::Stop;
		RunTimeSeconds = 0.0;
		break;
	case ETransportRequest::Play:
		if (CurrentSourceState == ESourceState::Stopped)
		{
			RunTimeSeconds = SeekRequestTime;
			CurrentSourceState = ESourceState::Running;
			LatestSourceEvent = ESourceEvent::Start;
		}
		break;
	case ETransportRequest::Pause:
		if (CurrentSourceState == ESourceState::Running)
		{
			CurrentSourceState = ESourceState::Paused;
			LatestSourceEvent = ESourceEvent::Pause;
		}
		break;
	case ETransportRequest::Continue:
		if (CurrentSourceState == ESourceState::Paused)
		{
			CurrentSourceState = ESourceState::Running;
			LatestSourceEvent = ESourceEvent::Continue;
		}
		break;
	case ETransportRequest::Seek:
		if (CurrentSourceState != ESourceState::Stopped)
		{
			RunTimeSeconds = SeekRequestTime;
			LatestSourceEvent = ESourceEvent::Seek;
		}
		break;
	}
	
	if (CurrentSourceState == ESourceState::Running)
	{
		RunTimeSeconds += DeltaTime;
		if (LatestSourceEvent == ESourceEvent::None)
		{
			LatestSourceEvent = ESourceEvent::Advance;
		}
	}

	TransportRequest = ETransportRequest::None;
}

Harmonix::ESourceState Harmonix::FWorldTimeSourceController::GetCurrentSourceState() const
{
	return CurrentSourceState;
}

Harmonix::ESourceEvent Harmonix::FWorldTimeSourceController::GetLatestSourceEvent() const
{
	return LatestSourceEvent;
}

double Harmonix::FWorldTimeSourceController::GetCurrentTime() const
{
	return RunTimeSeconds;
}

float Harmonix::FWorldTimeSourceController::GetSpeed() const
{
	if (TStrongObjectPtr<UObject> Obj = ContextObj.Pin())
	{
		if (const UWorld* World = Obj->GetWorld())
		{
			FGameTime GameTime = World->GetTime();
			return GameTime.GetTimeDilation() * Speed;
		}
	}

	return 0.0;
}

void Harmonix::FWorldTimeSourceController::SetSpeed(float InSpeed)
{
	Speed = FMath::Clamp(InSpeed, 0.01f, 10.0f);
}

void Harmonix::FWorldTimeSourceController::Start(float InStartTime)
{
	SeekRequestTime = FMath::Max(InStartTime, 0.0f);
	TransportRequest = ETransportRequest::Play;
}

void Harmonix::FWorldTimeSourceController::Stop()
{
	TransportRequest = ETransportRequest::Stop;
}

void Harmonix::FWorldTimeSourceController::Pause()
{
	TransportRequest = ETransportRequest::Pause;
}

void Harmonix::FWorldTimeSourceController::Continue()
{
	TransportRequest = ETransportRequest::Continue;
}

void Harmonix::FWorldTimeSourceController::Seek(float InSeekTime)
{
	TransportRequest = ETransportRequest::Seek;
	SeekRequestTime = FMath::Max(InSeekTime, 0.0f);
}