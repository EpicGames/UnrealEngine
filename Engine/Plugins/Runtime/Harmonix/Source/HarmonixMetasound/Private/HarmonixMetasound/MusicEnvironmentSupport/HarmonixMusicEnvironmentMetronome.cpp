// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMusicEnvironmentMetronome.h"

#include "MetasoundBuilderSubsystem.h"
#include "MetasoundStandardNodesNames.h"
#include "Components/AudioComponent.h"
#include "Engine/Engine.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/DataTypes/MidiAsset.h"
#include "HarmonixMetasound/Nodes/MetronomeNode.h"
#include "HarmonixMetasound/Nodes/MidiPlayerNode.h"
#include "HarmonixMetasound/Nodes/MidiClockSubdivisionTriggerNode.h"
#include "HarmonixMetasound/Nodes/MusicSeekTargetBuilder.h"
#include "HarmonixMetasound/Nodes/TriggerToTransportNode.h"
#include "HarmonixMidi/MidiConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixMusicEnvironmentMetronome)

namespace HarmonixMovieSceneMetronome
{
	static const FLazyName BuilderName(TEXT("HarmonixMovieMetronomeBuilder"));
}

bool UHarmonixMusicEnvironmentMetronome::Initialize(UWorld* InWorld)
{
	if (!bMetasoundIsPlaying)
	{
		if (!BuildAndStartMetasound(InWorld))
			return false;
	}
	return true;
}

void UHarmonixMusicEnvironmentMetronome::Tick(float DeltaSecs)
{
	// MusicClock no longer ticks, this function is empty
}

void UHarmonixMusicEnvironmentMetronome::Start(double FromSeconds)
{
	UE_LOGF(LogMIDI, Verbose, "UHarmonixMusicEnvironmentMetronome: Starting Metronome at %f seconds", FromSeconds);

	if (!AudioComponent || !MusicClockComponent)
	{
		return;
	}

	RebuildMidiFile();
	SecondsWhenStarted = FromSeconds;
	MusicalTimeWhenStarted = MidiFile->GetSongMaps()->GetMusicalTimeAtSeconds(FromSeconds);
	FAudioParameter MidiFileParam(TEXT("MIDI File"), MidiFile);
	FAudioParameter SeekTargetMs(TEXT("SeekTarget"), static_cast<float>(FromSeconds * 1000.0f));
	AudioComponent->Activate();
	AudioComponent->SetParameter(MoveTemp(MidiFileParam));
	AudioComponent->SetParameter(MoveTemp(SeekTargetMs));
	AudioComponent->SetTriggerParameter(TEXT("PlayMetronome"));
	MusicClockComponent->Start();
}

void UHarmonixMusicEnvironmentMetronome::Seek(double ToSeconds)
{
}

void UHarmonixMusicEnvironmentMetronome::Stop()
{
	UE_LOGF(LogMIDI, Verbose, "UHarmonixMusicEnvironmentMetronome: Stopping Metronome");

	if (MusicClockComponent)
	{
		MusicClockComponent->Stop();
	}
	if (AudioComponent)
	{
		AudioComponent->SetTriggerParameter(TEXT("StopMetronome"));
		AudioComponent->Deactivate();
	}
}

void UHarmonixMusicEnvironmentMetronome::Pause()
{
	if (AudioComponent)
	{
		AudioComponent->SetTriggerParameter(TEXT("PauseMetronome"));
	}
}

void UHarmonixMusicEnvironmentMetronome::Resume()
{
	if (AudioComponent)
	{
		AudioComponent->SetTriggerParameter(TEXT("ContinueMetronome"));
	}
}

float UHarmonixMusicEnvironmentMetronome::GetCurrentTempo() const
{
	return CurrentTempo;
}

float UHarmonixMusicEnvironmentMetronome::GetCurrentSpeed() const
{
	return CurrentSpeed;
}

bool UHarmonixMusicEnvironmentMetronome::IsMuted() const
{
	return bIsMuted;
}


float UHarmonixMusicEnvironmentMetronome::GetCurrentVolume() const
{
	return CurrentVolume;
}

double UHarmonixMusicEnvironmentMetronome::GetCurrentPositionSeconds() const
{
	return 0.0;
}

float UHarmonixMusicEnvironmentMetronome::GetPositionSeconds() const
{
	float TimeSeconds = (float)SecondsWhenStarted;
	if (MusicClockComponent)
	{
		return MusicClockComponent->GetPositionSeconds() < TimeSeconds ? TimeSeconds : MusicClockComponent->GetPositionSeconds();
	}
	return TimeSeconds;
}

FMusicalTime UHarmonixMusicEnvironmentMetronome::GetPositionMusicalTime() const
{
	FMusicalTime CurrentMusicalTime;
	if (MusicClockComponent)
	{
		CurrentMusicalTime = MusicClockComponent->GetPositionMusicalTime();
		return MusicalTimeWhenStarted > CurrentMusicalTime ? MusicalTimeWhenStarted : CurrentMusicalTime;
	}
	return CurrentMusicalTime;
}

int32 UHarmonixMusicEnvironmentMetronome::GetPositionAbsoluteTick() const
{
	if (MusicClockComponent)
	{
		return MusicClockComponent->GetPositionAbsoluteTick();
	}
	return 0;
}

FMusicalTime UHarmonixMusicEnvironmentMetronome::GetPositionMusicalTime(const FMusicalTime& SourceSpaceOffset) const
{
	FMusicalTime MusicalTime;
	if (MusicClockComponent)
	{
		return MusicClockComponent->GetPositionMusicalTime(SourceSpaceOffset);
	}
	return MusicalTime;
}

int32 UHarmonixMusicEnvironmentMetronome::GetPositionAbsoluteTick(const FMusicalTime& SourceSpaceOffset) const
{
	if (MusicClockComponent)
	{
		return MusicClockComponent->GetPositionAbsoluteTick(SourceSpaceOffset);
	}
	return 0;
}

FMusicalTime UHarmonixMusicEnvironmentMetronome::Quantize(const FMusicalTime& MusicalTime, int32 QuantizationInterval, UFrameBasedMusicMap::EQuantizeDirection Direction) const
{
	if (!MusicClockComponent)
	{
		return MusicalTime;
	}
	else
	{
		return MusicClockComponent->Quantize(MusicalTime, QuantizationInterval, Direction);
	}
}

void UHarmonixMusicEnvironmentMetronome::BeginDestroy()
{
	if (MusicClockComponent)
	{
		MusicClockComponent->Stop();
	}
	UObject::BeginDestroy();
}

void UHarmonixMusicEnvironmentMetronome::OnMusicMapSet()
{
	RebuildMidiFile();

	if (AudioComponent)
	{
		FAudioParameter MidiFileParam(TEXT("MIDI File"), MidiFile);
		AudioComponent->SetParameter(MoveTemp(MidiFileParam));
	}
}

bool UHarmonixMusicEnvironmentMetronome::OnSetTempo(float Bpm)
{
	RebuildMidiFile();
	
	if (!AudioComponent)
	{
		return false;
	}

	FAudioParameter MidiFileParam(TEXT("MIDI File"), MidiFile);
	AudioComponent->SetParameter(MoveTemp(MidiFileParam));
	
	FAudioParameter TempoParam(TEXT("Tempo"), Bpm);
	AudioComponent->SetParameter(MoveTemp(TempoParam));
	CurrentTempo = Bpm;
	return true;
}

void UHarmonixMusicEnvironmentMetronome::OnSetSpeed(const float InSpeed)
{
	CurrentSpeed = InSpeed;
	if (!AudioComponent)
	{
		return;
	}

	FAudioParameter VolumeParam(TEXT("Speed"), CurrentSpeed);
	AudioComponent->SetParameter(MoveTemp(VolumeParam));
}

void UHarmonixMusicEnvironmentMetronome::OnSetVolume(float InVolume)
{
	CurrentVolume = InVolume;
	if (!AudioComponent)
	{
		return;
	}

	FAudioParameter VolumeParam(TEXT("BeepGain"), FMath::Clamp(CurrentVolume, 0.0f, 1.0f));
	AudioComponent->SetParameter(MoveTemp(VolumeParam));
}

void UHarmonixMusicEnvironmentMetronome::OnSetMuted(bool bInMuted)
{
	bIsMuted = bInMuted;
	if (!AudioComponent)
	{
		return;
	}

	FAudioParameter VolumeParam(TEXT("BeepEnabled"), !bIsMuted);
	AudioComponent->SetParameter(MoveTemp(VolumeParam));
}


bool UHarmonixMusicEnvironmentMetronome::RebuildMidiFile()
{
	MidiFile = NewObject<UMidiFile>(this, "MIDI File Music Map", RF_Transient);
	if (MusicMap)
	{
		for (const FFrameBasedTimeSignaturePoint& TimeSigPoint : MusicMap->BarMap)
		{
			const int32 Num = TimeSigPoint.TimeSignature.Numerator;
			const int32 Den = TimeSigPoint.TimeSignature.Denominator;
			MidiFile->GetSongMaps()->AddTimeSignatureAtBarIncludingCountIn(TimeSigPoint.OnBar, Num, Den);
		}

		for (const FFrameBasedTempoPoint& TempoPoint : MusicMap->TempoMap)
		{
			int32 MidiTempo = TempoPoint.MicrosecondsPerQuarterNote;
			MidiFile->GetSongMaps()->AddTempoInfoPoint(MidiTempo, TempoPoint.OnTick);
		}
	}
	else
	{
		MidiFile->GetSongMaps()->AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(CurrentTempo), 0);
		MidiFile->GetSongMaps()->AddTimeSignatureAtBarIncludingCountIn(0, 4, 4);
	}
	
	MidiFile->BuildConductorTrack();
	return true;
}


bool UHarmonixMusicEnvironmentMetronome::BuildAndStartMetasound(UWorld* InWorld)
{
	if (!BuildMetasound())
	{
		return false;
	}
	check(SourceBuilder);
	AudioComponent = NewObject<UAudioComponent>(this, "MetronomeAudio", RF_Transient);
	AudioComponent->SetUISound(true);
	SourceBuilder->Audition(this, AudioComponent, {});

	MusicClockComponent = NewObject<UMusicClockComponent>(this, "MetronomeClock", RF_Transient);
	MusicClockComponent->MetasoundOutputName = TEXT("MIDI Clock");
	MusicClockComponent->ConnectToMetasoundOnAudioComponent(AudioComponent);
	MusicClockComponent->Start();

	if (MusicMap)
	{
		CurrentTempo = MusicMap->GetInitialTempo();
	}
	
	bMetasoundIsPlaying = true;
	return true;
}

bool UHarmonixMusicEnvironmentMetronome::BuildMetasound()
{
	using namespace HarmonixMetasound::Nodes;

	if (UMetaSoundBuilderSubsystem* MetaSoundBuilder = GEngine->GetEngineSubsystem<UMetaSoundBuilderSubsystem>())
	{
		FMetaSoundBuilderNodeOutputHandle Out;
		FMetasoundFrontendLiteral IntLiteral;
		EMetaSoundBuilderResult Result;
		
		SourceBuilder = MetaSoundBuilder->CreateSourceBuilder(HarmonixMovieSceneMetronome::BuilderName, OnPlayNodeOutput, OnFinishedNodeInput, AudioOutNodeInputs, Result, EMetaSoundOutputAudioFormat::Mono, false);
		MetaSoundBuilder->RegisterSourceBuilder(HarmonixMovieSceneMetronome::BuilderName, SourceBuilder);
		if (!ensureMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Unable to create the metasound source builder for the HarmonixMovieSceneMetronome!")))
		{
			return false;
		}

		// Make the nodes and set their un-wired inputs to appropriate values...
		if (!AddGraphInput(TEXT("BeepGain"), GetCurrentVolume()).IsSet()) { return false; }
		if (!AddGraphInput(TEXT("BeepEnabled"), !IsMuted()).IsSet()) { return false; }
		
		// the midi file is used for the midi player node if there's a music map
		if (!AddGraphInput(TEXT("MIDI File"), nullptr).IsSet()) { return false; }
		
		if (!AddGraphInput(TEXT("SeekTarget"), 0.0f).IsSet()) { return false; }
		if (!AddGraphInputTrigger(TEXT("PlayMetronome")).IsSet()) { return false; }
		if (!AddGraphInputTrigger(TEXT("StopMetronome")).IsSet()) { return false; }
		if (!AddGraphInputTrigger(TEXT("PauseMetronome")).IsSet()) { return false; }
		if (!AddGraphInputTrigger(TEXT("ContinueMetronome")).IsSet()) { return false; }
		if (!AddGraphInput(TEXT("Speed"), GetCurrentSpeed()).IsSet()) { return false; }

		Metasound::FNodeClassName ClassName = {Metasound::StandardNodes::Namespace, TEXT("Trigger Delay"), TEXT("")};
		FMetaSoundNodeHandle PlayTriggerDelay = SourceBuilder->AddNodeByClassName(ClassName, Result);
		if (!SetNodeInputDefault(PlayTriggerDelay, TEXT("Delay Time"), 0.0001f)) { return false; }
		FMetaSoundNodeHandle SeekTriggerDelay = SourceBuilder->AddNodeByClassName(ClassName, Result);
		if (!SetNodeInputDefault(SeekTriggerDelay, TEXT("Delay Time"), 0.0001f)) { return false; }
		
		FMetaSoundNodeHandle SeekTargetConverter = SourceBuilder->AddNodeByClassName(TimeMsToSeekTarget::GetClassName(), Result, TimeMsToSeekTarget::GetCurrentMajorVersion());
	
		FMetaSoundNodeHandle TransportNode = SourceBuilder->AddNodeByClassName(TriggerToTransportNode::GetClassName(), Result, TriggerToTransportNode::GetCurrentMajorVersion());

		FMetaSoundNodeHandle MidiPlayerNode = SourceBuilder->AddNodeByClassName(MidiPlayerNode::GetClassName(), Result, MidiPlayerNode::GetCurrentMajorVersion());
		FMetaSoundNodeHandle ClockSourceNode = MidiPlayerNode;
		
		FMetaSoundNodeHandle BeatSubdivision = SourceBuilder->AddNodeByClassName(MidiClockSubdivisionTriggerNode::GetClassName(), Result, MidiClockSubdivisionTriggerNode::GetCurrentMajorVersion());
		
		FMetaSoundNodeHandle BarSubdivision = SourceBuilder->AddNodeByClassName(MidiClockSubdivisionTriggerNode::GetClassName(), Result, MidiClockSubdivisionTriggerNode::GetCurrentMajorVersion());
		if (!SetNodeInputDefault(BarSubdivision, MidiClockSubdivisionTriggerNode::Inputs::GridSizeUnitsName, static_cast<int32>(EMidiClockSubdivisionQuantization::Bar))) { return false; }

		FMetaSoundNodeHandle OffsetBarSubdivision = SourceBuilder->AddNodeByClassName(MidiClockSubdivisionTriggerNode::GetClassName(), Result, MidiClockSubdivisionTriggerNode::GetCurrentMajorVersion());
		if (!SetNodeInputDefault(OffsetBarSubdivision, MidiClockSubdivisionTriggerNode::Inputs::GridSizeUnitsName, static_cast<int32>(EMidiClockSubdivisionQuantization::Bar))) { return false; }
		if (!SetNodeInputDefault(OffsetBarSubdivision, MidiClockSubdivisionTriggerNode::Inputs::OffsetUnitsName, static_cast<int32>(EMidiClockSubdivisionQuantization::Beat))) { return false; }
		if (!SetNodeInputDefault(OffsetBarSubdivision, MidiClockSubdivisionTriggerNode::Inputs::OffsetMultName, 1)) { return false; }

		ClassName = {Metasound::StandardNodes::Namespace, TEXT("Trigger Counter"), TEXT("")};
		FMetaSoundNodeHandle TriggerCounter = SourceBuilder->AddNodeByClassName(ClassName, Result);
		
		ClassName = { "TriggerAny", TEXT("Trigger Any (2)"), FName() };
		FMetaSoundNodeHandle TriggerAny = SourceBuilder->AddNodeByClassName(ClassName, Result);

		ClassName = {FName("Array"), TEXT("Get"), TEXT("Float:Array")};
		FMetaSoundNodeHandle GetPitch = SourceBuilder->AddNodeByClassName(ClassName, Result);
		TArray<float> Pitches;
		Pitches.Add(1000.0f);
		Pitches.Add(600.0f);
		if (!SetNodeInputDefault(GetPitch, TEXT("Array"), Pitches)) { return false; }
		
		ClassName = { Metasound::StandardNodes::Namespace, TEXT("Sine"), Metasound::StandardNodes::AudioVariant };
		FMetaSoundNodeHandle SineOsc = SourceBuilder->AddNodeByClassName(ClassName, Result);
		ClassName = { "AD Envelope", "AD Envelope", Metasound::GetMetasoundDataTypeName<Metasound::FAudioBuffer>() };
		FMetaSoundNodeHandle Envelope = SourceBuilder->AddNodeByClassName(ClassName, Result);
		if (!SetNodeInputDefault(Envelope, TEXT("Attack Time"), 0.0001f)) { return false; }
		if (!SetNodeInputDefault(Envelope, TEXT("Decay Time"), 0.1f)) { return false; }

		ClassName = { Metasound::StandardNodes::Namespace, TEXT("Multiply"), TEXT("Audio") };
		FMetaSoundNodeHandle EnvelopeMultiplier = SourceBuilder->AddNodeByClassName(ClassName, Result);

		ClassName = { Metasound::StandardNodes::Namespace, TEXT("Multiply"), TEXT("Audio by Float") };
		FMetaSoundNodeHandle GainMultiplier = SourceBuilder->AddNodeByClassName(ClassName, Result);

		SourceBuilder->AddGraphOutputNode(TEXT("MIDI Clock"), Metasound::GetMetasoundDataTypeName<HarmonixMetasound::FMidiClock>(), FMetasoundFrontendLiteral(), Result);

		// Now wire up the connections...

		// Inputs...
		if (!ConnectGraphInputToNodeInput(TEXT("PlayMetronome"), SeekTriggerDelay, TEXT("In"))) { return false; }
		if (!ConnectGraphInputToNodeInput(TEXT("PauseMetronome"), TransportNode, TEXT("Pause"))) { return false; }
		if (!ConnectGraphInputToNodeInput(TEXT("ContinueMetronome"), TransportNode, TEXT("Continue"))) { return false; }
		if (!ConnectGraphInputToNodeInput(TEXT("StopMetronome"), TransportNode, TEXT("Stop"))) { return false; }
		if (!ConnectGraphInputToNodeInput(TEXT("BeepGain"), GainMultiplier, TEXT("AdditionalOperands"))) { return false; }
		if (!ConnectGraphInputToNodeInput(TEXT("BeepEnabled"), SineOsc, TEXT("Enabled"))) { return false; }

		if (!ConnectGraphInputToNodeInput(TEXT("MIDI File"), MidiPlayerNode,TEXT("MIDI File"))) { return false; }
		if (!ConnectGraphInputToNodeInput(TEXT("Speed"), MidiPlayerNode, TEXT("Speed"))) { return false; }
		
		if (!ConnectGraphInputToNodeInput(TEXT("SeekTarget"), SeekTargetConverter, TEXT("Time (Ms)"))) { return false; }

		// Interconnects...
		if (!ConnectNodes(SeekTriggerDelay, TEXT("Out"), PlayTriggerDelay, TEXT("In"))) { return false; }
		if (!ConnectNodes(SeekTriggerDelay, TEXT("Out"), TransportNode, TEXT("Trigger Seek"))) { return false; }
		if (!ConnectNodes(SeekTargetConverter, TEXT("Seek Target"), TransportNode, TEXT("Seek Target"))) { return false; }
		if (!ConnectNodes(PlayTriggerDelay, TEXT("Out"), TransportNode, TEXT("Play"))) { return false; }

		if (!ConnectNodes(TransportNode, TEXT("Transport"), ClockSourceNode, TEXT("Transport"))) { return false; }
		if (!ConnectNodes(ClockSourceNode, TEXT("MIDI Clock"), BarSubdivision, TEXT("MIDI Clock"))) { return false; }
		if (!ConnectNodes(ClockSourceNode, TEXT("MIDI Clock"), OffsetBarSubdivision, TEXT("MIDI Clock"))) { return false; }
		if (!ConnectNodes(ClockSourceNode, TEXT("MIDI Clock"), BeatSubdivision, TEXT("MIDI Clock"))) { return false; }
		
		if (!ConnectNodes(BeatSubdivision, TEXT("Trigger Out"), Envelope, TEXT("Trigger"))) { return false; }
		if (!ConnectNodes(Envelope, TEXT("Out Envelope"), EnvelopeMultiplier, TEXT("PrimaryOperand"))) { return false; }
		if (!ConnectNodes(BarSubdivision, TEXT("Trigger Out"), TriggerCounter, TEXT("Reset"))) { return false; }
		if (!ConnectNodes(OffsetBarSubdivision, TEXT("Trigger Out"), TriggerCounter, TEXT("In"))) { return false; }
		if (!ConnectNodes(TriggerCounter, TEXT("On Trigger"), TriggerAny, TEXT("In 0"))) { return false; }
		if (!ConnectNodes(TriggerCounter, TEXT("On Reset"), TriggerAny, TEXT("In 1"))) { return false; }
		if (!ConnectNodes(TriggerAny, TEXT("Out"), GetPitch, TEXT("Trigger"))) { return false; }
		if (!ConnectNodes(TriggerCounter, TEXT("Count"), GetPitch, TEXT("Index"))) { return false; }
		if (!ConnectNodes(GetPitch, TEXT("Element"), SineOsc, TEXT("Frequency"))) { return false; }
		if (!ConnectNodes(SineOsc, TEXT("Audio"), EnvelopeMultiplier, TEXT("AdditionalOperands"))) { return false; }
		if (!ConnectNodes(EnvelopeMultiplier, TEXT("Out"), GainMultiplier, TEXT("PrimaryOperand"))) { return false; }

		// Outputs...
		if (!ConnectNodeOutputToGraphOutput(ClockSourceNode, TEXT("MIDI Clock"), TEXT("MIDI Clock"))) { return false; }
		
		Out = SourceBuilder->FindNodeOutputByName(GainMultiplier, TEXT("Out"), Result);
		SourceBuilder->ConnectNodes(Out, AudioOutNodeInputs[0], Result);

		return Result == EMetaSoundBuilderResult::Succeeded; 
	}
	return false;
}

FMetaSoundBuilderNodeOutputHandle UHarmonixMusicEnvironmentMetronome::AddGraphInput(const FName& InputName, UMidiFile* InMidiFile) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;
	
	FMetasoundFrontendLiteral DefaultValue;
	DefaultValue.Set(InMidiFile);
	return SourceBuilder->AddGraphInputNode(InputName, Metasound::GetMetasoundDataTypeName<HarmonixMetasound::FMidiAsset>(), DefaultValue, Result);
}

FMetaSoundBuilderNodeOutputHandle UHarmonixMusicEnvironmentMetronome::AddGraphInput(const FName& InputName, const float Value) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;

	FMetasoundFrontendLiteral DefaultValue;
	DefaultValue.Set(Value);
	return SourceBuilder->AddGraphInputNode(InputName, Metasound::GetMetasoundDataTypeName<float>(), DefaultValue, Result);
}

FMetaSoundBuilderNodeOutputHandle UHarmonixMusicEnvironmentMetronome::AddGraphInput(const FName& InputName, const bool Value) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;

	FMetasoundFrontendLiteral DefaultValue;
	DefaultValue.Set(Value);
	return SourceBuilder->AddGraphInputNode(InputName, Metasound::GetMetasoundDataTypeName<bool>(), DefaultValue, Result);
}

FMetaSoundBuilderNodeOutputHandle UHarmonixMusicEnvironmentMetronome::AddGraphInputTrigger(const FName& InputName) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;
	return SourceBuilder->AddGraphInputNode(InputName, Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>(), FMetasoundFrontendLiteral(), Result);
}

bool UHarmonixMusicEnvironmentMetronome::SetNodeInputDefault(const FMetaSoundNodeHandle& Node, const FName& InputName, const int32 Value) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;

	FMetaSoundBuilderNodeInputHandle In = SourceBuilder->FindNodeInputByName(Node, InputName, Result);
	if (Result == EMetaSoundBuilderResult::Failed) { return false; }
	FMetasoundFrontendLiteral IntLiteral;
	IntLiteral.Set(Value);
	SourceBuilder->SetNodeInputDefault(In, IntLiteral, Result);
	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool UHarmonixMusicEnvironmentMetronome::SetNodeInputDefault(const FMetaSoundNodeHandle& Node, const FName& InputName, const float Value) const
{
	check(SourceBuilder);	
	EMetaSoundBuilderResult Result;

	FMetaSoundBuilderNodeInputHandle In = SourceBuilder->FindNodeInputByName(Node, InputName, Result);
	if (Result == EMetaSoundBuilderResult::Failed) { return false; }
	FMetasoundFrontendLiteral FloatLiteral;
	FloatLiteral.Set(Value);
	SourceBuilder->SetNodeInputDefault(In, FloatLiteral, Result);
	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool UHarmonixMusicEnvironmentMetronome::SetNodeInputDefault(const FMetaSoundNodeHandle& Node, const FName& InputName,
	const TArray<float>& Values) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;

	FMetaSoundBuilderNodeInputHandle In = SourceBuilder->FindNodeInputByName(Node, InputName, Result);
	if (Result == EMetaSoundBuilderResult::Failed) { return false; }
	FMetasoundFrontendLiteral ArrayLiteral;
	ArrayLiteral.Set(Values);
	SourceBuilder->SetNodeInputDefault(In, ArrayLiteral, Result);
	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool UHarmonixMusicEnvironmentMetronome::ConnectGraphInputToNodeInput(const FName& GraphInputName, const FMetaSoundNodeHandle& Node,
	const FName& NodeInputName) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;

	FMetaSoundBuilderNodeInputHandle NodeIn = SourceBuilder->FindNodeInputByName(Node, NodeInputName, Result);
	if (Result == EMetaSoundBuilderResult::Failed) { return false; }
	SourceBuilder->ConnectNodeInputToGraphInput(GraphInputName, NodeIn, Result);
	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool UHarmonixMusicEnvironmentMetronome::ConnectNodeOutputToGraphOutput(const FMetaSoundNodeHandle& Node, const FName& NodeOutputName,
	const FName& GraphOutputName) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;

	FMetaSoundBuilderNodeOutputHandle NodeOut = SourceBuilder->FindNodeOutputByName(Node, NodeOutputName, Result);
	if (Result == EMetaSoundBuilderResult::Failed) { return false; }
	SourceBuilder->ConnectNodeOutputToGraphOutput(GraphOutputName, NodeOut, Result);
	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool UHarmonixMusicEnvironmentMetronome::ConnectNodes(const FMetaSoundNodeHandle& SourceNode, const FName& OutName,
                                                      const FMetaSoundNodeHandle& DestinationNode, const FName& InName) const
{
	check(SourceBuilder);
	EMetaSoundBuilderResult Result;

	FMetaSoundBuilderNodeOutputHandle Out = SourceBuilder->FindNodeOutputByName(SourceNode, OutName, Result);
	if (Result == EMetaSoundBuilderResult::Failed) { return false; }
	FMetaSoundBuilderNodeInputHandle In = SourceBuilder->FindNodeInputByName(DestinationNode, InName, Result);
	if (Result == EMetaSoundBuilderResult::Failed) { return false; }
	SourceBuilder->ConnectNodes(Out, In, Result);
	return Result == EMetaSoundBuilderResult::Succeeded;
}
