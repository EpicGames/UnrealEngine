// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundBuilderBase.h"
#include "MusicEnvironmentMetronome.h"
#include "UObject/Object.h"

#include "HarmonixMusicEnvironmentMetronome.generated.h"

#define UE_API HARMONIXMETASOUND_API

class UAudioComponent;
class UMetaSoundSourceBuilder;
class UMusicClockComponent;
class UMidiFile;
class UWorld;
class AActor;

/**
 * Implementation of the generic MusicEnvironmentSubsystem's metronome interface. So that subsystem
 * can create metronomes based on the Harmonix tech.
 */
UCLASS(MinimalAPI)
class UHarmonixMusicEnvironmentMetronome : public UObject, public IMusicEnvironmentMetronome
{
	GENERATED_BODY()
	
public:
	UE_API virtual bool Initialize(UWorld* InWorld) override;
	UE_API virtual void Tick(float DeltaSecs) override;
	UE_API virtual void Start(double FromSeconds = 0.0) override;
	UE_API virtual void Seek(double ToSeconds) override;
	UE_API virtual void Stop() override;
	UE_API virtual void Pause() override;
	UE_API virtual void Resume() override;

	UE_API virtual float GetCurrentTempo() const override;
	UE_API virtual float GetCurrentVolume() const override;
	UE_API virtual float GetCurrentSpeed() const override;
	UE_API virtual double GetCurrentPositionSeconds() const override;
	UE_API virtual bool IsMuted() const override;

	// vvv BEGIN  IMusicEnvironmentClockSource Implementation vvv
	UE_API virtual float GetPositionSeconds() const override;
	UE_API virtual FMusicalTime GetPositionMusicalTime() const override;
	UE_API virtual FMusicalTime GetPositionMusicalTime(const FMusicalTime& SourceSpaceOffset) const override;
	UE_API virtual int32 GetPositionAbsoluteTick() const override;
	UE_API virtual int32 GetPositionAbsoluteTick(const FMusicalTime& SourceSpaceOffset) const override;
	UE_API virtual FMusicalTime Quantize(const FMusicalTime& MusicalTime, int32 QuantizationInterval, UFrameBasedMusicMap::EQuantizeDirection Direction = UFrameBasedMusicMap::EQuantizeDirection::Nearest) const override;
	virtual bool CanAuditionInEditor() const override { return true; }
	// ^^^ END IMusicEnvironmentClockSource Implementation ^^^

	UE_API virtual void BeginDestroy() override;
	
protected:
	UE_API virtual void OnMusicMapSet() override;
	UE_API virtual bool OnSetTempo(float Bpm) override;
	UE_API virtual void OnSetSpeed(const float Speed) override;
	UE_API virtual void OnSetVolume(float InVolume) override;
	UE_API virtual void OnSetMuted(bool bNewMuted) override;
	
private:
	UPROPERTY(Transient)
	TObjectPtr<UAudioComponent> AudioComponent;
	UPROPERTY(Transient)
	TObjectPtr<UMusicClockComponent> MusicClockComponent;

	UPROPERTY(Transient)
	TObjectPtr<UMidiFile> MidiFile;

	UPROPERTY(Transient)
	TObjectPtr<UMetaSoundSourceBuilder> SourceBuilder;
	UPROPERTY(Transient)
	FMetaSoundBuilderNodeOutputHandle OnPlayNodeOutput;
	UPROPERTY(Transient)
	FMetaSoundBuilderNodeInputHandle OnFinishedNodeInput;
	UPROPERTY(Transient)
	TArray<FMetaSoundBuilderNodeInputHandle> AudioOutNodeInputs;

	bool bMetasoundIsPlaying = false;
	float CurrentTempo = 120.0f;
	float CurrentSpeed = 1.0f;
	bool bIsMuted = false;
	float CurrentVolume = 1.0f;
	double SecondsWhenStarted = 0.0;
	FMusicalTime MusicalTimeWhenStarted;
	
	UE_API bool BuildAndStartMetasound(UWorld* InWorld);
	UE_API bool RebuildMidiFile();
	UE_API bool BuildMetasound();
	UE_API FMetaSoundBuilderNodeOutputHandle AddGraphInput(const FName& InputName, UMidiFile* MidiFile) const;
	UE_API FMetaSoundBuilderNodeOutputHandle AddGraphInput(const FName& InputName, const float Value) const;
	UE_API FMetaSoundBuilderNodeOutputHandle AddGraphInput(const FName& InputName, const bool Value) const;
	UE_API FMetaSoundBuilderNodeOutputHandle AddGraphInputTrigger(const FName& InputName) const;
	UE_API bool SetNodeInputDefault(const FMetaSoundNodeHandle& Node, const FName& InputName, const int32 Value) const;
	UE_API bool SetNodeInputDefault(const FMetaSoundNodeHandle& Node, const FName& InputName, const float Value) const;
	UE_API bool SetNodeInputDefault(const FMetaSoundNodeHandle& Node, const FName& InputName, const TArray<float>& Values) const;
	UE_API bool ConnectGraphInputToNodeInput(const FName& GraphInputName, const FMetaSoundNodeHandle& Node, const FName& NodeInputName) const;
	UE_API bool ConnectNodeOutputToGraphOutput(const FMetaSoundNodeHandle& Node, const FName& NodeOutputName, const FName& GraphOutputName) const;
	UE_API bool ConnectNodes(const FMetaSoundNodeHandle& SourceNode, const FName& OutName, const FMetaSoundNodeHandle& DestinationNode, const FName& InName) const;
};

#undef UE_API
