// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MusicSource/MetasoundMusicSource.h"
#include "Components/AudioComponent.h"
#include "Harmonix.h"
#include "HarmonixMetasound/Analysis/MidiSongPosVertexAnalyzer.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "MetasoundGenerator.h"
#include "MetasoundGeneratorHandle.h"
#include "MetasoundSource.h"

DEFINE_LOG_CATEGORY_STATIC(LogMetasoundMusicSource, Log, All)

namespace MetasoundMusicSourceSmoothing
{
	float kP = 0.18f;
	float LagSeconds = 0.030f;
	float MaxErrorSecondsBeforeJump = 0.060f;
	double MinSyncSpeed = 0.98;
	double MaxSyncSpeed = 1.02;
}

void UMetasoundMusicSource::BeginDestroy()
{
	UMidiClockUpdateSubsystem::StopTrackingMusicSource(this);
	Disconnect();
	Super::BeginDestroy();
}

// ---- Connection ----

bool UMetasoundMusicSource::ConnectToAudioComponent(UAudioComponent* InAudioComponent, FName OutputPinName)
{
	AudioComponentToWatch = InAudioComponent;
	MetasoundOutputName = OutputPinName;

	OnAttachedDelegate = UMetasoundGeneratorHandle::FOnAttached::FDelegate::CreateLambda([](){});
	OnDetachedDelegate = UMetasoundGeneratorHandle::FOnDetached::FDelegate::CreateLambda([](){});

	return AttemptToConnect(InAudioComponent);
}

void UMetasoundMusicSource::Disconnect()
{
	check(IsInGameThread());
	SetClockHistory(nullptr);
	DetachAllCallbacks();
	AudioComponentToWatch.Reset();
	CurrentGeneratorHandle.Reset();
	CachedSourceState = Harmonix::ESourceState::Stopped;
	CurrentPos.Reset();
	PrevPos.Reset();
}

// ---- Transport (IMusicSource) ----
// Transport is derived from the Metasound's actual state, not from explicit calls.
// These are no-ops — control the Metasound through its AudioComponent directly.

void UMetasoundMusicSource::Start() {}
void UMetasoundMusicSource::Pause() {}
void UMetasoundMusicSource::Continue() {}
void UMetasoundMusicSource::Stop() {}

Harmonix::ESourceState UMetasoundMusicSource::GetSourceState() const
{
	return CachedSourceState;
}

Harmonix::ESourceEvent UMetasoundMusicSource::GetLatestSourceEvent() const
{
	if (CachedSourceState != Harmonix::ESourceState::Running)
	{
		return Harmonix::ESourceEvent::None;
	}
	if (bLoopDetected)
	{
		return Harmonix::ESourceEvent::Loop;
	}
	if (bSeekDetected)
	{
		return Harmonix::ESourceEvent::Seek;
	}
	return Harmonix::ESourceEvent::Advance;
}

const FMidiSongPos& UMetasoundMusicSource::GetCurrentSongPos() const
{
	return CurrentPos;
}

const FMidiSongPos& UMetasoundMusicSource::GetPreviousSongPos() const
{
	return PrevPos;
}

const ISongMapEvaluator* UMetasoundMusicSource::GetCurrentSongMapEvaluator() const
{
	check(IsInGameThread());
	if (ClockHistory && CurrentMapChain && CurrentMapChain->SongMaps)
	{
		return CurrentMapChain->SongMaps.Get();
	}
	return nullptr;
}

float UMetasoundMusicSource::GetCurrentClockAdvanceRate() const
{
	return CurrentAdvanceRate;
}

bool UMetasoundMusicSource::LoopedThisFrame() const
{
	return bLoopDetected;
}

bool UMetasoundMusicSource::SeekedThisFrame() const
{
	return bSeekDetected;
}

bool UMetasoundMusicSource::IsLooping() const
{
	return CurrentMapChain && CurrentMapChain->LoopLengthTicks > 0;
}

float UMetasoundMusicSource::GetLoopStartMs() const
{
	if (CurrentMapChain && CurrentMapChain->SongMaps && CurrentMapChain->LoopLengthTicks > 0)
	{
		return CurrentMapChain->SongMaps->TickToMs(CurrentMapChain->FirstTickInLoop);
	}
	return 0.f;
}

float UMetasoundMusicSource::GetLoopLengthMs() const
{
	if (CurrentMapChain && CurrentMapChain->SongMaps && CurrentMapChain->LoopLengthTicks > 0)
	{
		float StartMs = CurrentMapChain->SongMaps->TickToMs(CurrentMapChain->FirstTickInLoop);
		float EndMs = CurrentMapChain->SongMaps->TickToMs(CurrentMapChain->FirstTickInLoop + CurrentMapChain->LoopLengthTicks);
		return EndMs - StartMs;
	}
	return 0.f;
}

TOptional<FVector> UMetasoundMusicSource::TryGetAudioSourceLocation() const
{
	if (UAudioComponent* AudioComponent = AudioComponentToWatch.Get())
	{
		return AudioComponent->GetComponentLocation();
	}
	return {};
}

#if !UE_BUILD_SHIPPING
FString UMetasoundMusicSource::GetDisplayName() const
{
#if WITH_EDITORONLY_DATA
	if (UAudioComponent* Component = AudioComponentToWatch.Get())
	{
		if (UMetaSoundSource* Source = Cast<UMetaSoundSource>(Component->GetSound()))
		{
			return FString::Printf(TEXT("MetasoundMusicSource: %s"), *Source->GetDisplayName().ToString());
		}
	}
#endif
	return TEXT("MetasoundMusicSource");
}
#endif

// ---- Update (called per frame by subsystem) ----

void UMetasoundMusicSource::Update()
{
	if (ensureMsgf(IsInGameThread(), TEXT("%hs called from non-game thread"), __FUNCTION__) == false)
	{
		return;
	}

	// Try to connect if not yet connected
	if (AudioComponentToWatch.IsValid() && !CurrentGeneratorHandle)
	{
		AttemptToConnect(AudioComponentToWatch.Get());
	}

	bSeekDetected = false;
	bLoopDetected = false;

	if (!ClockHistory)
	{
		// Connected but no history yet, or not connected at all
		CachedSourceState = AudioComponentToWatch.IsValid()
			? Harmonix::ESourceState::Preparing
			: Harmonix::ESourceState::Stopped;
		return;
	}

	// Read the Metasound's actual transport state from the clock history
	auto Entry = ClockHistory->Positions.GetEntry(ClockHistory->Positions.GetLastWriteIndex());
	bool bMetasoundIsPlaying = Entry->Item.CurrentTransportState == HarmonixMetasound::EMusicPlayerTransportState::Playing;

	if (bMetasoundIsPlaying)
	{
		CachedSourceState = Harmonix::ESourceState::Running;
		PrevPos = CurrentPos;
		RefreshFromHistory();
	}
	else
	{
		// Metasound exists but isn't playing
		Harmonix::ESourceState PrevState = CachedSourceState;
		CachedSourceState = Harmonix::ESourceState::Stopped;

		// If we just transitioned to stopped, reset position
		if (PrevState == Harmonix::ESourceState::Running)
		{
			CurrentPos.Reset();
			PrevPos.Reset();
		}
	}
}

// ---- History-Based Position (Smoothed Audio Render Time) ----

void UMetasoundMusicSource::RefreshFromHistory()
{
	using namespace HarmonixMetasound::Analysis;
	using namespace MetasoundMusicSourceSmoothing;

	check(IsInGameThread());

	if (!SmoothedClockHistoryCursor.DataAvailable())
	{
		return;
	}

	if (!SmoothedClockHistoryCursor.Queue)
	{
		return;
	}

	if (!CurrentMapChain || !CurrentMapChain->SongMaps || !CurrentMapChain->IsLatest())
	{
		CurrentMapChain = ClockHistory->GetLatestMapsForConsumer();
		if (!CurrentMapChain || !CurrentMapChain->SongMaps)
		{
			return;
		}
	}

	// Read raw position from the history ring
	auto Entry = ClockHistory->Positions.GetEntry(ClockHistory->Positions.GetLastWriteIndex());
	Metasound::FSampleCount LastRenderPosSampleCount = Entry->Item.SampleCount;
	float SpeedAtRawRenderTime = Entry->Item.CurrentSpeed;
	LastTickSeen = Entry->Item.UpToTick;
	bool bClockIsStopped = Entry->Item.CurrentTransportState != HarmonixMetasound::EMusicPlayerTransportState::Playing;

	double CurrentWallClockSeconds = FPlatformTime::Seconds();
	// Initialize sync point on first update
	if (RenderStartWallClockTimeSeconds == 0.0)
	{
		RenderStartSampleCount = LastRenderPosSampleCount;
		RenderStartWallClockTimeSeconds = CurrentWallClockSeconds - static_cast<double>(RenderStartSampleCount) / static_cast<double>(ClockHistory->SampleRate);
		RenderSmoothingLagSeconds = LagSeconds;
		ErrorTracker.Reset();
		LastRefreshWallClockTimeSeconds = RenderStartWallClockTimeSeconds;
	}
	
	DeltaSecondsBetweenRefreshes = CurrentWallClockSeconds - LastRefreshWallClockTimeSeconds;
	LastRefreshWallClockTimeSeconds = CurrentWallClockSeconds;

	double ExpectedRenderedSeconds = (CurrentWallClockSeconds - RenderStartWallClockTimeSeconds) * SyncSpeed;
	double RenderedSeconds = static_cast<double>(LastRenderPosSampleCount) / static_cast<double>(ClockHistory->SampleRate);
	double Error = RenderedSeconds - ExpectedRenderedSeconds;

	if (!bClockIsStopped)
	{
		ErrorTracker.Push(Error);

		if (FMath::Abs(ErrorTracker.Min()) > MaxErrorSecondsBeforeJump)
		{
			UE_LOG(LogMetasoundMusicSource, Verbose, TEXT("======== MASSIVE ERROR (%f) - SEEKING ==========="), static_cast<float>(Error));
			RenderStartSampleCount = LastRenderPosSampleCount;
			RenderStartWallClockTimeSeconds = CurrentWallClockSeconds - static_cast<double>(RenderStartSampleCount) / static_cast<double>(ClockHistory->SampleRate);
			ExpectedRenderedSeconds = RenderedSeconds;
			RenderSmoothingLagSeconds = LagSeconds;
			ErrorTracker.Reset();
			Error = 0;
			SyncSpeed = 1.0;
		}

		if (ExpectedRenderedSeconds > 0.0)
		{
			SyncSpeed += kP * ErrorTracker.Min() / ExpectedRenderedSeconds;
		}
		SyncSpeed = FMath::Clamp(SyncSpeed, MinSyncSpeed, MaxSyncSpeed);
	}

	Metasound::FSampleCount ExpectedRenderPosSampleCount = static_cast<Metasound::FSampleCount>(ExpectedRenderedSeconds * static_cast<double>(ClockHistory->SampleRate));

	// Smooth the audio render position
	const ISongMapEvaluator* Maps = GetCurrentSongMapEvaluator();
	float SpeedAtSmoothedTime = 1.f;
	float SmoothedTick = Maps->MsToTick(CurrentPos.SecondsIncludingCountIn * 1000.f);
	float SmoothedTempoMapTick = SmoothedState.TempoMapTick;

	EHistoryFailureType Result = CalculateSmoothedTick(
		ExpectedRenderPosSampleCount, LastRenderPosSampleCount,
		SmoothedTick, SmoothedTempoMapTick, SpeedAtSmoothedTime,
		SmoothedClockHistoryCursor, RenderSmoothingLagSeconds);

	if (Result != EHistoryFailureType::None)
	{
		if (LastRenderPosSampleCount > static_cast<Metasound::FSampleCount>(RenderSmoothingLagSeconds * ClockHistory->SampleRate * 2))
		{
			if (RenderSmoothingLagSeconds < 0.250f)
			{
				RenderSmoothingLagSeconds += 0.005f;
				UE_LOG(LogMetasoundMusicSource, Verbose, TEXT("Bumping smoothing lag to %f"), RenderSmoothingLagSeconds);
			}
		}
		return;
	}

	if (SmoothedTempoMapTick != SmoothedTick && CurrentMapChain->LoopLengthTicks <= 0)
	{
		UpdateSmoothedPositionForOffsetClock(SmoothedTick, SmoothedTempoMapTick);
	}
	else
	{
		UpdateSmoothedPosition(SmoothedTick, SmoothedTempoMapTick);
	}

	CurrentAdvanceRate = SpeedAtRawRenderTime;
}

void UMetasoundMusicSource::UpdateSmoothedPosition(float SmoothedTick, float SmoothedTempoMapTick)
{
	SmoothedState.TempoMapMs = CurrentMapChain->SongMaps->TickToMs(SmoothedTempoMapTick);
	SmoothedState.TempoMapMs += RenderSmoothingLagSeconds * 1000.f;

	SmoothedState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(SmoothedState.TempoMapMs);
	CurrentPos = CalculateSongPosForLoopingOrMonotonic(
		SmoothedState.TempoMapMs, SmoothedState.LocalTick, bSeekDetected, bLoopDetected);
}

void UMetasoundMusicSource::UpdateSmoothedPositionForOffsetClock(float SmoothedTick, float SmoothedTempoMapTick)
{
	float SmoothedPositionMs = CurrentMapChain->SongMaps->TickToMs(SmoothedTick);
	SmoothedPositionMs += RenderSmoothingLagSeconds * 1000.f;

	CurrentPos = CalculateSongPosForOffsetClock(
		SmoothedPositionMs, SmoothedTick - SmoothedTempoMapTick,
		SmoothedState.LocalTick, bSeekDetected);

	float SmoothedTempoMapMs = CurrentMapChain->SongMaps->TickToMs(SmoothedTempoMapTick);
	SmoothedState.TempoMapMs = SmoothedTempoMapMs + (RenderSmoothingLagSeconds * 1000.f);
	SmoothedState.TempoMapTick = CurrentMapChain->SongMaps->MsToTick(SmoothedState.TempoMapMs);
}

// ---- Song Position Calculation ----

FMidiSongPos UMetasoundMusicSource::CalculateSongPosForLoopingOrMonotonic(
	float AbsoluteMs, float& InOutPositionTick, bool& OutSeekDetected, bool& OutLoopDetected) const
{
	FMidiSongPos OutSongPos;
	if (!ClockHistory || !CurrentMapChain || !CurrentMapChain->SongMaps)
	{
		InOutPositionTick = 0.f;
		return OutSongPos;
	}

	float NewPositionTick = 0.f;
	if (CurrentMapChain->LoopLengthTicks > 0)
	{
		float DrivingTick = CurrentMapChain->SongMaps->MsToTick(AbsoluteMs);
		float TickPastLoop = static_cast<float>(CurrentMapChain->FirstTickInLoop + CurrentMapChain->LoopLengthTicks);
		if (DrivingTick >= TickPastLoop)
		{
			float WrappedTick = FMath::Fmod(DrivingTick - CurrentMapChain->FirstTickInLoop, static_cast<float>(CurrentMapChain->LoopLengthTicks));
			OutLoopDetected = (InOutPositionTick - WrappedTick) > static_cast<float>(CurrentMapChain->LoopLengthTicks - 240);

			OutSongPos.SetByTick(WrappedTick, *CurrentMapChain->SongMaps);
			OutSongPos.Tempo = CurrentMapChain->SongMaps->GetTempoAtTick(FMath::FloorToInt32(DrivingTick));

			if (!OutLoopDetected)
			{
				OutSeekDetected = CheckForSeek(InOutPositionTick, WrappedTick, OutSongPos.Tempo, CurrentMapChain->SongMaps->GetTicksPerQuarterNote());
			}

			InOutPositionTick = WrappedTick;
			return OutSongPos;
		}
		NewPositionTick = DrivingTick;
	}
	else
	{
		NewPositionTick = CurrentMapChain->SongMaps->MsToTick(AbsoluteMs);
	}

	OutSongPos.SetByTimeAndTick(AbsoluteMs, NewPositionTick, *CurrentMapChain->SongMaps);
	OutSeekDetected = CheckForSeek(InOutPositionTick, NewPositionTick, OutSongPos.Tempo, CurrentMapChain->SongMaps->GetTicksPerQuarterNote());
	InOutPositionTick = NewPositionTick;
	return OutSongPos;
}

FMidiSongPos UMetasoundMusicSource::CalculateSongPosForOffsetClock(
	float PositionMs, float ClockTickOffset, float& InOutPositionTick, bool& OutSeekDetected) const
{
	FMidiSongPos OutSongPos;
	if (!ClockHistory || !CurrentMapChain || !CurrentMapChain->SongMaps)
	{
		InOutPositionTick = 0.f;
		return OutSongPos;
	}

	float NewPositionTick = CurrentMapChain->SongMaps->MsToTick(PositionMs);
	OutSongPos.SetByTick(NewPositionTick, *CurrentMapChain->SongMaps);
	OutSongPos.Tempo = CurrentMapChain->SongMaps->GetTempoAtTick(FMath::FloorToInt32(NewPositionTick - ClockTickOffset));

	OutSeekDetected = CheckForSeek(InOutPositionTick, NewPositionTick, OutSongPos.Tempo, CurrentMapChain->SongMaps->GetTicksPerQuarterNote());
	InOutPositionTick = NewPositionTick;
	return OutSongPos;
}

bool UMetasoundMusicSource::CheckForSeek(float FirstTick, float NextTick, float CurrentTempo, int32 TicksPerQuarter) const
{
	float QuartersPerSecond = CurrentTempo / 60.f;
	float ExpectedDeltaQuarters = QuartersPerSecond * static_cast<float>(DeltaSecondsBetweenRefreshes);
	float ExpectedDeltaTicks = ExpectedDeltaQuarters * static_cast<float>(TicksPerQuarter);
	return FMath::Abs(ExpectedDeltaTicks - (NextTick - FirstTick)) > (ExpectedDeltaTicks * 2.f);
}

// ---- Smoothed Tick from History ----

UMetasoundMusicSource::EHistoryFailureType UMetasoundMusicSource::CalculateSmoothedTick(
	Metasound::FSampleCount ExpectedRenderPosSampleCount,
	Metasound::FSampleCount LastRenderPosSampleCount,
	float& SmoothedLocalTick, float& SmoothedTempoMapTick, float& CurrentSpeed,
	HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor& ReadCursor,
	float LookBehindSeconds)
{
	using namespace HarmonixMetasound::Analysis;

	Metasound::FSampleCount LookingForSampleFrame = ExpectedRenderPosSampleCount - static_cast<Metasound::FSampleCount>(LookBehindSeconds * ClockHistory->SampleRate);

	int32 NumHistoryAvailable = ReadCursor.NumDataAvailable();
	if (LookingForSampleFrame >= LastRenderPosSampleCount && NumHistoryAvailable > 1)
	{
		while (ReadCursor.NumDataAvailable() > 1)
		{
			ReadCursor.ConsumeNext();
		}
		NumHistoryAvailable = ReadCursor.NumDataAvailable();
	}

	if (NumHistoryAvailable == 0)
	{
		return EHistoryFailureType::NotEnoughDataInTheHistoryRing;
	}

	FMidiClockSongPositionHistory::FScopedItemPeekRef PeekNextRef = ReadCursor.PeekNext();

	if (NumHistoryAvailable == 1 || PeekNextRef->SampleCount > LookingForSampleFrame)
	{
		SmoothedLocalTick = static_cast<float>(PeekNextRef->UpToTick);
		SmoothedTempoMapTick = static_cast<float>(PeekNextRef->TempoMapTick);
		CurrentSpeed = PeekNextRef->CurrentSpeed;
		return EHistoryFailureType::None;
	}

	FMidiClockSongPositionHistory::FScopedItemPeekRef PeekOneAheadRef(ReadCursor.PeekAhead(1));
	while (PeekOneAheadRef && PeekOneAheadRef->SampleCount <= LookingForSampleFrame)
	{
		ReadCursor.PeekAhead(2, PeekOneAheadRef);
		ReadCursor.PeekAhead(1, PeekNextRef);
		ReadCursor.ConsumeNext();
	}

	if (!PeekOneAheadRef)
	{
		return EHistoryFailureType::CaughtUpToRenderPosition;
	}

	check(LookingForSampleFrame >= PeekNextRef->SampleCount && LookingForSampleFrame < PeekOneAheadRef->SampleCount);

	float LerpAlpha = static_cast<float>(LookingForSampleFrame - PeekNextRef->SampleCount) / static_cast<float>(PeekOneAheadRef->SampleCount - PeekNextRef->SampleCount);
	SmoothedLocalTick = FMath::Lerp<float>(static_cast<float>(PeekNextRef->UpToTick), static_cast<float>(PeekOneAheadRef->UpToTick), LerpAlpha);
	SmoothedTempoMapTick = FMath::Lerp<float>(static_cast<float>(PeekNextRef->TempoMapTick), static_cast<float>(PeekOneAheadRef->TempoMapTick), LerpAlpha);
	CurrentSpeed = PeekNextRef->CurrentSpeed;

	return EHistoryFailureType::None;
}

FString UMetasoundMusicSource::HistoryFailureTypeToString(EHistoryFailureType Error)
{
	switch (Error)
	{
	case EHistoryFailureType::None: return TEXT("None");
	case EHistoryFailureType::NotEnoughDataInTheHistoryRing: return TEXT("NotEnoughDataInTheHistoryRing");
	case EHistoryFailureType::NotEnoughHistory: return TEXT("NotEnoughHistory");
	case EHistoryFailureType::LookingForTimeInTheFutureOfWhatHasEvenRendered: return TEXT("LookingForTimeInTheFutureOfWhatHasEvenRendered");
	case EHistoryFailureType::CaughtUpToRenderPosition: return TEXT("CaughtUpToRenderPosition");
	}
	return TEXT("Unrecognized");
}

// ---- Generator Connection ----

bool UMetasoundMusicSource::AttemptToConnect(UAudioComponent* InAudioComponent)
{
	using namespace HarmonixMetasound::Analysis;

	check(IsInGameThread());
	if (!AudioComponentToWatch.IsValid() || MetasoundOutputName.IsNone())
	{
		return false;
	}

	if (!Cast<UMetaSoundSource>(AudioComponentToWatch->GetSound()))
	{
		return false;
	}

	DetachAllCallbacks();
	CurrentGeneratorHandle.Reset(UMetasoundGeneratorHandle::CreateMetaSoundGeneratorHandle(AudioComponentToWatch.Get()));
	if (!CurrentGeneratorHandle)
	{
		return false;
	}

	bool bWatchingOutput = CurrentGeneratorHandle->WatchOutput(MetasoundOutputName,
		FOnMetasoundOutputValueChangedNative::CreateLambda([](FName, const FMetaSoundOutput&) {}),
		FMidiSongPosVertexAnalyzer::GetAnalyzerName(),
		FMidiSongPosVertexAnalyzer::SongPosition.Name);

	if (bWatchingOutput)
	{
		ensure(CurrentGeneratorHandle->TryCreateAnalyzerAddress(MetasoundOutputName,
			FMidiSongPosVertexAnalyzer::GetAnalyzerName(),
			FMidiSongPosVertexAnalyzer::SongPosition.Name,
			MidiSongPosAnalyzerAddress));
	}

	GeneratorAttachedCallbackHandle = CurrentGeneratorHandle->OnGeneratorHandleAttached.AddLambda([this]()
	{
		OnGeneratorAttached();
	});
	GeneratorDetachedCallbackHandle = CurrentGeneratorHandle->OnGeneratorHandleDetached.AddLambda([this]()
	{
		OnGeneratorDetached();
	});
	GeneratorIOUpdatedCallbackHandle = CurrentGeneratorHandle->OnIOUpdatedWithChanges.AddLambda([this](const TArray<Metasound::FVertexInterfaceChange>& Changes)
	{
		OnGeneratorIOUpdatedWithChanges(Changes);
	});

	UMetasoundGeneratorHandle::FOnSetGraph::FDelegate OnSetGraph;
	OnSetGraph.BindLambda([this]() { OnGraphSet(); });
	GraphChangedCallbackHandle = CurrentGeneratorHandle->AddGraphSetCallback(MoveTemp(OnSetGraph));

	return true;
}

void UMetasoundMusicSource::DetachAllCallbacks()
{
	if (CurrentGeneratorHandle)
	{
		CurrentGeneratorHandle->OnGeneratorHandleAttached.Remove(GeneratorAttachedCallbackHandle);
		GeneratorAttachedCallbackHandle.Reset();
		CurrentGeneratorHandle->OnGeneratorHandleDetached.Remove(GeneratorDetachedCallbackHandle);
		GeneratorDetachedCallbackHandle.Reset();
		CurrentGeneratorHandle->OnIOUpdatedWithChanges.Remove(GeneratorIOUpdatedCallbackHandle);
		GeneratorIOUpdatedCallbackHandle.Reset();
		CurrentGeneratorHandle->RemoveGraphSetCallback(GraphChangedCallbackHandle);
		GraphChangedCallbackHandle.Reset();
	}
	SetClockHistory(nullptr);
}

void UMetasoundMusicSource::OnGeneratorAttached()
{
	UnderlyingGenerator = CurrentGeneratorHandle->GetGenerator();
	SetClockHistory(UMidiClockUpdateSubsystem::GetOrCreateClockHistory(MidiSongPosAnalyzerAddress));
	if (!ClockHistory)
	{
		UE_LOG(LogMetasoundMusicSource, Warning, TEXT("OnGeneratorAttached: Failed to create clock history for address '%s'"), *MidiSongPosAnalyzerAddress.ToString());
		return;
	}
	SmoothedClockHistoryCursor = ClockHistory->CreateReadCursor();

	// Reset PLL sync state so it re-initializes on first RefreshFromHistory()
	RenderStartSampleCount = 0;
	RenderStartWallClockTimeSeconds = 0.0;
}

void UMetasoundMusicSource::OnGraphSet()
{
	SetClockHistory(UMidiClockUpdateSubsystem::GetOrCreateClockHistory(MidiSongPosAnalyzerAddress));
	if (!ClockHistory)
	{
		UE_LOG(LogMetasoundMusicSource, Warning, TEXT("OnGraphSet: Failed to create clock history for address '%s'"), *MidiSongPosAnalyzerAddress.ToString());
		return;
	}
	SmoothedClockHistoryCursor = ClockHistory->CreateReadCursor();
}

void UMetasoundMusicSource::OnGeneratorIOUpdatedWithChanges(const TArray<Metasound::FVertexInterfaceChange>& VertexInterfaceChanges)
{
	if (!MetasoundOutputName.IsNone() && VertexInterfaceChanges.Num() > 0)
	{
		SetClockHistory(UMidiClockUpdateSubsystem::GetOrCreateClockHistory(MidiSongPosAnalyzerAddress));
		if (!ClockHistory)
		{
			UE_LOG(LogMetasoundMusicSource, Warning, TEXT("OnGeneratorIOUpdatedWithChanges: Failed to create clock history for address '%s'"), *MidiSongPosAnalyzerAddress.ToString());
			return;
		}
		SmoothedClockHistoryCursor = ClockHistory->CreateReadCursor();
	}
}

void UMetasoundMusicSource::OnGeneratorDetached()
{
	TWeakPtr<Metasound::FMetasoundGenerator> DetachingGenerator = CurrentGeneratorHandle->GetGenerator();
	if (DetachingGenerator != UnderlyingGenerator)
	{
		return;
	}

	SetClockHistory(nullptr);
	SmoothedClockHistoryCursor = HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor();
}

void UMetasoundMusicSource::SetClockHistory(const UMidiClockUpdateSubsystem::FClockHistoryPtr& NewHistory)
{
	if (NewHistory != ClockHistory)
	{
		ClockHistory = NewHistory;
		CurrentMapChain = nullptr;
	}
}
