// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MusicSource/MusicSourceBlueprintLibrary.h"
#include "HarmonixMetasound/MusicClock/MusicClock.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"

TScriptInterface<IMusicSource> UMusicSourceBlueprintLibrary::CreateMusicSource(
	UObject* Outer,
	const TInstancedStruct<FMusicSourceSettings>& Settings)
{
	if (!Outer || !Settings.IsValid())
	{
		return {};
	}

	return Settings.Get().CreateSource(Outer);
}

UMusicClock* UMusicSourceBlueprintLibrary::CreateMusicClock(
	UObject* Outer,
	const TScriptInterface<IMusicSource>& Source)
{
	if (!Outer || !Source)
	{
		return nullptr;
	}

	UMusicClock* Clock = NewObject<UMusicClock>(Outer);
	Clock->SetSource(Source);
	UMidiClockUpdateSubsystem::TrackMusicClock(Clock);
	return Clock;
}
