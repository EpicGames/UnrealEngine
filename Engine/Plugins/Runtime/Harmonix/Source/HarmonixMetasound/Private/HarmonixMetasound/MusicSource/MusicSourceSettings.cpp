// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MusicSource/MusicSourceSettings.h"
#include "HarmonixMetasound/MusicSource/MetasoundMusicSource.h"
#include "HarmonixMetasound/MusicSource/RuntimeMusicSource.h"
#include "HarmonixMetasound/MusicSource/OffsetMusicSource.h"
#include "HarmonixMetasound/MusicClock/WorldTimeSourceController.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"
#include "Components/AudioComponent.h"
#include "HarmonixMidi/MidiFile.h"

TScriptInterface<IMusicSource> FMusicSourceSettings::CreateSource(UObject* Outer) const
{
	checkNoEntry(); // Base class — subclasses must override. Stripped in shipping.
	UE_LOG(LogTemp, Warning, TEXT("FMusicSourceSettings::CreateSource called on base type. Use a concrete settings subclass."));
	return {};
}

TScriptInterface<IMusicSource> FMetasoundMusicSourceSettings::CreateSource(UObject* Outer) const
{
	if (!AudioComponent)
	{
		return {};
	}

	UMetasoundMusicSource* Source = NewObject<UMetasoundMusicSource>(Outer);
	Source->ConnectToAudioComponent(AudioComponent, OutputPinName);
	UMidiClockUpdateSubsystem::TrackMusicSource(Source);
	return TScriptInterface<IMusicSource>(Source);
}

TScriptInterface<IMusicSource> FMidiMusicSourceSettings::CreateSource(UObject* Outer) const
{
	if (!MidiFile)
	{
		return {};
	}

	TSharedPtr<Harmonix::FWorldTimeSourceController> TimeSource = MakeShared<Harmonix::FWorldTimeSourceController>(Outer);
	URuntimeMusicSource* Source = NewObject<URuntimeMusicSource>(Outer);
	Source->InitializeWithMidi(TimeSource, MidiFile);
	UMidiClockUpdateSubsystem::TrackMusicSource(Source);
	return TScriptInterface<IMusicSource>(Source);
}

TScriptInterface<IMusicSource> FManualMusicSourceSettings::CreateSource(UObject* Outer) const
{
	TSharedPtr<Harmonix::FWorldTimeSourceController> TimeSource = MakeShared<Harmonix::FWorldTimeSourceController>(Outer);
	URuntimeMusicSource* Source = NewObject<URuntimeMusicSource>(Outer);
	Source->Initialize(TimeSource, nullptr);
	Source->SetTempo(Tempo);
	Source->SetTimeSignature(TimeSigNumerator, TimeSigDenominator);
	UMidiClockUpdateSubsystem::TrackMusicSource(Source);
	return TScriptInterface<IMusicSource>(Source);
}

TScriptInterface<IMusicSource> FOffsetMusicSourceSettings::CreateSource(UObject* Outer) const
{
	if (!ParentSource)
	{
		return {};
	}

	UOffsetMusicSource* Source = NewObject<UOffsetMusicSource>(Outer);
	Source->SetParentSource(ParentSource);
	Source->SetOffsetMs(OffsetMs);
	UMidiClockUpdateSubsystem::TrackMusicSource(Source);
	return TScriptInterface<IMusicSource>(Source);
}
