// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MusicSource/OffsetMusicSource.h"

void UOffsetMusicSource::SetParentSource(const TScriptInterface<IMusicSource>& InParentSource)
{
	ParentSourceObject = InParentSource.GetObject();
	ParentSourceCache = InParentSource ? Cast<IMusicSource>(InParentSource.GetObject()) : nullptr;
	bParentHasLooped = false;
}

TScriptInterface<IMusicSource> UOffsetMusicSource::GetParentSource() const
{
	if (UObject* Obj = ParentSourceObject.Get())
	{
		return TScriptInterface<IMusicSource>(Obj);
	}
	return {};
}

IMusicSource* UOffsetMusicSource::GetValidParentSource() const
{
	return ParentSourceObject.IsValid() ? ParentSourceCache : nullptr;
}

void UOffsetMusicSource::SetOffsetMs(float InOffsetMs)
{
	OffsetMs = InOffsetMs;
}

float UOffsetMusicSource::GetOffsetMs() const
{
	return OffsetMs;
}

Harmonix::ESourceState UOffsetMusicSource::GetSourceState() const
{
	IMusicSource* Parent = GetValidParentSource();
	return Parent ? Parent->GetSourceState() : Harmonix::ESourceState::Stopped;
}

Harmonix::ESourceEvent UOffsetMusicSource::GetLatestSourceEvent() const
{
	IMusicSource* Parent = GetValidParentSource();
	if (!Parent)
	{
		return Harmonix::ESourceEvent::None;
	}

	if (bLoopedThisFrame)
	{
		return Harmonix::ESourceEvent::Loop;
	}
	if (bSeekedThisFrame)
	{
		return Harmonix::ESourceEvent::Seek;
	}

	if (Parent->GetSourceState() == Harmonix::ESourceState::Running)
	{
		return Harmonix::ESourceEvent::Advance;
	}
	return Harmonix::ESourceEvent::None;
}

const FMidiSongPos& UOffsetMusicSource::GetCurrentSongPos() const
{
	return CurrentPos;
}

const FMidiSongPos& UOffsetMusicSource::GetPreviousSongPos() const
{
	return PrevPos;
}

const ISongMapEvaluator* UOffsetMusicSource::GetCurrentSongMapEvaluator() const
{
	IMusicSource* Parent = GetValidParentSource();
	return Parent ? Parent->GetCurrentSongMapEvaluator() : nullptr;
}

float UOffsetMusicSource::GetCurrentClockAdvanceRate() const
{
	IMusicSource* Parent = GetValidParentSource();
	return Parent ? Parent->GetCurrentClockAdvanceRate() : 1.f;
}

bool UOffsetMusicSource::LoopedThisFrame() const
{
	return bLoopedThisFrame;
}

bool UOffsetMusicSource::SeekedThisFrame() const
{
	return bSeekedThisFrame;
}

bool UOffsetMusicSource::IsLooping() const
{
	IMusicSource* Parent = GetValidParentSource();
	return Parent ? Parent->IsLooping() : false;
}

float UOffsetMusicSource::GetLoopStartMs() const
{
	IMusicSource* Parent = GetValidParentSource();
	return Parent ? Parent->GetLoopStartMs() : 0.f;
}

float UOffsetMusicSource::GetLoopLengthMs() const
{
	IMusicSource* Parent = GetValidParentSource();
	return Parent ? Parent->GetLoopLengthMs() : 0.f;
}

TOptional<FVector> UOffsetMusicSource::TryGetAudioSourceLocation() const
{
	IMusicSource* Parent = GetValidParentSource();
	return Parent ? Parent->TryGetAudioSourceLocation() : TOptional<FVector>{};
}

#if !UE_BUILD_SHIPPING
FString UOffsetMusicSource::GetDisplayName() const
{
	IMusicSource* Parent = GetValidParentSource();
	if (Parent)
	{
		return FString::Printf(TEXT("OffsetMusicSource(%+.1fms): %s"), OffsetMs, *Parent->GetDisplayName());
	}
	return TEXT("OffsetMusicSource (no parent)");
}
#endif

void UOffsetMusicSource::Update()
{
	IMusicSource* Parent = GetValidParentSource();
	if (!Parent)
	{
		if (ParentSourceCache != nullptr)
		{
			ParentSourceCache = nullptr;
		}
		CurrentPos.Reset();
		PrevPos.Reset();
		bLoopedThisFrame = false;
		bSeekedThisFrame = false;
		return;
	}

	if (Parent->GetSourceState() != Harmonix::ESourceState::Running)
	{
		if (Parent->GetSourceState() == Harmonix::ESourceState::Stopped)
		{
			CurrentPos.Reset();
			PrevPos.Reset();
		}
		bLoopedThisFrame = false;
		bSeekedThisFrame = false;
		return;
	}

	const ISongMapEvaluator* Maps = Parent->GetCurrentSongMapEvaluator();
	if (!Maps)
	{
		return;
	}

	PrevPos = CurrentPos;

	const FMidiSongPos& ParentPos = Parent->GetCurrentSongPos();
	float ParentMs = ParentPos.SecondsIncludingCountIn * 1000.f;
	float OffsetTimeMs = ParentMs + OffsetMs;

	bSeekedThisFrame = Parent->SeekedThisFrame();
	bLoopedThisFrame = false;

	if (Parent->IsLooping())
	{
		float LoopStart = Parent->GetLoopStartMs();
		float LoopLength = Parent->GetLoopLengthMs();

		// Track whether the parent has completed at least one full loop pass.
		// Reset on seek (e.g., seeking back to start).
		if (bSeekedThisFrame)
		{
			bParentHasLooped = false;
		}
		if (Parent->LoopedThisFrame())
		{
			bParentHasLooped = true;
		}

		if (LoopLength > 0.f)
		{
			// Only wrap around the loop region if:
			//  - The parent has already looped (steady-state), OR
			//  - The offset time is naturally within or past the loop region
			//
			// This prevents a negative offset from jumping to the end of the
			// loop on the first pass (before the parent has ever crossed the
			// loop boundary). In that case, the offset time stays negative —
			// the source is "before" the music start.
			//
			// Example with loop 0-4000ms, offset -100ms:
			//   First pass, parent at 50ms  -> OffsetTime = -50ms  -> stays -50ms (no wrap)
			//   First pass, parent at 200ms -> OffsetTime = 100ms  -> no wrap needed (in bounds)
			//   After loop, parent at 50ms  -> OffsetTime = -50ms  -> wraps to 3950ms
			if (bParentHasLooped || OffsetTimeMs >= LoopStart)
			{
				float RelativeMs = OffsetTimeMs - LoopStart;
				RelativeMs = FMath::Fmod(RelativeMs + LoopLength, LoopLength);
				OffsetTimeMs = LoopStart + RelativeMs;
			}

			// Detect if we looped: large position delta relative to loop length
			float PrevMs = PrevPos.SecondsIncludingCountIn * 1000.f;
			if (PrevMs > 0.f)
			{
				float DeltaMs = OffsetTimeMs - PrevMs;
				if (FMath::Abs(DeltaMs) > (LoopLength * 0.5f))
				{
					bLoopedThisFrame = true;
				}
			}
		}
	}
	else
	{
		bLoopedThisFrame = Parent->LoopedThisFrame();
	}

	CurrentPos.SetByTime(OffsetTimeMs, *Maps);
}
