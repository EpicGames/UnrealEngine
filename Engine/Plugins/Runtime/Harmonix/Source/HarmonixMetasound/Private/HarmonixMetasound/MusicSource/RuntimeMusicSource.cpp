// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MusicSource/RuntimeMusicSource.h"
#include "Harmonix.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"
#include "HarmonixMidi/MidiFile.h"

void URuntimeMusicSource::BeginDestroy()
{
	UMidiClockUpdateSubsystem::StopTrackingMusicSource(this);
	Super::BeginDestroy();
}

void URuntimeMusicSource::Initialize(TSharedPtr<Harmonix::ITimeSource> InTimeSource, const ISongMapEvaluator* InSongMaps)
{
	TimeSource = InTimeSource;
	MidiFile = nullptr;

	DefaultMaps.EmptyAllMaps();
	if (InSongMaps)
	{
		DefaultMaps.Copy(*InSongMaps, 0, InSongMaps->GetSongLengthData().LastTick);
	}
	else
	{
		DefaultMaps.Init(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt);
		DefaultMaps.GetTempoMap().AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(120.0f), 0);
		DefaultMaps.GetBarMap().AddTimeSignatureAtBarIncludingCountIn(0, 4, 4);
	}
}

void URuntimeMusicSource::InitializeWithMidi(TSharedPtr<Harmonix::ITimeSource> InTimeSource, UMidiFile* InMidiFile)
{
	TimeSource = InTimeSource;
	MidiFile = InMidiFile;
}

void URuntimeMusicSource::Start()
{
	if (TimeSource)
	{
		TimeSource->RequestStart();
	}
}

void URuntimeMusicSource::Pause()
{
	if (TimeSource)
	{
		TimeSource->RequestPause();
	}
}

void URuntimeMusicSource::Continue()
{
	if (TimeSource)
	{
		TimeSource->RequestContinue();
	}
}

void URuntimeMusicSource::Stop()
{
	if (TimeSource)
	{
		TimeSource->RequestStop();
	}
	CurrentPos.Reset();
	PrevPos.Reset();
}

Harmonix::ESourceState URuntimeMusicSource::GetSourceState() const
{
	if (TimeSource)
	{
		return TimeSource->GetCurrentSourceState();
	}
	return Harmonix::ESourceState::Stopped;
}

Harmonix::ESourceEvent URuntimeMusicSource::GetLatestSourceEvent() const
{
	if (TimeSource)
	{
		return TimeSource->GetLatestSourceEvent();
	}
	return Harmonix::ESourceEvent::None;
}

const FMidiSongPos& URuntimeMusicSource::GetCurrentSongPos() const
{
	return CurrentPos;
}

const FMidiSongPos& URuntimeMusicSource::GetPreviousSongPos() const
{
	return PrevPos;
}

const ISongMapEvaluator* URuntimeMusicSource::GetCurrentSongMapEvaluator() const
{
	return GetMaps();
}

float URuntimeMusicSource::GetCurrentClockAdvanceRate() const
{
	if (TimeSource)
	{
		return TimeSource->GetSpeed();
	}
	return 1.0f;
}

bool URuntimeMusicSource::LoopedThisFrame() const
{
	return TimeSource && TimeSource->GetLatestSourceEvent() == Harmonix::ESourceEvent::Loop;
}

bool URuntimeMusicSource::SeekedThisFrame() const
{
	return TimeSource && TimeSource->GetLatestSourceEvent() == Harmonix::ESourceEvent::Seek;
}

bool URuntimeMusicSource::IsLooping() const
{
	return LoopNumBars > 0 && CachedLoopLengthMs > 0.f;
}

float URuntimeMusicSource::GetLoopStartMs() const
{
	return CachedLoopStartMs;
}

float URuntimeMusicSource::GetLoopLengthMs() const
{
	return CachedLoopLengthMs;
}

void URuntimeMusicSource::SetLoopRegionByBars(int32 StartBar, int32 NumBars)
{
	LoopStartBar = StartBar;
	LoopNumBars = FMath::Max(NumBars, 0);
	RecomputeLoopRegionMs();
}

void URuntimeMusicSource::ClearLoopRegion()
{
	LoopStartBar = 0;
	LoopNumBars = 0;
	CachedLoopStartMs = 0.f;
	CachedLoopLengthMs = 0.f;
}

void URuntimeMusicSource::RecomputeLoopRegionMs()
{
	if (LoopNumBars <= 0)
	{
		CachedLoopStartMs = 0.f;
		CachedLoopLengthMs = 0.f;
		return;
	}

	const ISongMapEvaluator* Maps = GetMaps();
	if (!Maps)
	{
		CachedLoopStartMs = 0.f;
		CachedLoopLengthMs = 0.f;
		return;
	}

	int32 EndBar = LoopStartBar + LoopNumBars;

	// Convert bar numbers to ticks, then to ms
	float StartTick = Maps->BarIncludingCountInToTick(LoopStartBar);
	float EndTick = Maps->BarIncludingCountInToTick(EndBar);

	CachedLoopStartMs = Maps->TickToMs(StartTick);
	CachedLoopLengthMs = Maps->TickToMs(EndTick) - CachedLoopStartMs;
}

void URuntimeMusicSource::Seek(const FMusicTimestamp& Timestamp)
{
	if (!TimeSource)
	{
		return;
	}

	const ISongMapEvaluator* Maps = GetMaps();
	if (!Maps)
	{
		return;
	}

	float Tick = Maps->MusicTimestampToTick(Timestamp);
	float Ms = Maps->TickToMs(Tick);
	float Seconds = Ms / 1000.f;

	TimeSource->RequestSeek(Seconds);
}

void URuntimeMusicSource::SetTempo(float BPM)
{
	if (BPM <= 0.f)
	{
		return;
	}

	// When driven by a MIDI file, we can't modify external maps — copy to DefaultMaps first
	if (MidiFile.IsValid())
	{
		const ISongMapEvaluator* ExternalMaps = MidiFile->GetSongMaps();
		if (ExternalMaps)
		{
			DefaultMaps.Copy(*ExternalMaps, 0, ExternalMaps->GetSongLengthData().LastTick);
		}
		MidiFile = nullptr;
	}

	DefaultMaps.EmptyTempoMap();
	DefaultMaps.GetTempoMap().AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(BPM), 0);

	RecomputeLoopRegionMs();
}

void URuntimeMusicSource::SetTimeSignature(int32 Numerator, int32 Denominator)
{
	if (Numerator <= 0 || Denominator <= 0)
	{
		return;
	}

	// When driven by a MIDI file, copy to DefaultMaps first
	if (MidiFile.IsValid())
	{
		const ISongMapEvaluator* ExternalMaps = MidiFile->GetSongMaps();
		if (ExternalMaps)
		{
			DefaultMaps.Copy(*ExternalMaps, 0, ExternalMaps->GetSongLengthData().LastTick);
		}
		MidiFile = nullptr;
	}

	DefaultMaps.EmptyBarMap();
	DefaultMaps.EmptyBeatMap();
	DefaultMaps.GetBarMap().AddTimeSignatureAtBarIncludingCountIn(0, Numerator, Denominator);

	RecomputeLoopRegionMs();
}

void URuntimeMusicSource::SetSpeed(float Speed)
{
	if (TimeSource)
	{
		TimeSource->RequestSetSpeed(Speed);
	}
}

TOptional<FVector> URuntimeMusicSource::TryGetAudioSourceLocation() const
{
	if (TimeSource)
	{
		return TimeSource->TryGetAudioSourceLocation();
	}
	return {};
}

void URuntimeMusicSource::Update()
{
	check(IsInGameThread());

	if (!TimeSource)
	{
		return;
	}

	TimeSource->Update();

	if (TimeSource->GetCurrentSourceState() != Harmonix::ESourceState::Running)
	{
		return;
	}

	const ISongMapEvaluator* Maps = GetMaps();
	if (!Maps)
	{
		return;
	}

	PrevPos = CurrentPos;

	double SourceTimeSeconds = TimeSource->GetCurrentTime();
	float SourceTimeMs = static_cast<float>(SourceTimeSeconds * 1000.0);

	if (LoopNumBars > 0 && CachedLoopLengthMs > 0.f)
	{
		float LoopEndMs = CachedLoopStartMs + CachedLoopLengthMs;
		if (SourceTimeMs >= LoopEndMs)
		{
			float Relative = FMath::Fmod(SourceTimeMs - CachedLoopStartMs, CachedLoopLengthMs);
			SourceTimeMs = CachedLoopStartMs + Relative;
		}
	}

	CurrentPos.SetByTime(SourceTimeMs, *Maps);
}

#if !UE_BUILD_SHIPPING
FString URuntimeMusicSource::GetDisplayName() const
{
	if (TimeSource)
	{
		return FString::Printf(TEXT("RuntimeMusicSource: %s"), *TimeSource->GetDisplayName());
	}
	return TEXT("RuntimeMusicSource (no time source)");
}
#endif

const ISongMapEvaluator* URuntimeMusicSource::GetMaps() const
{
	if (MidiFile.IsValid())
	{
		return MidiFile->GetSongMaps();
	}
	return &DefaultMaps;
}
