// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/DelayNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "HarmonixDsp/Effects/Delay.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/DelayFilterType.h"
#include "HarmonixMetasound/DataTypes/DelayStereoType.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/TimeSyncOption.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::DelayNode
{
	using namespace Metasound;
	
	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(AudioLeft, CommonPinNames::Inputs::AudioLeft);
		DEFINE_METASOUND_PARAM_ALIAS(AudioRight, CommonPinNames::Inputs::AudioRight);
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_INPUT_METASOUND_PARAM(DelayTimeType, "Delay Sync Type", "Specifies if and how the delay should sync to a clock");
		DEFINE_INPUT_METASOUND_PARAM(DelayTime, "Delay Time", "The period of the delay. If Delay Sync Type is TempoSync, the unit is quarter notes. Otherwise, the unit is seconds.");
		DEFINE_INPUT_METASOUND_PARAM(Feedback, "Feedback", "The feedback amount, range [0.0, 1.0]");
		DEFINE_INPUT_METASOUND_PARAM(DryLevel, "Dry Gain", "The amount of dry signal to pass through, range [0.0, 1.0]");
		DEFINE_INPUT_METASOUND_PARAM(WetLevel, "Wet Gain", "The amount of wet signal to output, range [0.0, 1.0]");
		DEFINE_INPUT_METASOUND_PARAM(WetFilterEnabled, "Wet Filter Enabled", "Enables or disables the filter on the wet signal");
		DEFINE_INPUT_METASOUND_PARAM(FeedbackFilterEnabled, "Feedback Filter Enabled", "Enables or disables the filter on the feedback signal");
		DEFINE_INPUT_METASOUND_PARAM(FilterType, "Filter Type", "The type of filter to apply to the signal");
		DEFINE_INPUT_METASOUND_PARAM(FilterCutoff, "Filter Cutoff", "The cutoff frequency of the filter, in Hz");
		DEFINE_INPUT_METASOUND_PARAM(FilterQ, "Filter Q", "The Q of the filter");
		DEFINE_INPUT_METASOUND_PARAM(LFOEnabled, "LFO Enabled", "Enables or disables the LFO on the delay time");
		DEFINE_INPUT_METASOUND_PARAM(LFOTimeType, "LFO Sync Type", "Specifies if and how the LFO should sync to a clock");
		DEFINE_METASOUND_PARAM_ALIAS(LFOFrequency, CommonPinNames::Inputs::LFOFrequency);
		DEFINE_INPUT_METASOUND_PARAM(LFODepth, "LFO Depth", "The amount of delay to be added by the LFO. If Delay Sync Type is TempoSync, the unit is quarter notes. Otherwise, the unit is seconds.");
		DEFINE_INPUT_METASOUND_PARAM(StereoType, "Stereo Type", "Specifies method for panning and spreading the sound");
		DEFINE_INPUT_METASOUND_PARAM(StereoSpreadLeft, "Stereo Spread Left", "The amount of stereo spread on the left channel");
		DEFINE_INPUT_METASOUND_PARAM(StereoSpreadRight, "Stereo Spread Right", "The amount of stereo spread on the right channel");
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(AudioLeft, CommonPinNames::Outputs::AudioLeft);
		DEFINE_METASOUND_PARAM_ALIAS(AudioRight, CommonPinNames::Outputs::AudioRight);
	}
	
	class FDelayOperator final : public TExecutableOperator<FDelayOperator>
	{
	public:
		static constexpr int32 NumChannels = 1;

		struct FInputs
		{
			FAudioBufferReadRef AudioLeft;
			FAudioBufferReadRef AudioRight;
			TOptional<FMidiClockReadRef> MidiClock;
			FEnumTimeSyncOptionReadReference DelayTimeType;
			FFloatReadRef DelayTime;
			FFloatReadRef Feedback;
			FFloatReadRef DryLevel;
			FFloatReadRef WetLevel;
			FBoolReadRef WetFilterEnabled;
			FBoolReadRef FeedbackFilterEnabled;
			FEnumDelayFilterTypeReadReference FilterType;
			FFloatReadRef FilterCutoff;
			FFloatReadRef FilterQ;
			FBoolReadRef LFOEnabled;
			FEnumTimeSyncOptionReadReference LFOTimeType;
			FFloatReadRef LFOFrequency;
			FFloatReadRef LFODepth;
			FEnumDelayStereoTypeReadReference StereoType;
			FFloatReadRef StereoSpreadLeft;
			FFloatReadRef StereoSpreadRight;
		};

		struct FOutputs
		{
			FAudioBufferWriteRef AudioLeft;
			FAudioBufferWriteRef AudioRight;
		};
		
		FDelayOperator(const FBuildOperatorParams& Params, FInputs&& Inputs, FOutputs&& Outputs)
			: Inputs(MoveTemp(Inputs))
			, Outputs(MoveTemp(Outputs))
		{
			Reset(Params);
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			auto InitVertexInterface = []() -> FVertexInterface
			{
				const Harmonix::Dsp::Effects::FDelay DelayForDefaults;
				return
				{
					FInputVertexInterface
					{
						TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudioLeft)),
						TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudioRight)),
						TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
						TInputDataVertex<FEnumTimeSyncOption>(
							METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DelayTimeType),
							static_cast<int32>(DelayForDefaults.GetTimeSyncOption())),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DelayTime),
							DelayForDefaults.GetDelaySeconds()),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Feedback),
							DelayForDefaults.GetFeedbackGain()),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DryLevel),
							DelayForDefaults.GetDryGain()),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::WetLevel),
							DelayForDefaults.GetWetGain()),
						TInputDataVertex<bool>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::WetFilterEnabled),
							DelayForDefaults.GetWetFilterEnabled()),
						TInputDataVertex<bool>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::FeedbackFilterEnabled),
							DelayForDefaults.GetFeedbackFilterEnabled()),
						TInputDataVertex<FEnumDelayFilterType>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::FilterType),
							static_cast<int32>(DelayForDefaults.GetFilterType())),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::FilterCutoff),
							DelayForDefaults.GetFilterFreq()),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::FilterQ),
							DelayForDefaults.GetFilterQ()),
						TInputDataVertex<bool>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::LFOEnabled),
							DelayForDefaults.GetLfoEnabled()),
						TInputDataVertex<FEnumTimeSyncOption>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::LFOTimeType),
							static_cast<int32>(DelayForDefaults.GetLfoTimeSyncOption())),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::LFOFrequency),
							DelayForDefaults.GetLfoFreq()),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::LFODepth),
							DelayForDefaults.GetLfoDepth()),
						TInputDataVertex<FEnumDelayStereoType>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::StereoType),
							static_cast<int32>(DelayForDefaults.GetStereoType())),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::StereoSpreadLeft),
							DelayForDefaults.GetStereoSpreadLeft()),
						TInputDataVertex<float>(
							METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(Inputs::StereoSpreadRight),
							DelayForDefaults.GetStereoSpreadRight())
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::AudioLeft)),
						TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::AudioRight))
					}
				};
			};
			
			static const FVertexInterface Interface = InitVertexInterface();

			return Interface;
		}
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = FNodeClassMetadata::GetEmpty();
				Info.ClassName = { HarmonixNodeNamespace, TEXT("Delay"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("Delay_DisplayName", "Clock-Synced Delay");
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Delays };
				Info.Description = METASOUND_LOCTEXT("Delay_Description", "Delay effect with optional music clock sync.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FInputs InputReferences
			{
				InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(Inputs::AudioLeftName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(Inputs::AudioRightName, InParams.OperatorSettings),
				{},
				InputData.GetOrCreateDefaultDataReadReference<FEnumTimeSyncOption>(
					Inputs::DelayTimeTypeName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::DelayTimeName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::FeedbackName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::DryLevelName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::WetLevelName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(
					Inputs::WetFilterEnabledName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(
					Inputs::FeedbackFilterEnabledName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FEnumDelayFilterType>(
					Inputs::FilterTypeName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::FilterCutoffName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::FilterQName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(
					Inputs::LFOEnabledName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FEnumTimeSyncOption>(
					Inputs::LFOTimeTypeName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::LFOFrequencyName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::LFODepthName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FEnumDelayStereoType>(
					Inputs::StereoTypeName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::StereoSpreadLeftName,
					InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::StereoSpreadRightName,
					InParams.OperatorSettings)
			};

			// Find the MIDI clock if one is connected
			if (const FAnyDataReference* DataRef = InputData.FindDataReference(Inputs::MidiClockName))
			{
				InputReferences.MidiClock = DataRef->GetDataReadReference<FMidiClock>();
			}

			FOutputs OutputReferences
			{
				FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings),
				FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings)
			};

			return MakeUnique<FDelayOperator>(InParams, MoveTemp(InputReferences), MoveTemp(OutputReferences));
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::AudioLeftName, Inputs.AudioLeft);
			InVertexData.BindReadVertex(Inputs::AudioRightName, Inputs.AudioRight);
			if (Inputs.MidiClock.IsSet())
			{
				InVertexData.BindReadVertex(Inputs::MidiClockName, *Inputs.MidiClock);

				// Set the tempo and speed after we bind here
				const FMidiClockReadRef Clock = *Inputs.MidiClock;
				LastTempo = Clock->GetTempoAtStartOfBlock();
				LastSpeed = Clock->GetSpeedAtStartOfBlock();
				Delay.SetTempo(LastTempo);
				Delay.SetSpeed(LastSpeed);
			}
			InVertexData.BindReadVertex(Inputs::DelayTimeTypeName, Inputs.DelayTimeType);
			InVertexData.BindReadVertex(Inputs::DelayTimeName, Inputs.DelayTime);
			InVertexData.BindReadVertex(Inputs::FeedbackName, Inputs.Feedback);
			InVertexData.BindReadVertex(Inputs::DryLevelName, Inputs.DryLevel);
			InVertexData.BindReadVertex(Inputs::WetLevelName, Inputs.WetLevel);
			InVertexData.BindReadVertex(Inputs::WetFilterEnabledName, Inputs.WetFilterEnabled);
			InVertexData.BindReadVertex(Inputs::FeedbackFilterEnabledName, Inputs.FeedbackFilterEnabled);
			InVertexData.BindReadVertex(Inputs::FilterTypeName, Inputs.FilterType);
			InVertexData.BindReadVertex(Inputs::FilterCutoffName, Inputs.FilterCutoff);
			InVertexData.BindReadVertex(Inputs::FilterQName, Inputs.FilterQ);
			InVertexData.BindReadVertex(Inputs::LFOEnabledName, Inputs.LFOEnabled);
			InVertexData.BindReadVertex(Inputs::LFOTimeTypeName, Inputs.LFOTimeType);
			InVertexData.BindReadVertex(Inputs::LFOFrequencyName, Inputs.LFOFrequency);
			InVertexData.BindReadVertex(Inputs::LFODepthName, Inputs.LFODepth);
			InVertexData.BindReadVertex(Inputs::StereoTypeName, Inputs.StereoType);
			InVertexData.BindReadVertex(Inputs::StereoSpreadLeftName, Inputs.StereoSpreadLeft);
			InVertexData.BindReadVertex(Inputs::StereoSpreadRightName, Inputs.StereoSpreadRight);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::AudioLeftName, Outputs.AudioLeft);
			InVertexData.BindReadVertex(Outputs::AudioRightName, Outputs.AudioRight);
		}
		
		void Reset(const FResetParams& Params)
		{
			Delay.Unprepare();
			Delay.Prepare(Params.OperatorSettings.GetSampleRate(), Constants::NumChannels, Constants::MaxDelayTime);
			LastTempo = -1;
			LastSpeed = -1;
			Outputs.AudioLeft->Zero();
			Outputs.AudioRight->Zero();
		}

		void Execute()
		{
			// set params
			Delay.SetTimeSyncOption(*Inputs.DelayTimeType);
			Delay.SetDelaySeconds(*Inputs.DelayTime);
			Delay.SetFeedbackGain(*Inputs.Feedback);
			Delay.SetDryGain(*Inputs.DryLevel);
			Delay.SetWetGain(*Inputs.WetLevel);
			Delay.SetWetFilterEnabled(*Inputs.WetFilterEnabled);
			Delay.SetFeedbackFilterEnabled(*Inputs.FeedbackFilterEnabled);
			Delay.SetFilterType(*Inputs.FilterType);
			Delay.SetFilterFreq(*Inputs.FilterCutoff);
			Delay.SetFilterQ(*Inputs.FilterQ);
			Delay.SetLfoEnabled(*Inputs.LFOEnabled);
			Delay.SetLfoTimeSyncOption(*Inputs.LFOTimeType);
			Delay.SetLfoFreq(*Inputs.LFOFrequency);
			Delay.SetLfoDepth(*Inputs.LFODepth);
			Delay.SetStereoType(*Inputs.StereoType);
			Delay.SetStereoSpreadLeft(*Inputs.StereoSpreadLeft);
			Delay.SetStereoSpreadRight(*Inputs.StereoSpreadRight);

			const int32 NumSamplesInBlock = Outputs.AudioLeft->Num();
			
			// if there's a clock, update tempo and process in chunks
			if (Inputs.MidiClock.IsSet())
			{
				const FMidiClockReadRef Clock = *Inputs.MidiClock;
				int32 SamplesRendered = 0;
				
				// if the tempo or speed changes, render in chunks
				for (const FMidiClockEvent& ClockEvent : Clock->GetMidiClockEventsInBlock())
				{
					bool ShouldRenderNow = false;
					
					const MidiClockMessageTypes::FTempoChange* AsTempoChange = ClockEvent.TryGet<MidiClockMessageTypes::FTempoChange>();
					const MidiClockMessageTypes::FSpeedChange* AsSpeedChange = ClockEvent.TryGet<MidiClockMessageTypes::FSpeedChange>();

					if (AsTempoChange && AsTempoChange->Tempo != LastTempo)
					{
						LastTempo = AsTempoChange->Tempo;
						ShouldRenderNow = true;
					}

					if (AsSpeedChange && AsSpeedChange->Speed != LastSpeed)
					{
						LastSpeed = AsSpeedChange->Speed;
						ShouldRenderNow = true;
					}

					if (ShouldRenderNow)
					{
						// render the delay
						const int32 SamplesToRender = ClockEvent.BlockFrameIndex - SamplesRendered;
						RenderSubBlock(SamplesRendered, SamplesToRender);
						SamplesRendered += SamplesToRender;

						// set the tempo and speed *after* rendering, because we're rendering up to the change
						Delay.SetTempo(LastTempo);
						Delay.SetSpeed(LastSpeed);
					}
				}

				// if there are left over samples in the block (or there were no tempo changes), render the last bit
				if (SamplesRendered < NumSamplesInBlock)
				{
					// alias the buffers, then process
					const int32 NumSamples = NumSamplesInBlock - SamplesRendered;
					RenderSubBlock(SamplesRendered, NumSamples);
				}
			}
			// otherwise, process in one pass
			else
			{
				RenderSubBlock(0, NumSamplesInBlock);
			}
		}

	private:
		void RenderSubBlock(int32 StartSample, int32 NumSamples)
		{
			check(StartSample >= 0);
			check(NumSamples >= 0);
			
			if (NumSamples == 0)
			{
				return;
			}

			// copy the input audio to the output, then process in place
			FMemory::Memcpy(Outputs.AudioLeft->GetData() + StartSample, Inputs.AudioLeft->GetData() + StartSample, NumSamples * sizeof(float));
			FMemory::Memcpy(Outputs.AudioRight->GetData() + StartSample, Inputs.AudioRight->GetData() + StartSample, NumSamples * sizeof(float));
			BufferView.Reset();
			BufferView.Emplace(Outputs.AudioLeft->GetData() + StartSample, NumSamples);
			BufferView.Emplace(Outputs.AudioRight->GetData() + StartSample, NumSamples);
			Delay.Process(BufferView);
		}
		
		Harmonix::Dsp::Effects::FDelay Delay;
		Audio::FMultichannelBufferView BufferView;
		float LastTempo{ -1 };
		float LastSpeed{ -1 };
		FInputs Inputs;
		FOutputs Outputs;
	};

	using FDelayNode = Metasound::TNodeFacade<FDelayOperator>;
	METASOUND_REGISTER_NODE(FDelayNode);
}

#undef LOCTEXT_NAMESPACE
