// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "HarmonixDsp/AudioUtility.h"
#include "HarmonixDsp/Effects/Settings/BiquadFilterSettings.h"
#include "HarmonixDsp/Effects/BiquadFilter.h"
#include "HarmonixDsp/Effects/Settings/DistortionSettings.h"
#include "HarmonixDsp/Effects/DistortionV2.h"
#include "HarmonixMetasound/Common.h"

#include "HAL/Platform.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_DistortionNode"

namespace HarmonixMetasound::Effects::Distortion
{
	// enumerate num-passes to limit what can be entered in the UI
	enum class EFilterPasses
	{
		k1Pass,
		k2Pass,
		k3Pass
	};

}

namespace Metasound
{
	using namespace HarmonixMetasound::Effects;
	using namespace Harmonix::Dsp::Effects;
	DECLARE_METASOUND_ENUM(EDistortionTypeV2, EDistortionTypeV2::Clean, HARMONIXMETASOUND_API, FEnumDistortionType, FEnumDistortionTypeInfo, FDistortionTypeReadRef, FDistortionTypeWriteRef);
	DECLARE_METASOUND_ENUM(EBiquadFilterType, EBiquadFilterType::LowPass, HARMONIXMETASOUND_API, FEnumHmxBiquadFilterType, FEnumHmxBiquadFilterTypeInfo, FHmxBiquadFilterTypeReadRef, FHmxBiquadFilterTypeWriteRef);
	DECLARE_METASOUND_ENUM(Distortion::EFilterPasses, Distortion::EFilterPasses::k1Pass, HARMONIXMETASOUND_API, FEnumDistortionFilterPasses, FEnumDistortionFilterPassesInfo, FDistortionFilterPassesReadRef, FDistortionFilterPassesWriteRef);

#define DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(NAME, TOOLTIP) DEFINE_METASOUND_ENUM_ENTRY(EDistortionTypeV2::NAME, "DistortionType" #NAME "Description", #NAME, "DistortionType" #NAME "DescriptionToolTip", TOOLTIP)

	DEFINE_METASOUND_ENUM_BEGIN(EDistortionTypeV2, FEnumDistortionType, "DistortionType")
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(Clean, "Clean distortion"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(Warm, "warm distortion"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(Clip, "clipped distortion"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(Soft, "soft distortion"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(Asymmetric, "assymetric distortion"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(Cruncher, "cruncher"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(CaptCrunch, "cpt. crunch"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_TYPE(Rectifier, "rectifier"),
	DEFINE_METASOUND_ENUM_END()

#define DEFINE_METASOUND_ENUM_ENTRY_FILTER_TYPE(NAME, TOOLTIP) DEFINE_METASOUND_ENUM_ENTRY(EBiquadFilterType::NAME, "Harmonix:BiquadFilterType" #NAME "Description", #NAME, "Harmonix:BiquadFilterType" #NAME "DescriptionToolTip", TOOLTIP)

	DEFINE_METASOUND_ENUM_BEGIN(EBiquadFilterType, FEnumHmxBiquadFilterType, "Harmonix:BiquadFilterType")
		DEFINE_METASOUND_ENUM_ENTRY_FILTER_TYPE(LowPass, "Low Pass Filter"),
		DEFINE_METASOUND_ENUM_ENTRY_FILTER_TYPE(HighPass, "High Pass Filter"),
		DEFINE_METASOUND_ENUM_ENTRY_FILTER_TYPE(BandPass, "Band Pass Filter"),
		DEFINE_METASOUND_ENUM_ENTRY_FILTER_TYPE(Peaking, "Peaking Filter"),
		DEFINE_METASOUND_ENUM_ENTRY_FILTER_TYPE(LowShelf, "Low Shelf Filter"),
		DEFINE_METASOUND_ENUM_ENTRY_FILTER_TYPE(HighShelf, "High Shelf Filter"),
	DEFINE_METASOUND_ENUM_END()

#define DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_FILTER_PASSES(NAME, DESCRIPTION) DEFINE_METASOUND_ENUM_ENTRY(Distortion::EFilterPasses::NAME, "Distortion:FilterPassesType" #NAME "Description", DESCRIPTION, "Distortion:FilterPassesType" #NAME "DescriptionToolTip", DESCRIPTION)

	DEFINE_METASOUND_ENUM_BEGIN(Distortion::EFilterPasses, FEnumDistortionFilterPasses, "Distortion:FilterPasses")
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_FILTER_PASSES(k1Pass, "1 Pass"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_FILTER_PASSES(k2Pass, "2 Pass"),
		DEFINE_METASOUND_ENUM_ENTRY_DISTORTION_FILTER_PASSES(k3Pass, "3 Pass"),
	DEFINE_METASOUND_ENUM_END()

}

namespace HarmonixMetasound
{
	using namespace Metasound;
	using namespace Harmonix::Dsp::Effects;

	namespace Effects::Distortion::PinNames
	{
		namespace Inputs
		{
			METASOUND_PARAM(AudioIn, "In", "Audio Input");
			METASOUND_PARAM(InputGain, "Input Gain", "Gain applied to the input signal");
			METASOUND_PARAM(OutputGain, "Output Gain", "Gain applied to the output signal");
			METASOUND_PARAM(WetGain, "Wet Gain", "The wet gain of the Distortion");
			METASOUND_PARAM(DryGain, "Dry Gain", "The Dry gain of the Distortion");
			METASOUND_PARAM(DCAdjust, "DC Adjust", "The DC Adjust of the Distortion");
			METASOUND_PARAM(DistortionType, "Type", "The type of Distortion being used");
			METASOUND_PARAM(Oversample, "Oversample", "Whether to oversample.");

			// per filter input pin names. {0} is used with Format to show the filter index
			METASOUND_PARAM(FilterEnabled, "{0}: Filter Enabled", "Whether this filter is enabled");
			METASOUND_PARAM(FilterType, "{0}: Filter Type", "Filter type to use for this biquad filter");
			METASOUND_PARAM(Freq, "{0}: Freq", "The center frequency for this filter");
			METASOUND_PARAM(Q, "{0}: Q" , "Filter Q, or resonance, controls the steepness of the filter.");
			METASOUND_PARAM(Passes, "{0}: Num Passes", "Number of passes to use for this filter");
			METASOUND_PARAM(PreClip, "{0}: Pre-Clipping", "Whether to enable pre-clipping for the distortion filter");
		}

		namespace Outputs
		{
			METASOUND_PARAM(AudioOut, "Out", "Audio Output");
		}

	}


	class FDistortionOperator final : public TExecutableOperator<FDistortionOperator>
	{
	public:
		static constexpr int32 NumChannels = 1;
		static constexpr int32 NumFilters = FDistortionSettingsV2::kNumFilters;


		struct FFilterSettings
		{
			FFilterSettings(
				FBoolReadRef&& InEnabled,
				FHmxBiquadFilterTypeReadRef&& InFilterType,
				FFloatReadRef&& InFreq,
				FFloatReadRef&& InQ,
				FBoolReadRef&& InPreClip,
				FDistortionFilterPassesReadRef&& InPasses
			)
				: Enabled(InEnabled)
				, FilterType(InFilterType)
				, Freq(InFreq)
				, Q(InQ)
				, PreClip(InPreClip)
				, Passes(InPasses)
			{}

			FBoolReadRef Enabled;
			FHmxBiquadFilterTypeReadRef FilterType;
			FFloatReadRef Freq;
			FFloatReadRef Q;
			FBoolReadRef PreClip;
			FDistortionFilterPassesReadRef Passes;
		};

		FDistortionOperator(
			const FBuildOperatorParams& InParams,
			const FAudioBufferReadRef& InAudio,
			const FFloatReadRef& InInputGain,
			const FFloatReadRef& InOutputGain,
			const FFloatReadRef& InDryGain,
			const FFloatReadRef& InWetGain,
			const FFloatReadRef& InDCAdjust,
			const FDistortionTypeReadRef& InDistortionType,
			const FBoolReadRef& InOversample,
			const FFilterSettings& InFilterSettings1,
			const FFilterSettings& InFilterSettings2,
			const FFilterSettings& InFilterSettings3
		)
			: Distortion(InParams.OperatorSettings.GetSampleRate(), InParams.OperatorSettings.GetNumFramesPerBlock())
			, AudioIn(InAudio)
			, InputGain(InInputGain)
			, OutputGain(InOutputGain)
			, DryGain(InDryGain)
			, WetGain(InWetGain)
			, DCAdjust(InDCAdjust)
			, DistortionType(InDistortionType)
			, Oversample(InOversample)
			, FilterSettings{ InFilterSettings1, InFilterSettings2, InFilterSettings3 }
			, AudioOut(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
		{
			check(AudioOut->Num() == InParams.OperatorSettings.GetNumFramesPerBlock());

			SampleRate = InParams.OperatorSettings.GetSampleRate();
			FramesPerBlock = InParams.OperatorSettings.GetNumFramesPerBlock();
			FDistortionSettingsV2 DistortionSettings;
			GetCurrentSettings(DistortionSettings);
			Distortion.Reset();
			Distortion.Setup(DistortionSettings, InParams.OperatorSettings.GetSampleRate(), InParams.OperatorSettings.GetNumFramesPerBlock(), true);

			Reset(InParams);
		}

		static const FVertexInterface& GetVertexInterface()
		{

#if WITH_EDITOR
#define METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(NAME, INDEX) METASOUND_GET_PARAM_NAME_WITH_INDEX(NAME, INDEX), FDataVertexMetadata { NAME##Tooltip, FText::Format(NAME##DisplayName, INDEX), true }
#else
#define METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(NAME, INDEX) METASOUND_GET_PARAM_NAME_WITH_INDEX(NAME, INDEX), FDataVertexMetadata { FText::GetEmpty(), FText::GetEmpty(), true }
#endif

			auto InitVertexInterface = []() -> FVertexInterface
			{
				using namespace Effects::Distortion::PinNames;
				const FDistortionSettingsV2 DefaultSettings;

				FInputVertexInterface InputInterface;

				InputInterface.Add(TInputDataVertex<Metasound::FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudioIn)));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::InputGain), HarmonixDsp::DBToLinear(DefaultSettings.InputGainDb)));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::OutputGain), HarmonixDsp::DBToLinear(DefaultSettings.OutputGainDb)));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DryGain), DefaultSettings.DryGain));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::WetGain), DefaultSettings.WetGain));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DCAdjust), DefaultSettings.DCAdjust));
				InputInterface.Add(TInputDataVertex<FEnumDistortionType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DistortionType), 0));
				InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Oversample), DefaultSettings.Oversample));

				for (int Index = 0; Index < FDistortionOperator::NumFilters; ++Index)
				{
					InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(Inputs::FilterEnabled, Index), false));
					InputInterface.Add(TInputDataVertex<FEnumHmxBiquadFilterType>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(Inputs::FilterType, Index), 0));
					InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(Inputs::Freq, Index), 630.0f));
					InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(Inputs::Q, Index), 1.0f));
					InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(Inputs::PreClip, Index), false));
					InputInterface.Add(TInputDataVertex<FEnumDistortionFilterPasses>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA_ADVANCED_DISPLAY(Inputs::Passes, Index), 0));
				}

				FOutputVertexInterface OutputInterface;
				OutputInterface.Add(TOutputDataVertex<Metasound::FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::AudioOut)));

				return { InputInterface, OutputInterface };
			};

			static const FVertexInterface Interface = InitVertexInterface();
			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { HarmonixNodeNamespace, TEXT("Distortion"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("Distortion_DisplayName", "Distortion");
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Dynamics };
				Info.Description = METASOUND_LOCTEXT("Distortion_Description", "Distortion audio effect.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace Effects::Distortion::PinNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef InAudio = InputData.GetOrCreateDefaultDataReadReference<Metasound::FAudioBuffer>(
				METASOUND_GET_PARAM_NAME(Inputs::AudioIn), InParams.OperatorSettings);
			FFloatReadRef InInputGain = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::InputGain), InParams.OperatorSettings);
			FFloatReadRef InOutputGain = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::OutputGain), InParams.OperatorSettings);
			FFloatReadRef InWetGain = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::WetGain), InParams.OperatorSettings);
			FFloatReadRef InDryGain = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::DryGain), InParams.OperatorSettings);
			FFloatReadRef InDCAdjust = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::DCAdjust), InParams.OperatorSettings);
			FDistortionTypeReadRef InDistortionType = InputData.GetOrCreateDefaultDataReadReference<FEnumDistortionType>(
				METASOUND_GET_PARAM_NAME(Inputs::DistortionType), InParams.OperatorSettings);
			FBoolReadRef InOversample = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Oversample), InParams.OperatorSettings);

			auto MakeFilterSettings = [&InputData, &InParams](int Index)-> FFilterSettings
			{			
				return FFilterSettings(
					InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::FilterEnabled, Index), InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FEnumHmxBiquadFilterType>(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::FilterType, Index), InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Freq, Index), InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Q, Index), InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::PreClip, Index), InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FEnumDistortionFilterPasses>(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Passes, Index), InParams.OperatorSettings)
				);
			};
				
			FFilterSettings FilterSettings[NumFilters] = {
				MakeFilterSettings(0),
				MakeFilterSettings(1),
				MakeFilterSettings(2)
			};

			return MakeUnique<FDistortionOperator>(
				InParams,
				InAudio,
				InInputGain,
				InOutputGain,
				InDryGain,
				InWetGain,
				InDCAdjust,
				InDistortionType,
				InOversample,
				FilterSettings[0],
				FilterSettings[1],
				FilterSettings[2]);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace Distortion::PinNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::AudioIn), AudioIn);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::InputGain), InputGain);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::OutputGain), OutputGain);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::DryGain), DryGain);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::WetGain), WetGain);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::DCAdjust), DCAdjust);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::DistortionType), DistortionType);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Oversample), Oversample);

			for (int Index = 0; Index < NumFilters; ++Index)
			{
				InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::FilterEnabled, Index), FilterSettings[Index].Enabled);
				InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::FilterType, Index), FilterSettings[Index].FilterType);
				InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Freq, Index), FilterSettings[Index].Freq);
				InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Q, Index), FilterSettings[Index].Q);
				InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::PreClip, Index), FilterSettings[Index].PreClip);
				InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Passes, Index), FilterSettings[Index].Passes);
			}

			InputBufferView.Reset();
			InputBufferView.Emplace(*AudioIn);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace Effects::Distortion::PinNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::AudioOut), AudioOut);

			OutputBufferView.Reset();
			OutputBufferView.Emplace(*AudioOut);
			AudioOut->Zero();
		}

		void Reset(const FResetParams& Params)
		{
			SampleRate = Params.OperatorSettings.GetSampleRate();
			FramesPerBlock = Params.OperatorSettings.GetNumFramesPerBlock();

			FDistortionSettingsV2 DistortionSettings;
			GetCurrentSettings(DistortionSettings);
			Distortion.Reset();
			Distortion.Setup(DistortionSettings, SampleRate, FramesPerBlock, true);

			InputBufferView.Reset();
			InputBufferView.Emplace(*AudioIn);
			OutputBufferView.Reset();
			OutputBufferView.Emplace(*AudioOut);
			AudioOut->Zero();
		}

		void Execute()
		{
			FDistortionSettingsV2 DistortionSettings;
			GetCurrentSettings(DistortionSettings);

			Distortion.SetInputGain(*InputGain);
			Distortion.SetOutputGain(*OutputGain);
			Distortion.SetWetGain(*WetGain);
			Distortion.SetDryGain(*DryGain);
			Distortion.SetDCOffset(*DCAdjust);
			Distortion.SetType(*DistortionType);
			Distortion.SetOversample(*Oversample, FramesPerBlock);

			for (int32 Index = 0; Index < NumFilters; ++Index)
			{
				FDistortionFilterSettings Settings;
				Settings.Filter.Type = *(FilterSettings[Index].FilterType);
				Settings.Filter.Freq = *(FilterSettings[Index].Freq);
				Settings.Filter.Q = *(FilterSettings[Index].Q);
				Settings.FilterPreClip = *(FilterSettings[Index].PreClip);
				Effects::Distortion::EFilterPasses FilterPasses = *(FilterSettings[Index].Passes);
				Settings.NumPasses = static_cast<int32>(FilterPasses);
				Settings.Filter.IsEnabled = *(FilterSettings[Index].Enabled);

				Distortion.SetupFilter(Index, Settings);
			}

			Distortion.Process(InputBufferView, OutputBufferView);
		}

	private:

		// INPUT
		FDistortionV2 Distortion;
		FAudioBufferReadRef AudioIn;
		FFloatReadRef InputGain;
		FFloatReadRef OutputGain;
		FFloatReadRef DryGain;
		FFloatReadRef WetGain;
		FFloatReadRef DCAdjust;
		FDistortionTypeReadRef DistortionType;
		FBoolReadRef Oversample;
		FFilterSettings FilterSettings[NumFilters];

		// OUTPUT
		FAudioBufferWriteRef AudioOut;

		// DATA
		TArray<TArrayView<const float>> InputBufferView;
		TArray<TArrayView<float>> OutputBufferView;
		int32 FramesPerBlock = 0;
		int32 SampleRate = 0;

		void GetCurrentSettings(FDistortionSettingsV2& OutSettings)
		{
			OutSettings.InputGainDb = HarmonixDsp::LinearToDB(*InputGain);
			OutSettings.OutputGainDb = HarmonixDsp::LinearToDB(*OutputGain);
			OutSettings.WetGain = *WetGain;
			OutSettings.DryGain = *DryGain;
			OutSettings.DCAdjust = *DCAdjust;
			OutSettings.Type = *DistortionType;
			OutSettings.Oversample = *Oversample;

			for (int32 Index = 0; Index < NumFilters; ++Index)
			{
				OutSettings.Filters[Index].Filter.Type = *(FilterSettings[Index].FilterType);
				OutSettings.Filters[Index].Filter.Freq = *(FilterSettings[Index].Freq);
				OutSettings.Filters[Index].Filter.Q = *(FilterSettings[Index].Q);
				OutSettings.Filters[Index].FilterPreClip = *(FilterSettings[Index].PreClip);
				Effects::Distortion::EFilterPasses FilterPasses = *(FilterSettings[Index].Passes);
				OutSettings.Filters[Index].NumPasses = static_cast<int32>(FilterPasses);
				OutSettings.Filters[Index].Filter.IsEnabled = *(FilterSettings[Index].Enabled);
			}
		}
	};

	using FDistortionNode = Metasound::TNodeFacade<FDistortionOperator>;
	METASOUND_REGISTER_NODE(FDistortionNode);
}

#undef LOCTEXT_NAMESPACE
