// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/DjFilterNode.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixDsp/Effects/DjFilter.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::DjFilter
{
	const Metasound::FNodeClassName& GetClassName()
	{
		static Metasound::FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
			"DjFilter",
			""
		};
		return ClassName;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(AudioMono, CommonPinNames::Inputs::AudioMono);
		DEFINE_INPUT_METASOUND_PARAM(Amount, "Amount", "The knob position. -1 to DeadZoneSize will low-pass the signal. DeadZoneSize to 1 will high-pass the signal. Within the dead zone will fade to dry.");
		DEFINE_INPUT_METASOUND_PARAM(Resonance, "Resonance", "The filter resonance");
		DEFINE_INPUT_METASOUND_PARAM(LowPassMinFrequency, "Low-Pass Min Frequency", "The frequency the low-pass will be at when Amount is set to -1");
		DEFINE_INPUT_METASOUND_PARAM(LowPassMaxFrequency, "Low-Pass Max Frequency", "The frequency the low-pass will be at when Amount is set to -DeadZoneSize");
		DEFINE_INPUT_METASOUND_PARAM(HighPassMinFrequency, "High-Pass Min Frequency", "The frequency the high-pass will be at when Amount is set to DeadZoneSize");
		DEFINE_INPUT_METASOUND_PARAM(HighPassMaxFrequency, "High-Pass Max Frequency", "The frequency the high-pass will be at when Amount is set to 1");
		DEFINE_INPUT_METASOUND_PARAM(DeadZoneSize, "Dead Zone Size", "The portion of the Amount that will cross-fade between the filtered signal and the dry signal");
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(AudioMono, CommonPinNames::Outputs::AudioMono);
	}

	class FOp final : public Metasound::TExecutableOperator<FOp>
	{
	public:
		static const Metasound::FVertexInterface& GetVertexInterface()
		{
			auto InitVertexInterface = []() -> Metasound::FVertexInterface
			{
				using namespace Metasound;
				const Harmonix::Dsp::Effects::FDjFilter FilterForDefaults{ 0 };
				
				return {
					FInputVertexInterface
					{
						TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudioMono)),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Amount), FilterForDefaults.Amount.Default),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Resonance), FilterForDefaults.Resonance.Default),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::LowPassMinFrequency), FilterForDefaults.LowPassMinFrequency.Default),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::LowPassMaxFrequency), FilterForDefaults.LowPassMaxFrequency.Default),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::HighPassMinFrequency), FilterForDefaults.HighPassMinFrequency.Default),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::HighPassMaxFrequency), FilterForDefaults.HighPassMaxFrequency.Default),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DeadZoneSize), FilterForDefaults.DeadZoneSize.Default),
					},
					Metasound::FOutputVertexInterface
					{
						TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::AudioMono))
					}
				};
			};
			
			static const Metasound::FVertexInterface Interface = InitVertexInterface();
			return Interface;
		}
		
		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> Metasound::FNodeClassMetadata
			{
				Metasound::FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("DjFilter_DisplayName", "DJ Filter");
				Info.Description = METASOUND_LOCTEXT("DjFilter_Description", "A filter that cross-fades between a low-pass filter, the dry signal, and a high-pass filter on one knob.");
				Info.Author = Metasound::PluginAuthor;
				Info.PromptIfMissing = Metasound::PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, Metasound::NodeCategories::Filters };
				return Info;
			};

			static const Metasound::FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		struct FInputs
		{
			Metasound::FAudioBufferReadRef Audio;
			Metasound::FFloatReadRef Amount;
			Metasound::FFloatReadRef Resonance;
			Metasound::FFloatReadRef LowPassMinFreq;
			Metasound::FFloatReadRef LowPassMaxFreq;
			Metasound::FFloatReadRef HighPassMinFreq;
			Metasound::FFloatReadRef HighPassMaxFreq;
			Metasound::FFloatReadRef DeadZoneSize;
		};

		struct FOutputs
		{
			Metasound::FAudioBufferWriteRef Audio;
		};

		static TUniquePtr<IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			const Metasound::FOperatorSettings& OperatorSettings = InParams.OperatorSettings;
			const Metasound::FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<Metasound::FAudioBuffer>(
					Inputs::AudioMonoName,
					OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::AmountName,
					OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::ResonanceName,
					OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::LowPassMinFrequencyName,
					OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::LowPassMaxFrequencyName,
					OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::HighPassMinFrequencyName,
					OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::HighPassMaxFrequencyName,
					OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					Inputs::DeadZoneSizeName,
					OperatorSettings)
			};

			FOutputs Outputs
			{
				Metasound::FAudioBufferWriteRef::CreateNew(OperatorSettings)
			};

			return MakeUnique<FOp>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FOp(const Metasound::FBuildOperatorParams& Params, FInputs&& InInputs, FOutputs&& InOutputs)
		: Inputs(MoveTemp(InInputs))
		, Outputs(MoveTemp(InOutputs))
		, Filter(Params.OperatorSettings.GetSampleRate())
		{
			Reset(Params);
		}

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::AudioMonoName, Inputs.Audio);
			InVertexData.BindReadVertex(Inputs::AmountName, Inputs.Amount);
			InVertexData.BindReadVertex(Inputs::ResonanceName, Inputs.Resonance);
			InVertexData.BindReadVertex(Inputs::LowPassMinFrequencyName, Inputs.LowPassMinFreq);
			InVertexData.BindReadVertex(Inputs::LowPassMaxFrequencyName, Inputs.LowPassMaxFreq);
			InVertexData.BindReadVertex(Inputs::HighPassMinFrequencyName, Inputs.HighPassMinFreq);
			InVertexData.BindReadVertex(Inputs::HighPassMaxFrequencyName, Inputs.HighPassMaxFreq);
			InVertexData.BindReadVertex(Inputs::DeadZoneSizeName, Inputs.DeadZoneSize);
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::AudioMonoName, Outputs.Audio);
		}

		void Reset(const FResetParams& ResetParams)
		{
			Outputs.Audio->Zero();
			Filter.Reset(ResetParams.OperatorSettings.GetSampleRate());
		}

		void Execute()
		{
			Filter.Amount = *Inputs.Amount;
			Filter.Resonance = *Inputs.Resonance;
			Filter.LowPassMinFrequency = *Inputs.LowPassMinFreq;
			Filter.LowPassMaxFrequency = *Inputs.LowPassMaxFreq;
			Filter.HighPassMinFrequency = *Inputs.HighPassMinFreq;
			Filter.HighPassMaxFrequency = *Inputs.HighPassMaxFreq;
			Filter.DeadZoneSize = *Inputs.DeadZoneSize;
			Filter.Process(*Inputs.Audio, *Outputs.Audio);
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;
		Harmonix::Dsp::Effects::FDjFilter Filter;
	};

	using FDjFilterNode = Metasound::TNodeFacade<FOp>;
	METASOUND_REGISTER_NODE(FDjFilterNode);
}

#undef LOCTEXT_NAMESPACE
