// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "HarmonixDsp/AudioAnalysis/FFTAnalyzer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/FFTAnalyzerResult.h"

#include "HAL/Platform.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace Metasound
{
	using namespace Harmonix::Dsp::AudioAnalysis;

}

namespace HarmonixMetasound
{
	using namespace Metasound;
	using namespace Harmonix::Dsp;

	namespace AudioAnalysis::FFTAnalyzer::PinNames
	{
		namespace Inputs
		{
			DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
			METASOUND_PARAM(Audio, "In", "Audio Input");
			METASOUND_PARAM(FFTSize, "FFT Size", "The size of the FFT window, which determines the frequency resolution of the resulting spectrum analysis (a larger size will result in higher frequency resolution).");
			METASOUND_PARAM(MinFrequencyHz, "Min Frequency Hz", "The minimum frequency that will be included in the spectrum analysis.");
			METASOUND_PARAM(MaxFrequencyHz, "Max Frequency Hz", "The maximum frequency that will be included in the spectrum analysis.");
			METASOUND_PARAM(MelScaleBinning, "Mel Scale Binning", " If true, the resulting spectrum analysis will be arranged in bins that more closely represent aural perception of pitch.");
			METASOUND_PARAM(NumResultBins, "Num Result Bins", "The number of frequency ranges to include in the resulting spectrum analysis. Must be less than or equal to FFTSize / 2, will be FFTSize / 2 if greater.");
			METASOUND_PARAM(RiseMs, "Rise Ms", "The amount of time over which rising energy values will be smoothed");
			METASOUND_PARAM(FallMs, "Fall Ms", "The amount of time over which falling energy values will be smoothed");
		}

		namespace Outputs
		{
			METASOUND_PARAM(Results, "Results", "The results of the FFT Analysis");
		}
	}

	class FFFTAnalyzerOperator final : public TExecutableOperator<FFFTAnalyzerOperator>
	{
	public:
		static constexpr int32 NumChannels = 1;

		struct FInputs
		{
			FBoolReadRef Enable;
			FAudioBufferReadRef Audio;
			FInt32ReadRef FFTSize;
			FFloatReadRef MinFrequencyHz;
			FFloatReadRef MaxFrequencyHz;
			FBoolReadRef MelScaleBinning;
			FInt32ReadRef NumResultBins;
			FFloatReadRef RiseMs;
			FFloatReadRef FallMs;
		};

		FFFTAnalyzerOperator(const FBuildOperatorParams& CreateOperatorParams, FInputs&& Inputs)
			: Inputs(MoveTemp(Inputs))
			, FFTAnalyzer(CreateOperatorParams.OperatorSettings.GetSampleRate())
			, ResultsOut(FHarmonixFFTAnalyzerResultsWriteRef::CreateNew())
		{
			Reset(CreateOperatorParams);
		}

		static const FVertexInterface& GetVertexInterface()
		{
			auto InitVertexInterface = []() -> FVertexInterface
			{
				using namespace AudioAnalysis::FFTAnalyzer::PinNames;
				const FHarmonixFFTAnalyzerSettings DefaultSettings;

				FInputVertexInterface InputInterface;

				InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true));
				InputInterface.Add(TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Audio)));
				InputInterface.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::FFTSize), DefaultSettings.FFTSize));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MinFrequencyHz), DefaultSettings.MinFrequencyHz));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MaxFrequencyHz), DefaultSettings.MaxFrequencyHz));
				InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MelScaleBinning), DefaultSettings.MelScaleBinning));
				InputInterface.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::NumResultBins), DefaultSettings.NumResultBins));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::RiseMs), DefaultSettings.OutputSettings.RiseMs));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::FallMs), DefaultSettings.OutputSettings.FallMs));

				FOutputVertexInterface OutputInterface;
				OutputInterface.Add(TOutputDataVertex<FHarmonixFFTAnalyzerResults>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Results)));

				return { InputInterface, OutputInterface };
			};

			static const FVertexInterface Interface = InitVertexInterface();
			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { HarmonixNodeNamespace, TEXT("FFT Analyzer"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("FFTAnalyzer_DisplayName", "FFT Analyzer");
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, MetasoundNodeCategories::Analysis };
				Info.Description = METASOUND_LOCTEXT("FFTAnalyzer_Description", "Fast Fourier Transform Analysis.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace AudioAnalysis::FFTAnalyzer::PinNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(
					Inputs::EnableName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(
					METASOUND_GET_PARAM_NAME(Inputs::Audio), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(
					METASOUND_GET_PARAM_NAME(Inputs::FFTSize), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					METASOUND_GET_PARAM_NAME(Inputs::MinFrequencyHz), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					METASOUND_GET_PARAM_NAME(Inputs::MaxFrequencyHz), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(
					METASOUND_GET_PARAM_NAME(Inputs::MelScaleBinning), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(
					METASOUND_GET_PARAM_NAME(Inputs::NumResultBins), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					METASOUND_GET_PARAM_NAME(Inputs::RiseMs), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(
					METASOUND_GET_PARAM_NAME(Inputs::FallMs), InParams.OperatorSettings)
			};

			return MakeUnique<FFFTAnalyzerOperator>(InParams, MoveTemp(Inputs));
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace AudioAnalysis::FFTAnalyzer::PinNames;
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enable);
			InVertexData.BindReadVertex(Inputs::AudioName, Inputs.Audio);
			InVertexData.BindReadVertex(Inputs::FFTSizeName, Inputs.FFTSize);
			InVertexData.BindReadVertex(Inputs::MinFrequencyHzName, Inputs.MinFrequencyHz);
			InVertexData.BindReadVertex(Inputs::MaxFrequencyHzName, Inputs.MaxFrequencyHz);
			InVertexData.BindReadVertex(Inputs::MelScaleBinningName, Inputs.MelScaleBinning);
			InVertexData.BindReadVertex(Inputs::NumResultBinsName, Inputs.NumResultBins);
			InVertexData.BindReadVertex(Inputs::RiseMsName, Inputs.RiseMs);
			InVertexData.BindReadVertex(Inputs::FallMsName, Inputs.FallMs);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace AudioAnalysis::FFTAnalyzer::PinNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::Results), ResultsOut);
		}

		void Reset(const FResetParams& Params)
		{
			SampleRate = Params.OperatorSettings.GetSampleRate();
			FramesPerBlock = Params.OperatorSettings.GetNumFramesPerBlock();
			
			FFTAnalyzer.Reset();

			ResultsOut->Spectrum.Reset();
		}

		void Execute()
		{
			if (!*Inputs.Enable)
			{
				// Reset the results if there are some
				if (ResultsOut->Spectrum.Num() > 0)
				{
					ResultsOut->Spectrum.Reset();
				}
				
				return;
			}
			
			RefreshParams();
			FFTAnalyzer.Process(*Inputs.Audio, *ResultsOut);
		}

	private:
		void RefreshParams()
		{
			bool SomethingChanged = false;

			if (FFTSettings.FFTSize != *Inputs.FFTSize)
			{
				FFTSettings.FFTSize = *Inputs.FFTSize;
				SomethingChanged = true;
			}
			if (FFTSettings.MinFrequencyHz != *Inputs.MinFrequencyHz)
			{
				FFTSettings.MinFrequencyHz = *Inputs.MinFrequencyHz;
				SomethingChanged = true;
			}
			if (FFTSettings.MaxFrequencyHz != *Inputs.MaxFrequencyHz)
			{
				FFTSettings.MaxFrequencyHz = *Inputs.MaxFrequencyHz;
				SomethingChanged = true;
			}
			if (FFTSettings.MelScaleBinning != *Inputs.MelScaleBinning)
			{
				FFTSettings.MelScaleBinning = *Inputs.MelScaleBinning;
				SomethingChanged = true;
			}
			if (FFTSettings.NumResultBins != *Inputs.NumResultBins)
			{
				FFTSettings.NumResultBins = *Inputs.NumResultBins;
				SomethingChanged = true;
			}
			if (FFTSettings.OutputSettings.RiseMs != *Inputs.RiseMs)
			{
				FFTSettings.OutputSettings.RiseMs = *Inputs.RiseMs;
				SomethingChanged = true;
			}
			if (FFTSettings.OutputSettings.FallMs != *Inputs.FallMs)
			{
				FFTSettings.OutputSettings.FallMs = *Inputs.FallMs;
				SomethingChanged = true;
			}

			if (SomethingChanged)
			{
				FFTAnalyzer.SetSettings(FFTSettings);
			}
		}
		
		FInputs Inputs;
		FHarmonixFFTAnalyzerSettings FFTSettings;
		FFFTAnalyzer FFTAnalyzer;

		// OUTPUT
		FHarmonixFFTAnalyzerResultsWriteRef ResultsOut;

		// DATA
		int32 FramesPerBlock = 0;
		int32 SampleRate = 0;
	};

	using FFFTAnalyzerNode = Metasound::TNodeFacade<FFFTAnalyzerOperator>;
	METASOUND_REGISTER_NODE(FFFTAnalyzerNode);
}

#undef LOCTEXT_NAMESPACE
