// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/FusionSyncLink.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/FusionPatchRenderableAsset.h"

#include "Harmonix/AudioRenderableProxy.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "HarmonixDsp/FusionSampler/FusionSampler.h"
#include "HarmonixDsp/FusionSampler/FusionVoicePool.h"
#include "HarmonixDsp/AudioUtility.h"

#include "HarmonixMetasound/MidiOps/StuckNoteGuard.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

DEFINE_LOG_CATEGORY_STATIC(LogFusionSamplerPlayer, Log, All);

namespace HarmonixMetasound
{
	using namespace Metasound;

	namespace FusionSamplerNodePinNames
	{
		METASOUND_PARAM(MidiTrackNumber, "Track Number", "Track number (1 based). 0 will play all tracks (track-omni).")
		METASOUND_PARAM(Lfo0DepthOverride, "LFO 0 Depth Override", "Overrides the patch's lfo0 depth setting if connected AND if the value is >= 0.")
		METASOUND_PARAM(Lfo0FrequencyOverride, "LFO 0 Frequency Override", "Overrides the patch's lfo0 frequency setting if connected AND if the vaue is > 0.")
		METASOUND_PARAM(Lfo1DepthOverride, "LFO 1 Depth Override", "Overrides the patch's lfo1 depth setting if connected AND if the value is >= 0..")
		METASOUND_PARAM(Lfo1FrequencyOverride, "LFO 1 Frequency Override", "Overrides the patch's lfo1 frequency setting if connected AND if the vaue is > 0.")
		METASOUND_PARAM(FineTuneCents, "Fine Tune Cents", "Adds to the patch's fine tune cents setting if connected AND if the vaue is > -1200 and < 1200.")
		METASOUND_PARAM(EnableMTFusion, "Multithreaded Rendering", "Turn on to allow Fusion rendering to be done across multiple tasks. NOTE: YOU MUST then connect this node's "
		                                 "Audio Output AND Render Sync Output to a Fusion Synchronizer Node, and reference THAT NODE'S audio outputs elsewhere in the graph!")
		METASOUND_PARAM(VoicePoolName, "Voice Pool Name", "The name of a specific voice pool for this node to use. Uses the default pool if the named pool does not exist, or if the name is \"None\"")
		METASOUND_PARAM(UsePitchShifters, "Use Pitch Shifters", "Set whether to use pitch shifters. If Disabled, pitch shifting will be disabled, and voices that are set to \"Maintain Time\" won't play.")
		METASOUND_PARAM(RenderSync, "Render Sync", "YOU MUST connect this AND this node's audio output to a Fusion Synchronizer Node if you have enabled Multithreaded Rendering for this node!")
	}

	bool bDisableMultithreadedRender = false;
	FAutoConsoleVariableRef CVarMetaSoundProfileAllEnabled(
		TEXT("au.Fusion.DisableThreadedRender"),
		bDisableMultithreadedRender,
		TEXT("Disable Multithreaded Rendering of Fusion Sampler Nodes."),
		ECVF_Default);

	/**********************************************************************************************************
	***********************************************************************************************************
	* FFusionSamplerOperatorBase
	***********************************************************************************************************
	*********************************************************************************************************/
	class FFusionSamplerOperatorBase
		: public TExecutableOperator<FFusionSamplerOperatorBase>, protected FFusionSampler
	{
	public:
		struct FConstructionArgs
		{
			const FBuildOperatorParams Params;
			const FOperatorSettings* InSettings;
			FBoolReadRef             InEnabled;
			FMidiStreamReadRef       InMidiStream;
			FInt32ReadRef            InTrackNumber;
			FInt32ReadRef            InTransposition;
			FFloatReadRef            InLfo0RateOverride;
			FFloatReadRef            InLfo0DepthOverride;
			FFloatReadRef            InLfo1RateOverride;
			FFloatReadRef            InLfo1DepthOverride;
			FFloatReadRef            InFineTuneCents;
			FFusionPatchAssetReadRef InPatch;
			FBoolReadRef             InClockSpeedToPitch;
			bool		             InMTRenderingEnable;
			FString					 InVoicePoolName;
			bool                     InUsePitchShifters;
			EAudioBufferChannelLayout InOutputChannelLayout;

			FConstructionArgs(const FBuildOperatorParams& InParams, EAudioBufferChannelLayout OutputChannelLayout)
				: Params(InParams)
				, InSettings(& InParams.OperatorSettings)
				, InEnabled(InParams.InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(CommonPinNames::Inputs::Enable), InParams.OperatorSettings))
				, InMidiStream(InParams.InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(CommonPinNames::Inputs::MidiStream), InParams.OperatorSettings))
				, InTrackNumber(InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::MidiTrackNumber), InParams.OperatorSettings))
				, InTransposition(InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(CommonPinNames::Inputs::Transposition), InParams.OperatorSettings))
				, InLfo0RateOverride(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo0FrequencyOverride), InParams.OperatorSettings))
				, InLfo0DepthOverride(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo0DepthOverride), InParams.OperatorSettings))
				, InLfo1RateOverride(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo1FrequencyOverride), InParams.OperatorSettings))
				, InLfo1DepthOverride(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo1DepthOverride), InParams.OperatorSettings))
				, InFineTuneCents(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::FineTuneCents), InParams.OperatorSettings))
				, InPatch(InParams.InputData.GetOrCreateDefaultDataReadReference<FFusionPatchAsset>(METASOUND_GET_PARAM_NAME(CommonPinNames::Inputs::SynthPatch), InParams.OperatorSettings))
				, InClockSpeedToPitch(InParams.InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(CommonPinNames::Inputs::ClockSpeedToPitch), InParams.OperatorSettings))
				, InMTRenderingEnable(InParams.InputData.GetOrCreateDefaultValue<bool>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::EnableMTFusion), InParams.OperatorSettings))
				, InVoicePoolName(InParams.InputData.GetOrCreateDefaultValue<FString>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::VoicePoolName), InParams.OperatorSettings))
			    , InUsePitchShifters(InParams.InputData.GetOrCreateDefaultValue<bool>(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::UsePitchShifters), InParams.OperatorSettings))
				, InOutputChannelLayout(OutputChannelLayout)
			{
			}
		};

		static const FInputVertexInterface& GetInputVertexInterface()
		{
			using namespace CommonPinNames;

			static const FInputVertexInterface InputInterface = FInputVertexInterface(
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
				TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream)),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(FusionSamplerNodePinNames::MidiTrackNumber), 1),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Transposition), 0),
				TInputDataVertex<FFusionPatchAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::SynthPatch)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(FusionSamplerNodePinNames::Lfo0FrequencyOverride), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(FusionSamplerNodePinNames::Lfo0DepthOverride), -1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(FusionSamplerNodePinNames::Lfo1FrequencyOverride), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(FusionSamplerNodePinNames::Lfo1DepthOverride), -1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(FusionSamplerNodePinNames::FineTuneCents), -1200.0f),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::ClockSpeedToPitch), true),
				TInputConstructorVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(FusionSamplerNodePinNames::EnableMTFusion), false),
				TInputConstructorVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(FusionSamplerNodePinNames::VoicePoolName), FString(TEXT(""))),
				TInputConstructorVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(FusionSamplerNodePinNames::UsePitchShifters), false)
			);

			return InputInterface;
		}

		FFusionSamplerOperatorBase(const FConstructionArgs& Args)
			: FFusionSampler(Args.Params.OperatorSettings.GetSampleRate(), Args.InUsePitchShifters)
		    , SyncLinkOutPin(FFusionSyncLinkWriteRef::CreateNew())
			, EnableInPin(Args.InEnabled)
			, MidiStreamInPin(Args.InMidiStream)
			, TrackNumberInPin(Args.InTrackNumber)
			, TranspositionInPin(Args.InTransposition)
			, Lfo0RateInPin(Args.InLfo0RateOverride)
			, Lfo0DepthInPin(Args.InLfo0DepthOverride)
			, Lfo1RateInPin(Args.InLfo1RateOverride)
			, Lfo1DepthInPin(Args.InLfo1DepthOverride)
			, FineTuneCentsInPin(Args.InFineTuneCents)
			, PatchInPin(Args.InPatch)
			, ClockSpeedAffectsPitchInPin(Args.InClockSpeedToPitch)
			, EnableMTRenderingInPin(Args.InMTRenderingEnable)
			, VoicePoolName(Args.InVoicePoolName)
			, UsePitchShifters(Args.InUsePitchShifters)
			, CostReporter(Args.Params.GraphRenderCost ? Args.Params.GraphRenderCost->AddNode(Args.Params.Node.GetInstanceID(), Args.Params.Environment) : FNodeRenderCost{})
			, OutputChannelLayout(Args.InOutputChannelLayout)
		{
			Reset(Args.Params);
			Init();
		}

		virtual ~FFusionSamplerOperatorBase()
		{
			SyncLinkOutPin->Reset();
		}

		void Init()
		{
			SetTicksPerQuarterNote(MidiStreamInPin->GetTicksPerQuarterNote());
			SetPatchAndOverrides(PatchInPin->GetRenderable());
			SetRawTransposition(*TranspositionInPin);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace CommonPinNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable), EnableInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), MidiStreamInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::MidiTrackNumber), TrackNumberInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Transposition), TranspositionInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo0FrequencyOverride), Lfo0RateInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo0DepthOverride), Lfo0DepthInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo1FrequencyOverride), Lfo1RateInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::Lfo1DepthOverride), Lfo1DepthInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::FineTuneCents), FineTuneCentsInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::SynthPatch), PatchInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::ClockSpeedToPitch), ClockSpeedAffectsPitchInPin);
			InVertexData.SetValue(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::EnableMTFusion), EnableMTRenderingInPin);
			InVertexData.SetValue(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::VoicePoolName), VoicePoolName);
			InVertexData.SetValue(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::UsePitchShifters), UsePitchShifters);
			Init();
		}

		void Reset(const FResetParams& Params)
		{
			SyncLinkOutPin->Reset();
			
			Prepare(Params.OperatorSettings.GetSampleRate(), OutputChannelLayout, AudioRendering::kFramesPerRenderBuffer, true);
			BusBuffer.SetNumValidFrames(0);
			ResidualBuffer.SetNumValidFrames(0);
			FramesPerBlock = Params.OperatorSettings.GetNumFramesPerBlock();
			ResetInstrumentState();
			SetSampleRate(Params.OperatorSettings.GetSampleRate());
			
			SetVoicePool(FFusionVoicePool::GetNamedPool(FName(VoicePoolName), Params.OperatorSettings.GetSampleRate()), false);
		}

		void Execute();
		void DoRender();
		void PostExecute();

	protected:
		virtual void ResetInternal(const FResetParams& Params) = 0;
		virtual void ZeroOutput() = 0;
		virtual int32 CopyFramesToOutput(int32 OutputOffset, const TAudioBuffer<float>& InputBuffer, int32 NumFrames) = 0;
		virtual void SetupOutputAlias(TAudioBuffer<float>& AliasBuffer, int32 Offset) = 0;

		//** OUTPUTS
		FFusionSyncLinkWriteRef SyncLinkOutPin;

		//** DATA
		TAudioBuffer<float> ResidualBuffer;

	private:

		//** INPUTS
		FBoolReadRef EnableInPin;
		FMidiStreamReadRef MidiStreamInPin;
		FInt32ReadRef TrackNumberInPin;
		FInt32ReadRef TranspositionInPin;
		FFloatReadRef Lfo0RateInPin;
		FFloatReadRef Lfo0DepthInPin;
		FFloatReadRef Lfo1RateInPin;
		FFloatReadRef Lfo1DepthInPin;
		FFloatReadRef FineTuneCentsInPin;
		FFusionPatchAssetReadRef PatchInPin;
		FBoolReadRef ClockSpeedAffectsPitchInPin;
		bool EnableMTRenderingInPin;
		FString VoicePoolName;
		const bool UsePitchShifters;

		//** DATA
		FNodeRenderCost CostReporter;
		int32 FramesPerBlock = 0;
		int32 CurrentTrackNumber = 0;
		bool MadeAudioLastFrame = false;
		FFusionPatchDataProxy::NodePtr FusionPatchDataPtr = nullptr;
		int32 SliceIndex = 0;
		EAudioBufferChannelLayout OutputChannelLayout = EAudioBufferChannelLayout::Mono;
		Harmonix::Midi::Ops::FStuckNoteGuard StuckNoteGuard;

		TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> MidiClock;

		void SetPatchAndOverrides(FFusionPatchDataProxy::NodePtr NewPatch)
		{
			KillAllVoices();
			ResetInstrumentState();
			ResetPatchRelatedState();
			
			SetPatch(NewPatch);
			FusionPatchDataPtr = NewPatch;

			UpdatePatchOverrides();
		}

		void UpdatePatchOverrides()
		{
			static_assert(FFusionPatchSettings::kNumLfos == 2);

			using namespace Harmonix::Midi::Constants;
			
			if (*Lfo0RateInPin > 0.0f)
			{
				SetController(EControllerID::LFO0Frequency, *Lfo0RateInPin);
			}
			else if (FusionPatchDataPtr)
			{
				SetController(EControllerID::LFO0Frequency, FusionPatchDataPtr->GetSettings().Lfo[0].Freq);
			}
			if (*Lfo0DepthInPin >= 0.0f)
			{
				SetController(EControllerID::LFO0Depth, *Lfo0DepthInPin);
			}
			else if (FusionPatchDataPtr)
			{
				SetController(EControllerID::LFO0Depth, FusionPatchDataPtr->GetSettings().Lfo[0].Depth);
			}
			if (*Lfo1RateInPin > 0.0f)
			{
				SetController(EControllerID::LFO1Frequency, *Lfo1RateInPin);
			}
			else if (FusionPatchDataPtr)
			{
				SetController(EControllerID::LFO1Frequency, FusionPatchDataPtr->GetSettings().Lfo[1].Freq);
			}
			if (*Lfo1DepthInPin >= 0.0f)
			{
				SetController(EControllerID::LFO1Depth, *Lfo1DepthInPin);
			}
			else if (FusionPatchDataPtr)
			{
				SetController(EControllerID::LFO1Depth, FusionPatchDataPtr->GetSettings().Lfo[1].Depth);
			}

			float FineTuneCentsTotal = 0;
			if (*FineTuneCentsInPin > -1200.0f && *FineTuneCentsInPin < 1200.0f)
			{
				FineTuneCentsTotal += *FineTuneCentsInPin;
			}
			if (FusionPatchDataPtr)
			{
				FineTuneCentsTotal += FusionPatchDataPtr->GetSettings().FineTuneCents;
			}
			SetFineTuneCents(FineTuneCentsTotal);
		}

		void CheckForUpdatedFusionPatchData()
		{
			using namespace Harmonix;
			// First, if patch data has changed at the pin level. Update it!
			FFusionPatchDataProxy::NodePtr Tester = PatchInPin->GetRenderable();
			if (Tester != FusionPatchDataPtr)
			{
				SetPatchAndOverrides(Tester);
				return;
			}

			// Second, if there are queued changes get those.
			if (FusionPatchDataPtr)
			{
				// implicit cast from NodePtr to QueueType::NodeType
				// an alias: FFusionPatchDataProxy::QueueType::NodeType*
				// only a slightly shorter name
				//FFusionPatchDataProxy::QueueType::NodeType*
				TRefCountedAudioRenderableWithQueuedChanges<FFusionPatchData>* PatchData = FusionPatchDataPtr;
				if (PatchData->HasUpdate())
				{
					UE_LOGF(LogFusionSamplerPlayer, Log, "Has update, resetting fusion patch data.");
					SetPatchAndOverrides(PatchData->GetUpdate());
				}
			}
		}
	};

	/**********************************************************************************************************
	***********************************************************************************************************
	* TEMPLATE FFusionSamplerOperator
	***********************************************************************************************************
	*********************************************************************************************************/
	template<int32 NUM_CHANNELS>
	class FFusionSamplerOperator : public FFusionSamplerOperatorBase
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { HarmonixNodeNamespace, GetMetasoundClassName(), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = GetDisplayName();
				Info.Description = METASOUND_LOCTEXT("FusionSamplerNode_Description", "Renders incoming MIDI stream using the Fusion Sampler configured with the specified patch.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Generators };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static const FVertexInterface& GetVertexInterface();

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			EAudioBufferChannelLayout OutputChannelLayout = EAudioBufferChannelLayout::Raw;
			if constexpr (NUM_CHANNELS == 1)
			{
				OutputChannelLayout = EAudioBufferChannelLayout::Mono;
			}
			if constexpr (NUM_CHANNELS == 2)
			{
				OutputChannelLayout = EAudioBufferChannelLayout::Stereo;
			}
			check (OutputChannelLayout != EAudioBufferChannelLayout::Raw);

			const TNodeFacade<FFusionSamplerOperator<NUM_CHANNELS>>& PlayerNode = static_cast<const TNodeFacade<FFusionSamplerOperator<NUM_CHANNELS>>&>(InParams.Node);
			FFusionSamplerOperatorBase::FConstructionArgs ConstructionArgs(InParams, OutputChannelLayout);
			return MakeUnique<FFusionSamplerOperator<NUM_CHANNELS>>(ConstructionArgs);
		}

		FFusionSamplerOperator(const FFusionSamplerOperatorBase::FConstructionArgs& Args)
			: FFusionSamplerOperatorBase(Args)
		{
			for (int32 i = 0; i < NUM_CHANNELS; ++i)
			{
				AudioOuts.Add(FAudioBufferWriteRef::CreateNew(*Args.InSettings));
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

	protected:
		virtual void ResetInternal(const FResetParams& Params) override
		{
			ZeroOutput();
		}
		
		virtual void ZeroOutput() override
		{
			for (int32 i = 0; i < NUM_CHANNELS; ++i)
			{
				AudioOuts[i]->Zero();
			}
		}

		virtual int32 CopyFramesToOutput(int32 OutputOffset, const TAudioBuffer<float>& InputBuffer, int32 NumFrames) override
		{
			int32 CopyFrames = FMath::Min(InputBuffer.GetNumValidFrames(), NumFrames);
			int32 CopySizeBytes = sizeof(float) * CopyFrames;
			for (int32 i = 0; i < NUM_CHANNELS; ++i)
			{
				check(CopyFrames <= AudioOuts[i]->Num() - OutputOffset);
				const float* SourceSamples = InputBuffer.GetRawChannelData(i);
				FMemory::Memcpy(AudioOuts[i]->GetData() + OutputOffset, SourceSamples, CopySizeBytes);
			}
			return CopyFrames;
		}

		virtual void SetupOutputAlias(TAudioBuffer<float>& AliasBuffer, int32 Offset) override
		{
			float* ChannelDataPtrs[NUM_CHANNELS];
			for (int32 i = 0; i < NUM_CHANNELS; ++i)
			{
				ChannelDataPtrs[i] = AudioOuts[i]->GetData() + Offset;
			}
			AliasBuffer.AliasChannelDataPointers(FAudioBufferConfig(NUM_CHANNELS, AudioRendering::kFramesPerRenderBuffer), ChannelDataPtrs);
		}

	private:
		//** OUTPUTS
		TArray<FAudioBufferWriteRef> AudioOuts;

		static const FName& GetMetasoundClassName();
		static const FText& GetDisplayName();
	};

	/**********************************************************************************************************
	***********************************************************************************************************
	* TEMPLATE FFusionSamplerOperator MONO SPECIALIZATION
	***********************************************************************************************************
	*********************************************************************************************************/
	template<>
	const FVertexInterface& FFusionSamplerOperator<1>::GetVertexInterface()
	{
		using namespace CommonPinNames;
		static const FVertexInterface Interface(
			FFusionSamplerOperatorBase::GetInputVertexInterface(),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::AudioMono)),
				TOutputDataVertex<FFusionSyncLink>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(FusionSamplerNodePinNames::RenderSync))
			)
		);
		return Interface;
	}

	template<>
	const FName& FFusionSamplerOperator<1>::GetMetasoundClassName()
	{
		static const FName MyName(TEXT("FusionSampler"));
		return MyName;
	}
	template<>
	const FText& FFusionSamplerOperator<1>::GetDisplayName()
	{
		static const FText MyText(METASOUND_LOCTEXT("FusionSamplerNode_DisplayName", "Fusion Sampler"));
		return MyText;
	}
	template<>
	void FFusionSamplerOperator<1>::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::AudioMono), AudioOuts[0]);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::RenderSync), SyncLinkOutPin);
	}

	/**********************************************************************************************************
	***********************************************************************************************************
	* TEMPLATE FFusionSamplerOperator STEREO SPECIALIZATION
	***********************************************************************************************************
	*********************************************************************************************************/
	template<>
	const FVertexInterface& FFusionSamplerOperator<2>::GetVertexInterface()
	{
		using namespace CommonPinNames;
		static const FVertexInterface Interface(
			FFusionSamplerOperatorBase::GetInputVertexInterface(),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::AudioLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::AudioRight)),
				TOutputDataVertex<FFusionSyncLink>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(FusionSamplerNodePinNames::RenderSync))
			)
		);
		return Interface;
	}
	template<>
	const FName& FFusionSamplerOperator<2>::GetMetasoundClassName()
	{
		static const FName MyName(TEXT("FusionSamplerStereo"));
		return MyName;
	}
	template<>
	const FText& FFusionSamplerOperator<2>::GetDisplayName()
	{
		static const FText MyText(METASOUND_LOCTEXT("FusionSamplerNodeStereo_DisplayName", "Fusion Sampler Stereo"));
		return MyText;
	}
	template<>
	void FFusionSamplerOperator<2>::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::AudioLeft), AudioOuts[0]);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::AudioRight), AudioOuts[1]);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FusionSamplerNodePinNames::RenderSync), SyncLinkOutPin);
	}

	/**********************************************************************************************************
	***********************************************************************************************************
	* TEMPLATE FFusionSamplerOperator & FFusionSamplerNode EXPLICIT INSTANTIATIONS
	***********************************************************************************************************
	*********************************************************************************************************/
	template class FFusionSamplerOperator<1>;
	template class FFusionSamplerOperator<2>;
	using FFusionSamplerNodeMono   = TNodeFacade<FFusionSamplerOperator<1>>;
	using FFusionSamplerNodeStereo = TNodeFacade<FFusionSamplerOperator<2>>;
	METASOUND_REGISTER_NODE(FFusionSamplerNodeMono)
	METASOUND_REGISTER_NODE(FFusionSamplerNodeStereo)

	/**********************************************************************************************************
	***********************************************************************************************************
	* FFusionSamplerOperatorBase::Execute()
	***********************************************************************************************************
	*********************************************************************************************************/
	void FFusionSamplerOperatorBase::Execute()
	{
		StuckNoteGuard.UnstickNotes(*MidiStreamInPin, [this](const FMidiStreamEvent& Event)
		{
			NoteOff(Event.GetVoiceId(), Event.MidiMessage.GetStdData1(), Event.MidiMessage.GetStdChannel());
		});
		
		if (EnableMTRenderingInPin)
		{
			if (!SyncLinkOutPin->GetTask().IsCompleted())
			{
				UE_LOGF(LogFusionSamplerPlayer, Warning, "FusionSamplerOperator is running multi-threaded, and the Execute method was called again before the last render was complete!");
			}
			SyncLinkOutPin->Reset();
		}
		
		if (!*EnableInPin)
		{
			if (MadeAudioLastFrame)
			{
				KillAllVoices();
				MadeAudioLastFrame = false;
			}
			ZeroOutput();
			return;
		}

		int32 TrackIndex = *TrackNumberInPin;
		if (TrackIndex != CurrentTrackNumber)
		{
			AllNotesOff();
			CurrentTrackNumber = TrackIndex;
		}

		CheckForUpdatedFusionPatchData();
		UpdatePatchOverrides();

		SetRawTransposition(*TranspositionInPin);
		
		MidiClock = MidiStreamInPin->GetClock();
		
		if (EnableMTRenderingInPin && !bDisableMultithreadedRender)
		{
			SyncLinkOutPin->KickAsyncRender([this]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("FusionAsyncRender");
				DoRender();
				CostReporter.SetRenderCost(GetNumVoicesInUse());
			});
		}
		else
		{
			DoRender();
			CostReporter.SetRenderCost(GetNumVoicesInUse());
		}
	}

	void FFusionSamplerOperatorBase::DoRender()
	{
		int32 FramesRequired = FramesPerBlock;
		int32 CurrentBlockFrameIndex = 0;

		MadeAudioLastFrame = false;

		if (ResidualBuffer.GetNumValidFrames() > 0)
		{
			// we have some samples from the last render we need to use...
			int32 NumFramesCopied = CopyFramesToOutput(CurrentBlockFrameIndex, ResidualBuffer, FramesRequired);
			ResidualBuffer.AdvanceAliasedDataPointers(NumFramesCopied);

			FramesRequired -= NumFramesCopied;
			CurrentBlockFrameIndex += NumFramesCopied;
			MadeAudioLastFrame = true;
		}
		
		TAudioBuffer<float> AliasedOutputBuffer;
		SetupOutputAlias(AliasedOutputBuffer, CurrentBlockFrameIndex);

		// create an iterator for midi events in the block
		const TArray<FMidiStreamEvent>& MidiEvents = MidiStreamInPin->GetEventsInBlock();
		auto MidiEventIterator = MidiEvents.begin();
		
		while (FramesRequired > 0)
		{
			while (MidiEventIterator != MidiEvents.end())
			{
				if ((*MidiEventIterator).BlockSampleFrameIndex <= CurrentBlockFrameIndex)
				{
					const FMidiMsg& MidiMessage = (*MidiEventIterator).MidiMessage;
					if (MidiMessage.IsStd() && (CurrentTrackNumber == 0 || (*MidiEventIterator).TrackIndex == CurrentTrackNumber))
					{
						// If using -1 for the track number to play all tracks, There's currently an issue
						// With voice IDs that playing the same note and channel from the same source but with different tracks
						// Will create the same ID. This means that notes with the same pitch and data will have the same ID,
						// So they won't play as separate fusion voices.
						HandleMidiMessage(
							(*MidiEventIterator).GetVoiceId(),
							MidiMessage.GetStdStatus(),
							MidiMessage.GetStdData1(),
							MidiMessage.GetStdData2(),
							(*MidiEventIterator).AuthoredMidiTick,
							(*MidiEventIterator).CurrentMidiTick,
							0.0f);
					}
					else if (MidiMessage.IsAllNotesOff())
					{
						AllNotesOff();
					}
					else if (MidiMessage.IsAllNotesKill())
					{
						KillAllVoices();
					}
					++MidiEventIterator;
				}
				else
				{
					break;
				}
			}

			if (MidiClock.IsValid())
			{
				const float ClockSpeed = MidiClock->GetSpeedAtBlockSampleFrame(CurrentBlockFrameIndex);
				SetSpeed(ClockSpeed, !(*ClockSpeedAffectsPitchInPin));
				const float ClockTempo = MidiClock->GetTempoAtBlockSampleFrame(CurrentBlockFrameIndex);
				SetTempo(ClockTempo);
				// This next thing is *close* to being accurate, but not really. 
				// It gets the last processed beat in this metasound render block, which may or may not
				// be correct for this specific sample in the block we are processing at the moment. 
				// It should be close enough. If not, we will need to ask the clock specifically what
				// its tick was at this point in the metasound rendering block. 
				const int32 Tick = MidiClock->GetNextMidiTickToProcess();
				const ISongMapEvaluator& ClocksMap = MidiClock->GetSongMapEvaluator();
				const float MidiQuarterNote = (float)Tick / (float)ClocksMap.GetTicksPerQuarterNote();
				SetQuarterNote(MidiQuarterNote);
			}

			if (FramesRequired > AudioRendering::kFramesPerRenderBuffer)
			{
				// we can render this block in place and avoid a copy
				Process(SliceIndex++, 0, AliasedOutputBuffer);
				MadeAudioLastFrame = MadeAudioLastFrame || !AliasedOutputBuffer.GetIsSilent();
				AliasedOutputBuffer.IncrementChannelDataPointers(AudioRendering::kFramesPerRenderBuffer);
				FramesRequired -= AudioRendering::kFramesPerRenderBuffer;
				CurrentBlockFrameIndex += AudioRendering::kFramesPerRenderBuffer;
			}
			else
			{
				// we only have room for a portion of the block we are about to render, so render to our scratch buffer...
				BusBuffer.SetNumValidFrames(AudioRendering::kFramesPerRenderBuffer);
				Process(SliceIndex++, 0, BusBuffer);
				MadeAudioLastFrame = MadeAudioLastFrame || !BusBuffer.GetIsSilent();
				
				int32 CopiedFrames = CopyFramesToOutput(CurrentBlockFrameIndex, BusBuffer, FramesRequired);
				// Alias the BusBuffer and initialize it with the offset of the num frames copied
				// this allows us to advance through the frames as we copy them from the residual buffer
				ResidualBuffer.Alias(BusBuffer, CopiedFrames);

				FramesRequired -= CopiedFrames;
				CurrentBlockFrameIndex += CopiedFrames;
			}
		}

		// there may be some remaining midi at the end of the block that we need to render next block
		// so pass them to the fusion sampler now...
		while (MidiEventIterator != MidiEvents.end())
		{
			const FMidiMsg& MidiMessage = (*MidiEventIterator).MidiMessage;
			if (MidiMessage.IsStd() && (CurrentTrackNumber == 0 || (*MidiEventIterator).TrackIndex == CurrentTrackNumber))
			{
				HandleMidiMessage(
					(*MidiEventIterator).GetVoiceId(),
					MidiMessage.GetStdStatus(),
					MidiMessage.GetStdData1(),
					MidiMessage.GetStdData2(),
					(*MidiEventIterator).AuthoredMidiTick,
					(*MidiEventIterator).CurrentMidiTick,
					0.0f);
			}
			++MidiEventIterator;
		}
	}
	void FFusionSamplerOperatorBase::PostExecute()
	{
		if (EnableMTRenderingInPin)
		{
			// This will ensure that our deferred renderer HAS, in fact, completed,
			// and will clear out the task handle. 
			SyncLinkOutPin->PostExecute();
		}
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
