// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MetronomeNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include <algorithm>

#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMath.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

DEFINE_LOG_CATEGORY_STATIC(LogMetronomeNode, Log, All);

namespace HarmonixMetasound::Nodes::MetronomeNode
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("Metronome"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	bool bSkipMetronomeLastProcessedClockTickCheck = true;
	FAutoConsoleVariableRef CVarSkipMetronomeLastProcessedClockTickCheck(
		TEXT("au.Metronome.SkipLastProcessedClockTickCheck"),
		bSkipMetronomeLastProcessedClockTickCheck,
		TEXT("Skip Last Processed Clock Tick conditions when executing a Metronome Metasound Node."),
		ECVF_Default);

	class FMetronomeOperator : public FMetronomeOperatorBase
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMetronomeOperator(const FBuildOperatorParams& InParams,
			const FMusicTransportEventStreamReadRef& InTransport,
			const bool  InLoop,
			const int32 InLoopLengthBars,
			const FInt32ReadRef& InTimSigNumerator,
			const FInt32ReadRef& InTimeSigDenominator,
			const FFloatReadRef& InTempo,
			const FFloatReadRef& InSpeedMultiplier,
			const int32 InSeekPrerollBars);
	};

	using FMetronomeNode = Metasound::TNodeFacade<FMetronomeOperator>;
	METASOUND_REGISTER_NODE(FMetronomeNode)

	const FNodeClassMetadata& FMetronomeOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = GetClassName();
			Info.MajorVersion     = GetCurrentMajorVersion();
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("MetronomeNode_DisplayName", "Metronome MIDI Clock Generator");
			Info.Description      = METASOUND_LOCTEXT("MetronomeNode_Description", "Provides a MIDI clock at the specified tempo and speed.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Transport, CommonPinNames::Inputs::Transport);
		DEFINE_METASOUND_PARAM_ALIAS(Loop, CommonPinNames::Inputs::Loop);
		DEFINE_METASOUND_PARAM_ALIAS(LoopLengthBars, CommonPinNames::Inputs::LoopLengthBars);
		DEFINE_METASOUND_PARAM_ALIAS(TimeSigNumerator, CommonPinNames::Inputs::TimeSigNumerator);
		DEFINE_METASOUND_PARAM_ALIAS(TimeSigDenominator, CommonPinNames::Inputs::TimeSigDenominator);
		DEFINE_METASOUND_PARAM_ALIAS(Tempo, CommonPinNames::Inputs::Tempo);
		DEFINE_METASOUND_PARAM_ALIAS(Speed, CommonPinNames::Inputs::Speed);
		DEFINE_METASOUND_PARAM_ALIAS(PrerollBars, CommonPinNames::Inputs::PrerollBars);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Outputs::MidiClock);
	}
	
	const FVertexInterface& FMetronomeOperator::GetVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Transport)),
				TInputConstructorVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Loop), false),
				TInputConstructorVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::LoopLengthBars), 4),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TimeSigNumerator), 4),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TimeSigDenominator), 4),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Tempo), 120.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Speed), 1.0f),
				TInputConstructorVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::PrerollBars), 8)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiClock))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMetronomeOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FMetronomeNode& TempoClockNode = static_cast<const FMetronomeNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		const FOperatorSettings& Settings = InParams.OperatorSettings;

		FMusicTransportEventStreamReadRef InTransport = InputData.GetOrCreateDefaultDataReadReference<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME(Inputs::Transport), Settings);
		bool InLoop = InputData.GetOrCreateDefaultValue<bool>(METASOUND_GET_PARAM_NAME(Inputs::Loop), Settings);
		int32 InLoopLengthBars = InputData.GetOrCreateDefaultValue<int32>(METASOUND_GET_PARAM_NAME(Inputs::LoopLengthBars), Settings);
		FInt32ReadRef InTimeSigNumerator   = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::TimeSigNumerator), Settings);
		FInt32ReadRef InTimeSigDenominator = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::TimeSigDenominator), Settings);
		FFloatReadRef InTempo = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::Tempo), Settings);
		FFloatReadRef InSpeed = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::Speed), Settings);
		int32 InPreRollBars = InputData.GetOrCreateDefaultValue<int32>(METASOUND_GET_PARAM_NAME(Inputs::PrerollBars), Settings);

		return MakeUnique<FMetronomeOperator>(InParams, InTransport, InLoop, InLoopLengthBars, InTimeSigNumerator, InTimeSigDenominator, InTempo, InSpeed, InPreRollBars);
	}

	FMetronomeOperatorBase::FMetronomeOperatorBase(const FBuildOperatorParams& InParams,
	                                       const FMusicTransportEventStreamReadRef& InTransport,
										   const bool  InLoop,
										   const int32 InLoopLengthBars,
	                                       const FInt32ReadRef& InTimSigNumerator,
	                                       const FInt32ReadRef& InTimeSigDenominator,
	                                       const FFloatReadRef& InTempo,
	                                       const FFloatReadRef& InSpeedMultiplier,
		                                   const int32 InPreRollBars)
		: FMusicTransportControllable(EMusicPlayerTransportState::Prepared) 
		, TransportInPin(InTransport)
		, LoopInPin(InLoop)
		, LoopLengthBarsInPin(InLoopLengthBars)
		, TimeSigNumInPin(InTimSigNumerator)
		, TimeSigDenomInPin(InTimeSigDenominator)
		, TempoInPin(InTempo)
		, SpeedMultInPin(InSpeedMultiplier)
		, SeekPreRollBarsInPin(InPreRollBars)
		, MidiClockOutPin(FMidiClockWriteRef::CreateNew(InParams.OperatorSettings))
		, MonotonicallyIncreasingClock(MakeShared<FMidiClock, ESPMode::NotThreadSafe>(InParams.OperatorSettings))
		, BlockSize(InParams.OperatorSettings.GetNumFramesPerBlock())
		, SampleRate(InParams.OperatorSettings.GetSampleRate())
	{
		CurrentTempo = GetInputTempoClamped();
		CurrentTimeSigNum = GetInputTimeSigNumeratorClamped();
		CurrentTimeSigDenom = GetInputTimeSigDenominatorClamped();
		Reset(InParams);
		Init();
	}

	FMetronomeOperator::FMetronomeOperator(const FBuildOperatorParams& InParams,
		const FMusicTransportEventStreamReadRef& InTransport,
		const bool  InLoop,
		const int32 InLoopLengthBars,
		const FInt32ReadRef& InTimSigNumerator,
		const FInt32ReadRef& InTimeSigDenominator,
		const FFloatReadRef& InTempo,
		const FFloatReadRef& InSpeedMultiplier,
		const int32 InSeekPrerollBars)
		: FMetronomeOperatorBase(InParams, InTransport, InLoop, InLoopLengthBars, InTimSigNumerator, InTimeSigDenominator, InTempo, InSpeedMultiplier, InSeekPrerollBars)
	{
	}

	void FMetronomeOperatorBase::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Transport), TransportInPin);
		InVertexData.SetValue(METASOUND_GET_PARAM_NAME(Inputs::Loop), LoopInPin);
		InVertexData.SetValue(METASOUND_GET_PARAM_NAME(Inputs::LoopLengthBars), LoopLengthBarsInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TimeSigNumerator), TimeSigNumInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TimeSigDenominator), TimeSigDenomInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Tempo), TempoInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Speed), SpeedMultInPin);
		InVertexData.SetValue(METASOUND_GET_PARAM_NAME(Inputs::PrerollBars), SeekPreRollBarsInPin);

		Init();
	}

	void FMetronomeOperatorBase::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiClock), MidiClockOutPin);
	}

	void FMetronomeOperatorBase::Reset(const FResetParams& Params)
	{
		MidiClockOutPin->Reset(Params.OperatorSettings);
		BlockSize = Params.OperatorSettings.GetNumFramesPerBlock();
		SampleRate = Params.OperatorSettings.GetSampleRate();
		LastProcessedClockTick = -1;
		NextClockTickToProcess = 0;
		CurrentTempo = GetInputTempoClamped();
		CurrentTimeSigNum = GetInputTimeSigNumeratorClamped();
		CurrentTimeSigDenom = GetInputTimeSigDenominatorClamped();
        Init();
	}

	void FMetronomeOperatorBase::Init()
	{
		bClocksArePreparedForExecute = false;
		PrepareClocksForExecute();
		
		MonotonicallyIncreasingClock->SetSpeed(0, 1.0f);
		MidiClockOutPin->SetSpeed(0, 1.0f);
		if (LoopInPin)
		{
			MidiClockOutPin->SetDrivingClock(MonotonicallyIncreasingClock);
		}

		BuildSongMaps();
		
		FTransportInitFn InitFn = [this](EMusicPlayerTransportState CurrentState)
		{
			FMidiClock& DrivingMidiClock = GetDrivingMidiClock();
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Starting:
				MidiClockOutPin->SeekTo(0, 0, 0);
				if (LoopInPin)
				{
					MonotonicallyIncreasingClock->SeekTo(0, 0, 0);
				}
				break;
			}

			const EMusicPlayerTransportState NextState = GetNextTransportState(CurrentState);
			HandleTransportChange(0, NextState);
			return NextState;
		};
		
		FMusicTransportControllable::Init(*TransportInPin, MoveTemp(InitFn));
	}

	void FMetronomeOperatorBase::Execute()
	{
		PrepareClocksForExecute();

		FMidiClock& DrivingMidiClock = GetDrivingMidiClock();

		if (*SpeedMultInPin != DrivingMidiClock.GetSpeedAtEndOfBlock())
		{
			DrivingMidiClock.SetSpeed(0, *SpeedMultInPin);
		}

		// only update our midi data if the clock is advancing
		// update the tempo and time sig before we advance our clock 
		int32 ClockTick = DrivingMidiClock.GetLastProcessedMidiTick();
		if ((ClockTick >= 0 && ClockTick > LastProcessedClockTick) || bSkipMetronomeLastProcessedClockTickCheck)
		{
			UpdateMidi();
			LastProcessedClockTick = ClockTick;
		}

		TransportSpanPostProcessor HandleMidiClockEvents = [this, &DrivingMidiClock](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
		{
			int32 NumFrames = EndFrameIndex - StartFrameIndex;
			HandleTransportChange(StartFrameIndex, CurrentState);
			DrivingMidiClock.SetSpeed(StartFrameIndex, *SpeedMultInPin);
			if (CurrentState == EMusicPlayerTransportState::Playing || CurrentState == EMusicPlayerTransportState::Continuing)
			{
				DrivingMidiClock.Advance(StartFrameIndex, NumFrames);
			}
		};

		TransportSpanProcessor TransportHandler = [this, &DrivingMidiClock](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
		{
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Starting:
				// Play from the beginning if we haven't received a seek call while we were stopped...
				if (!ReceivedSeekWhileStopped())
				{
					BuildSongMaps(true);
					LastProcessedClockTick = -1;
					NextClockTickToProcess = 0;
				}
				if (!ReceivedSeekWhileStopped())
				{
					DrivingMidiClock.SeekTo(StartFrameIndex, 0, 0);
				}
				return EMusicPlayerTransportState::Playing;
				
			case EMusicPlayerTransportState::Seeking:
				BuildSongMaps(false);
				DrivingMidiClock.SeekTo(StartFrameIndex, TransportInPin->GetNextSeekDestination());
				LastProcessedClockTick = DrivingMidiClock.GetLastProcessedMidiTick();
				NextClockTickToProcess = DrivingMidiClock.GetNextMidiTickToProcess();
				// Here we will return that we want to be in the same state we were in before this request to 
				// seek since we can seek "instantaneously"...
				return GetTransportState();
			}

			return GetNextTransportState(CurrentState);
		};
		ExecuteTransportSpans(TransportInPin, BlockSize, TransportHandler, HandleMidiClockEvents);

		if (MidiClockOutPin->HasPersistentLoop())
		{
			MidiClockOutPin->Advance(*MonotonicallyIncreasingClock, 0, BlockSize);
		}

		MarkClocksAsExecuted();
	}

	float FMetronomeOperatorBase::GetInputTempoClamped() const
	{
		return FMath::Clamp(*TempoInPin, kMinTempoBpm, kMaxTempoBpm);
	}

	int32 FMetronomeOperatorBase::GetInputTimeSigNumeratorClamped() const
	{
		return FMath::Clamp(*TimeSigNumInPin, kMinTimeSigNumerator, kMaxTimeSigNumerator);;
	}

	int32 FMetronomeOperatorBase::GetInputTimeSigDenominatorClamped() const
	{
		return FMath::Clamp(*TimeSigDenomInPin, kMinTimeSigDenominator, kMaxTimeSigDenominator);
	}
	
	void FMetronomeOperatorBase::BuildSongMaps(bool ResetToStart)
	{
		// make sure we have valid values
		CurrentTempo = GetInputTempoClamped();
		CurrentTimeSigNum = GetInputTimeSigNumeratorClamped();
		CurrentTimeSigDenom = GetInputTimeSigDenominatorClamped();
		SongMaps = MakeShared<FSongMaps>(CurrentTempo, CurrentTimeSigNum, CurrentTimeSigDenom);
		
		if (LoopInPin)
		{
			MonotonicallyIncreasingClock->AttachToSongMapEvaluator(SongMaps, ResetToStart);

			// midi clock out will follow the tempo of the monotonically increasing clock
			// so just assign it to some reasonable values. They will be ignored
			TSharedPtr<FSongMaps> SongMapsOut = MakeShared<FSongMaps>(120.0f, CurrentTimeSigNum, CurrentTimeSigDenom);
			MidiClockOutPin->AttachToSongMapEvaluator(SongMapsOut, ResetToStart);
			
			int32 LoopEndTick = SongMapsOut->BarIncludingCountInToTick(FMath::Max(LoopLengthBarsInPin, 1));
			// LoopEndTick == Loop Length since we are starting the loop at tick 0...
			MidiClockOutPin->SetupPersistentLoop(0, LoopEndTick);
		}
		else
		{
			// if we're not looping, then the MidiClockOut is going to be the driving midi clock
			// so attach the midi data directly to it
			MidiClockOutPin->AttachToSongMapEvaluator(SongMaps);
			MidiClockOutPin->ClearPersistentLoop();
		}
	}

	void FMetronomeOperatorBase::UpdateMidi()
	{
		bool HasMidiChanges = false;
		float NewTempo = GetInputTempoClamped();
		if (!FMath::IsNearlyEqual(CurrentTempo, NewTempo))
		{
			AddTempoChangeForMidi(NewTempo);
			HasMidiChanges = true;
		};

		int32 InTimeSigNum = GetInputTimeSigNumeratorClamped();
		int32 InTimeSigDenom = GetInputTimeSigDenominatorClamped();
		if (InTimeSigNum != CurrentTimeSigNum || InTimeSigDenom != CurrentTimeSigDenom)
		{
			CurrentTimeSigNum = InTimeSigNum;
			CurrentTimeSigDenom = InTimeSigDenom;
			if (IsEffectivelyStopped() || !MidiClockOutPin->HasPersistentLoop())
			{
				HandleTimeSigChangeForMidi(InTimeSigNum, InTimeSigDenom);
				HasMidiChanges = true;
			}
			else
			{
				UE_LOGF(LogMetronomeNode, Warning, "Changing Time Sig. on looping metronome not supported." 
					"Changing time signature will require changing loop length which is currently not supported.")
			}
		}

		if (HasMidiChanges)
		{
			// We only have to tell the MonotonicallyIncreasingClock that the song maps have changed
			// if we are looping...
			if (LoopInPin)
			{
				MonotonicallyIncreasingClock->SongMapsChanged();
			}
			// Regardless of which clock is currently driving, MonotonicallyIncreasingClock or MidiClockOutPin,
			// we need to inform the MidiClockOutPin that the song maps have changed. 
			MidiClockOutPin->SongMapsChanged();
		}
	}

	void FMetronomeOperatorBase::AddTempoChangeForMidi(float InTempoBPM)
	{
		CurrentTempo = InTempoBPM;
		int32 AtTick = GetDrivingMidiClock().GetNextMidiTickToProcess();
		SongMaps->AddTempoChange(AtTick, CurrentTempo);
	}

	void FMetronomeOperatorBase::HandleTimeSigChangeForMidi(int32 InTimeSigNum, int32 InTimeSigDenom)
	{
		CurrentTimeSigNum = InTimeSigNum;
		CurrentTimeSigDenom = InTimeSigDenom;
		int32 AtTick = GetDrivingMidiClock().GetNextMidiTickToProcess();
		// round to the next bar boundary, the bar we're actually going to apply the time sig change to
		int32 AtBar = FMath::CeilToInt32(SongMaps->GetBarIncludingCountInAtTick(AtTick));
		int32 NumTimeSigPoints = SongMaps->GetNumTimeSignatureChanges();

		// check if there's already a time signature point at the bar we're trying to update
		// the metronome clock increases monotonically, so we just have to check the _last_ point
		FTimeSignaturePoint* Point = SongMaps->GetMutableTimeSignaturePoint(NumTimeSigPoints - 1);
		check(Point);
		if (Point->BarIndex == AtBar)
		{
			// update that instead of adding a new one
			Point->TimeSignature.Numerator = InTimeSigNum;
			Point->TimeSignature.Denominator = InTimeSigDenom;
		}
		else
		{
			SongMaps->AddTimeSigChange(AtTick, CurrentTimeSigNum, CurrentTimeSigDenom);
		}
	}

	void FMetronomeOperatorBase::HandleTransportChange(int32 StartFrameIndex, EMusicPlayerTransportState NewTransportState)
	{
		GetDrivingMidiClock().SetTransportState(StartFrameIndex, NewTransportState);
	}

	void FMetronomeOperatorBase::PrepareClocksForExecute()
	{
		if (bClocksArePreparedForExecute)
		{
			return;
		}

		MidiClockOutPin->PrepareBlock();

		if (LoopInPin)
		{
			MonotonicallyIncreasingClock->PrepareBlock();
		}

		bClocksArePreparedForExecute = true;

	}

	void FMetronomeOperatorBase::MarkClocksAsExecuted()
	{
		bClocksArePreparedForExecute = false;
	}

}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
