// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiCCTriggerNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "MetasoundTrigger.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMidi/MidiMsg.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "HarmonixMetasound/DataTypes/MidiControllerID.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiCCTriggerNode
{
	using namespace Metasound;
	
	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
				TEXT("MidiCCTrigger"),
				""
		};
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 1;
	}
	
	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Inputs::MidiStream);
		DEFINE_INPUT_METASOUND_PARAM(InputMidiControllerID, "Control Number", "MIDI Control Number");

		// Removed in V1
		DECLARE_METASOUND_PARAM_ALIAS(MidiTrackNumber);
		DEFINE_METASOUND_PARAM_ALIAS(MidiTrackNumber, CommonPinNames::Inputs::MidiTrackNumber);
		DECLARE_METASOUND_PARAM_ALIAS(MidiChannelNumber);
		DEFINE_METASOUND_PARAM_ALIAS(MidiChannelNumber, CommonPinNames::Inputs::MidiChannelNumber);
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(OutputControlChangeValueInt32, "Value", "Control Change Value (0-127)");

		// Removed in V1
		DECLARE_METASOUND_PARAM_EXTERN(OutputTrigger);
		DEFINE_OUTPUT_METASOUND_PARAM(OutputTrigger, "Trigger Out", "A trigger when a MIDI Control Change message is found");
		DECLARE_METASOUND_PARAM_EXTERN(OutputControlNumber);
		DEFINE_OUTPUT_METASOUND_PARAM(OutputControlNumber, "Control Number", "Control Number (0-127)");
		DECLARE_METASOUND_PARAM_EXTERN(OutputControlChangeValueFloat);
		DEFINE_OUTPUT_METASOUND_PARAM(OutputControlChangeValueFloat, "Value (Normalized)", "Normalized Control Change value (0.0-1.0)");
	}

	class FMidiCCTriggerOperator_V1 final : public TExecutableOperator<FMidiCCTriggerOperator_V1>
	{
	public:
		struct FInputs
		{
			FBoolReadRef Enable;
			FMidiStreamReadRef MidiStream;
			FEnumStdMidiControllerIDReadRef ControllerID;
		};

		struct FOutputs
		{
			FInt32WriteRef ControlChangeValueInt32;
		};

		static const FVertexInterface& GetVertexInterface()
		{
			const auto MakeInterface = []() -> FVertexInterface
			{
				using namespace Metasound;

				return
				{
					FInputVertexInterface
					{
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
						TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream)),
						TInputDataVertex<FEnumStdMidiControllerID>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::InputMidiControllerID))
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::OutputControlChangeValueInt32))
					}
				};
			};

			static const FVertexInterface Interface = MakeInterface();

			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("MidiCCTriggerNode_V1_DisplayName", "MIDI CC Trigger");
				Info.Description = METASOUND_LOCTEXT("MidiCCTriggerNode_V1_Description", "Find a MIDI Control Change message in a MIDI stream and output the value.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FOperatorSettings& OperatorSettings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName,OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(Inputs::MidiStreamName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FEnumStdMidiControllerID>(Inputs::InputMidiControllerIDName, OperatorSettings)
			};

			FOutputs Outputs
			{
				FInt32WriteRef::CreateNew(0)
			};

			return MakeUnique<FMidiCCTriggerOperator_V1>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiCCTriggerOperator_V1(const FBuildOperatorParams& Params, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
		{
			Reset(Params);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enable);
			InVertexData.BindReadVertex(Inputs::MidiStreamName, Inputs.MidiStream);
			InVertexData.BindReadVertex(Inputs::InputMidiControllerIDName, Inputs.ControllerID);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::OutputControlChangeValueInt32Name, Outputs.ControlChangeValueInt32);
		}

		void Reset(const FResetParams& ResetParams)
		{
			*Outputs.ControlChangeValueInt32 = 0;
		}

		void Execute()
		{
			if (!*Inputs.Enable)
			{
				return;
			}

			const TArray<FMidiStreamEvent>& MidiEvents = Inputs.MidiStream->GetEventsInBlock();
			for (const FMidiStreamEvent& Event : MidiEvents)
			{
				if (!Event.MidiMessage.IsControlChange())
				{
					continue;
				}
				
				if (Event.MidiMessage.GetStdData1() != static_cast<uint8>(Inputs.ControllerID->Get()))
				{
					continue;
				}

				*Outputs.ControlChangeValueInt32 = static_cast<int32>(Event.MidiMessage.GetStdData2());
			}
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;
	};

	using FMidiCCTriggerNode_V1 = Metasound::TNodeFacade<FMidiCCTriggerOperator_V1>;
	METASOUND_REGISTER_NODE(FMidiCCTriggerNode_V1);

	class FMidiCCTriggerOperator_V0 final : public Metasound::TExecutableOperator<FMidiCCTriggerOperator_V0>
	{
	public:
		struct FInputs
		{
			FBoolReadRef Enable;
			FInt32ReadRef TrackNumber;
			FInt32ReadRef ChannelNumber;
			FEnumStdMidiControllerIDReadRef ControllerID;
			FMidiStreamReadRef MidiStream;
		};

		struct FOutputs
		{
			FInt32WriteRef ControlChangeValueInt32;
			FFloatWriteRef ControlChangeValueFloat;
			FTriggerWriteRef TriggerOut;
		};

		static const FVertexInterface& GetVertexInterface()
		{
			const auto MakeInterface = []() -> FVertexInterface
			{
				using namespace Metasound;

				return
				{
					FInputVertexInterface
					{
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiTrackNumber), 1),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiChannelNumber), 1),
						TInputDataVertex<FEnumStdMidiControllerID>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::InputMidiControllerID)),
						TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream))
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::OutputControlChangeValueInt32)),
						TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::OutputControlChangeValueFloat)),
						TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::OutputTrigger))
					}
				};
			};

			static const FVertexInterface Interface = MakeInterface();

			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("MIDICCTriggerNode_DisplayName", "MIDI CC Trigger");
				Info.Description = METASOUND_LOCTEXT("MIDICCTriggerNode_Description", "Find the MIDI CC messages in a MIDI stream and output them as triggers, floats, ints by the specified MIDI track, MIDI channel, and MIDI Controller ID.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Music);
				EnumAddFlags(Info.AccessFlags, ENodeClassAccessFlags::Deprecated);
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FOperatorSettings& OperatorSettings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName,OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MidiTrackNumberName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MidiChannelNumberName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FEnumStdMidiControllerID>(Inputs::InputMidiControllerIDName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(Inputs::MidiStreamName, OperatorSettings)
			};

			FOutputs Outputs
			{
				FInt32WriteRef::CreateNew(0), FFloatWriteRef::CreateNew(0.0f), FTriggerWriteRef::CreateNew(InParams.OperatorSettings)
			};

			return MakeUnique<FMidiCCTriggerOperator_V0>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiCCTriggerOperator_V0(const Metasound::FBuildOperatorParams& Params, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
		{
			Reset(Params);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enable);
			InVertexData.BindReadVertex(Inputs::MidiTrackNumberName, Inputs.TrackNumber);
			InVertexData.BindReadVertex(Inputs::MidiChannelNumberName, Inputs.ChannelNumber);
			InVertexData.BindReadVertex(Inputs::InputMidiControllerIDName, Inputs.ControllerID);
			InVertexData.BindReadVertex(Inputs::MidiStreamName, Inputs.MidiStream);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::OutputControlChangeValueInt32Name, Outputs.ControlChangeValueInt32);
			InVertexData.BindReadVertex(Outputs::OutputControlChangeValueFloatName, Outputs.ControlChangeValueFloat);
			InVertexData.BindReadVertex(Outputs::OutputTriggerName, Outputs.TriggerOut);
		}

		void Reset(const FResetParams& ResetParams)
		{
			Outputs.TriggerOut->Reset();
			*Outputs.ControlChangeValueInt32 = 0;
			*Outputs.ControlChangeValueFloat = 0.0f;
		}

		void Execute()
		{
			Outputs.TriggerOut->AdvanceBlock();

			//reset output values to 0 upon entering a new block
			*Outputs.ControlChangeValueInt32 = 0;
			*Outputs.ControlChangeValueFloat = 0.0f;
			
			if (!*Inputs.Enable)
			{
				return;
			}

			const TArray<FMidiStreamEvent>& MidiEvents = Inputs.MidiStream->GetEventsInBlock();
			for (const FMidiStreamEvent& Event : MidiEvents)
			{
				if (!Event.MidiMessage.IsStd())
				{
					continue;
				}

				if (Event.TrackIndex != *Inputs.TrackNumber
					|| !Event.MidiMessage.IsControlChange()
					|| Event.MidiMessage.GetStdChannel() + 1 != *Inputs.ChannelNumber)
				{
					continue;
				}

				EStdMidiControllerID InCCID = (*Inputs.ControllerID);
				//check if current CC message's controller ID matches with input Controller ID
				if (Event.MidiMessage.GetStdData1() == static_cast<uint8>(InCCID))
				{
					//raw midi control change value (0-127)
					*Outputs.ControlChangeValueInt32 = (int32)Event.MidiMessage.GetStdData2();

					//normalize control change value(0.0-1.0)
					*Outputs.ControlChangeValueFloat = (float)Event.MidiMessage.GetStdData2() / 127.0f;

					Outputs.TriggerOut->TriggerFrame(Event.BlockSampleFrameIndex);
				}
				
			}
		}
		private:
			FInputs Inputs;
			FOutputs Outputs;
	};

	using FMidiCCTriggerNode_V0 = Metasound::TNodeFacade<FMidiCCTriggerOperator_V0>;
	METASOUND_REGISTER_NODE(FMidiCCTriggerNode_V0);
}

#undef LOCTEXT_NAMESPACE