// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiChannelFilterNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"

#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/MidiOps/MidiChannelFilter.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiChannelFilter
{
	Metasound::FNodeClassName GetClassName()
	{
		return { HarmonixNodeNamespace, TEXT("MidiChannelFilterNode"), ""};
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Inputs::MidiStream);
		DEFINE_METASOUND_PARAM_ALIAS(Channel, CommonPinNames::Inputs::MidiChannelNumber);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
	}

	class FMidiChannelFilterOperator final : public Metasound::TExecutableOperator<FMidiChannelFilterOperator>
	{
	public:
		static const Metasound::FVertexInterface& GetVertexInterface()
		{
			auto InitVertexInterface = []() -> Metasound::FVertexInterface
			{
				using namespace Metasound;
				
				return {
					FInputVertexInterface
					{
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
						TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream)),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Channel), 0)
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
					}
				};
			};
			
			static const Metasound::FVertexInterface Interface = InitVertexInterface();
			return Interface;
		}
		
		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> Metasound::FNodeClassMetadata
			{
				Metasound::FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("MidiChannelFilterNode_DisplayName", "MIDI Channel Filter");
				Info.Description = METASOUND_LOCTEXT("MidiChannelFilterNode_Description", "Passes through MIDI events on the specified channel (or all channels)");
				Info.Author = Metasound::PluginAuthor;
				Info.PromptIfMissing = Metasound::PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, Metasound::NodeCategories::Music };
				return Info;
			};

			static const Metasound::FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		struct FInputs
		{
			Metasound::FBoolReadRef Enabled;
			FMidiStreamReadRef MidiStream;
			Metasound::FInt32ReadRef Channel;
		};

		struct FOutputs
		{
			FMidiStreamWriteRef MidiStream;
		};

		static TUniquePtr<IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			const Metasound::FOperatorSettings& OperatorSettings = InParams.OperatorSettings;
			const Metasound::FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(Inputs::MidiStreamName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::ChannelName, OperatorSettings)
			};

			FOutputs Outputs
			{
				FMidiStreamWriteRef::CreateNew()
			};

			return MakeUnique<FMidiChannelFilterOperator>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiChannelFilterOperator(const Metasound::FBuildOperatorParams& Params, FInputs&& InInputs, FOutputs&& InOutputs)
		: Inputs(MoveTemp(InInputs))
		, Outputs(MoveTemp(InOutputs))
		{
			Reset(Params);
		}

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enabled);
			InVertexData.BindReadVertex(Inputs::MidiStreamName, Inputs.MidiStream);
			InVertexData.BindReadVertex(Inputs::ChannelName, Inputs.Channel);
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::MidiStreamName, Outputs.MidiStream);
		}

		void Reset(const FResetParams&)
		{
			LastChannelValue = -1;
		}

		void Execute()
		{
			if (const int32 ClampedInputChannel = FMath::Clamp(*Inputs.Channel, 0, 16); ClampedInputChannel != LastChannelValue)
			{
				// Reset the filter in case we had "all" enabled
				Filter.SetChannelEnabled(0, false);
				// Set the channel (or all) enabled
				Filter.SetChannelEnabled(ClampedInputChannel, true);
				// Keep track of the last input value so we don't have to do this unless it changes
				LastChannelValue = ClampedInputChannel;
			}

			Outputs.MidiStream->PrepareBlock();

			if (*Inputs.Enabled)
			{
				Filter.Process(*Inputs.MidiStream, *Outputs.MidiStream);
			}
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;
		Harmonix::Midi::Ops::FMidiChannelFilter Filter;
		int32 LastChannelValue = -1;
	};

	using FMidiChannelFilterNode = Metasound::TNodeFacade<FMidiChannelFilterOperator>;
	METASOUND_REGISTER_NODE(FMidiChannelFilterNode)
}

#undef LOCTEXT_NAMESPACE
