// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MidiClockEvent.h"
#include "HarmonixMetasound/DataTypes/MidiAsset.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#include "HarmonixMidi/MidiVoiceId.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiClockOffset
{
	using namespace Metasound;

	FNodeClassName GetClassName()
	{
		return { HarmonixNodeNamespace, TEXT("MidiClockOffsetNode"), ""};
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_INPUT_METASOUND_PARAM(OffsetMs, "Offset (Ms)", "How much to offset the incoming clock by, in Milliseconds");
		DEFINE_INPUT_METASOUND_PARAM(OffsetBars, "Offset (Bars)", "How much to offset the incoming clock by, in Bars");
		DEFINE_INPUT_METASOUND_PARAM(OffsetBeats, "Offset (Beats)", "How much to offset the incoming clock by, in Beats");
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Outputs::MidiClock);
	}
	
	class FMidiClockOffsetOperator : public TExecutableOperator<FMidiClockOffsetOperator>, public FMusicTransportControllable
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName			= GetClassName();
				Info.MajorVersion		= GetCurrentMajorVersion();
				Info.MinorVersion		= 1;
				Info.DisplayName		= METASOUND_LOCTEXT("MIDIClockFollower_DisplayName", "MIDI Clock Offset");
				Info.Description		= METASOUND_LOCTEXT("MIDIClockFollower_Description", "Offset the incoming clock by some combination of Bars, Beats, and Milliseconds.");
				Info.Author				= PluginAuthor;
				Info.PromptIfMissing	= PluginNodeMissingPrompt;
				Info.DefaultInterface	= GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::OffsetBars), 0),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::OffsetBeats), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::OffsetMs), 0.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiClock))
				)
			);

			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FOperatorSettings& Settings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FMidiClockReadRef InMidiClock = InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), Settings);
			FInt32ReadRef InOffsetBars = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::OffsetBars), Settings);
			FFloatReadRef InOffsetBeats = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::OffsetBeats), Settings);
			FFloatReadRef InOffsetMs = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::OffsetMs), Settings);
			return MakeUnique<FMidiClockOffsetOperator>(InParams.OperatorSettings, InMidiClock, InOffsetBars, InOffsetBeats, InOffsetMs);	
		}

		FMidiClockOffsetOperator(const FOperatorSettings& InSettings, const FMidiClockReadRef& InMidiClock, const FInt32ReadRef& InOffsetBars, const FFloatReadRef& InOffsetBeats, const FFloatReadRef& InOffsetMs)
			: FMusicTransportControllable(EMusicPlayerTransportState::Prepared)
			, MidiClockIn(InMidiClock)
			, OffsetBarsInPin(InOffsetBars)
			, OffsetBeatsInPin(InOffsetBeats)
			, OffsetMsInPin(InOffsetMs)
			, MidiClockOut(FMidiClockWriteRef::CreateNew(InSettings))
			, BlockSize(InSettings.GetNumFramesPerBlock())
		{
			TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(120.0f, 4, 4);
			MidiClockOut->AttachToSongMapEvaluator(SongMaps, true);
			MidiClockOut->SetDrivingClock(MidiClockIn->AsShared().ToSharedPtr());
			bClockOutNeedsPrepare = false;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), MidiClockIn);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::OffsetBars), OffsetBarsInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::OffsetBeats), OffsetBeatsInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::OffsetMs), OffsetMsInPin);
			MidiClockOut->SetDrivingClock(MidiClockIn->AsShared().ToSharedPtr());
		}
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiClock), MidiClockOut);
		}

		virtual void Reset(const FResetParams& Params)
		{
			BlockSize = Params.OperatorSettings.GetNumFramesPerBlock();
			CurrentBlockSpanStart = 0;
			MidiClockOut->SeekTo(0,0,0);
			MidiClockOut->SetTransportState(0, EMusicPlayerTransportState::Playing);
			bClockOutNeedsPrepare = false;

			PrevOffsetBars = 0;
			PrevOffsetBeats = 0.0f;
			PrevOffsetMs = 0.0f;
		}

		int32 GetTickWithOffset(int32 InTick, int32 OffsetBars, float OffsetBeats, float OffsetMs)
		{
			int32 OffsetTick = InTick;
			if (!FMath::IsNearlyZero(OffsetBeats) || OffsetBars != 0)
			{
				int32 BeatsPerBar = 0;
				FMusicTimestamp OffsetTimestamp = MidiClockIn->GetSongMapEvaluator().TickToMusicTimestamp(InTick, &BeatsPerBar);
				if (BeatsPerBar != 0)
				{
					float Beats = ((OffsetTimestamp.Bar - 1) * BeatsPerBar) + (OffsetTimestamp.Beat - 1);
					Beats += OffsetBars * BeatsPerBar + OffsetBeats;
					OffsetTimestamp.Bar = 1 + FMath::Floor(Beats / BeatsPerBar);
					OffsetTimestamp.Beat = 1 + FMath::Fmod(BeatsPerBar + FMath::Fmod(Beats, BeatsPerBar), BeatsPerBar);
					OffsetTick = MidiClockIn->GetSongMapEvaluator().MusicTimestampToTick(OffsetTimestamp);
				}
			}
			if (!FMath::IsNearlyZero(OffsetMs))
			{
				const float Ms = MidiClockIn->GetSongMapEvaluator().TickToMs(OffsetTick);
				OffsetTick = MidiClockIn->GetSongMapEvaluator().MsToTick(OffsetMs + Ms);
			}

			return OffsetTick;
		}
		
		virtual void Execute()
		{
			using namespace MidiClockMessageTypes;

			float OffsetMs = *OffsetMsInPin;
			int32 OffsetBars = *OffsetBarsInPin;
			float OffsetBeats = *OffsetBeatsInPin;

			// We might not want to "prepare" the clock output because
			// we may have just initialized it with a tempo map, etc.
			// in which case it already has events in it for this block!
			if (bClockOutNeedsPrepare)
			{
				MidiClockOut->PrepareBlock();
			}

			if (MidiClockIn->GetSongMapsChangedInBlock())
			{
				MidiClockOut->SongMapsChanged();
			}

			// next time we definitely want to prepare the block.
			bClockOutNeedsPrepare = true;

			const TArray<FMidiClockEvent>& MidiClockEvents = MidiClockIn->GetMidiClockEventsInBlock();
			for (int32 EventIndex = 0; EventIndex < MidiClockEvents.Num(); ++EventIndex)
			{
				const FMidiClockEvent& Event = MidiClockEvents[EventIndex];
				if (const FSeek* AsSeek = Event.TryGet<FSeek>())
				{
					const int32 NewNextTick = GetTickWithOffset(AsSeek->NewNextTick, OffsetBars, OffsetBeats, OffsetMs);
					MidiClockOut->SeekTo(Event.BlockFrameIndex, NewNextTick, AsSeek->TempoMapTick);
				}
				else if (const FLoop* AsLoop = Event.TryGet<FLoop>())
				{
					const int32 FirstTickInLoop = GetTickWithOffset(AsLoop->FirstTickInLoop, OffsetBars, OffsetBeats, OffsetMs);
					const int32 LastTickAfterLoop = GetTickWithOffset(AsLoop->FirstTickInLoop + AsLoop->LengthInTicks, OffsetBars, OffsetBeats, OffsetMs);
					MidiClockOut->AdvanceToTick(Event.BlockFrameIndex, LastTickAfterLoop, AsLoop->TempoMapTick);
					MidiClockOut->AddTransientLoop(Event.BlockFrameIndex, FirstTickInLoop, LastTickAfterLoop - FirstTickInLoop);
				}
				else if (const FAdvance* AsAdvance = Event.TryGet<FAdvance>())
				{
					bool bOffsetChanged = !FMath::IsNearlyEqual(PrevOffsetMs, OffsetMs) || !FMath::IsNearlyEqual(PrevOffsetBeats, OffsetBeats) || PrevOffsetBars != OffsetBars;

					if (bOffsetChanged)
					{
						PrevOffsetMs = OffsetMs;
						PrevOffsetBars = OffsetBars;
						PrevOffsetBeats = OffsetBeats;
					}
					const int32 FirstickToProcess = GetTickWithOffset(AsAdvance->FirstTickToProcess, OffsetBars, OffsetBeats, OffsetMs);
					// ONLY seek if there is a discontinuity AND the transport changed OR the offset changed.
					// Otherwise we just want to advance from where we sit to the appropriate destination.
					if (bOffsetChanged || (!bAdvancedSinceTransportChange && FirstickToProcess != MidiClockOut->GetNextMidiTickToProcess()))
					{
						MidiClockOut->SeekTo(Event.BlockFrameIndex, FirstickToProcess, AsAdvance->TempoMapTick);
					}
					const int32 ProcessUpToTick = GetTickWithOffset(AsAdvance->FirstTickToProcess + AsAdvance->NumberOfTicksToProcess, OffsetBars, OffsetBeats, OffsetMs);
					MidiClockOut->AdvanceToTick(Event.BlockFrameIndex, ProcessUpToTick, AsAdvance->TempoMapTick);
					bAdvancedSinceTransportChange = true;
				}
				else if (const MidiClockMessageTypes::FTempoChange* AsTempoChange = Event.TryGet<MidiClockMessageTypes::FTempoChange>())
				{
					MidiClockOut->SetTempo(Event.BlockFrameIndex, MidiClockOut->GetNextMidiTickToProcess(), AsTempoChange->Tempo, AsTempoChange->TempoMapTick);
				}
				else if (const MidiClockMessageTypes::FTimeSignatureChange* AsTimeSigChange = Event.TryGet<MidiClockMessageTypes::FTimeSignatureChange>())
				{
					MidiClockOut->SetTimeSignature(Event.BlockFrameIndex, MidiClockOut->GetNextMidiTickToProcess(), AsTimeSigChange->TimeSignature, AsTimeSigChange->TempoMapTick);
				}
				else if (const MidiClockMessageTypes::FSpeedChange* AsSpeedChange = Event.TryGet<MidiClockMessageTypes::FSpeedChange>())
				{
					MidiClockOut->SetSpeed(Event.BlockFrameIndex, AsSpeedChange->Speed);
				}
				else if (const FTransportChange* AsTransportChange = Event.TryGet<FTransportChange>())
				{
					CurrentTransportState = AsTransportChange->TransportState;
					// We need to know that the transport changed when we get the next advance
					// so that we know if we should seek on a discontinuity. Otherwise we just
					// want to advance from where the clock last left off!
					bAdvancedSinceTransportChange = false;
				}
			}
		}

	protected:
		//** INPUTS
		FMidiClockReadRef MidiClockIn;
		FInt32ReadRef OffsetBarsInPin;
		FFloatReadRef OffsetBeatsInPin;
		FFloatReadRef OffsetMsInPin;
		int32 PrerollBars = 0;

		//** OUTPUTS
		FMidiClockWriteRef MidiClockOut;

		//** DATA
		FSampleCount BlockSize      = 0;
		int32 CurrentBlockSpanStart = 0;
		EMusicPlayerTransportState CurrentTransportState = EMusicPlayerTransportState::Prepared;
		bool bAdvancedSinceTransportChange = false;
		bool bClockOutNeedsPrepare = false;
		float PrevOffsetMs = 0.0f;
		int32 PrevOffsetBars = 0;
		float PrevOffsetBeats = 0.0f;
	};

	using FMidiClockOffsetNode = Metasound::TNodeFacade<FMidiClockOffsetOperator>;
	METASOUND_REGISTER_NODE(FMidiClockOffsetNode)
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
