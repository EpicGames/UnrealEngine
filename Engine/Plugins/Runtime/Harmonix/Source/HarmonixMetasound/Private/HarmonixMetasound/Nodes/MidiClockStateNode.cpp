// Copyright Epic Games, Inc. All Rights Reserved.

#include <limits>

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound
{
	using namespace Metasound;

	class FMidiClockStateOperator : public TExecutableOperator<FMidiClockStateOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMidiClockStateOperator(const FBuildOperatorParams& InParams,
		                            const FBoolReadRef&      InEnabled,
		                            const FMidiClockReadRef& InMidiClock);

		virtual ~FMidiClockStateOperator() override;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;
		
		void Reset(const FResetParams& ResetParams);

		void Execute();

	private:
		//** INPUTS
		FMidiClockReadRef   MidiClockInPin;
		FBoolReadRef        EnableInPin;

		//** OUTPUTS
		FTriggerWriteRef   TempoOrSpeedChangedTriggerOutPin;
		FFloatWriteRef     CurrentTempoOutPin;
		FFloatWriteRef     CurrentSpeedOutPin;
		FFloatWriteRef     CurrentSecondsPerQuarterNoteOutPin;
		FFloatWriteRef     CurrentSecondsPerBeatOutPin;
		FInt32WriteRef     CurrentBarOutPin;
		FFloatWriteRef     CurrentBeatOutPin;
		FInt32WriteRef     CurrentTimeSigNumOutPin;
		FInt32WriteRef     CurrentTimeSigDenomOutPin;

 		//** DATA (current state)
		int32 LastMidiTick = -1;

		static constexpr float kTinyTempoSpeed = 0.0001f;
	};

	using FMidiClockStateNode = Metasound::TNodeFacade<FMidiClockStateOperator>;
	METASOUND_REGISTER_NODE(FMidiClockStateNode)
		
	const FNodeClassMetadata& FMidiClockStateOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = { HarmonixNodeNamespace, TEXT("MidiClockState"), TEXT("")};
			Info.MajorVersion     = 0;
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("MidiClockStateNode_DisplayName", "MIDI Clock State");
			Info.Description      = METASOUND_LOCTEXT("MidiClockStateNode_Description", "Provides the current tempo, current speed, and current position of the attached MIDI clock.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace MidiClockStatePinNames
	{
		METASOUND_PARAM(TempoChangedTriggerOutput, "Tempo-Speed Changed", "Triggers when either the tempo or speed changes.")
	}

	const FVertexInterface& FMidiClockStateOperator::GetVertexInterface()
	{
		using namespace MidiClockStatePinNames;
		using namespace CommonPinNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
				TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock))
				),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(TempoChangedTriggerOutput)),
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Tempo)),
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Speed)),
				TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TimeSigNumerator)),
				TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TimeSigDenominator)),
				TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MusicTimespanBar)),
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MusicTimespanBeat)),
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::SecsPerQuarter)),
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::SecsPerBeat))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMidiClockStateOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace MidiClockStatePinNames;
		using namespace CommonPinNames;

		const FMidiClockStateNode& TheNode = static_cast<const FMidiClockStateNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FMidiClockReadRef InMidiClock = InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), InParams.OperatorSettings);
		FBoolReadRef InEnabled        = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings);

		return MakeUnique<FMidiClockStateOperator>(InParams, InEnabled, InMidiClock);
	}

	FMidiClockStateOperator::FMidiClockStateOperator(const FBuildOperatorParams& InParams,
															 const FBoolReadRef&      InEnabled,
															 const FMidiClockReadRef& InMidiClock)
		: MidiClockInPin(InMidiClock)
		, EnableInPin(InEnabled)
		, TempoOrSpeedChangedTriggerOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
		, CurrentTempoOutPin(FFloatWriteRef::CreateNew())
		, CurrentSpeedOutPin(FFloatWriteRef::CreateNew())
		, CurrentSecondsPerQuarterNoteOutPin(FFloatWriteRef::CreateNew())
		, CurrentSecondsPerBeatOutPin(FFloatWriteRef::CreateNew())
		, CurrentBarOutPin(FInt32WriteRef::CreateNew())
		, CurrentBeatOutPin(FFloatWriteRef::CreateNew())
		, CurrentTimeSigNumOutPin(FInt32WriteRef::CreateNew())
		, CurrentTimeSigDenomOutPin(FInt32WriteRef::CreateNew())
	{
		Reset(InParams);
	}

	FMidiClockStateOperator::~FMidiClockStateOperator()
	{
	}

	void FMidiClockStateOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		using namespace MidiClockStatePinNames;
		using namespace CommonPinNames;

		InVertexData.BindReadVertex(Inputs::EnableName, EnableInPin);
		InVertexData.BindReadVertex(Inputs::MidiClockName, MidiClockInPin);
	}

	void FMidiClockStateOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{

		using namespace MidiClockStatePinNames;
		using namespace CommonPinNames;

		InVertexData.BindReadVertex(TempoChangedTriggerOutputName, TempoOrSpeedChangedTriggerOutPin);
		InVertexData.BindReadVertex(Outputs::TempoName, CurrentTempoOutPin);
		InVertexData.BindReadVertex(Outputs::SpeedName, CurrentSpeedOutPin);
		InVertexData.BindReadVertex(Outputs::MusicTimespanBarName, CurrentBarOutPin);
		InVertexData.BindReadVertex(Outputs::MusicTimespanBeatName, CurrentBeatOutPin);
		InVertexData.BindReadVertex(Outputs::TimeSigNumeratorName, CurrentTimeSigNumOutPin);
		InVertexData.BindReadVertex(Outputs::TimeSigDenominatorName, CurrentTimeSigDenomOutPin);
		InVertexData.BindReadVertex(Outputs::SecsPerQuarterName, CurrentSecondsPerQuarterNoteOutPin);
		InVertexData.BindReadVertex(Outputs::SecsPerBeatName, CurrentSecondsPerBeatOutPin);
	}

	void FMidiClockStateOperator::Reset(const FResetParams& ResetParams)
	{
		TempoOrSpeedChangedTriggerOutPin->Reset();
		*CurrentTempoOutPin = kTinyTempoSpeed; // help users avoid divide by 0!
		*CurrentSpeedOutPin = 1.0f;
		*CurrentBarOutPin = 1;
		*CurrentBeatOutPin = 1.0f;
		*CurrentTimeSigNumOutPin = 4;
		*CurrentTimeSigDenomOutPin = 4;
		*CurrentSecondsPerQuarterNoteOutPin = std::numeric_limits<float>::max();
		*CurrentSecondsPerBeatOutPin = std::numeric_limits<float>::max();

		LastMidiTick = -1;
	}

	void FMidiClockStateOperator::Execute()
	{
		TempoOrSpeedChangedTriggerOutPin->AdvanceBlock();

		if (!*EnableInPin)
		{
			return;
		}
	
		const int32 NextTick = MidiClockInPin->GetNextMidiTickToProcess();

		bool HasTempoSpeedOrTimeSigChange = false;
		if (MidiClockInPin->GetTempoAtEndOfBlock() != *CurrentTempoOutPin)
		{
			HasTempoSpeedOrTimeSigChange = true;
			*CurrentTempoOutPin = FMath::Max(kTinyTempoSpeed, MidiClockInPin->GetTempoAtEndOfBlock());
		}
		if (MidiClockInPin->GetSpeedAtEndOfBlock() != *CurrentSpeedOutPin)
		{
			HasTempoSpeedOrTimeSigChange = true;
			*CurrentSpeedOutPin = FMath::Max(kTinyTempoSpeed, MidiClockInPin->GetSpeedAtEndOfBlock());
		}
			
		const ISongMapEvaluator& Map = MidiClockInPin->GetSongMapEvaluator();
		const FTimeSignature* TimeSignature = Map.GetTimeSignatureAtTick(NextTick);
		if (ensure(TimeSignature))
		{
			if (TimeSignature->Numerator != *CurrentTimeSigNumOutPin || TimeSignature->Denominator != *CurrentTimeSigDenomOutPin)
			{
				HasTempoSpeedOrTimeSigChange = true;
				*CurrentTimeSigNumOutPin = TimeSignature->Numerator;
				*CurrentTimeSigDenomOutPin = FMath::Max(1, TimeSignature->Denominator);
			}

			if (NextTick != LastMidiTick)
			{
				LastMidiTick = NextTick;
				FMusicTimestamp CurrentTimestamp = Map.TickToMusicTimestamp(NextTick);
				*CurrentBarOutPin = CurrentTimestamp.Bar;
				*CurrentBeatOutPin = CurrentTimestamp.Beat;
			}
		}

		if (HasTempoSpeedOrTimeSigChange)
		{
			TempoOrSpeedChangedTriggerOutPin->TriggerFrame(0);
			*CurrentSecondsPerQuarterNoteOutPin = 1.0f / (*CurrentTempoOutPin) * 60.0f / (*CurrentSpeedOutPin);
			*CurrentSecondsPerBeatOutPin = (*CurrentSecondsPerQuarterNoteOutPin) * 4.0f / ((float)*CurrentTimeSigDenomOutPin);
		}
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
