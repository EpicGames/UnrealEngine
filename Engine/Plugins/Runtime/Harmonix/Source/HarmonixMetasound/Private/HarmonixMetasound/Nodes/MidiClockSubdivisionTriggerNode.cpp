// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tickable.h"
#include "Containers/Queue.h"

#include <atomic>
#include <limits>

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/MidiOps/PulseGenerator.h"
#include "HarmonixMetasound/Nodes/MidiClockSubdivisionTriggerNode.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_MidiClockSubdivisionTriggerNode"

namespace HarmonixMetasound::Nodes::MidiClockSubdivisionTriggerNode
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, "MidiClockSubdivisionTrigger", "" };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	class FMidiClockSubdivisionTriggerOperator : public TExecutableOperator<FMidiClockSubdivisionTriggerOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMidiClockSubdivisionTriggerOperator(const FBuildOperatorParams& InParams,
		                            const FBoolReadRef&      InEnabled,
		                            const FMidiClockReadRef& InMidiClock,
									const FInt32ReadRef&     InGridSizeMult,
									const FEnumMidiClockSubdivisionQuantizationReadRef& InGridUnits,
									const FInt32ReadRef&	 InOffsetMult,
									const FEnumMidiClockSubdivisionQuantizationReadRef& InOffsetUnits);
		virtual ~FMidiClockSubdivisionTriggerOperator() override;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		void Reset(const FResetParams& ResetParams);

		void Execute();

	private:
		//** INPUTS
		FMidiClockReadRef   MidiClockInPin;
		FBoolReadRef        EnableInPin;
		FInt32ReadRef		GridSizeMultInPin;
		FEnumMidiClockSubdivisionQuantizationReadRef GridSizeUnitsInPin;
		FInt32ReadRef		GridOffsetMultInPin;
		FEnumMidiClockSubdivisionQuantizationReadRef GridOffsetUnitsInPin;

		//** OUTPUTS
		FTriggerWriteRef   TriggerOutPin;

 		//** DATA (current state)
		Harmonix::Midi::Ops::FPulseGenerator PulseGenerator;
	};

	using FMidiClockSubdivisionTriggerNode = Metasound::TNodeFacade<FMidiClockSubdivisionTriggerOperator>;
	METASOUND_REGISTER_NODE(FMidiClockSubdivisionTriggerNode)
		
	const FNodeClassMetadata& FMidiClockSubdivisionTriggerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = GetClassName();
			Info.MajorVersion     = GetCurrentMajorVersion();
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("MIDIClockSubdivisionTriggerNode_DisplayName", "MIDI Clock Subdivision Trigger");
			Info.Description      = METASOUND_LOCTEXT("MIDIClockSubdivisionTriggerNode_Description", "Watches a MIDI clock and outputs triggers at musical subdivisions.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(GridSizeUnits, CommonPinNames::Inputs::GridSizeUnits);
		DEFINE_METASOUND_PARAM_ALIAS(GridSizeMult, CommonPinNames::Inputs::GridSizeMult);
		DEFINE_METASOUND_PARAM_ALIAS(OffsetUnits, CommonPinNames::Inputs::OffsetUnits);
		DEFINE_METASOUND_PARAM_ALIAS(OffsetMult, CommonPinNames::Inputs::OffsetMult);
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(TriggerOutput, "Trigger Out", "A series of triggers at the specified subdivision grid.")
	}

	const FVertexInterface& FMidiClockSubdivisionTriggerOperator::GetVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
				TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::GridSizeUnits), (int32)EMidiClockSubdivisionQuantization::Beat),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::GridSizeMult), 1),
				TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::OffsetUnits), (int32)EMidiClockSubdivisionQuantization::Beat),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::OffsetMult), 0)
				),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TriggerOutput))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMidiClockSubdivisionTriggerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FMidiClockSubdivisionTriggerNode& TheNode = static_cast<const FMidiClockSubdivisionTriggerNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FMidiClockReadRef InMidiClock = InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), InParams.OperatorSettings);
		FBoolReadRef InEnabled        = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings);
		FEnumMidiClockSubdivisionQuantizationReadRef InGridSizeUnits = InputData.GetOrCreateDefaultDataReadReference<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME(Inputs::GridSizeUnits), InParams.OperatorSettings);
		FInt32ReadRef InGridSizeMult = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::GridSizeMult), InParams.OperatorSettings);
		FEnumMidiClockSubdivisionQuantizationReadRef InOffsetUnits   = InputData.GetOrCreateDefaultDataReadReference<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME(Inputs::OffsetUnits), InParams.OperatorSettings);
		FInt32ReadRef InOffsetMult = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::OffsetMult), InParams.OperatorSettings);

		return MakeUnique<FMidiClockSubdivisionTriggerOperator>(InParams,
			InEnabled,
			InMidiClock,
			InGridSizeMult,
			InGridSizeUnits,
			InOffsetMult,
			InOffsetUnits);
	}

	FMidiClockSubdivisionTriggerOperator::FMidiClockSubdivisionTriggerOperator(const FBuildOperatorParams& InParams,
															 const FBoolReadRef&      InEnabled,
															 const FMidiClockReadRef& InMidiClock,
															 const FInt32ReadRef&	  InGridSizeMult,
															 const FEnumMidiClockSubdivisionQuantizationReadRef& InGridUnits,
															 const FInt32ReadRef&	  InOffsetMult,
															 const FEnumMidiClockSubdivisionQuantizationReadRef& InOffsetUnits)
		: MidiClockInPin(InMidiClock)
		, EnableInPin(InEnabled)
		, GridSizeMultInPin(InGridSizeMult)
		, GridSizeUnitsInPin(InGridUnits)
		, GridOffsetMultInPin(InOffsetMult)
		, GridOffsetUnitsInPin(InOffsetUnits)
		, TriggerOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
	{
		Reset(InParams);
	}

	FMidiClockSubdivisionTriggerOperator::~FMidiClockSubdivisionTriggerOperator()
	{
	}

	void FMidiClockSubdivisionTriggerOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable),    EnableInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), MidiClockInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::GridSizeMult), GridSizeMultInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::GridSizeUnits), GridSizeUnitsInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::OffsetMult), GridOffsetMultInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::OffsetUnits), GridOffsetUnitsInPin);

		PulseGenerator.Reset();
	}

	void FMidiClockSubdivisionTriggerOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TriggerOutput), TriggerOutPin);
	}
	
	void FMidiClockSubdivisionTriggerOperator::Reset(const FResetParams& ResetParams)
	{
		TriggerOutPin->Reset();
		
		PulseGenerator.Reset();
	}

	void FMidiClockSubdivisionTriggerOperator::Execute()
	{
		// Update the pulse generator
		PulseGenerator.Enable(*EnableInPin);
		PulseGenerator.SetInterval({
			*GridSizeUnitsInPin,
			*GridOffsetUnitsInPin,
			static_cast<uint16>(*GridSizeMultInPin),
			static_cast<uint16>(*GridOffsetMultInPin) });

		// If there were pulses, trigger the output
		TriggerOutPin->AdvanceBlock();
		PulseGenerator.Process(*MidiClockInPin,
			[this](const Harmonix::Midi::Ops::FPulseGenerator::FPulseInfo& Pulse)
			{
				if (*EnableInPin)
				{
					TriggerOutPin->TriggerFrame(Pulse.BlockFrameIndex);
				}
			},
			[](const Harmonix::Midi::Ops::FPulseGenerator::FPulseInfo&) {});
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
