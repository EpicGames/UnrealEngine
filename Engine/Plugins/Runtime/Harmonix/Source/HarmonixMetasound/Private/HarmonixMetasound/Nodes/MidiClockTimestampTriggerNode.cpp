// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tickable.h"
#include "Containers/Queue.h"

#include <atomic>
#include <limits>

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"

DEFINE_LOG_CATEGORY_STATIC(LogMidiClockTimestampTrigger, Log, All);

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_MidiClockTimestampTriggerNode"

namespace HarmonixMetasound
{
	using namespace Metasound;

	class FMidiClockTimestampTriggerOperator : public TExecutableOperator<FMidiClockTimestampTriggerOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMidiClockTimestampTriggerOperator(const FBuildOperatorParams& InParams,
									const FBoolReadRef&                                 InEnabled,
		                            const FMidiClockReadRef&                            InMidiClock,
									const FMusicTimestampReadRef&                       InTimestamp,
									const FEnumMidiClockSubdivisionQuantizationReadRef& InQuanitizationUnit,
									const FBoolReadRef&                                 InTriggerDuringSeek);
		virtual ~FMidiClockTimestampTriggerOperator() override;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		void Reset(const FResetParams& ResetParams);
		
		void Execute();

	private:
		//** INPUTS
		FMidiClockReadRef                            MidiClockInPin;
		FBoolReadRef                                 EnableInPin;
		FMusicTimestampReadRef                       TimestampInPin;
		FEnumMidiClockSubdivisionQuantizationReadRef QuantizeUnitInPin;
		FBoolReadRef		                         TriggerDuringSeekInPin;

		//** OUTPUTS
		FTriggerWriteRef   TriggerOutPin;

 		//** DATA (current state)
		FMusicTimestamp CurrentTimestamp{1, 1.0f};
		EMidiClockSubdivisionQuantization Quantize = EMidiClockSubdivisionQuantization::None;

		int32 TriggerTick = 0;

		void CalculateTriggerTick();
	};

	using FMidiClockTimestampTriggerNode = Metasound::TNodeFacade<FMidiClockTimestampTriggerOperator>;
	METASOUND_REGISTER_NODE(FMidiClockTimestampTriggerNode)
		
	const FNodeClassMetadata& FMidiClockTimestampTriggerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = { HarmonixNodeNamespace, TEXT("MidiClockTimestampTrigger"), TEXT("")};
			Info.MajorVersion     = 1;
			Info.MinorVersion     = 0;
			Info.DisplayName      = METASOUND_LOCTEXT("MIDIClockTimestampTriggerNode_DisplayName", "MIDI Clock Timestamp Trigger");
			Info.Description      = METASOUND_LOCTEXT("MIDIClockTimestampTriggerNode_Description", "Watches a MIDI clock and outputs a trigger at the specified musical timestamp. The floating point beat can optionally be quantized to a musical subdivision.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace MidiClockTimestampTriggerPinNames
	{
		METASOUND_PARAM(TriggerDuringSeek, "Trigger During Seek", "Whether a trigger should be generated is a seek over the timestamp is detected.")
		METASOUND_PARAM(TriggerOutput, "Trigger Out", "A trigger when the timestamp is detected.")
	}

	const FVertexInterface& FMidiClockTimestampTriggerOperator::GetVertexInterface()
	{
		using namespace MidiClockTimestampTriggerPinNames;
		using namespace CommonPinNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
				TInputDataVertex<FMusicTimestamp>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Timestamp)),
				TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::QuantizationUnit), (int32)EMidiClockSubdivisionQuantization::None),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(TriggerDuringSeek), false)
				),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(TriggerOutput))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMidiClockTimestampTriggerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace MidiClockTimestampTriggerPinNames;
		using namespace CommonPinNames;

		const FMidiClockTimestampTriggerNode& TheNode = static_cast<const FMidiClockTimestampTriggerNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FMidiClockReadRef InMidiClock      = InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), InParams.OperatorSettings);
		FBoolReadRef  InEnabled            = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings);
		FMusicTimestampReadRef InTimestamp = InputData.GetOrCreateDefaultDataReadReference<FMusicTimestamp>(METASOUND_GET_PARAM_NAME(Inputs::Timestamp), InParams.OperatorSettings);
		FEnumMidiClockSubdivisionQuantizationReadRef InQuantizeUnits = InputData.GetOrCreateDefaultDataReadReference<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME(Inputs::QuantizationUnit), InParams.OperatorSettings);
		FBoolReadRef  InTriggerDuringSeek  = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(TriggerDuringSeek), InParams.OperatorSettings);

		return MakeUnique<FMidiClockTimestampTriggerOperator>(InParams,
			InEnabled,
			InMidiClock,
			InTimestamp,
			InQuantizeUnits,
			InTriggerDuringSeek);
	}

	FMidiClockTimestampTriggerOperator::FMidiClockTimestampTriggerOperator(
			const FBuildOperatorParams&                         InParams,
			const FBoolReadRef&                                 InEnabled,
			const FMidiClockReadRef&                            InMidiClock,
			const FMusicTimestampReadRef&                       InTimestamp,
			const FEnumMidiClockSubdivisionQuantizationReadRef& InQuanitizationUnit,
			const FBoolReadRef&                                 InTriggerDuringSeek)
		: MidiClockInPin(InMidiClock)
		, EnableInPin(InEnabled)
		, TimestampInPin(InTimestamp)
		, QuantizeUnitInPin(InQuanitizationUnit)
		, TriggerDuringSeekInPin(InTriggerDuringSeek)
		, TriggerOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
	{
		Reset(InParams);
	}

	FMidiClockTimestampTriggerOperator::~FMidiClockTimestampTriggerOperator()
	{
	}

	void FMidiClockTimestampTriggerOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;
		using namespace MidiClockTimestampTriggerPinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable), EnableInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), MidiClockInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Timestamp), TimestampInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::QuantizationUnit), QuantizeUnitInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(TriggerDuringSeek), TriggerDuringSeekInPin);
	}

	void FMidiClockTimestampTriggerOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MidiClockTimestampTriggerPinNames::TriggerOutput), TriggerOutPin);
	}
	
	void FMidiClockTimestampTriggerOperator::Reset(const FResetParams& ResetParams)
	{
		TriggerOutPin->Reset();

		CurrentTimestamp = *TimestampInPin;
		
		TriggerTick = 0;
		CalculateTriggerTick();
	}

	void FMidiClockTimestampTriggerOperator::CalculateTriggerTick()
	{
		const ISongMapEvaluator& Map = MidiClockInPin->GetSongMapEvaluator();
		TriggerTick = Map.CalculateMidiTick(CurrentTimestamp, Quantize);
	}

	void FMidiClockTimestampTriggerOperator::Execute()
	{
		TriggerOutPin->AdvanceBlock();

		// first let's see if our configuration has changed at all...
		if (CurrentTimestamp != *TimestampInPin || Quantize != *QuantizeUnitInPin)
		{
			CurrentTimestamp = *TimestampInPin;
			Quantize = *QuantizeUnitInPin;
			CalculateTriggerTick();
		}

		if (*EnableInPin)
		{
			for (const FMidiClockEvent& ClockEvent : MidiClockInPin->GetMidiClockEventsInBlock())
			{
				using namespace MidiClockMessageTypes;

				bool ShouldTrigger = false;

				if (const FAdvance* AsAdvance = ClockEvent.TryGet<FAdvance>())
				{
					ShouldTrigger = AsAdvance->FirstTickToProcess <= TriggerTick && TriggerTick <= AsAdvance->LastTickToProcess();
				}
				else if (const FSeek* AsSeekTo = ClockEvent.TryGet<FSeek>())
				{
					if (*TriggerDuringSeekInPin)
					{
						ShouldTrigger = AsSeekTo->LastTickProcessedBeforeSeek < TriggerTick && AsSeekTo->NewNextTick > TriggerTick;
					}
				}
				
				if (ShouldTrigger)
				{
					TriggerOutPin->TriggerFrame(ClockEvent.BlockFrameIndex);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
