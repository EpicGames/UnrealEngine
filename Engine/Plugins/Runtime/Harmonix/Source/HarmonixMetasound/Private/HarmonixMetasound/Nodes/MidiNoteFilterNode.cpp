// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiNoteFilterNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"

#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/MidiOps/MidiNoteFilter.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiNoteFilter
{
	Metasound::FNodeClassName GetClassName()
	{
		return { HarmonixNodeNamespace, TEXT("MidiNoteFilterNode"), ""};
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Inputs::MidiStream);
		DEFINE_METASOUND_PARAM_ALIAS(MinNoteNumber, CommonPinNames::Inputs::MinMidiNote);
		DEFINE_METASOUND_PARAM_ALIAS(MaxNoteNumber, CommonPinNames::Inputs::MaxMidiNote);
		DEFINE_METASOUND_PARAM_ALIAS(MinVelocity, CommonPinNames::Inputs::MinMidiVelocity);
		DEFINE_METASOUND_PARAM_ALIAS(MaxVelocity, CommonPinNames::Inputs::MaxMidiVelocity);
		DEFINE_INPUT_METASOUND_PARAM(IncludeOtherEvents, "Include Other Events", "Toggle on to include non-note events in the output.");
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
	}

	class FMidiNoteFilterOperator final : public Metasound::TExecutableOperator<FMidiNoteFilterOperator>
	{
	public:
		static const Metasound::FVertexInterface& GetVertexInterface()
		{
			auto InitVertexInterface = []() -> Metasound::FVertexInterface
			{
				using namespace Metasound;

				const Harmonix::Midi::Ops::FMidiNoteFilter FilterForDefaults;
				
				return {
					FInputVertexInterface
					{
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
						TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream)),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MinNoteNumber), static_cast<int32>(FilterForDefaults.MinNoteNumber.Get())),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MaxNoteNumber), static_cast<int32>(FilterForDefaults.MaxNoteNumber.Get())),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MinVelocity), static_cast<int32>(FilterForDefaults.MinVelocity.Get())),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MaxVelocity), static_cast<int32>(FilterForDefaults.MaxVelocity.Get())),
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::IncludeOtherEvents), FilterForDefaults.IncludeOtherEvents.Get()),
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
					}
				};
			};
			
			static const Metasound::FVertexInterface Interface = InitVertexInterface();
			return Interface;
		}
		
		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> Metasound::FNodeClassMetadata
			{
				Metasound::FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("MidiNoteFilterNode_DisplayName", "MIDI Note Filter");
				Info.Description = METASOUND_LOCTEXT("MidiNoteFilterNode_Description", "Passes through MIDI notes based on note number and velocity");
				Info.Author = Metasound::PluginAuthor;
				Info.PromptIfMissing = Metasound::PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, Metasound::NodeCategories::Music };
				return Info;
			};

			static const Metasound::FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		struct FInputs
		{
			Metasound::FBoolReadRef Enabled;
			FMidiStreamReadRef MidiStream;
			Metasound::FInt32ReadRef MinNoteNumber;
			Metasound::FInt32ReadRef MaxNoteNumber;
			Metasound::FInt32ReadRef MinVelocity;
			Metasound::FInt32ReadRef MaxVelocity;
			Metasound::FBoolReadRef IncludeOtherEvents;
		};

		struct FOutputs
		{
			FMidiStreamWriteRef MidiStream;
		};

		static TUniquePtr<IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			const Metasound::FOperatorSettings& OperatorSettings = InParams.OperatorSettings;
			const Metasound::FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(Inputs::MidiStreamName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MinNoteNumberName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MaxNoteNumberName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MinVelocityName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MaxVelocityName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::IncludeOtherEventsName, OperatorSettings)
			};

			FOutputs Outputs
			{
				FMidiStreamWriteRef::CreateNew()
			};

			return MakeUnique<FMidiNoteFilterOperator>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiNoteFilterOperator(const Metasound::FBuildOperatorParams& Params, FInputs&& InInputs, FOutputs&& InOutputs)
		: Inputs(MoveTemp(InInputs))
		, Outputs(MoveTemp(InOutputs))
		{
			Reset(Params);
		}

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enabled);
			InVertexData.BindReadVertex(Inputs::MidiStreamName, Inputs.MidiStream);
			InVertexData.BindReadVertex(Inputs::MinNoteNumberName, Inputs.MinNoteNumber);
			InVertexData.BindReadVertex(Inputs::MaxNoteNumberName, Inputs.MaxNoteNumber);
			InVertexData.BindReadVertex(Inputs::MinVelocityName, Inputs.MinVelocity);
			InVertexData.BindReadVertex(Inputs::MaxVelocityName, Inputs.MaxVelocity);
			InVertexData.BindReadVertex(Inputs::IncludeOtherEventsName, Inputs.IncludeOtherEvents);
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::MidiStreamName, Outputs.MidiStream);
		}

		void Reset(const FResetParams&)
		{
		}

		void Execute()
		{
			Filter.MinNoteNumber = *Inputs.MinNoteNumber;
			Filter.MaxNoteNumber = *Inputs.MaxNoteNumber;
			Filter.MinVelocity = *Inputs.MinVelocity;
			Filter.MaxVelocity = *Inputs.MaxVelocity;
			Filter.IncludeOtherEvents.Set(*Inputs.IncludeOtherEvents);

			Outputs.MidiStream->PrepareBlock();

			if (*Inputs.Enabled)
			{
				Filter.Process(*Inputs.MidiStream, *Outputs.MidiStream);
			}
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;
		Harmonix::Midi::Ops::FMidiNoteFilter Filter;
	};

	using FMidiNoteFilterNode = Metasound::TNodeFacade<FMidiNoteFilterOperator>;
	METASOUND_REGISTER_NODE(FMidiNoteFilterNode)
}

#undef LOCTEXT_NAMESPACE
