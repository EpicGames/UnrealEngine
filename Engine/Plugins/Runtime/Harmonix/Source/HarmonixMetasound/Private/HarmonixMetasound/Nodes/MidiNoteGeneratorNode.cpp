// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiNoteGeneratorNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "MetasoundTrigger.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/MidiOps/StuckNoteGuard.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiNoteGeneratorNode
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName{ HarmonixNodeNamespace, TEXT("MidiNoteGenerator"), TEXT("")};
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 1;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_METASOUND_PARAM_ALIAS(MidiChannelNumber, CommonPinNames::Inputs::MidiChannelNumber);
	}
	
	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
	}

	class FMidiNoteGeneratorOperator final : public TExecutableOperator<FMidiNoteGeneratorOperator>
	{
	public:
		struct FInputs
		{
			FMidiClockReadRef MidiClock;
			FTriggerReadRef NoteOnTrigger;
			FInt32ReadRef TrackNumber;
			FInt32ReadRef ChannelNumber;
			FInt32ReadRef MidiNote;
			FInt32ReadRef MidiVelocity;
			FFloatReadRef NoteOffDelay;
		};

		struct FOutputs
		{
			FMidiStreamWriteRef MidiStream;
		};
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
				{
					FNodeClassMetadata Info;
					Info.ClassName        = GetClassName();
					Info.MajorVersion     = 1;
					Info.MinorVersion     = 0;
					Info.DisplayName      = METASOUND_LOCTEXT("MIDINoteGenerator_DisplayName", "MIDI Note Generator");
					Info.Description      = METASOUND_LOCTEXT("MIDINoteGenerator_Description", "Generator MIDI Note On/Off events base on input trigger");
					Info.Author           = PluginAuthor;
					Info.PromptIfMissing  = PluginNodeMissingPrompt;
					Info.DefaultInterface = GetVertexInterface();
					Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
					return Info;
				};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TriggerNote)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TrackNumber), 1),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiChannelNumber), 1),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiNote), 60),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiVelocity), 127),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::NoteOffDelay))
					),
				FOutputVertexInterface(
					TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
					)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults&)
		{
			using namespace CommonPinNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TriggerNote), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::TrackNumber), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::MidiChannelNumber), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::MidiNote), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::MidiVelocity), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::NoteOffDelay), InParams.OperatorSettings),
			};

			FOutputs Outputs
			{
				FMidiStreamWriteRef::CreateNew()
			};

			return MakeUnique<FMidiNoteGeneratorOperator>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiNoteGeneratorOperator(const FBuildOperatorParams& InParams, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
			, FramesPerBlock(InParams.OperatorSettings.GetNumFramesPerBlock())
			, SampleRate(InParams.OperatorSettings.GetSampleRate())
		{
			
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::MidiClockName, Inputs.MidiClock);
			InVertexData.BindReadVertex(Inputs::TriggerNoteName, Inputs.NoteOnTrigger);
			InVertexData.BindReadVertex(Inputs::TrackNumberName, Inputs.TrackNumber);
			InVertexData.BindReadVertex(Inputs::MidiChannelNumberName, Inputs.ChannelNumber);
			InVertexData.BindReadVertex(Inputs::MidiNoteName, Inputs.MidiNote);
			InVertexData.BindReadVertex(Inputs::MidiVelocityName, Inputs.MidiVelocity);
			InVertexData.BindReadVertex(Inputs::NoteOffDelayName, Inputs.NoteOffDelay);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::MidiStreamName, Outputs.MidiStream);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			MidiNoteTriggerCounters.Reset();
		}

		void Execute()
		{
			Outputs.MidiStream->PrepareBlock();
			
			int32 TrackNumber = *Inputs.TrackNumber;
			for (int32 Idx = MidiNoteTriggerCounters.Num() - 1; Idx >= 0; --Idx)
			{
				FMidiNoteTriggerCounter& Counter = MidiNoteTriggerCounters[Idx];
				if (Counter.SampleCount < FramesPerBlock)
				{
					uint8 Channel;
					uint8 MidiNote;
					Counter.MidiVoiceId.GetChannelAndNote(Channel, MidiNote);
					FMidiMsg Msg = FMidiMsg::CreateNoteOff(Channel, MidiNote);
					int32 EventTick = Inputs.MidiClock->GetNextTickToProcessAtBlockFrame(Counter.SampleCount);
					FMidiStreamEvent MidiStreamEvent = FMidiStreamEvent(&MidiVoiceIdGenerator, Msg);
					MidiStreamEvent.BlockSampleFrameIndex = Counter.SampleCount;
					MidiStreamEvent.AuthoredMidiTick = EventTick;
					MidiStreamEvent.CurrentMidiTick = EventTick;
					MidiStreamEvent.TrackIndex = TrackNumber;
					Outputs.MidiStream->InsertMidiEvent(MidiStreamEvent);
					MidiNoteTriggerCounters.RemoveAtSwap(Idx);
				}
				else
				{
					Counter.SampleCount -= FramesPerBlock;
				}
			}
			
			int32 Channel = *Inputs.ChannelNumber;
			int32 MidiNote = FMath::Clamp(*Inputs.MidiNote, Harmonix::Midi::Constants::GMinNote, Harmonix::Midi::Constants::GMaxNote);
			int32 MidiVelocity = FMath::Clamp(*Inputs.MidiVelocity, Harmonix::Midi::Constants::GMinVelocity, Harmonix::Midi::Constants::GMaxVelocity);
			Inputs.NoteOnTrigger->ExecuteBlock([](int32, int32){}, [this, MidiNote, MidiVelocity, Channel, TrackNumber](int32 StartFrame, int32 EndFrame)
				{
					int32 EventTick = Inputs.MidiClock->GetNextTickToProcessAtBlockFrame(StartFrame);
					FMidiMsg Msg = FMidiMsg::CreateNoteOn(Channel, MidiNote, MidiVelocity);
					FMidiStreamEvent MidiStreamEvent = FMidiStreamEvent(&MidiVoiceIdGenerator, Msg);
					FMidiVoiceId MidiVoiceId = MidiStreamEvent.GetVoiceId();
					MidiStreamEvent.BlockSampleFrameIndex = StartFrame;
					MidiStreamEvent.AuthoredMidiTick = EventTick;
					MidiStreamEvent.CurrentMidiTick = EventTick;
					MidiStreamEvent.TrackIndex = TrackNumber;
					Outputs.MidiStream->InsertMidiEvent(MidiStreamEvent);

					MidiNoteTriggerCounters.RemoveAll([MidiVoiceId](const FMidiNoteTriggerCounter& Counter) { return Counter.MidiVoiceId == MidiVoiceId; });

					float NoteOffDelay = FMath::Max(*Inputs.NoteOffDelay, 0.01f);
					if (NoteOffDelay >= 0.0f)
					{
						FSampleCount SampleCount = StartFrame + NoteOffDelay * SampleRate;
						MidiNoteTriggerCounters.Add({ MidiVoiceId, SampleCount });
					}
				});
		}
		

	private:

		FInputs Inputs;
		FOutputs Outputs;
		FSampleCount FramesPerBlock;
		FSampleRate SampleRate;

		struct FMidiNoteTriggerCounter
		{
			FMidiVoiceId MidiVoiceId;
			FSampleCount SampleCount;
		};

		TArray<FMidiNoteTriggerCounter> MidiNoteTriggerCounters;
		
		FMidiVoiceGeneratorBase MidiVoiceIdGenerator;
	};

	using FMidiNoteGeneratorNode = Metasound::TNodeFacade<FMidiNoteGeneratorOperator>;
	METASOUND_REGISTER_NODE(FMidiNoteGeneratorNode);
}

#undef LOCTEXT_NAMESPACE