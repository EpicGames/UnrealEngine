// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiNoteTriggerNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "MetasoundTrigger.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/MidiOps/StuckNoteGuard.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiNoteTriggerNode
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName{ HarmonixNodeNamespace, TEXT("MidiNoteTrigger"), TEXT("")};
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 1;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Inputs::MidiStream);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(NoteOnTrigger, CommonPinNames::Outputs::NoteOn);
		DEFINE_METASOUND_PARAM_ALIAS(NoteOffTrigger, CommonPinNames::Outputs::NoteOff);
		DEFINE_METASOUND_PARAM_ALIAS(MidiNoteNumber, CommonPinNames::Outputs::MidiNoteNumber);
		DEFINE_METASOUND_PARAM_ALIAS(MidiVelocity, CommonPinNames::Outputs::MidiVelocity);
	}

	class FMidiNoteTriggerOperator_V1 final : public TExecutableOperator<FMidiNoteTriggerOperator_V1>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = GetClassName();
				Info.MajorVersion     = 1;
				Info.MinorVersion     = 0;
				Info.DisplayName      = METASOUND_LOCTEXT("MIDINoteTriggerNodeV1_DisplayName", "MIDI Note Trigger");
				Info.Description      = METASOUND_LOCTEXT("MIDINoteTriggerNodeV1_Description", "Outputs triggers and info for incoming MIDI notes.");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
					TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream))
					),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::NoteOnTrigger)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::NoteOffTrigger)),
					TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiNoteNumber)),
					TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiVelocity))
					)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults&)
		{
			using namespace CommonPinNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FBoolReadRef InEnabled          = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings);
			FMidiStreamReadRef InMidiStream = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), InParams.OperatorSettings);

			return MakeUnique<FMidiNoteTriggerOperator_V1>(InParams, MoveTemp(InEnabled), MoveTemp(InMidiStream));
		}

		FMidiNoteTriggerOperator_V1(const FBuildOperatorParams& InParams, FBoolReadRef&& InEnabled, FMidiStreamReadRef&& InMidiStream)
			: EnableInPin(MoveTemp(InEnabled))
			, MidiStreamInPin(MoveTemp(InMidiStream))
			, NoteOnOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, NoteOffOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, NoteNumOutPin(FInt32WriteRef::CreateNew(0))
			, VelOutPin(FInt32WriteRef::CreateNew(0))
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace CommonPinNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable), EnableInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiStream),   MidiStreamInPin);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace CommonPinNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(CommonPinNames::Outputs::NoteOn), NoteOnOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(CommonPinNames::Outputs::NoteOff), NoteOffOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiNoteNumber), NoteNumOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiVelocity), VelOutPin);
		}

		void Reset(const FResetParams&)
		{
			NoteOnOutPin->Reset();
			NoteOffOutPin->Reset();
			*NoteNumOutPin = 0;
			*VelOutPin = 0;
			SoundingNote = -1;
			PlayingId = FMidiVoiceId::None();
		}

		void Execute()
		{
			NoteOnOutPin->AdvanceBlock();
			NoteOffOutPin->AdvanceBlock();

			int32 NoteOffTriggerFrame = -1;

			StuckNoteGuard.UnstickNotes(*MidiStreamInPin, [this, &NoteOffTriggerFrame](const FMidiStreamEvent& Event)
			{
				TriggerNoteOff(0);
				NoteOffTriggerFrame = 0;
			});

			if (!*EnableInPin)
			{
				if (SoundingNote >= 0)
				{
					TriggerNoteOff(0);
				}
				return;
			}

			// Note off if the transport has stopped and we have a sounding note
			if (SoundingNote >= 0)
			{
				const TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> Clock = MidiStreamInPin->GetClock();
				if (Clock.IsValid() && Clock->GetTransportStateAtEndOfBlock() != EMusicPlayerTransportState::Playing)
				{
					TriggerNoteOff(0);
					return;
				}
			}
			
			for (const FMidiStreamEvent& Event : MidiStreamInPin->GetEventsInBlock())
			{
				if (Event.MidiMessage.IsNoteOn()) 
				{
					if (SoundingNote > -1)
					{
						// Stop sounding note
						TriggerNoteOff(Event.BlockSampleFrameIndex);
						NoteOffTriggerFrame = Event.BlockSampleFrameIndex;
					}

					// Play new note
					PlayingId      = Event.GetVoiceId();
					*VelOutPin     = Event.MidiMessage.GetStdData2();
					*NoteNumOutPin = Event.MidiMessage.GetStdData1();
					NoteOnOutPin->TriggerFrame(NoteOffTriggerFrame == Event.BlockSampleFrameIndex ? NoteOffTriggerFrame + 1 : Event.BlockSampleFrameIndex);
					SoundingNote   = Event.MidiMessage.GetStdData1();
				}
				else if (Event.MidiMessage.IsNoteOff())
				{
					if (Event.GetVoiceId() == PlayingId)
					{
						TriggerNoteOff(Event.BlockSampleFrameIndex);
						NoteOffTriggerFrame = Event.BlockSampleFrameIndex;
						PlayingId = FMidiVoiceId::None();
						SoundingNote = -1;
					}
				}
				else if (Event.MidiMessage.IsAllNotesOff() || Event.MidiMessage.IsAllNotesKill())
				{
					if (SoundingNote > -1)
					{
						// Stop sounding note
						TriggerNoteOff(Event.BlockSampleFrameIndex);
						NoteOffTriggerFrame = Event.BlockSampleFrameIndex;
						PlayingId = FMidiVoiceId::None();
						SoundingNote = -1;
					}
				}
			}
		}
		
	private:
		void TriggerNoteOff(int32 BlockSampleFrameIndex)
		{
			*VelOutPin = 0;
			NoteOffOutPin->TriggerFrame(BlockSampleFrameIndex);
			SoundingNote = -1;
			PlayingId = FMidiVoiceId::None();
		}
		
		FBoolReadRef       EnableInPin;
		FMidiStreamReadRef MidiStreamInPin;

		FTriggerWriteRef NoteOnOutPin;
		FTriggerWriteRef NoteOffOutPin;
		FInt32WriteRef   NoteNumOutPin;
		FInt32WriteRef   VelOutPin;

		int8 SoundingNote = -1;
		FMidiVoiceId PlayingId = FMidiVoiceId::None();
		Harmonix::Midi::Ops::FStuckNoteGuard StuckNoteGuard;
	};

	using FMidiNoteTriggerNode_V1 = Metasound::TNodeFacade<FMidiNoteTriggerOperator_V1>;
	METASOUND_REGISTER_NODE(FMidiNoteTriggerNode_V1)
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"

