// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiPlayerNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MidiClockEvent.h"
#include "HarmonixMetasound/DataTypes/MidiAsset.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#include "HarmonixMidi/MidiVoiceId.h"
#include "HarmonixMidi/MidiCursor.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiPlayerNode
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("MIDIPlayer"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiFileAsset, CommonPinNames::Inputs::MidiFileAsset);
		DEFINE_METASOUND_PARAM_ALIAS(Transport, CommonPinNames::Inputs::Transport);
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_METASOUND_PARAM_ALIAS(Loop, CommonPinNames::Inputs::Loop);
		DEFINE_METASOUND_PARAM_ALIAS(Speed, CommonPinNames::Inputs::Speed);
		DEFINE_METASOUND_PARAM_ALIAS(PrerollBars, CommonPinNames::Inputs::PrerollBars);
		DEFINE_INPUT_METASOUND_PARAM(KillVoicesOnSeek, "Kill Voices On Seek", "If true, a \"Kill All Voices\" MIDI message will be sent when seeking. Otherwise an \"All Notes Off\" will be sent which allows to ADSR release phases.")
		DEFINE_INPUT_METASOUND_PARAM(KillVoicesOnMidiChange, "Kill Voices On MIDI File Change", "If true, a \"Kill All Voices\" MIDI message will be sent when the MIDI file asset is changed. Otherwise an \"All Notes Off\" will be sent which allows to ADSR release phases.")
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Outputs::MidiClock);
	}

	class FMidiPlayerOperator : public TExecutableOperator<FMidiPlayerOperator>, public FMidiCursor::FReceiver, public FMidiVoiceGeneratorBase, public FMusicTransportControllable
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface&   GetVertexInterface();
		static TUniquePtr<IOperator>     CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMidiPlayerOperator(const FOperatorSettings& InSettings, 
							const FMidiAssetReadRef& InMidiAsset,
							const FMusicTransportEventStreamReadRef& InTransport,
							const FBoolReadRef& InLoop,
							const FFloatReadRef& InSpeedMultiplier,
                            const int32 InPrerollBars,
							const bool bInKillVoicesOnSeek,
							const bool bInKillVoicesOnMidiChange);

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;
		
		virtual void Reset(const FResetParams& Params);
		virtual void Execute();
		bool IsPlaying() const;

		//~ BEGIN FMidiCursor::FReceiver Overrides
		virtual void OnMidiMessage(int32 TrackIndex, int32 Tick, uint8 Status, uint8 Data1, uint8 Data2, bool bIsPreroll) override;
		virtual void OnTempo(int32 TrackIndex, int32 Tick, int32 Tempo, bool bIsPreroll) override;
		virtual void OnText(int32 TrackIndex, int32 Tick, int32 TextIndex, const FString& Str, uint8 Type, bool bIsPreroll) override;
		virtual void OnPreRollNoteOn(int32 TrackIndex, int32 EventTick, int32 CurrentTick, float PreRollMs, uint8 Status, uint8 Data1, uint8 Data2) override;
		//~ END FMidiCursor::FReceiver Overrides

	protected:
		//** INPUTS
		FMidiAssetReadRef MidiAssetInPin;
		FMusicTransportEventStreamReadRef TransportInPin;
		FBoolReadRef  LoopInPin;
		FFloatReadRef SpeedMultInPin;
		int32 PrerollBars;
		bool bKillVoicesOnSeek;
		bool bKillVoicesOnMidiChange;

		//** OUTPUTS
		FMidiStreamWriteRef MidiOutPin;
		FMidiClockWriteRef  MidiClockOut;

		//** DATA
		FMidiFileProxyPtr CurrentMidiFile;
		FMidiCursor MidiCursor;
		FSampleCount BlockSize = 0;
		int32 CurrentBlockSpanStart = 0;
		bool NeedsTransportInit = true;
		int32 CurrentRenderBlockFrame = 0;
		
		virtual void SetupNewMidiFile(const FMidiFileProxyPtr& NewMidi);

		void InitTransportIfNeeded()
		{
			if (NeedsTransportInit)
			{
				InitTransportImpl();
				NeedsTransportInit = false;
			}
		}
		virtual void InitTransportImpl() = 0;

		void RenderMidiForClockEvents();
		void SendAllNotesOff(int32 BlockFrameIndex, int32 Tick);

		FORCEINLINE float GetInputClockSpeedClamped() const
		{
			return FMath::Clamp(*SpeedMultInPin, kMinClockSpeed, kMaxClockSpeed);
		}
	};

	class FExternallyClockedMidiPlayerOperator : public FMidiPlayerOperator
	{
	public:
		FExternallyClockedMidiPlayerOperator(const FOperatorSettings& InSettings,
		                                     const FMidiAssetReadRef& InMidiAsset,
		                                     const FMusicTransportEventStreamReadRef& InTransport,
		                                     const FMidiClockReadRef& InMidiClock,
											 const FBoolReadRef& InLoop,
		                                     const FFloatReadRef& InSpeedMultiplier,
			                                 const int32 InPrerollBars,
											 const bool bInKillVoicesOnSeek,
											 const bool bInKillVoicesOnMidiChange);

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		virtual void Reset(const FResetParams& Params) override;

		virtual void Execute() override;

	protected:
		virtual void InitTransportImpl() override;
	private:
		//** INPUTS **********************************
		FMidiClockReadRef MidiClockIn;
	};

	class FSelfClockedMidiPlayerOperator : public FMidiPlayerOperator
	{
	public:
		FSelfClockedMidiPlayerOperator(const FOperatorSettings& InSettings,
		                               const FMidiAssetReadRef& InMidiAsset,
		                               const FMusicTransportEventStreamReadRef& InTransport,
									   const FBoolReadRef& InLoop,
		                               const FFloatReadRef& InSpeedMultiplier,
			                           const int32 InPrerollBars,
									   const bool bInKillVoicesOnSeek,
									   const bool bInKillVoicesOnMidiChange );

		virtual void Reset(const FResetParams& Params) override;

		virtual void Execute() override;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

	protected:
		virtual void InitTransportImpl() override;
	};

	using FMidiPlayerNode = Metasound::TNodeFacade<FMidiPlayerOperator>;
	METASOUND_REGISTER_NODE(FMidiPlayerNode)

	const FNodeClassMetadata& FMidiPlayerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = GetClassName();
			Info.MajorVersion     = GetCurrentMajorVersion();
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("MIDIPlayerNode_DisplayName", "MIDI Player");
			Info.Description      = METASOUND_LOCTEXT("MIDIPlayerNode_Description", "Plays a standard MIDI file.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	const FVertexInterface& FMidiPlayerOperator::GetVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FMidiAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiFileAsset)),
				TInputDataVertex<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Transport)),
				TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Loop), false),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Speed), 1.0f),
				TInputConstructorVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::PrerollBars), 8),
				TInputConstructorVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::KillVoicesOnSeek), false),
				TInputConstructorVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::KillVoicesOnMidiChange), false)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream)),
				TOutputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiClock))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMidiPlayerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FMidiPlayerNode& MidiPlayerNode = static_cast<const FMidiPlayerNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		const FOperatorSettings& Settings = InParams.OperatorSettings;

		FMidiAssetReadRef InMidiAsset = InputData.GetOrCreateDefaultDataReadReference<FMidiAsset>(METASOUND_GET_PARAM_NAME(Inputs::MidiFileAsset), Settings);
		FMusicTransportEventStreamReadRef InTransport  = InputData.GetOrCreateDefaultDataReadReference<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME(Inputs::Transport), Settings);
		FBoolReadRef InLoop = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Loop), Settings);
		FFloatReadRef InSpeed = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::Speed), Settings);
		int32 InPrerollBars = InputData.GetOrCreateDefaultValue<int32>(METASOUND_GET_PARAM_NAME(Inputs::PrerollBars), Settings);
		bool bInKillVoicesOnSeek = InputData.GetOrCreateDefaultValue<bool>(METASOUND_GET_PARAM_NAME(Inputs::KillVoicesOnSeek), Settings);
		bool bInKillVoicesOnMidiChange = InputData.GetOrCreateDefaultValue<bool>(METASOUND_GET_PARAM_NAME(Inputs::KillVoicesOnMidiChange), Settings);
		if (InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(Inputs::MidiClock)))
		{
			FMidiClockReadRef InMidiClock = InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), Settings);
			return MakeUnique<FExternallyClockedMidiPlayerOperator>(InParams.OperatorSettings, InMidiAsset, InTransport, InMidiClock, InLoop, InSpeed, InPrerollBars, bInKillVoicesOnSeek, bInKillVoicesOnMidiChange);
		}

		return MakeUnique<FSelfClockedMidiPlayerOperator>(InParams.OperatorSettings, InMidiAsset, InTransport, InLoop, InSpeed, InPrerollBars, bInKillVoicesOnSeek, bInKillVoicesOnMidiChange);
	}

	FMidiPlayerOperator::FMidiPlayerOperator(const FOperatorSettings& InSettings, 
											 const FMidiAssetReadRef& InMidiAsset,
											 const FMusicTransportEventStreamReadRef& InTransport,
											 const FBoolReadRef& InLoop,
											 const FFloatReadRef& InSpeed,
		                                     const int32 InPrerollBars,
											 const bool bInKillVoicesOnSeek,
											 const bool bInKillVoicesOnMidiChange)
		: FMusicTransportControllable(EMusicPlayerTransportState::Prepared)
		, MidiAssetInPin(InMidiAsset)
		, TransportInPin(InTransport)
		, LoopInPin(InLoop)
		, SpeedMultInPin(InSpeed)
		, PrerollBars(InPrerollBars)
		, bKillVoicesOnSeek(bInKillVoicesOnSeek)
		, bKillVoicesOnMidiChange(bInKillVoicesOnMidiChange)
		, MidiOutPin(FMidiStreamWriteRef::CreateNew())
		, MidiClockOut(FMidiClockWriteRef::CreateNew(InSettings))
		, BlockSize(InSettings.GetNumFramesPerBlock())
	{
		MidiOutPin->SetClock(*MidiClockOut);
	}

	FExternallyClockedMidiPlayerOperator::FExternallyClockedMidiPlayerOperator(const FOperatorSettings& InSettings,
																			   const FMidiAssetReadRef& InMidiAsset, 
																			   const FMusicTransportEventStreamReadRef& InTransport,
																			   const FMidiClockReadRef& InMidiClock, 
																			   const FBoolReadRef& InLoop,
																			   const FFloatReadRef& InSpeedMultiplier,
		                                                                       const int32 InPrerollBars,
																			   const bool bInKillVoicesOnSeek,
																			   const bool bInKillVoicesOnMidiChange)
		: FMidiPlayerOperator(InSettings, InMidiAsset, InTransport, InLoop, InSpeedMultiplier, InPrerollBars, bInKillVoicesOnSeek, bInKillVoicesOnMidiChange)
		, MidiClockIn(InMidiClock)
	{
		MidiClockOut->SetDrivingClock(MidiClockIn->AsShared().ToSharedPtr());
	}

	FSelfClockedMidiPlayerOperator::FSelfClockedMidiPlayerOperator(const FOperatorSettings& InSettings, 
																   const FMidiAssetReadRef& InMidiAsset, 
																   const FMusicTransportEventStreamReadRef& InTransport,
																   const FBoolReadRef& InLoop,
																   const FFloatReadRef& InSpeedMultiplier,
		                                                           int32 InPrerollBars,
																   const bool bInKillVoicesOnSeek,
																   const bool bInKillVoicesOnMidiChange)
		: FMidiPlayerOperator(InSettings, InMidiAsset, InTransport, InLoop, InSpeedMultiplier, InPrerollBars, bInKillVoicesOnSeek, bInKillVoicesOnMidiChange)
	{}

	void FSelfClockedMidiPlayerOperator::Reset(const FResetParams& Params)
	{
		FMidiPlayerOperator::Reset(Params);
	}

	void FExternallyClockedMidiPlayerOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		FMidiPlayerOperator::BindInputs(InVertexData);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), MidiClockIn);
		MidiClockOut->SetDrivingClock(MidiClockIn->AsShared().ToSharedPtr());
	}

	void FExternallyClockedMidiPlayerOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		FMidiPlayerOperator::BindOutputs(InVertexData);
	}

	void FExternallyClockedMidiPlayerOperator::Reset(const FResetParams& Params)
	{
		FMidiPlayerOperator::Reset(Params);
		MidiClockOut->SetDrivingClock(MidiClockIn->AsShared().ToSharedPtr());
	}

	void FSelfClockedMidiPlayerOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		FMidiPlayerOperator::BindInputs(InVertexData);
	}

	void FSelfClockedMidiPlayerOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		FMidiPlayerOperator::BindOutputs(InVertexData);
	}

	void FSelfClockedMidiPlayerOperator::InitTransportImpl()
	{
		// Get the node caught up to its transport input
		FTransportInitFn InitFn = [this](EMusicPlayerTransportState CurrentState)
		{
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Invalid:
			case EMusicPlayerTransportState::Preparing:
			case EMusicPlayerTransportState::Prepared:
			case EMusicPlayerTransportState::Stopping:
			case EMusicPlayerTransportState::Killing:
				MidiClockOut->SetTransportState(0, EMusicPlayerTransportState::Prepared);
				return EMusicPlayerTransportState::Prepared;

			case EMusicPlayerTransportState::Starting:
			case EMusicPlayerTransportState::Playing:
			case EMusicPlayerTransportState::Continuing:
				MidiClockOut->SetTransportState(0, EMusicPlayerTransportState::Playing);
				if (!ReceivedSeekWhileStopped())
				{
					MidiClockOut->SeekTo(0, 0, 0);
				}
				return EMusicPlayerTransportState::Playing;

			case EMusicPlayerTransportState::Seeking: // seeking is omitted from init, shouldn't happen
				checkNoEntry();
				return EMusicPlayerTransportState::Invalid;

			case EMusicPlayerTransportState::Pausing:
			case EMusicPlayerTransportState::Paused:
				MidiClockOut->SetTransportState(0, EMusicPlayerTransportState::Paused);
				return EMusicPlayerTransportState::Paused;

			default:
				checkNoEntry();
				return EMusicPlayerTransportState::Invalid;
			}
		};
		Init(*TransportInPin, MoveTemp(InitFn));
	}

	void FMidiPlayerOperator::Execute()
	{
		MidiOutPin->PrepareBlock();
		CurrentRenderBlockFrame = 0;

		MidiClockOut->PrepareBlock();

		if (MidiClockOut->HasPersistentLoop() != *LoopInPin || CurrentMidiFile != MidiAssetInPin->GetMidiProxy())
		{
			SetupNewMidiFile(MidiAssetInPin->GetMidiProxy());
		}

		InitTransportIfNeeded();
	}

	bool FMidiPlayerOperator::IsPlaying() const
	{
		return GetTransportState() == EMusicPlayerTransportState::Playing
			|| GetTransportState() == EMusicPlayerTransportState::Starting
			|| GetTransportState() == EMusicPlayerTransportState::Continuing;
	}

	void FExternallyClockedMidiPlayerOperator::Execute()
	{
		FMidiPlayerOperator::Execute();

		if (MidiClockIn->GetSongMapsChangedInBlock())
		{
			MidiClockOut->SongMapsChanged();
		}

		TransportSpanPostProcessor HandleMidiClockEventsInBlock = [this](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
		{
			// clock should always process in post processor
			int32 NumFrames = EndFrameIndex - StartFrameIndex;
			MidiClockOut->SetTransportState(StartFrameIndex, CurrentState);
			if (CurrentState == EMusicPlayerTransportState::Playing || CurrentState == EMusicPlayerTransportState::Continuing)
			{
				MidiClockOut->SetSpeed(StartFrameIndex, GetInputClockSpeedClamped());
				MidiClockOut->Advance(*MidiClockIn, StartFrameIndex, NumFrames);
			}
		};

		TransportSpanProcessor TransportHandler = [&](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
		{
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Starting:
				if (!ReceivedSeekWhileStopped())
				{
					MidiClockOut->SeekTo(StartFrameIndex, 0, MidiClockIn->GetNextTickToProcessAtBlockFrame(StartFrameIndex));
				}
				break;
			case EMusicPlayerTransportState::Stopping:
				{
					FMidiStreamEvent MidiEvent(this, FMidiMsg::CreateAllNotesOff());
					MidiEvent.BlockSampleFrameIndex = StartFrameIndex;
					MidiEvent.AuthoredMidiTick = MidiCursor.GetNextTick();
					MidiEvent.CurrentMidiTick = MidiCursor.GetNextTick();
					MidiEvent.TrackIndex = 0;
					MidiOutPin->AddMidiEvent(MidiEvent);
				}
				break;
			}
			return GetNextTransportState(CurrentState);
		};
		ExecuteTransportSpans(TransportInPin, BlockSize, TransportHandler, HandleMidiClockEventsInBlock);
		RenderMidiForClockEvents();
	}

	void FExternallyClockedMidiPlayerOperator::InitTransportImpl()
	{
		// Get the node caught up to its transport input
		// We don't send clock events for the externally-clocked player because those should already be handled
		FTransportInitFn InitFn = [this](EMusicPlayerTransportState CurrentState)
		{
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Seeking: // seeking is omitted from init, shouldn't happen
				checkNoEntry();
				return EMusicPlayerTransportState::Invalid;
			}

			return GetNextTransportState(CurrentState);
		};
		Init(*TransportInPin, MoveTemp(InitFn));
	}

	void FSelfClockedMidiPlayerOperator::Execute()
	{
		FMidiPlayerOperator::Execute();

		MidiClockOut->SetSpeed(0, GetInputClockSpeedClamped());

		TransportSpanPostProcessor MidiClockTransportHandler = [&](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
		{
			int32 NumFrames = EndFrameIndex - StartFrameIndex;
			MidiClockOut->SetTransportState(StartFrameIndex, CurrentState);
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Playing:
			case EMusicPlayerTransportState::Continuing:
				MidiClockOut->Advance(StartFrameIndex, NumFrames);
			}
		};

		TransportSpanProcessor TransportHandler = [this](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
		{
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Starting:
				if (!ReceivedSeekWhileStopped())
				{
					MidiClockOut->SeekTo(StartFrameIndex, 0, 0);
				}
				break;
			case EMusicPlayerTransportState::Seeking:
				MidiClockOut->SeekTo(StartFrameIndex, TransportInPin->GetNextSeekDestination());
				break;
			}
			return GetNextTransportState(CurrentState);
		};
		ExecuteTransportSpans(TransportInPin, BlockSize, TransportHandler, MidiClockTransportHandler);
		RenderMidiForClockEvents();
	}

	void FMidiPlayerOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiFileAsset), MidiAssetInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Transport), TransportInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Loop), LoopInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Speed), SpeedMultInPin);
		InVertexData.SetValue(METASOUND_GET_PARAM_NAME(Inputs::PrerollBars), PrerollBars);
		InVertexData.SetValue(METASOUND_GET_PARAM_NAME(Inputs::KillVoicesOnSeek), bKillVoicesOnSeek);
		InVertexData.SetValue(METASOUND_GET_PARAM_NAME(Inputs::KillVoicesOnMidiChange), bKillVoicesOnMidiChange);

		NeedsTransportInit = true;
	}

	void FMidiPlayerOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiStream), MidiOutPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiClock), MidiClockOut);

		NeedsTransportInit = true;
	}
	
	void FMidiPlayerOperator::Reset(const FResetParams& Params)
	{
		BlockSize = Params.OperatorSettings.GetNumFramesPerBlock();
		CurrentBlockSpanStart = 0;
		CurrentRenderBlockFrame = 0;
		CurrentMidiFile = nullptr;
		
		MidiOutPin->SetClock(*MidiClockOut);
		MidiClockOut->Reset(Params.OperatorSettings);

		NeedsTransportInit = true;
	}

	void FMidiPlayerOperator::OnMidiMessage(int32 TrackIndex, int32 Tick, uint8 Status, uint8 Data1, uint8 Data2, bool bIsPreroll)
	{
		if (IsPlaying())
		{
			FMidiStreamEvent MidiEvent(this, FMidiMsg(Status, Data1, Data2));
			MidiEvent.BlockSampleFrameIndex = CurrentRenderBlockFrame;
			MidiEvent.AuthoredMidiTick = Tick;
			MidiEvent.CurrentMidiTick = Tick;
			MidiEvent.TrackIndex = TrackIndex;
			MidiOutPin->AddMidiEvent(MidiEvent);
		}
	}

	void FMidiPlayerOperator::OnTempo(int32 TrackIndex, int32 Tick, int32 tempo, bool bIsPreroll)
	{
		if (IsPlaying())
		{
			FMidiStreamEvent MidiEvent(this, FMidiMsg((int32)tempo));
			MidiEvent.BlockSampleFrameIndex  = CurrentRenderBlockFrame;
			MidiEvent.AuthoredMidiTick = Tick;
			MidiEvent.CurrentMidiTick  = Tick;
			MidiEvent.TrackIndex       = TrackIndex;
			MidiOutPin->AddMidiEvent(MidiEvent);
		}
	}

	void FMidiPlayerOperator::OnText(int32 TrackIndex, int32 Tick, int32 TextIndex, const FString& Str, uint8 Type, bool bIsPreroll)
	{
		if (!bIsPreroll && IsPlaying())
		{
			FMidiStreamEvent MidiEvent(this, FMidiMsg::CreateText(TextIndex, Type));
			MidiEvent.BlockSampleFrameIndex = CurrentRenderBlockFrame;
			MidiEvent.AuthoredMidiTick = Tick;
			MidiEvent.CurrentMidiTick = Tick;
			MidiEvent.TrackIndex = TrackIndex;
			MidiOutPin->AddMidiEvent(MidiEvent);
		}
	}

	void FMidiPlayerOperator::OnPreRollNoteOn(int32 TrackIndex, int32 EventTick, int32 InCurrentTick, float InPrerollMs, uint8 InStatus, uint8 Data1, uint8 Data2)
	{
		if (IsPlaying())
		{
			FMidiStreamEvent MidiEvent(this, FMidiMsg(InStatus, Data1, Data2));
			MidiEvent.BlockSampleFrameIndex = CurrentRenderBlockFrame;
			MidiEvent.AuthoredMidiTick = EventTick;
			MidiEvent.CurrentMidiTick = InCurrentTick;
			MidiEvent.TrackIndex = TrackIndex;
			MidiOutPin->AddMidiEvent(MidiEvent);
		}
	}

	void FMidiPlayerOperator::SetupNewMidiFile(const FMidiFileProxyPtr& NewMidi)
	{
		CurrentMidiFile = NewMidi;
		MidiOutPin->SetMidiFile(CurrentMidiFile);

		FMidiStreamEvent MidiEvent(this, bKillVoicesOnMidiChange ? FMidiMsg::CreateAllNotesKill() : FMidiMsg::CreateAllNotesOff());
		MidiEvent.BlockSampleFrameIndex = 0;
		MidiEvent.AuthoredMidiTick = 0;
		MidiEvent.CurrentMidiTick = 0;
		MidiEvent.TrackIndex = 0;
		MidiOutPin->InsertMidiEvent(MidiEvent);

		if (CurrentMidiFile.IsValid())
		{			
			MidiClockOut->AttachToSongMapEvaluator(CurrentMidiFile->GetMidiFile(), !IsPlaying());
			MidiCursor.Prepare(CurrentMidiFile->GetMidiFile());
			MidiOutPin->SetTicksPerQuarterNote(CurrentMidiFile->GetMidiFile()->TicksPerQuarterNote);
			if (*LoopInPin)
			{
				const FSongLengthData& SongLengthData = CurrentMidiFile->GetMidiFile()->SongMaps.GetSongLengthData();
				int32 LoopStartTick = 0;
				int32 LoopEndTick = SongLengthData.LengthTicks;

				// Round the content authored ticks to bar boundaries based on the bar map. 
				const ISongMapEvaluator& SongMap = CurrentMidiFile->GetMidiFile()->SongMaps;
				int32 LoopStartBarIndex = FMath::RoundToInt32(SongMap.TickToFractionalBarIncludingCountIn(LoopStartTick));
				int32 LoopEndBarIndex = FMath::RoundToInt32(SongMap.TickToFractionalBarIncludingCountIn(LoopEndTick));

				// Make sure there's at least 1 bar of looping
				if (LoopEndBarIndex <= LoopStartBarIndex)
				{
					LoopEndBarIndex = LoopStartBarIndex + 1;
				}
				LoopStartTick = SongMap.BarIncludingCountInToTick(LoopStartBarIndex);
				LoopEndTick = SongMap.BarIncludingCountInToTick(LoopEndBarIndex);
				
				MidiClockOut->SetupPersistentLoop(LoopStartTick, LoopEndTick - LoopStartTick);
				
				// remap our current tick based on the looping behavior
				// maybe this should happen automatically when resetting a loop or changing midi files?
				int32 NewTick = MidiClockOut->WrapTickIfLooping(MidiClockOut->GetNextMidiTickToProcess());
				
				// Hmmmm. Sending in 0 for the DrivingClock's tick because in this base clase we don't know
				// if we have a driving clock. This may need to be changed (eg moved in to a subclass 'OnNewMidi' 
				// callback function.
				MidiClockOut->SeekTo(0, NewTick, 0);
			}
			else
			{
				MidiClockOut->ClearPersistentLoop();
			}
			MidiCursor.SeekToNextTick(MidiClockOut->GetNextMidiTickToProcess(), PrerollBars, this);
		}
		else
		{
			MidiClockOut->AttachToSongMapEvaluator(nullptr, !IsPlaying());
			MidiCursor.Prepare(nullptr);
		}
	}

	void FMidiPlayerOperator::RenderMidiForClockEvents()
	{
		const TArray<FMidiClockEvent>& ClockEvents = MidiClockOut->GetMidiClockEventsInBlock();
		for (const FMidiClockEvent& Event : ClockEvents)
		{
			CurrentRenderBlockFrame = Event.BlockFrameIndex;
			if (auto AsAdvance = Event.TryGet<MidiClockMessageTypes::FAdvance>())
			{
				MidiCursor.Process(AsAdvance->FirstTickToProcess, AsAdvance->LastTickToProcess(), *this);
			}
			else if (auto AsSeekTo = Event.TryGet<MidiClockMessageTypes::FSeek>())
			{
				SendAllNotesOff(Event.BlockFrameIndex, AsSeekTo->NewNextTick);
				MidiCursor.SeekToNextTick(AsSeekTo->NewNextTick, PrerollBars, this);
			}
			else if (auto AsLoop = Event.TryGet<MidiClockMessageTypes::FLoop>())
			{
				// When looping we don't preroll the events prior to the 
				// loop start point.
				SendAllNotesOff(Event.BlockFrameIndex, AsLoop->FirstTickInLoop);
				MidiCursor.SeekToNextTick(AsLoop->FirstTickInLoop);
			}
		}
	}

	void FMidiPlayerOperator::SendAllNotesOff(int32 BlockFrameIndex, int32 Tick)
	{
		FMidiStreamEvent MidiEvent(this, bKillVoicesOnSeek ? FMidiMsg::CreateAllNotesKill() : FMidiMsg::CreateAllNotesOff());
		MidiEvent.BlockSampleFrameIndex = BlockFrameIndex;
		MidiEvent.AuthoredMidiTick = Tick;
		MidiEvent.CurrentMidiTick = Tick;
		MidiEvent.TrackIndex = 0;
		MidiOutPin->AddMidiEvent(MidiEvent);
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
