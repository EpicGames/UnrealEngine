// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiPulseGeneratorNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"

#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/MidiOps/PulseGenerator.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_MidiPulseGeneratorNode"

namespace HarmonixMetasound::Nodes::MidiPulseGeneratorNode
{
	const Metasound::FNodeClassName& GetClassName()
	{
		static const Metasound::FNodeClassName ClassName { HarmonixNodeNamespace, "MidiPulseGenerator", "" };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_INPUT_METASOUND_PARAM(DutyCycle, "Duty Cycle", "The pulse width as a percentage of the Interval [0.0, 1.0]")
		DEFINE_INPUT_METASOUND_PARAM(Interval, "Interval", "The musical time interval at which to send out the pulse");
		DEFINE_INPUT_METASOUND_PARAM(IntervalMultiplier, "Interval Multiplier", "Multiplies the interval, 1 to use just the value of Interval");
		DEFINE_INPUT_METASOUND_PARAM(Offset, "Offset", "Offsets the pulse by a musical time");
		DEFINE_INPUT_METASOUND_PARAM(OffsetMultiplier, "Offset Multiplier", "Multiplies the offset, 0 for no offset");
		DEFINE_METASOUND_PARAM_ALIAS(MidiTrack, CommonPinNames::Inputs::MidiTrackNumber);
		DEFINE_METASOUND_PARAM_ALIAS(MidiChannel, CommonPinNames::Inputs::MidiChannelNumber);
		DEFINE_INPUT_METASOUND_PARAM(MidiNoteNumber, "Note Number", "The note number to play");
		DEFINE_INPUT_METASOUND_PARAM(MidiVelocity, "Velocity", "The velocity at which to play the note");
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
	}

	class FMidiPulseGeneratorOperator final : public Metasound::TExecutableOperator<FMidiPulseGeneratorOperator>
	{
	public:
		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			using namespace Metasound;
			
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = GetClassName();
				Info.MajorVersion     = 0;
				Info.MinorVersion     = 1;
				Info.DisplayName      = METASOUND_LOCTEXT("MidiPulseGeneratorNode_DisplayName", "MIDI Pulse Generator");
				Info.Description      = METASOUND_LOCTEXT("MidiPulseGeneratorNode_Description", "Outputs a repeated MIDI note at the specified musical time interval");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const Metasound::FVertexInterface& GetVertexInterface()
		{
			using namespace Metasound;

			static const Harmonix::Midi::Ops::FMidiPulseGenerator PulseGeneratorForDefaults;
			const auto DefaultInterval = PulseGeneratorForDefaults.GetInterval();

			static const FVertexInterface Interface
			{
				FInputVertexInterface {
					TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::DutyCycle), 1.0f),
					TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Interval), static_cast<int32>(DefaultInterval.Interval)),
					TInputDataVertex<int32>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::IntervalMultiplier), static_cast<int32>(DefaultInterval.IntervalMultiplier)),
					TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Offset), static_cast<int32>(DefaultInterval.Offset)),
					TInputDataVertex<int32>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::OffsetMultiplier), static_cast<int32>(DefaultInterval.OffsetMultiplier)),
					TInputDataVertex<int32>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiTrack), static_cast<int32>(PulseGeneratorForDefaults.Track)),
					TInputDataVertex<int32>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiChannel), static_cast<int32>(PulseGeneratorForDefaults.Channel)),
					TInputDataVertex<int32>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiNoteNumber), static_cast<int32>(PulseGeneratorForDefaults.NoteNumber)),
					TInputDataVertex<int32>(
						METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiVelocity), static_cast<int32>(PulseGeneratorForDefaults.Velocity))
				},
				FOutputVertexInterface
				{
					TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
				}
			};

			return Interface;
		}

		struct FInputs
		{
			FMidiClockReadRef Clock;
			Metasound::FBoolReadRef Enable;
			Metasound::FFloatReadRef DutyCycle;
			Metasound::FEnumMidiClockSubdivisionQuantizationReadRef Interval;
			Metasound::FInt32ReadRef IntervalMultiplier;
			Metasound::FEnumMidiClockSubdivisionQuantizationReadRef Offset;
			Metasound::FInt32ReadRef OffsetMultiplier;
			Metasound::FInt32ReadRef Track;
			Metasound::FInt32ReadRef Channel;
			Metasound::FInt32ReadRef NoteNumber;
			Metasound::FInt32ReadRef Velocity;
		};

		struct FOutputs
		{
			FMidiStreamWriteRef MidiStream;
		};
		
		static TUniquePtr<IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			FInputs Inputs
			{
				InParams.InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(Inputs::MidiClockName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(Inputs::DutyCycleName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<Metasound::FEnumMidiClockSubdivisionQuantizationType>(
					Inputs::IntervalName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::IntervalMultiplierName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<Metasound::FEnumMidiClockSubdivisionQuantizationType>(
					Inputs::OffsetName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::OffsetMultiplierName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MidiTrackName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MidiChannelName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MidiNoteNumberName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MidiVelocityName, InParams.OperatorSettings),
			};

			FOutputs Outputs
			{
				FMidiStreamWriteRef::CreateNew()
			};

			return MakeUnique<FMidiPulseGeneratorOperator>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiPulseGeneratorOperator(const Metasound::FBuildOperatorParams& InParams, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
		{
			Reset(InParams);
		}

		void Reset(const FResetParams&)
		{
			PulseGenerator.Reset();
			ApplyParameters();
		}

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::MidiClockName, Inputs.Clock);
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enable);
			InVertexData.BindReadVertex(Inputs::DutyCycleName, Inputs.DutyCycle);
			InVertexData.BindReadVertex(Inputs::IntervalName, Inputs.Interval);
			InVertexData.BindReadVertex(Inputs::IntervalMultiplierName, Inputs.IntervalMultiplier);
			InVertexData.BindReadVertex(Inputs::OffsetName, Inputs.Offset);
			InVertexData.BindReadVertex(Inputs::OffsetMultiplierName, Inputs.OffsetMultiplier);
			InVertexData.BindReadVertex(Inputs::MidiTrackName, Inputs.Track);
			InVertexData.BindReadVertex(Inputs::MidiChannelName, Inputs.Channel);
			InVertexData.BindReadVertex(Inputs::MidiNoteNumberName, Inputs.NoteNumber);
			InVertexData.BindReadVertex(Inputs::MidiVelocityName, Inputs.Velocity);

			PulseGenerator.Reset();
			ApplyParameters();
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::MidiStreamName, Outputs.MidiStream);
		}

		void Execute()
		{
			ApplyParameters();
			
			PulseGenerator.Process(*Inputs.Clock, *Outputs.MidiStream);
		}
	private:
		void ApplyParameters()
		{
			PulseGenerator.Enable(*Inputs.Enable);
			PulseGenerator.Track = *Inputs.Track;
			PulseGenerator.Channel = *Inputs.Channel;
			PulseGenerator.NoteNumber = *Inputs.NoteNumber;
			PulseGenerator.Velocity = *Inputs.Velocity;
			PulseGenerator.SetInterval(
				{
					*Inputs.Interval,
					*Inputs.Offset,
					static_cast<uint16>(*Inputs.IntervalMultiplier),
					static_cast<uint16>(*Inputs.OffsetMultiplier)
				});
			PulseGenerator.SetDutyCycle(*Inputs.DutyCycle);
		}
		
		FInputs Inputs;
		FOutputs Outputs;
		Harmonix::Midi::Ops::FMidiPulseGenerator PulseGenerator;
	};

	using FMidiPulseGeneratorNode = Metasound::TNodeFacade<FMidiPulseGeneratorOperator>;
	METASOUND_REGISTER_NODE(FMidiPulseGeneratorNode)
}

#undef LOCTEXT_NAMESPACE
