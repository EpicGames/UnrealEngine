// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/MidiOps/PulseGenerator.h"
#include "HarmonixMetasound/Nodes/MidiQuantizeTriggerNode.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_MidiQuantizeTriggerNode"

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixMetasoundMidiQuantizeTriggerNode, Log, All);

namespace HarmonixMetasound::Nodes::MidiQuantizeTriggerNode
{
	using namespace Metasound;

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_INPUT_METASOUND_PARAM(TriggerInput, "Trigger Input", "The input trigger to quantize");
		DEFINE_INPUT_METASOUND_PARAM(Quantization, "Subdivision", "The subdivision to quantize the trigger to")
		DEFINE_INPUT_METASOUND_PARAM(Tolerance, "Tolerance", "The tolerance that will allow late triggers to process immediately (as percentage of the subdivision)")
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(TriggerOutput, "Trigger Out", "The output trigger to quantize")
	}

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, "MidiQuantizeTriggerNode", "" };
		return ClassName;
	}
	
	int32 GetCurrentMajorVersion()
    {
    	return 0;
    }

	struct FInputs
	{
		FMidiClockReadRef MidiClock;
		FTriggerReadRef Trigger;
		FEnumMidiClockSubdivisionQuantizationReadRef Quantization;
		FFloatReadRef Tolerance;
	};

	struct FOutputs
	{
		FTriggerWriteRef Trigger;	
	};
	
	class FMidiQuantizeTriggerOperator : public TExecutableOperator<FMidiQuantizeTriggerOperator>
	{
	public:

		FMidiQuantizeTriggerOperator(const FBuildOperatorParams& InParams, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs)), Outputs(MoveTemp(InOutputs))
		{
			PulseGenerator.Enable(true);
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = GetClassName();
				Info.MajorVersion     = GetCurrentMajorVersion();
				Info.MinorVersion     = 1;
				Info.DisplayName      = METASOUND_LOCTEXT("MidiQuantizeTriggerNode_DisplayName", "MIDI Quantize/Delay Trigger Node");
				Info.Description      = METASOUND_LOCTEXT("MidiQuantizeTriggernode_Description", "Quantize an input trigger to a musical subdivision. Will delay the trigger to the next best subdivision or immediately play if it's close enough to the last subdivision based on the given tolerance.");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.Keywords         = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace Metasound;

			auto CreateVertexInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputVertexInterface
				{
					TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TriggerInput)),
					TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Quantization), (int32)EMidiClockSubdivisionQuantization::Beat),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Tolerance), 0.1f)
				};

				FOutputVertexInterface OutputVertexInterface
				{
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TriggerOutput))
				};

				return FVertexInterface(InputVertexInterface, OutputVertexInterface);
			};

			static FVertexInterface VertexInterface = CreateVertexInterface();
			return VertexInterface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(Inputs::MidiClockName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::TriggerInputName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FEnumMidiClockSubdivisionQuantizationType>(Inputs::QuantizationName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(Inputs::ToleranceName, InParams.OperatorSettings),
			};

			FOutputs Outputs
			{
				FTriggerWriteRef::CreateNew(InParams.OperatorSettings)
			};
			
			return MakeUnique<FMidiQuantizeTriggerOperator>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::MidiClockName, Inputs.MidiClock);
			InVertexData.BindReadVertex(Inputs::TriggerInputName, Inputs.Trigger);
			InVertexData.BindReadVertex(Inputs::QuantizationName, Inputs.Quantization);
			InVertexData.BindReadVertex(Inputs::ToleranceName, Inputs.Tolerance);
		}

		void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::TriggerOutputName, Outputs.Trigger);
		}

		void Reset(const FResetParams& Params)
		{
			Outputs.Trigger->Reset();
			PulseGenerator.Reset();
			bCanTriggerImmediate = true;
			bTriggerOnSubdivision = false;
			PulseGenerator.Reset();
		}

		void Execute()
		{
			using namespace HarmonixMetasound;
			using namespace HarmonixMetasound::MidiClockMessageTypes;
			using namespace Harmonix::Midi::Ops;

			Outputs.Trigger->AdvanceBlock();
			PulseGenerator.SetInterval({
				*Inputs.Quantization,
				*Inputs.Quantization,
				1,
				0
			});

			const float Tolerance = FMath::Clamp(*Inputs.Tolerance, 0.0f, 1.0f);
			Inputs.Trigger->ExecuteBlock(
				[](int32, int32) {},
				[this, &Tolerance](int32 StartFrame, int32 EndFrame)
				{
					// we can ignore triggers if we have already cued up a trigger for the next subdivision
					// AND we can't trigger anything immediately
					if (bTriggerOnSubdivision && !bCanTriggerImmediate)
					{
						return;
					}
					
					int32 TickAtFrame = Inputs.MidiClock->GetNextTickToProcessAtBlockFrame(StartFrame);
					const ISongMapEvaluator& SongMaps = Inputs.MidiClock->GetSongMapEvaluator();

					int32 LowerTick = 0;
					int32 UpperTick = 0;
					SongMaps.GetTicksForNearestSubdivision(TickAtFrame, *Inputs.Quantization, LowerTick, UpperTick);
					int32 NearestTick =  ((TickAtFrame - LowerTick) < (UpperTick - TickAtFrame)) ? LowerTick : UpperTick;

					// invalid tick range
					if (LowerTick >= UpperTick)
					{
						return;
					}
					
					if (NearestTick >= TickAtFrame)
					{
						// the nearest tick is in the future, so we can cue it up to trigger with the subdivision trigger node
						// we also know at this point we can't trigger anything immediately anymore
						bTriggerOnSubdivision = true;
						bCanTriggerImmediate = false;
					}
					else if (UpperTick > TickAtFrame)
					{
						if (!bCanTriggerImmediate)
						{
							bTriggerOnSubdivision = true;
						}
						else
						{
							// the nearest tick is in the past, and the upper tick is in the future...
							// see if the tick is within our tolerance
							float PctInInterval = 2.0f * static_cast<float>(TickAtFrame - LowerTick) / static_cast<float>(UpperTick - LowerTick);
							if (PctInInterval <= Tolerance)
							{
								Outputs.Trigger->TriggerFrame(StartFrame);
							}
							else
							{
								// otherwise, cue it up for the future
								bTriggerOnSubdivision = true;
							}
							// prevent other immediate triggers from happening until the next subdivision
							bCanTriggerImmediate = false;
						}
					}
				});

			PulseGenerator.Process(*Inputs.MidiClock,
				[this](const FPulseGenerator::FPulseInfo& Pulse)
				{
					if (bTriggerOnSubdivision)
					{
						Outputs.Trigger->TriggerFrame(Pulse.BlockFrameIndex);
					}

					// if we triggered this subdivision, don't allow immediate triggers to happen
					// since it would be for the same subdivision
					bCanTriggerImmediate = !bTriggerOnSubdivision;
					bTriggerOnSubdivision = false;
				},
				[](const FPulseGenerator::FPulseInfo&) {});
		}

	private:

		FInputs Inputs;
		FOutputs Outputs;

		bool bTriggerOnSubdivision = false;
		bool bCanTriggerImmediate = true;

		//** DATA (current state)
		Harmonix::Midi::Ops::FPulseGenerator PulseGenerator;

	};

	using FMidiQuantizeTriggerNode = Metasound::TNodeFacade<FMidiQuantizeTriggerOperator>;
	METASOUND_REGISTER_NODE(FMidiQuantizeTriggerNode);
}

#undef LOCTEXT_NAMESPACE