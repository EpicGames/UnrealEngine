// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

DEFINE_LOG_CATEGORY_STATIC(LogMIDIStreamLogger, Log, All);

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound
{
	using namespace Metasound;

	class FMidiStreamLoggerOperator : public TExecutableOperator<FMidiStreamLoggerOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMidiStreamLoggerOperator(const FBuildOperatorParams& InParams,
								  const FBoolReadRef& InEnabled,
								  const FMidiStreamReadRef& InMidiStream);

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		void Reset(const FResetParams& ResetParams);

		void Execute();

	private:
		//** INPUTS
		FBoolReadRef       EnableInPin;
		FMidiStreamReadRef MidiStreamInPin;

		//** DATA
		void DumpMidiEvent(const FMidiStreamEvent& MidiEvent);
	};

	using FMidiStreamLoggerNode = Metasound::TNodeFacade<FMidiStreamLoggerOperator>;
	METASOUND_REGISTER_NODE(FMidiStreamLoggerNode)
		
	const FNodeClassMetadata& FMidiStreamLoggerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = { HarmonixNodeNamespace, TEXT("MIDIStreamLogger"), TEXT("")};
			Info.MajorVersion     = 0;
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("MIDIStreamLoggerNode_DisplayName", "MIDI Logger");
			Info.Description      = METASOUND_LOCTEXT("MIDIStreamLoggerNode_Description", "Receives MIDI messages and writes them to the log.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music, NodeCategories::Debug };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	const FVertexInterface& FMidiStreamLoggerOperator::GetVertexInterface()
	{
		using namespace CommonPinNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
				TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream))
			),
			FOutputVertexInterface()
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMidiStreamLoggerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace CommonPinNames;

		const FMidiStreamLoggerNode& LoggerNode = static_cast<const FMidiStreamLoggerNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FBoolReadRef InEnabled = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings);
		FMidiStreamReadRef InMidiStream = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), InParams.OperatorSettings);

		return MakeUnique<FMidiStreamLoggerOperator>(InParams, InEnabled, InMidiStream);
	}

	FMidiStreamLoggerOperator::FMidiStreamLoggerOperator(
		const FBuildOperatorParams& InParams,
		const FBoolReadRef& InEnabled,
		const FMidiStreamReadRef& InMidiStream)
	: EnableInPin(InEnabled)
	, MidiStreamInPin(InMidiStream)
	{
		Reset(InParams);
	}

	void FMidiStreamLoggerOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable),     EnableInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), MidiStreamInPin);
	}

	void FMidiStreamLoggerOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
	}

	void FMidiStreamLoggerOperator::Reset(const FResetParams& ResetParams)
	{
	}

	void FMidiStreamLoggerOperator::Execute()
	{
		if (!*EnableInPin)
		{
			return;
		}

		for (const FMidiStreamEvent& Event : MidiStreamInPin->GetEventsInBlock())
		{
			DumpMidiEvent(Event);
		}
	}

	void FMidiStreamLoggerOperator::DumpMidiEvent(const FMidiStreamEvent& MidiEvent)
	{
		UE_LOGF(LogMIDIStreamLogger, Log, "[%d] Track: %d, Auth Tick %d, Render Tick %d, Message: %ls",
			MidiEvent.BlockSampleFrameIndex,
			MidiEvent.TrackIndex,
			MidiEvent.AuthoredMidiTick,
			MidiEvent.CurrentMidiTick,
			*FMidiMsg::ToString(MidiEvent.MidiMessage));
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
