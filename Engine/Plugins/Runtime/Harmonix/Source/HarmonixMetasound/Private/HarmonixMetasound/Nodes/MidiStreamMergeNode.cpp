// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound
{
	using namespace Metasound;

	class FMidiStreamMergeOperator : public TExecutableOperator<FMidiStreamMergeOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMidiStreamMergeOperator(const FBuildOperatorParams& InParams,
								  const FMidiStreamReadRef& InMidiStreamA,
								  const FMidiStreamReadRef& InMidiStreamB);

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		void Reset(const FResetParams& ResetParams);
		
		void Execute();

	private:
		//** INPUTS
		FMidiStreamReadRef MidiStreamAInPin;
		FMidiStreamReadRef MidiStreamBInPin;

		//** OUTPUTS
		FMidiStreamWriteRef MidiStreamOutPin;
	};

	using FMidiStreamMergeNode = Metasound::TNodeFacade<FMidiStreamMergeOperator>;
	METASOUND_REGISTER_NODE(FMidiStreamMergeNode)
		
	const FNodeClassMetadata& FMidiStreamMergeOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = { HarmonixNodeNamespace, TEXT("MidiStreamMerge"), TEXT("")};
			Info.MajorVersion     = 0;
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("MIDIStreamMergeNode_DisplayName", "MIDI Merge");
			Info.Description      = METASOUND_LOCTEXT("MIDIStreamMergeNode_Description", "Combines two MIDI streams into a single stream.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace MergePinNames
	{
		METASOUND_PARAM(InputMidiStreamA, "MIDI Stream A", "The first MIDI stream to merge");
		METASOUND_PARAM(InputMidiStreamB, "MIDI Stream B", "The second MIDI stream to merge");
	}

	const FVertexInterface& FMidiStreamMergeOperator::GetVertexInterface()
	{
		using namespace CommonPinNames;
		using namespace MergePinNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMidiStreamA)),
				TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMidiStreamB))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMidiStreamMergeOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace MergePinNames;

		const FMidiStreamMergeNode& LoggerNode = static_cast<const FMidiStreamMergeNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FMidiStreamReadRef InMidiStreamA = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(InputMidiStreamA), InParams.OperatorSettings);
		FMidiStreamReadRef InMidiStreamB = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(InputMidiStreamB), InParams.OperatorSettings);

		return MakeUnique<FMidiStreamMergeOperator>(InParams, InMidiStreamA, InMidiStreamB);
	}

	FMidiStreamMergeOperator::FMidiStreamMergeOperator(const FBuildOperatorParams& InParams,
											   const FMidiStreamReadRef& InMidiStreamA,
											   const FMidiStreamReadRef& InMidiStreamB)
		: MidiStreamAInPin(InMidiStreamA)
		, MidiStreamBInPin(InMidiStreamB)
		, MidiStreamOutPin(FMidiStreamWriteRef::CreateNew())
	{
		Reset(InParams);
	}

	void FMidiStreamMergeOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		using namespace MergePinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMidiStreamA), MidiStreamAInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMidiStreamB), MidiStreamBInPin);
	}

	void FMidiStreamMergeOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiStream), MidiStreamOutPin);
	}

	void FMidiStreamMergeOperator::Reset(const FResetParams&)
	{
	}

	void FMidiStreamMergeOperator::Execute()
	{
		MidiStreamOutPin->PrepareBlock();
		FMidiStream::Merge(*MidiStreamAInPin, *MidiStreamBInPin, *MidiStreamOutPin);
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
