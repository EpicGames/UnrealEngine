// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMidi/MidiMsg.h"

DEFINE_LOG_CATEGORY_STATIC(LogMIDIStreamSelect, Log, All);

#define LOCTEXT_NAMESPACE "HaronixMetaSound"

namespace HarmonixMetasound
{
	using namespace Metasound;

	class FMidiStreamSelectOperator : public TExecutableOperator<FMidiStreamSelectOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMidiStreamSelectOperator(const FBuildOperatorParams& InParams,
			const FMidiStreamReadRef& InMidiStreamA,
			const FMidiStreamReadRef& InMidiStreamB,
			const FInt32ReadRef& InStreamIndex,
			const FBoolReadRef& InImmediateNoteOff,
			const FBoolReadRef& InCopyInactiveNoteOffs);

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		void Execute();
		void Reset(const FResetParams& ResetParams);

	private:
		//** INPUTS
		FMidiStreamReadRef MidiStreamAInPin;
		FMidiStreamReadRef MidiStreamBInPin;
		FInt32ReadRef	   StreamIndexInPin;
		FBoolReadRef	   ImmediateNoteOffInPin;
		FBoolReadRef	   CopyInactiveNoteOffsInPin;

		//** OUTPUTS
		FMidiStreamWriteRef MidiStreamOutPin;

		//** DATA
		TSet<FMidiVoiceId> PlayingVoices;
		int32 CurrentStreamIndex = -1;

		void CopyFromSelectedStream(int32 SelectedStreamIndex);
		void MergeOrCreateNoteOffsForDeselectedStream(int32 SelectedStreamIndex);
	};

	using FMidiStreamSelectNode = Metasound::TNodeFacade<FMidiStreamSelectOperator>;
	METASOUND_REGISTER_NODE(FMidiStreamSelectNode)

	const FNodeClassMetadata& FMidiStreamSelectOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { HarmonixNodeNamespace, TEXT("MidiStreamSelect"), TEXT("") };
			Info.MajorVersion = 0;
			Info.MinorVersion = 1;
			Info.DisplayName = METASOUND_LOCTEXT("MIDIStreamSelectNode_DisplayName", "MIDI Select");
			Info.Description = METASOUND_LOCTEXT("MIDIStreamSelectNode_Description", "Copies the output of one of the inputs to the output.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace SelectPinNames
	{
		METASOUND_PARAM(InputMidiStreamA, "MIDI Stream A", "The first MIDI stream to select.");
		METASOUND_PARAM(InputMidiStreamB, "MIDI Stream B", "The second MIDI stream to select.");
		METASOUND_PARAM(InputStreamIndex, "Index", "The index of the MIDI stream to copy.");
		METASOUND_PARAM(InputImmediateNoteOff, "Immediately Note Off", "Should all active MIDI in the stream turn off when the index changes?");
		METASOUND_PARAM(InputCopyInactiveNoteOffs, "Copy Inactive Note Offs", "Should note offs for active MIDI be copied from inactive streams?");
	}

	const FVertexInterface& FMidiStreamSelectOperator::GetVertexInterface()
	{
		using namespace CommonPinNames;
		using namespace SelectPinNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMidiStreamA)),
				TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMidiStreamB)),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStreamIndex)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputImmediateNoteOff), true),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputCopyInactiveNoteOffs), true)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMidiStreamSelectOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace SelectPinNames;

		const FMidiStreamSelectNode& LoggerNode = static_cast<const FMidiStreamSelectNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FMidiStreamReadRef InMidiStreamA = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(InputMidiStreamA), InParams.OperatorSettings);
		FMidiStreamReadRef InMidiStreamB = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(InputMidiStreamB), InParams.OperatorSettings);
		FInt32ReadRef InStreamIndex = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputStreamIndex), InParams.OperatorSettings);
		FBoolReadRef InImmediateNoteOff = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputImmediateNoteOff), InParams.OperatorSettings);
		FBoolReadRef InCopyInactiveNoteOffs = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputCopyInactiveNoteOffs), InParams.OperatorSettings);

		return MakeUnique<FMidiStreamSelectOperator>(InParams, InMidiStreamA, InMidiStreamB, InStreamIndex, InImmediateNoteOff, InCopyInactiveNoteOffs);
	}

	FMidiStreamSelectOperator::FMidiStreamSelectOperator(const FBuildOperatorParams& InParams,
		const FMidiStreamReadRef& InMidiStreamA,
		const FMidiStreamReadRef& InMidiStreamB,
		const FInt32ReadRef& InStreamIndex,
		const FBoolReadRef& InImmediateNoteOff,
		const FBoolReadRef& InCopyInactiveNoteOffs)
		: MidiStreamAInPin(InMidiStreamA)
		, MidiStreamBInPin(InMidiStreamB)
		, StreamIndexInPin(InStreamIndex)
		, ImmediateNoteOffInPin(InImmediateNoteOff)
		, CopyInactiveNoteOffsInPin(InCopyInactiveNoteOffs)
		, MidiStreamOutPin(FMidiStreamWriteRef::CreateNew())
	{
		Reset(InParams);
	}

	void FMidiStreamSelectOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		using namespace SelectPinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMidiStreamA), MidiStreamAInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMidiStreamB), MidiStreamBInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStreamIndex), StreamIndexInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputImmediateNoteOff), ImmediateNoteOffInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputCopyInactiveNoteOffs), CopyInactiveNoteOffsInPin);
	}

	void FMidiStreamSelectOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiStream), MidiStreamOutPin);
	}

	void FMidiStreamSelectOperator::Execute()
	{
		MidiStreamOutPin->PrepareBlock();

		if (*StreamIndexInPin > 1 || *StreamIndexInPin < 0)
		{
			return;
		}

		int32 NewCurrentStreamIndex = *StreamIndexInPin;

		CopyFromSelectedStream(NewCurrentStreamIndex);
		MergeOrCreateNoteOffsForDeselectedStream(NewCurrentStreamIndex);

		CurrentStreamIndex = NewCurrentStreamIndex;
	}

	void FMidiStreamSelectOperator::CopyFromSelectedStream(int32 SelectedStreamIndex)
	{
		const FMidiStreamReadRef SourceStream = SelectedStreamIndex == 0 ? MidiStreamAInPin : MidiStreamBInPin;

		// Copy, but keep track of 'note-on' messages so we can appropriately send 'note-off' messages later...
		FMidiStream::Copy(
			*SourceStream,
			*MidiStreamOutPin,
			[this](const FMidiStreamEvent& Event)
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					PlayingVoices.Add(Event.GetVoiceId());
				}
				else if (Event.MidiMessage.IsNoteOff())
				{
					PlayingVoices.Remove(Event.GetVoiceId());
				}
				
				return true;
			});
	}

	void FMidiStreamSelectOperator::MergeOrCreateNoteOffsForDeselectedStream(int32 SelectedStreamIndex)
	{
		FMidiStreamReadRef InactiveStream = SelectedStreamIndex == 0 ? MidiStreamBInPin : MidiStreamAInPin;

		if (*ImmediateNoteOffInPin)
		{
			// Immediately send note offs - generate NoteOffs with matching voice IDs and send those through
			if (CurrentStreamIndex != SelectedStreamIndex && !PlayingVoices.IsEmpty())
			{
				for (FMidiVoiceId VoiceId : PlayingVoices)
				{
					uint8 MidiCh;
					uint8 MidiNote;
					VoiceId.GetChannelAndNote(MidiCh, MidiNote);
					FMidiStreamEvent MidiEvent(VoiceId, FMidiMsg::CreateNoteOff(MidiCh, MidiNote)); // Do we need a note? VoiceId should handle the note off
					MidiEvent.BlockSampleFrameIndex = 0;
					MidiEvent.AuthoredMidiTick = 0;
					MidiEvent.CurrentMidiTick = 0;
					MidiEvent.TrackIndex = 1;
					MidiStreamOutPin->InsertNoteOffEventOrCancelPendingNoteOn(MidiEvent);
				}
				PlayingVoices.Empty();
			}
		}
		else if (*CopyInactiveNoteOffsInPin)
		{
			// Merge note offs from inactive midi stream
			FMidiStream::Merge(
				*InactiveStream,
				*MidiStreamOutPin,
				[this](const FMidiStreamEvent& Event)
				{
					if (Event.MidiMessage.IsNoteOff() && PlayingVoices.Contains(Event.GetVoiceId()))
					{
						PlayingVoices.Remove(Event.GetVoiceId());
						return true;
					}

					return false;
				});
		}
	}

	void FMidiStreamSelectOperator::Reset(const FResetParams& ResetParams)
	{
		CurrentStreamIndex = -1;
		PlayingVoices.Empty();

		MidiStreamOutPin->PrepareBlock();
		MidiStreamOutPin->ResetClock();
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
