// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/Nodes/MidiStreamTrackFilterNode.h"

#include "HarmonixMetasound/MidiOps/MidiTrackFilter.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiTrackFilter
{
	using namespace Metasound;
	
	const FNodeClassName& GetClassName()
	{
		static FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
			"MidiStreamTrackFilter",
			""
		};
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 1;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Inputs::MidiStream);
		DEFINE_INPUT_METASOUND_PARAM(MinTrackIndex, "Min. Track Index", "The first track index to include (1-based, inclusive)");
		DEFINE_INPUT_METASOUND_PARAM(MaxTrackIndex, "Max. Track Index", "The last track index to include (1-based, inclusive)");
		DEFINE_INPUT_METASOUND_PARAM(IncludeConductorTrack, "Include Conductor Track", "Enable to include the conductor track (AKA track 0)");
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
	}
	
	class FMidiStreamTrackFilterOperator_V1 final : public TExecutableOperator<FMidiStreamTrackFilterOperator_V1>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = GetClassName();
				Info.MajorVersion     = 1;
				Info.MinorVersion     = 0;
				Info.DisplayName      = METASOUND_LOCTEXT("MIDIStreamTrackFilterNodeV1_DisplayName", "MIDI Stream Track Filter");
				Info.Description      = METASOUND_LOCTEXT("MIDIStreamTrackFilterNodeV1_Description", "Duplicates the incoming MIDI stream to its output after filtering tracks from the input MIDI Stream");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
					TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MinTrackIndex), 0),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MaxTrackIndex), 0),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::IncludeConductorTrack), false)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
				)
			);

			return Interface;
		}

		struct FInputs
		{
			FBoolReadRef Enabled;
			FMidiStreamReadRef MidiStream;
			FInt32ReadRef MinTrackIndex;
			FInt32ReadRef MaxTrackIndex;
			FBoolReadRef IncludeConductorTrack;
		};

		struct FOutputs
		{
			FMidiStreamWriteRef MidiStream;
		};

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(Inputs::MidiStreamName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MinTrackIndexName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(Inputs::MaxTrackIndexName, InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::IncludeConductorTrackName, InParams.OperatorSettings)
			};

			FOutputs Outputs
			{
				FMidiStreamWriteRef::CreateNew()
			};
			
			return MakeUnique<FMidiStreamTrackFilterOperator_V1>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiStreamTrackFilterOperator_V1(const FBuildOperatorParams& InParams, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
		{
			Reset(InParams);
		}

		
		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enabled);
			InVertexData.BindReadVertex(Inputs::MidiStreamName, Inputs.MidiStream);
			InVertexData.BindReadVertex(Inputs::MinTrackIndexName, Inputs.MinTrackIndex);
			InVertexData.BindReadVertex(Inputs::MaxTrackIndexName, Inputs.MaxTrackIndex);
			InVertexData.BindReadVertex(Inputs::IncludeConductorTrackName, Inputs.IncludeConductorTrack);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::MidiStreamName, Outputs.MidiStream);
		}

		void Reset(const FResetParams&)
		{
		}

		void Execute()
		{
			Filter.SetTrackRange(*Inputs.MinTrackIndex, *Inputs.MaxTrackIndex, *Inputs.IncludeConductorTrack);

			Outputs.MidiStream->PrepareBlock();

			if (*Inputs.Enabled)
			{
				Filter.Process(*Inputs.MidiStream, *Outputs.MidiStream);
			}
		}
	private:
		FInputs Inputs;
		FOutputs Outputs;
		Harmonix::Midi::Ops::FMidiTrackFilter Filter;
	};

	using FMidiStreamTrackFilterNode_V1 = Metasound::TNodeFacade<FMidiStreamTrackFilterOperator_V1>;
	METASOUND_REGISTER_NODE(FMidiStreamTrackFilterNode_V1)
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
