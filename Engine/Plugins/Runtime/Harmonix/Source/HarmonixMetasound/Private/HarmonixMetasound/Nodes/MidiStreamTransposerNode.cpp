// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiStreamTransposerNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiNoteTranspose
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName{ HarmonixNodeNamespace, TEXT("MidiStreamTransposer"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 1;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Inputs::MidiStream);
		DEFINE_METASOUND_PARAM_ALIAS(Transposition, CommonPinNames::Inputs::Transposition);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
	}

	class FMidiNoteTransposeOperator final : public TExecutableOperator<FMidiNoteTransposeOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = { HarmonixNodeNamespace, TEXT("MidiStreamTransposer"), TEXT("")};
				Info.MajorVersion     = 1;
				Info.MinorVersion     = 0;
				Info.DisplayName      = METASOUND_LOCTEXT("MIDINoteTransposeNode_DisplayName", "MIDI Note Transpose");
				Info.Description      = METASOUND_LOCTEXT("MIDINoteTransposeNode_Description", "Duplicates the incoming MIDI stream to its output with the note on/off messages transposed by the specified number of semitones.");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
					TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Transposition), 0)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
				)
			);

			return Interface;
		}

		struct FInputs
		{
			FBoolReadRef Enabled;
			FMidiStreamReadRef MidiStream;
			FInt32ReadRef Transposition;
		};

		struct FOutputs
		{
			FMidiStreamWriteRef MidiStream;
		};

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::Transposition), InParams.OperatorSettings)
			};

			FOutputs Outputs
			{
				FMidiStreamWriteRef::CreateNew()
			};

			return MakeUnique<FMidiNoteTransposeOperator>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FMidiNoteTransposeOperator(const FBuildOperatorParams& InParams, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable), Inputs.Enabled);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), Inputs.MidiStream);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Transposition), Inputs.Transposition);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiStream), Outputs.MidiStream);
		}

		void Reset(const FResetParams&)
		{
		}

		void Execute()
		{
			Outputs.MidiStream->PrepareBlock();
			
			if (!*Inputs.Enabled)
			{
				return;
			}

			const int32 Transposition = *Inputs.Transposition;

			const auto Transformer = [Transposition](const FMidiStreamEvent& InEvent) -> FMidiStreamEvent
			{
				FMidiStreamEvent NewEvent = InEvent;
				
				if (NewEvent.MidiMessage.IsNoteOn() || NewEvent.MidiMessage.IsNoteOff())
				{
					NewEvent.MidiMessage.Data1 = FMath::Clamp(NewEvent.MidiMessage.Data1 + Transposition, 0, 127);
				}
				
				return NewEvent;
			};

			FMidiStream::Copy(*Inputs.MidiStream, *Outputs.MidiStream, FMidiStream::NoOpFilter, Transformer);
		}
		
	private:
		FInputs Inputs;
		FOutputs Outputs;
	};

	using FMidiNoteTransposeNode = TNodeFacade<FMidiNoteTransposeOperator>;
	METASOUND_REGISTER_NODE(FMidiNoteTransposeNode)
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
