// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MidiStreamWriterNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/MidiOps/MidiStreamWriter.h"

#include "Misc/Paths.h"
#include "Misc/DateTime.h"
#include "HAL/FileManager.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MidiStreamWriter
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
			"MIDIStreamWriter",
			""
		};
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Inputs::MidiStream);
		DEFINE_INPUT_METASOUND_PARAM(FilenamePrefix,  "Filename Prefix", "Filename Prefix of file to write to");
	}

	class FMidiStreamWriterOperator final : public TExecutableOperator<FMidiStreamWriterOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace HarmonixMetasound;
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = GetCurrentMajorVersion();
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("MidiStreamWriterNode_DisplayName", "MIDI Writer");
				Info.Description = METASOUND_LOCTEXT("MidiStreamWriterNode_Description", "Writes the input midi stream to a standard midi file");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music}; 

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
					TInputDataVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::FilenamePrefix)),
					TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream))
				),
				FOutputVertexInterface()
			);
			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FBoolReadRef InEnabled = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings);
			FStringReadRef InFilenamePrefix = InputData.GetOrCreateDefaultDataReadReference<FString>(METASOUND_GET_PARAM_NAME(Inputs::FilenamePrefix), InParams.OperatorSettings);
			FMidiStreamReadRef InMidiStream = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), InParams.OperatorSettings);
			
			return MakeUnique<FMidiStreamWriterOperator>(InParams, InEnabled, InFilenamePrefix, InMidiStream);
		}

		FMidiStreamWriterOperator(const FBuildOperatorParams& InParams, const FBoolReadRef& InEnable, const FStringReadRef& InFilenamePrefix, const FMidiStreamReadRef& InMidiStream)
			: EnableInPin(InEnable)
			, FilenamePrefixInPin(InFilenamePrefix)
			, MidiStreamInPin(InMidiStream)
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable), EnableInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::FilenamePrefix), FilenamePrefixInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), MidiStreamInPin);
		}
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			
		}
		
		void Reset(const FResetParams& ResetParams)
		{
			MidiStreamWriter.Reset();
		}
		
		void Execute()
		{
			SetEnabled(*EnableInPin);
			if (bEnabled && MidiStreamWriter.IsValid() && MidiStreamInPin->GetClock())
			{
				MidiStreamWriter->Process(*MidiStreamInPin);
			}
		}

	private:

		void SetEnabled(bool NewEnabled)
		{
			// enable
			if (!bEnabled && NewEnabled)
			{
				const FString Filepath = GetMidiCaptureDir() / GenerateTimestampedFilename(*FilenamePrefixInPin);
				TUniquePtr<FArchive> Archive = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Filepath, IO_WRITE));
				MidiStreamWriter = MakeUnique<Harmonix::Midi::Ops::FMidiStreamWriter>(MoveTemp(Archive));
				
			}
			// disbale
			else if (bEnabled && !NewEnabled)
			{
				MidiStreamWriter.Reset();
			}
			bEnabled = NewEnabled;
		}

		static FString GetMidiCaptureDir()
		{
			static const FString MidiCaptureDir = FPaths::AudioCaptureDir() / TEXT("../MIDICaptures");
			return MidiCaptureDir;
		}

		static FString GenerateTimestampedFilename(const FString& BaseName)
		{
			static const FString Separator = TEXT("_");
			static const FString Extension = TEXT(".midi");
			const FString FileID = FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S"));
			return BaseName + Separator + FileID + Extension;
		}
		
		//** INPUTS
		FBoolReadRef EnableInPin;
		FStringReadRef FilenamePrefixInPin;
		FMidiStreamReadRef MidiStreamInPin;

		//** DATA
		bool bEnabled = false;
		TUniquePtr<Harmonix::Midi::Ops::FMidiStreamWriter> MidiStreamWriter;
	};

	using FMidiStreamWriterNode = Metasound::TNodeFacade<FMidiStreamWriterOperator>;
	METASOUND_REGISTER_NODE(FMidiStreamWriterNode);
}


#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
