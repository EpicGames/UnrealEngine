// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "MetasoundTrigger.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_MIDITextTriggerNode"

namespace HarmonixMetasound
{
	using namespace Metasound;

	namespace MidiTextTriggerPinNames
	{
		METASOUND_PARAM(TextInput, "Text", "String of characters to look for.")
		METASOUND_PARAM(TriggerOutput, "Trigger Out", "A trigger when the text is encountered.")
	}

	class FMidiTextTriggerOperator_V1 final : public TExecutableOperator<FMidiTextTriggerOperator_V1>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = { HarmonixNodeNamespace, TEXT("MidiTextTrigger"), TEXT("")};
				Info.MajorVersion     = 1;
				Info.MinorVersion     = 0;
				Info.DisplayName      = METASOUND_LOCTEXT("MIDITextTriggerNode_DisplayName", "MIDI Text Trigger");
				Info.Description      = METASOUND_LOCTEXT("MIDITextTriggerNode_Description", "Receives a MIDI stream, filters for the desired messages, and outputs triggers.");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MidiTextTriggerPinNames;
			using namespace CommonPinNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
					TInputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiStream)),
					TInputDataVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA(TextInput))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(TriggerOutput))
					)
				);

			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace MidiTextTriggerPinNames;
			using namespace CommonPinNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FBoolReadRef InEnabled          = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enable), InParams.OperatorSettings);
			FMidiStreamReadRef InMidiStream = InputData.GetOrCreateDefaultDataReadReference<FMidiStream>(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), InParams.OperatorSettings);
			FStringReadRef InText           = InputData.GetOrCreateDefaultDataReadReference<FString>(METASOUND_GET_PARAM_NAME(TextInput), InParams.OperatorSettings);

			return MakeUnique<FMidiTextTriggerOperator_V1>(InParams, InEnabled, InMidiStream, InText);
		}

		FMidiTextTriggerOperator_V1(
			const FBuildOperatorParams& InParams,
			const FBoolReadRef& InEnabled,
			const FMidiStreamReadRef& InMidiStream,
			const FStringReadRef& InText)
		: EnableInPin(InEnabled)
		, MidiStreamInPin(InMidiStream)
		, TextInPin(InText)
		, TriggerOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace MidiTextTriggerPinNames;
			using namespace CommonPinNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enable), EnableInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiStream), MidiStreamInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(TextInput), TextInPin);
		}
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace MidiTextTriggerPinNames;
			using namespace CommonPinNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(TriggerOutput), TriggerOutPin);
		}

		void Reset(const FResetParams&)
		{
			TriggerOutPin->Reset();
		}
		
		void Execute()
		{
			TriggerOutPin->AdvanceBlock();

			if (!*EnableInPin)
			{
				return;
			}
			
			const TArray<FMidiStreamEvent>& MidiEvents = MidiStreamInPin->GetEventsInBlock();
			for (auto& Event : MidiEvents)
			{
				if (!Event.MidiMessage.IsText())
				{
					continue;
				}

				const int32 TrackIndex = Event.TrackIndex;
				const FString* EventString = MidiStreamInPin->GetMidiTrackText(TrackIndex, Event.MidiMessage.GetTextIndex());
				if (EventString && EventString->Compare(*TextInPin) == 0)
				{
					TriggerOutPin->TriggerFrame(Event.BlockSampleFrameIndex);
				}
			}
		}

	protected:
		FBoolReadRef EnableInPin;
		FMidiStreamReadRef MidiStreamInPin;
		FStringReadRef TextInPin;

		FTriggerWriteRef TriggerOutPin;
	};

	
	using FMidiTextTriggerNode_V1 = Metasound::TNodeFacade<FMidiTextTriggerOperator_V1>;
	METASOUND_REGISTER_NODE(FMidiTextTriggerNode_V1)
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"

