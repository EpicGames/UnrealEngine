// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MorphingLfoNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"

#include "HarmonixDsp/Modulators/MorphingLfo.h"

#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/TimeSyncOption.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MorphingLFO
{
	using namespace Metasound;
	
	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_METASOUND_PARAM_ALIAS(LFOSyncType, CommonPinNames::Inputs::LFOSyncType);
		DEFINE_METASOUND_PARAM_ALIAS(LFOFrequency, CommonPinNames::Inputs::LFOFrequency);
		DEFINE_METASOUND_PARAM_ALIAS(LFOInvert, CommonPinNames::Inputs::LFOInvert);
		DEFINE_METASOUND_PARAM_ALIAS(LFOShape, CommonPinNames::Inputs::LFOShape);
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(LFO, "LFO Output", "The output of the LFO, range [0.0, 1.0]")
	}

	template<typename OutputDataType>
	class TOp final : public TExecutableOperator<TOp<OutputDataType>>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			auto InitVertexInterface = []() -> FVertexInterface
			{
				const Harmonix::Dsp::Modulators::FMorphingLFO LFOForDefaults{ 0 };
				
				return {
					FInputVertexInterface
					{
						TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
						TInputDataVertex<FEnumTimeSyncOption>(
							METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::LFOSyncType),
							static_cast<int32>(LFOForDefaults.SyncType.Get())),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::LFOFrequency), LFOForDefaults.Frequency.Get()),
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::LFOInvert), LFOForDefaults.Invert.Get()),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::LFOShape), LFOForDefaults.Shape.Get())
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<OutputDataType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::LFO))
					}
				};
			};
			
			static const FVertexInterface Interface = InitVertexInterface();

			return Interface;
		}
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = GetClassName<OutputDataType>();
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT_FORMAT(
					"MorphingLFO_DisplayName",
					"Morphing LFO ({0})",
					GetMetasoundDataTypeDisplayText<OutputDataType>());
				Info.Description = METASOUND_LOCTEXT("MorphingLFO_Description", "Morphing, clock-syncable LFO.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, MetasoundNodeCategories::Modulation };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		struct FInputs
		{
			TOptional<FMidiClockReadRef> MidiClock;
			FEnumTimeSyncOptionReadReference SyncType;
			FFloatReadRef Frequency;
			FBoolReadRef Invert;
			FFloatReadRef Shape;
		};

		struct FOutputs
		{
			TDataWriteReference<OutputDataType> LFO;
		};

		TOp(const FBuildOperatorParams& Params, FInputs&& InInputs, FOutputs&& InOutputs)
		: Inputs(MoveTemp(InInputs))
		, Outputs(MoveTemp(InOutputs))
		, LFO(Params.OperatorSettings.GetSampleRate())
		{
			Reset(Params);
		}

		static FOutputs CreateOutputs(const FOperatorSettings& OperatorSettings);

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FOperatorSettings& OperatorSettings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FInputs Inputs
			{
					{},
					InputData.GetOrCreateDefaultDataReadReference<FEnumTimeSyncOption>(
						Inputs::LFOSyncTypeName,
						OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<float>(
						Inputs::LFOFrequencyName,
						OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<bool>(
						Inputs::LFOInvertName,
						OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<float>(
						Inputs::LFOShapeName,
						OperatorSettings)
				};

			// MIDI clock is optional
			if (const FAnyDataReference* DataRef = InputData.FindDataReference(Inputs::MidiClockName))
			{
				Inputs.MidiClock = DataRef->GetDataReadReference<FMidiClock>();
			}

			return MakeUnique<TOp>(InParams, MoveTemp(Inputs), CreateOutputs(InParams.OperatorSettings));
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			if (Inputs.MidiClock.IsSet())
			{
				InVertexData.BindReadVertex(Inputs::MidiClockName, *Inputs.MidiClock);
			}
			InVertexData.BindReadVertex(Inputs::LFOSyncTypeName, Inputs.SyncType);
			InVertexData.BindReadVertex(Inputs::LFOFrequencyName, Inputs.Frequency);
			InVertexData.BindReadVertex(Inputs::LFOInvertName, Inputs.Invert);
			InVertexData.BindReadVertex(Inputs::LFOShapeName, Inputs.Shape);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::LFOName, Outputs.LFO);
		}

		void Reset(const IOperator::FResetParams& ResetParams)
		{
			NumFramesPerBlock = ResetParams.OperatorSettings.GetNumFramesPerBlock();
			LastTempo = 120;
			LastSpeed = 1;
			LFO.Reset(ResetParams.OperatorSettings.GetSampleRate());
			ResetImpl(ResetParams);
		}

		void Execute()
		{
			LFO.SyncType = *Inputs.SyncType;
			LFO.Frequency = *Inputs.Frequency;
			LFO.Shape = *Inputs.Shape;
			LFO.Invert = *Inputs.Invert;
			
			// if there's a clock, update tempo and process in chunks
			if (Inputs.MidiClock.IsSet())
			{
				const FMidiClockReadRef Clock = *Inputs.MidiClock;
				int32 SamplesRendered = 0;
				
				// if the tempo or speed changes, render in chunks
				for (int SampleIdx = 0; SampleIdx < NumFramesPerBlock; ++SampleIdx)
				{
					const float Tempo = Clock->GetTempoAtBlockSampleFrame(SampleIdx);
					const float Speed = Clock->GetSpeedAtBlockSampleFrame(SampleIdx);

					// Render a block if the tempo or speed changed
					if (Tempo != LastTempo || Speed != LastSpeed)
					{
						// Render
						const int32 SamplesToRender = SampleIdx - SamplesRendered;
						ExecuteSubBlock(SamplesRendered, SamplesToRender);
						SamplesRendered += SamplesToRender;

						// Update the last tempo and speed
						LastTempo = Tempo;
						LastSpeed = Speed;
					}
				}

				// if there are left over samples in the block (or there were no tempo changes), render the last bit
				if (SamplesRendered < NumFramesPerBlock)
				{
					// alias the buffers, then process
					const int32 NumSamples = NumFramesPerBlock - SamplesRendered;
					ExecuteSubBlock(SamplesRendered, NumSamples);
				}
			}
			// otherwise, process in one pass
			else
			{
				ExecuteSubBlock(0, NumFramesPerBlock);
			}
		}
		
	private:
		void ResetImpl(const IOperator::FResetParams& ResetParams);
		
		void ExecuteSubBlock(const int32 StartSample, const int32 NumSamples);
		
		int32 NumFramesPerBlock;
		FInputs Inputs;
		FOutputs Outputs;
		float LastTempo;
		float LastSpeed;
		Harmonix::Dsp::Modulators::FMorphingLFO LFO;
	};

	template <>
	TOp<float>::FOutputs TOp<float>::CreateOutputs(const FOperatorSettings& OperatorSettings)
	{
		return { FFloatWriteRef::CreateNew() };
	}

	template <>
	void TOp<float>::ResetImpl(const IOperator::FResetParams& ResetParams)
	{
		*Outputs.LFO = 0;
	}

	template <>
	void TOp<float>::ExecuteSubBlock(const int32 StartSample, const int32 NumSamples)
	{
		check(nullptr != Outputs.LFO.Get());

		if (Inputs.MidiClock.IsSet())
		{
			Harmonix::Dsp::Modulators::FMorphingLFO::FMusicTimingInfo MusicTimingInfo;
			MusicTimingInfo.Tempo = LastTempo;
			MusicTimingInfo.Speed = LastSpeed;
			MusicTimingInfo.Timestamp = (*Inputs.MidiClock)->GetMusicTimestampAtBlockOffset(StartSample);
			const FTimeSignature* TimeSigPtr = (*Inputs.MidiClock)->GetSongMapEvaluator().GetTimeSignatureAtBar(MusicTimingInfo.Timestamp.Bar);
			if (!TimeSigPtr)
			{
				MusicTimingInfo.TimeSignature = FTimeSignature(4, 4);
			}
			else
			{
				MusicTimingInfo.TimeSignature = *TimeSigPtr;
			}
			LFO.Advance(NumSamples, *Outputs.LFO.Get(), &MusicTimingInfo);
		}
		else
		{
			LFO.Advance(NumSamples, *Outputs.LFO.Get());
		}
	}

	template <>
	TOp<FAudioBuffer>::FOutputs TOp<FAudioBuffer>::CreateOutputs(const FOperatorSettings& OperatorSettings)
	{
		return { FAudioBufferWriteRef::CreateNew(OperatorSettings) };
	}
	
	template <>
	void TOp<FAudioBuffer>::ResetImpl(const IOperator::FResetParams& ResetParams)
	{
		Outputs.LFO->Zero();
	}

	template <>
	void TOp<FAudioBuffer>::ExecuteSubBlock(const int32 StartSample, const int32 NumSamples)
	{
		check(nullptr != Outputs.LFO.Get());

		float* Output = Outputs.LFO->GetData() + StartSample;

		if (Inputs.MidiClock.IsSet())
		{
			Harmonix::Dsp::Modulators::FMorphingLFO::FMusicTimingInfo MusicTimingInfo;
			MusicTimingInfo.Tempo = LastTempo;
			MusicTimingInfo.Speed = LastSpeed;
			MusicTimingInfo.Timestamp = (*Inputs.MidiClock)->GetMusicTimestampAtBlockOffset(StartSample);
			const FTimeSignature* TimeSigPtr = (*Inputs.MidiClock)->GetSongMapEvaluator().GetTimeSignatureAtBar(MusicTimingInfo.Timestamp.Bar);
			if (!TimeSigPtr)
			{
				MusicTimingInfo.TimeSignature = FTimeSignature(4, 4);
			}
			else
			{
				MusicTimingInfo.TimeSignature = *TimeSigPtr;
			}
			LFO.Advance(Output, NumSamples, &MusicTimingInfo);
		}
		else
		{
			LFO.Advance(Output, NumSamples);
		}
	}

	using FMorphingLFOFloatNode = Metasound::TNodeFacade<TOp<float>>;
	METASOUND_REGISTER_NODE(FMorphingLFOFloatNode);
	using FMorphingLFOAudioNode = Metasound::TNodeFacade<TOp<FAudioBuffer>>;
	METASOUND_REGISTER_NODE(FMorphingLFOAudioNode);
}

#undef LOCTEXT_NAMESPACE
