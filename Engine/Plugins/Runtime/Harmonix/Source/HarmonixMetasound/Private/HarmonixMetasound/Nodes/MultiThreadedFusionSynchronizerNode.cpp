// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAudioBuffer.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/FusionSyncLink.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

REGISTER_METASOUND_DATATYPE(HarmonixMetasound::FFusionSyncLink, "Fusion Sync Link")
DEFINE_LOG_CATEGORY(LogFusionAsync);

namespace HarmonixMetasound
{
	using namespace Metasound;

	namespace FusionSyncNode
	{
		struct FPinName
		{
			const TCHAR* Name;
			const FText  DisplayName;
			const FText  TooTip;
		};

		struct FNameSet
		{
			const FPinName AudioInLeft;
			const FPinName AudioInRight;
			const FPinName AudioInMono;
			const FPinName SyncIn;
			const FPinName AudioOutLeft;
			const FPinName AudioOutRight;
			const FPinName AudioOutMono;
			const FName MonoClassName;
			const FText MonoDisplayName;
			const FName StereoClassName;
			const FText StereoDisplayName;
		};

#define PIN_NAME_SET_ITEM(i,name)	\
		{{ TEXT("Sampler " #i " In L"), LOCTEXT("Sampler" #i "AudioInLeft_DisplayName",  "Sampler " #i " In Left"), LOCTEXT("Sampler" #i "AudioInLeft_ToolTip", "The left audio input from the " #name " Fusion Sampler Node.") },                       \
		 { TEXT("Sampler " #i " In R"), LOCTEXT("Sampler" #i "AudioInRight_DisplayName",  "Sampler " #i " In Right"), LOCTEXT("Sampler" #i "AudioInRight_ToolTip", "The right audio input from the " #name " Fusion Sampler Node.") },                   \
		 { TEXT("Sampler " #i " In Mono"), LOCTEXT("Sampler" #i "AudioInMono_DisplayName",  "Sampler " #i " In Mono"), LOCTEXT("Sampler" #i "AudioInMono_ToolTip", "The mono audio input from the " #name " Fusion Sampler Node.") },                    \
		 { TEXT("Sampler " #i " Sync"), LOCTEXT("Sampler" #i "SyncIn_DisplayName",  "Sampler " #i " Render Sync"), LOCTEXT("Sampler" #i "In_ToolTip", "The sync input from the " #name " Fusion Sampler Node.") },                                       \
		 { TEXT("Sampler " #i " Out L"), LOCTEXT("Sampler" #i "AudioOutLeft_DisplayName",  "Sampler " #i " Out Left"), LOCTEXT("Sampler" #i "AudioOutLeft_ToolTip", "The syncrhonized left audio output from the " #name " Fusion Sampler Node.") },     \
		 { TEXT("Sampler " #i " Out R"), LOCTEXT("Sampler" #i "AudioOutRight_DisplayName",  "Sampler " #i " Out Right"), LOCTEXT("Sampler" #i "AudioOutRight_ToolTip", "The syncrhonized right audio output from the " #name " Fusion Sampler Node.") }, \
		 { TEXT("Sampler " #i " Out Mono"), LOCTEXT("Sampler" #i "AudioOutMono_DisplayName",  "Sampler " #i " Out Mono"), LOCTEXT("Sampler" #i "AudioOutMono_ToolTip", "The syncrhonized mono audio output from the " #name " Fusion Sampler Node.") },  \
		 #i "InstanceMonoFusionSynchronizer",                                                                                                                                                                                                            \
		 LOCTEXT(#i "InstanceMonoFusionSyncrhonizerNode_DisplayName", #i " Instance Mono Fusion Synchronizer Node"),                                                                                                                                     \
		 #i "InstanceStereoFusionSynchronizer",                                                                                                                                                                                                          \
		 LOCTEXT(#i "InstanceStereoFusionSyncrhonizerNode_DisplayName", #i " Instance Stereo Fusion Synchronizer Node")                                                                                                                                  \
		 }

		static const FNameSet Names[8] = 
			{
				PIN_NAME_SET_ITEM(1, first), PIN_NAME_SET_ITEM(2, second), PIN_NAME_SET_ITEM(3, third), PIN_NAME_SET_ITEM(4, forth),
				PIN_NAME_SET_ITEM(5, fifth), PIN_NAME_SET_ITEM(6, sixth), PIN_NAME_SET_ITEM(7, seventh),PIN_NAME_SET_ITEM(8, eighth)
			};

#undef PIN_NAME_SET_ITEM	
	}

	/**********************************************************************************************************
	***********************************************************************************************************
	* TEMPLATE TMultiThreadedFusionSynchronizerOperatorBase
	***********************************************************************************************************
	*********************************************************************************************************/

	template <int32 NUM_SAMPLERS, int NUM_CHANNELS>
	class TMultiThreadedFusionSynchronizerOperator : public TExecutableOperator<TMultiThreadedFusionSynchronizerOperator<NUM_SAMPLERS,NUM_CHANNELS>>
	{
	public:
		static_assert(NUM_SAMPLERS > 1 && NUM_SAMPLERS <= 8); // Any more than 8 will require modifying the "Names" array above.  
		static_assert(NUM_CHANNELS > 0 && NUM_CHANNELS <= 2);

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
				{
					FNodeClassMetadata Info;
					Info.ClassName = { HarmonixNodeNamespace, GetMetasoundClassName(), TEXT("") };
					Info.MajorVersion = 0;
					Info.MinorVersion = 1;
					Info.DisplayName = GetDisplayName();
					Info.Description = METASOUND_LOCTEXT("FusionSynchronizerNode_Description", "Allows a set of Fusion Sampler Nodes' Execute functions to run in parallel.");
					Info.Author = PluginAuthor;
					Info.PromptIfMissing = PluginNodeMissingPrompt;
					Info.DefaultInterface = GetVertexInterface();
					Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Generators };
					return Info;
				};

			static const FNodeClassMetadata Info = InitNodeInfo(); 

			return Info;
		}

		static constexpr const FName& GetMetasoundClassName()
		{
			if constexpr (NUM_CHANNELS == 1)
			{
				return FusionSyncNode::Names[NUM_SAMPLERS - 1].MonoClassName;
			}
			else
			{
				return FusionSyncNode::Names[NUM_SAMPLERS-1].StereoClassName;
			}
		}

		static constexpr const FText& GetDisplayName()
		{
			if constexpr (NUM_CHANNELS == 1)
			{
				return FusionSyncNode::Names[NUM_SAMPLERS-1].MonoDisplayName;
			}
			else
			{
				return FusionSyncNode::Names[NUM_SAMPLERS - 1].StereoDisplayName;

			}
		}

		static FInputVertexInterface MakeInputVertexInterface()
		{
			FInputVertexInterface Interface;
			for (int32 SamplerIndex = 0; SamplerIndex < NUM_SAMPLERS; ++SamplerIndex)
			{
				const FusionSyncNode::FNameSet& PinNameSet = FusionSyncNode::Names[SamplerIndex];
				if constexpr (NUM_CHANNELS == 1)
				{
					Interface.Add(TInputDataVertex<FAudioBuffer>(PinNameSet.AudioInMono.Name, {PinNameSet.AudioInMono.TooTip, PinNameSet.AudioInMono.DisplayName}));
				}
				else
				{
					Interface.Add(TInputDataVertex<FAudioBuffer>(PinNameSet.AudioInLeft.Name, {PinNameSet.AudioInLeft.TooTip, PinNameSet.AudioInLeft.DisplayName}));
					Interface.Add(TInputDataVertex<FAudioBuffer>(PinNameSet.AudioInRight.Name, {PinNameSet.AudioInRight.TooTip, PinNameSet.AudioInRight.DisplayName}));
				}
				Interface.Add(TInputDataVertex<FFusionSyncLink>(PinNameSet.SyncIn.Name, {PinNameSet.SyncIn.TooTip, PinNameSet.SyncIn.DisplayName}));
			}
			return Interface;
		}

		static FOutputVertexInterface MakeOutputVertexInterface()
		{
			FOutputVertexInterface Interface;
			for (int32 SamplerIndex = 0; SamplerIndex < NUM_SAMPLERS; ++SamplerIndex)
			{
				const FusionSyncNode::FNameSet& PinNameSet = FusionSyncNode::Names[SamplerIndex];
				if constexpr (NUM_CHANNELS == 1)
				{
					Interface.Add(TOutputDataVertex<FAudioBuffer>(PinNameSet.AudioOutMono.Name, {PinNameSet.AudioOutMono.TooTip, PinNameSet.AudioOutMono.DisplayName}));
				}
				else
				{
					Interface.Add(TOutputDataVertex<FAudioBuffer>(PinNameSet.AudioOutLeft.Name, {PinNameSet.AudioOutLeft.TooTip, PinNameSet.AudioOutLeft.DisplayName}));
					Interface.Add(TOutputDataVertex<FAudioBuffer>(PinNameSet.AudioOutRight.Name, {PinNameSet.AudioOutRight.TooTip, PinNameSet.AudioOutRight.DisplayName}));
				}
			}
			return Interface;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(MakeInputVertexInterface(),	MakeOutputVertexInterface());
			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			FConstructionArgs ConstructionArgs(InParams);
			return MakeUnique<TMultiThreadedFusionSynchronizerOperator<NUM_SAMPLERS,NUM_CHANNELS>>(ConstructionArgs);
		}

		struct SamplerInstanceInRefs
		{
			FAudioBufferReadRef    AudioInput[NUM_CHANNELS];
			FFusionSyncLinkReadRef Sync;
		};

		struct FConstructionArgs
		{
			const FOperatorSettings* InSettings;
			TArray<SamplerInstanceInRefs> InRefs;

			FConstructionArgs(const FBuildOperatorParams& InParams)
				: InSettings(&InParams.OperatorSettings)
			{
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				for (int32 InstanceIndex = 0; InstanceIndex < NUM_SAMPLERS; ++InstanceIndex)
				{
					if constexpr (NUM_CHANNELS == 1)
					{
						SamplerInstanceInRefs InRef
						{
							{
								InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(FusionSyncNode::Names[InstanceIndex].AudioInMono.Name, InParams.OperatorSettings)
							},
							InputData.GetOrCreateDefaultDataReadReference<FFusionSyncLink>(FusionSyncNode::Names[InstanceIndex].SyncIn.Name, InParams.OperatorSettings)
						};
						InRefs.Add(MoveTemp(InRef));
					}
					else
					{
						SamplerInstanceInRefs InRef
						{
							{
								InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(FusionSyncNode::Names[InstanceIndex].AudioInLeft.Name, InParams.OperatorSettings),
								InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(FusionSyncNode::Names[InstanceIndex].AudioInRight.Name, InParams.OperatorSettings)
							},
							InputData.GetOrCreateDefaultDataReadReference<FFusionSyncLink>(FusionSyncNode::Names[InstanceIndex].SyncIn.Name, InParams.OperatorSettings)
						};
						InRefs.Add(MoveTemp(InRef));
					}
				}
			}
		};

		TMultiThreadedFusionSynchronizerOperator(const FConstructionArgs& InArgs)
		{
			RenderTasks.Reserve(NUM_SAMPLERS);
			for (int32 InstanceIndex = 0; InstanceIndex < NUM_SAMPLERS; ++InstanceIndex)
			{
				IO.Add(InArgs.InRefs[InstanceIndex]);
			}
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			for (int32 InstanceIndex = 0; InstanceIndex < NUM_SAMPLERS; ++InstanceIndex)
			{
				InVertexData.BindReadVertex(FusionSyncNode::Names[InstanceIndex].SyncIn.Name, IO[InstanceIndex].Sync);
				if constexpr (NUM_CHANNELS == 1)
				{
					InVertexData.BindReadVertex(FusionSyncNode::Names[InstanceIndex].AudioInMono.Name, IO[InstanceIndex].AudioInput[0]);
				}
				else
				{
					InVertexData.BindReadVertex(FusionSyncNode::Names[InstanceIndex].AudioInLeft.Name, IO[InstanceIndex].AudioInput[0]);
					InVertexData.BindReadVertex(FusionSyncNode::Names[InstanceIndex].AudioInRight.Name, IO[InstanceIndex].AudioInput[1]);
				}
			}


		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			for (int32 InstanceIndex = 0; InstanceIndex < NUM_SAMPLERS; ++InstanceIndex)
			{
				if constexpr (NUM_CHANNELS == 1)
				{
					InVertexData.BindReadVertex(FusionSyncNode::Names[InstanceIndex].AudioOutMono.Name, IO[InstanceIndex].AudioInput[0]);
				}
				else
				{
					InVertexData.BindReadVertex(FusionSyncNode::Names[InstanceIndex].AudioOutLeft.Name, IO[InstanceIndex].AudioInput[0]);
					InVertexData.BindReadVertex(FusionSyncNode::Names[InstanceIndex].AudioOutRight.Name, IO[InstanceIndex].AudioInput[1]);
				}
			}
		}

		void Reset(const IOperator::FResetParams& Params)
		{
			RenderTasks.Empty(NUM_SAMPLERS);
		}
		
		void Execute()
		{
			RenderTasks.Empty(NUM_SAMPLERS);
			for (int32 InstanceIndex = 0; InstanceIndex < NUM_SAMPLERS; ++InstanceIndex)
			{
				RenderTasks.Add(IO[InstanceIndex].Sync->GetTask());
			}
			UE::Tasks::Wait(RenderTasks);
		}

	private:
		TArray<SamplerInstanceInRefs> IO;
		TArray<UE::Tasks::FTask> RenderTasks;
	};

	// For now just make a subset of all of the possible configurations. 
	// It is competely fine to have unconnected inputs, so the user can 
	// use one with more inputs than they actually need. 
	template class TMultiThreadedFusionSynchronizerOperator<4,1>;
	template class TMultiThreadedFusionSynchronizerOperator<8,1>;
	template class TMultiThreadedFusionSynchronizerOperator<4,2>;
	template class TMultiThreadedFusionSynchronizerOperator<8,2>;

	/**********************************************************************************************************
	***********************************************************************************************************
	* TEMPLATE FMultiThreadedFusionSynchronizerNode
	***********************************************************************************************************
	*********************************************************************************************************/

	using F4MonoInstanceSynchronizerNode = Metasound::TNodeFacade<TMultiThreadedFusionSynchronizerOperator<4,1>>;
	using F8MonoInstanceSynchronizerNode = Metasound::TNodeFacade<TMultiThreadedFusionSynchronizerOperator<8,1>>;
	METASOUND_REGISTER_NODE(F4MonoInstanceSynchronizerNode)
	METASOUND_REGISTER_NODE(F8MonoInstanceSynchronizerNode)
	using F4StereoInstanceSynchronizerNode = Metasound::TNodeFacade<TMultiThreadedFusionSynchronizerOperator<4,2>>;
	using F8StereoInstanceSynchronizerNode = Metasound::TNodeFacade<TMultiThreadedFusionSynchronizerOperator<8,2>>;
	METASOUND_REGISTER_NODE(F4StereoInstanceSynchronizerNode)
	METASOUND_REGISTER_NODE(F8StereoInstanceSynchronizerNode)
}

#undef LOCTEXT_NAMESPACE