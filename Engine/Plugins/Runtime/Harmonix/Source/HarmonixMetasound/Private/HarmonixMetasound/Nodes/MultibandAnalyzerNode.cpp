// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/MultibandAnalyzerNode.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundEnvelopeFollowerTypes.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundStandardNodesCategories.h"

#include "DSP/EnvelopeFollower.h"
#include "DSP/Filter.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::MultibandAnalyzer
{
	constexpr int32 NumChannels = 1;

	const Metasound::FNodeClassName& GetClassName()
	{
		static const Metasound::FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
			TEXT("MultibandAnalyzer"),
			""
		};
		return ClassName;
	}

	
	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(AudioMono, CommonPinNames::Inputs::AudioMono);
		DEFINE_INPUT_METASOUND_PARAM(CrossoverFrequencies, "Crossover Frequencies", "The frequencies at which the analysis bands should be split");
		DEFINE_INPUT_METASOUND_PARAM(ApplySmoothing, "Apply Smoothing", "If enabled, will use an envelope follower on each band to smooth the output levels. Otherwise, the raw peak value for each band will be reported.");
		DEFINE_INPUT_METASOUND_PARAM(AttackTime, "Attack Time", "The amount of smoothing on rising amplitude, in seconds");
		DEFINE_INPUT_METASOUND_PARAM(ReleaseTime, "Release Time", "The amount of smoothing on falling amplitudes, in seconds");
		DEFINE_INPUT_METASOUND_PARAM(PeakMode, "Peak Mode", "The method for reporting output levels for each band");
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(BandLevels, "Band Levels", "The levels for each band");
	}

	/**
	 * Similar to FLinkwitzRileyBandSplitter, but skips a lot of phase correction that's not really
	 * necessary for audio reactivity.
	 */
	class FCheapBandSplitter final
	{
	public:
		FCheapBandSplitter(const float SampleRate, const TArray<float>& CrossoverFrequencies)
		{
			const int32 NumBands = CrossoverFrequencies.Num() + 1;
			check(NumBands >= 2);
			
			// Bandwidth to get -6dB at the corner frequency. See FLinkwitzRileyBandSplitter::GetQ for explanation.
			const float FilterBandwidth = Audio::GetBandwidthFromQ(Audio::ConvertToLinear(-3.0f));

			BandFilters.AddDefaulted(NumBands);
			
			for (int32 BandIdx = 0; BandIdx < NumBands; ++BandIdx)
			{
				// Low band is just a lowpass
				if (BandIdx == 0)
				{
					BandFilters[BandIdx].Filters.AddDefaulted(1);
					
					const float LowpassCutoff = CrossoverFrequencies[BandIdx];
					
					BandFilters[BandIdx].Filters[0].Init(
						SampleRate,
						NumChannels,
						Audio::EBiquadFilter::ButterworthLowPass,
						LowpassCutoff,
						FilterBandwidth);
				}

				// High band is just a highpass
				else if (BandIdx == NumBands - 1)
				{
					BandFilters[BandIdx].Filters.AddDefaulted(1);
					
					const float HighpassCutoff = CrossoverFrequencies[BandIdx - 1];
					
					BandFilters[BandIdx].Filters[0].Init(
						SampleRate,
						NumChannels,
						Audio::EBiquadFilter::ButterworthHighPass,
						HighpassCutoff,
						FilterBandwidth);
				}

				// Other bands have a lowpass and highpass
				else
				{
					BandFilters[BandIdx].Filters.AddDefaulted(2);

					const float LowpassCutoff = CrossoverFrequencies[BandIdx];
					
					BandFilters[BandIdx].Filters[0].Init(
						SampleRate,
						NumChannels,
						Audio::EBiquadFilter::ButterworthLowPass,
						LowpassCutoff,
						FilterBandwidth);

					const float HighpassCutoff = CrossoverFrequencies[BandIdx - 1];
					
					BandFilters[BandIdx].Filters[1].Init(
						SampleRate,
						NumChannels,
						Audio::EBiquadFilter::ButterworthHighPass,
						HighpassCutoff,
						FilterBandwidth);
				}
			}
		}

		void Process(const Metasound::FAudioBuffer& InBufferMono, TArray<Audio::FAlignedFloatBuffer>& OutBuffers)
		{
			const int32 NumBands = BandFilters.Num();
			check(NumBands >= 2);
			check(OutBuffers.Num() == NumBands);

			const int32 NumFrames = InBufferMono.Num();
			WorkBuffer.SetNumUninitialized(NumFrames);

			const float* InData = InBufferMono.GetData();

			for (int32 BandIdx = 0; BandIdx < NumBands; ++BandIdx)
			{
				check(OutBuffers[BandIdx].Num() == NumFrames);

				// If we're processing the first or last buffer, we can skip some copies
				if (BandIdx == 0 || BandIdx == NumBands - 1)
				{
					float* OutData = OutBuffers[BandIdx].GetData();
					check(BandFilters[BandIdx].Filters.Num() == 1);
					BandFilters[BandIdx].Filters[0].ProcessAudio(InData, NumFrames, OutData);
				}
				else
				{
					// copy the input to the work buffer
					float* WorkData = WorkBuffer.GetData();
					FMemory::Memcpy(WorkData, InData, NumFrames * sizeof(float));

					// process the filters
					for (Audio::FBiquadFilter& Filter : BandFilters[BandIdx].Filters)
					{
						Filter.ProcessAudio(WorkData, NumFrames, WorkData);
					}

					// copy the work buffer to the output
					float* OutData = OutBuffers[BandIdx].GetData();
					FMemory::Memcpy(OutData, WorkData, NumFrames * sizeof(float));
				}
			}
		}

	private:
		struct FBandFilter
		{
			TArray<Audio::FBiquadFilter> Filters;
		};

		Audio::FAlignedFloatBuffer WorkBuffer;
		TArray<FBandFilter> BandFilters;
	};

	class FOp final : public Metasound::TExecutableOperator<FOp>
	{
	public:
		struct FInputs
		{
			Metasound::FBoolReadRef Enable;
			Metasound::FAudioBufferReadRef Audio;
			TArray<float> CrossoverFrequencies;
			bool ApplySmoothing;
			Metasound::FTime AttackTime;
			Metasound::FTime ReleaseTime;
			Metasound::EEnvelopePeakMode PeakMode;
		};

		struct FOutputs
		{
			Metasound::TDataWriteReference<TArray<float>> BandLevels;
		};

		static const Metasound::FVertexInterface& GetVertexInterface()
		{
			const auto MakeInterface = []() -> Metasound::FVertexInterface
			{
				using namespace Metasound;
				
				const FSettings DefaultSettings;

				return
				{
					FInputVertexInterface
					{
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), DefaultSettings.Enable),
						TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudioMono)),
						TInputConstructorVertex<TArray<float>>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::CrossoverFrequencies), DefaultSettings.CrossoverFrequencies),
						TInputConstructorVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::ApplySmoothing), DefaultSettings.ApplySmoothing),
						TInputConstructorVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AttackTime), static_cast<float>(DefaultSettings.AttackTime.GetSeconds())),
						TInputConstructorVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::ReleaseTime),static_cast<float>(DefaultSettings.ReleaseTime.GetSeconds())),
						TInputConstructorVertex<FEnumEnvelopePeakMode>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::PeakMode), static_cast<int32>(DefaultSettings.PeakMode))
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<TArray<float>>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::BandLevels))
					}
				};
			};

			static const Metasound::FVertexInterface Interface = MakeInterface();
			
			return Interface;
		}

		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> Metasound::FNodeClassMetadata
			{
				Metasound::FNodeClassMetadata Info;
				Info.ClassName = { HarmonixNodeNamespace, TEXT("MultibandAnalyzer"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("MultibandAnalyzer_DisplayName", "Multiband Analyzer");
				Info.Description = METASOUND_LOCTEXT("MultibandAnalyzer_Description", "Reports levels in frequency bands for an audio signal.");
				Info.Author = Metasound::PluginAuthor;
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, MetasoundNodeCategories::Analysis };
				Info.PromptIfMissing = Metasound::PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();

				return Info;
			};

			static const Metasound::FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			FInputs Inputs
			{
				InParams.InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<Metasound::FAudioBuffer>(Inputs::AudioMonoName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultValue<TArray<float>>(Inputs::CrossoverFrequenciesName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultValue<bool>(Inputs::ApplySmoothingName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultValue<Metasound::FTime>(Inputs::AttackTimeName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultValue<Metasound::FTime>(Inputs::ReleaseTimeName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultValue<Metasound::FEnumEnvelopePeakMode>(Inputs::PeakModeName, InParams.OperatorSettings)
			};

			FOutputs Outputs
			{
				Metasound::TDataWriteReference<TArray<float>>::CreateNew()
			};
			
			return MakeUnique<FOp>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FOp(const Metasound::FBuildOperatorParams& BuildParams, FInputs&& InInputs, FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
			, BandSplitter(BuildParams.OperatorSettings.GetSampleRate(), Inputs.CrossoverFrequencies)
		{
			Reset(BuildParams);
		}

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enable);
			InVertexData.BindReadVertex(Inputs::AudioMonoName, Inputs.Audio);
			InVertexData.SetValue(Inputs::CrossoverFrequenciesName, Inputs.CrossoverFrequencies);
			InVertexData.SetValue(Inputs::ApplySmoothingName, Inputs.ApplySmoothing);
			InVertexData.SetValue(Inputs::AttackTimeName, Inputs.AttackTime);
			InVertexData.SetValue(Inputs::ReleaseTimeName, Inputs.ReleaseTime);
			InVertexData.SetValue(Inputs::PeakModeName, Metasound::FEnumEnvelopePeakMode(Inputs.PeakMode));
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::BandLevelsName, Outputs.BandLevels);
		}

		void Reset(const FResetParams& ResetParams)
		{
			const int32 NumBands = Inputs.CrossoverFrequencies.Num() + 1;

			// Init the envelope followers if enabled
			EnvelopeFollowers.Reset();
			if (Inputs.ApplySmoothing)
			{
				Audio::FEnvelopeFollowerInitParams InitParams;
				InitParams.NumChannels = NumChannels;
				InitParams.SampleRate = ResetParams.OperatorSettings.GetSampleRate();
				InitParams.AttackTimeMsec = Metasound::FTime::ToMilliseconds(Inputs.AttackTime);
				InitParams.ReleaseTimeMsec = Metasound::FTime::ToMilliseconds(Inputs.ReleaseTime);

				switch (Inputs.PeakMode)
				{
				case Metasound::EEnvelopePeakMode::Peak:
					InitParams.Mode = Audio::EPeakMode::Type::Peak;
					break;
				case Metasound::EEnvelopePeakMode::MeanSquared:
					InitParams.Mode = Audio::EPeakMode::Type::MeanSquared;
					break;
				case Metasound::EEnvelopePeakMode::RootMeanSquared:
					InitParams.Mode = Audio::EPeakMode::Type::RootMeanSquared;
					break;
				default:
					checkNoEntry();
					break;
				}
			
				for (int32 i = 0; i < NumBands; ++i)
				{
					EnvelopeFollowers.Emplace(InitParams);
				}
			}

			// Init the band buffers
			BandBuffers.Empty();
			BandBuffers.AddDefaulted(NumBands);

			// Init the output band levels
			Outputs.BandLevels->Reset();
			Outputs.BandLevels->SetNumZeroed(NumBands);
			BypassedOutputValid = true;
		}

		void Execute()
		{
			const int32 NumFrames = Inputs.Audio->Num();
			
			if (!*Inputs.Enable)
			{
				if (!BypassedOutputValid)
				{
					const int32 NumBands = Inputs.CrossoverFrequencies.Num() + 1;
					Outputs.BandLevels->Reset();
					Outputs.BandLevels->SetNumZeroed(NumBands);
					BypassedOutputValid = true;
				}
				return;
			}
			BypassedOutputValid = false;

			// Process the band splitter
			for (Audio::FAlignedFloatBuffer& Buffer : BandBuffers)
			{
				Buffer.SetNumUninitialized(NumFrames);
			}
			
			BandSplitter.Process(*Inputs.Audio, BandBuffers);

			// If enabled, for each band, find the envelope value for the block and set the output
			if (Inputs.ApplySmoothing)
			{
				for (int32 i = 0; i < EnvelopeFollowers.Num(); ++i)
				{
					EnvelopeFollowers[i].ProcessAudio(BandBuffers[i].GetData(), NumFrames);
					(*Outputs.BandLevels)[i] = EnvelopeFollowers[i].GetEnvelopeValues()[0];
				}
			}
			// Otherwise, just get the peak for each band
			else
			{
				for (int32 i = 0; i < BandBuffers.Num(); ++i)
				{
					(*Outputs.BandLevels)[i] = Audio::ArrayMaxAbsValue(BandBuffers[i]);
				}
			}
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;
		FCheapBandSplitter BandSplitter;
		TArray<Audio::FAlignedFloatBuffer> BandBuffers;
		TArray<Audio::FEnvelopeFollower> EnvelopeFollowers;
		bool BypassedOutputValid{ false };
	};

	using FMultibandAnalyzerNode = Metasound::TNodeFacade<FOp>;
	METASOUND_REGISTER_NODE(FMultibandAnalyzerNode);
}

#undef LOCTEXT_NAMESPACE
