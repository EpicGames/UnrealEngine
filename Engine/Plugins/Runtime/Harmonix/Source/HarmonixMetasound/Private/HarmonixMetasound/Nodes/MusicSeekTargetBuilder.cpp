// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MusicSeekRequest.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::BarBeatToSeekTarget
{
	using namespace Metasound;
	
	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("BarBeatToSeekTarget"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}
	
	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Bar, CommonPinNames::Inputs::Bar);
		DEFINE_METASOUND_PARAM_ALIAS(Beat, CommonPinNames::Inputs::FloatBeat);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(SeekTarget, CommonPinNames::Outputs::SeekTarget);
	}

	class FBarBeatToSeekTargetOperator : public TExecutableOperator<FBarBeatToSeekTargetOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = GetClassName();
				Info.MajorVersion     = GetCurrentMajorVersion();
				Info.MinorVersion     = 1;
				Info.DisplayName      = METASOUND_LOCTEXT("BarBeatToSeekTarget_DisplayName", "Bar & Beat To Seek Target");
				Info.Description      = METASOUND_LOCTEXT("BarBeatToSeekTarget_Description", "Build a Seek Target from a Bar and Beat.");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Bar), 1),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Beat), 1.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FMusicSeekTarget>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::SeekTarget))
				)
			);

			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FInt32ReadRef InBar = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::Bar), InParams.OperatorSettings);
			FFloatReadRef InBeat = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::Beat), InParams.OperatorSettings);

			return MakeUnique<FBarBeatToSeekTargetOperator>(InParams, InBar, InBeat);
		}

		FBarBeatToSeekTargetOperator(
			const FBuildOperatorParams& InParams,
			const FInt32ReadRef& InBar,
			const FFloatReadRef& InBeat
		)
			: Bar(InBar)
			, FloatBeat(InBeat)	 
			, SeekTargetOutPin(FMusicSeekTargetWriteRef::CreateNew())
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Bar), Bar);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Beat), FloatBeat);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::SeekTarget), SeekTargetOutPin);
		}

		void Reset(const FResetParams& ResetParams)
		{
			SeekTargetOutPin->Type = ESeekPointType::BarBeat;
			SeekTargetOutPin->BarBeat = FMusicTimestamp(1, 1.0f);
		}

		void Execute()
		{
			SeekTargetOutPin->Type = ESeekPointType::BarBeat;
			SeekTargetOutPin->BarBeat.Bar = *Bar;
			SeekTargetOutPin->BarBeat.Beat = *FloatBeat;
		}
	protected:

		FInt32ReadRef Bar;
		FFloatReadRef FloatBeat;
		FMusicSeekTargetWriteRef SeekTargetOutPin;
	};

	using FBarBeatToSeekTargetNode = Metasound::TNodeFacade<FBarBeatToSeekTargetOperator>;
	METASOUND_REGISTER_NODE(FBarBeatToSeekTargetNode);
}

namespace HarmonixMetasound::Nodes::TimeMsToSeekTarget
{
	using namespace Metasound;
	
	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("TimeMsToSeekTarget"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}
	
	namespace Inputs
	{
		DEFINE_INPUT_METASOUND_PARAM(TimeMs, "Time (Ms)", "Time in Milliseconds");
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(SeekTarget, CommonPinNames::Outputs::SeekTarget);
	}

	class FTimeMsToSeekTargetOperator : public TExecutableOperator<FTimeMsToSeekTargetOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = GetClassName();
				Info.MajorVersion     = GetCurrentMajorVersion();
				Info.MinorVersion     = 1;
				Info.DisplayName      = METASOUND_LOCTEXT("TimeMsToSeekTarget_DisplayName", "Time (Ms) To Seek Target");
				Info.Description      = METASOUND_LOCTEXT("TimeMsToSeekTarget_Description", "Build a Seek Target from Time in Milliseconds.");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Music);
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TimeMs), 0.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FMusicSeekTarget>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::SeekTarget))
				)
			);

			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FFloatReadRef InTimeMs = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::TimeMs), InParams.OperatorSettings);

			return MakeUnique<FTimeMsToSeekTargetOperator>(InParams, InTimeMs);
		}

		FTimeMsToSeekTargetOperator(
			const FBuildOperatorParams& InParams,
			const FFloatReadRef& InTimeMs
		)
			: TimeMsInPin(InTimeMs)
			, SeekTargetOutPin(FMusicSeekTargetWriteRef::CreateNew())
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TimeMs), TimeMsInPin);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::SeekTarget), SeekTargetOutPin);
		}

		void Reset(const FResetParams& ResetParams)
		{
			SeekTargetOutPin->Type = ESeekPointType::Millisecond;
			SeekTargetOutPin->Ms = 0.0f;
		}

		void Execute()
		{
			SeekTargetOutPin->Type = ESeekPointType::Millisecond;
			SeekTargetOutPin->Ms = *TimeMsInPin;
		}
	protected:

		FFloatReadRef TimeMsInPin;
		FMusicSeekTargetWriteRef SeekTargetOutPin;
	};

	using FTimeMsToSeekTargetNode = Metasound::TNodeFacade<FTimeMsToSeekTargetOperator>;
	METASOUND_REGISTER_NODE(FTimeMsToSeekTargetNode);
};

namespace HarmonixMetasound::Nodes::MusicTimestampToSeekTarget
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("MusicTimestampToSeekTarget"), TEXT("") };
		return ClassName;
	}

	int32 GetMajorVersion()
	{
		return 0;
	}
	
	namespace Inputs
	{
		DEFINE_INPUT_METASOUND_PARAM(MusicTimestamp, "Music Timestamp", "A single structure representing a musical time (bar & beat).");
	}

	namespace Outputs
	{
		 DEFINE_METASOUND_PARAM_ALIAS(SeekTarget, CommonPinNames::Outputs::SeekTarget);
	}
	
	class FMusicTimestampToSeekTargetOperator : public TExecutableOperator<FMusicTimestampToSeekTargetOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName        = GetClassName();
				Info.MajorVersion     = GetMajorVersion();
				Info.MinorVersion     = 1;
				Info.DisplayName      = METASOUND_LOCTEXT("MusicTimestampToSeekTarget_DisplayName", "Music Timestamp to Seek Target");
				Info.Description      = METASOUND_LOCTEXT("MusicTimestampToSeekTarget_Description", "Build a Seek Target from a Music Timestamp.");
				Info.Author           = PluginAuthor;
				Info.PromptIfMissing  = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = {MetasoundNodeCategories::Harmonix, NodeCategories::Music};
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FMusicTimestamp>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MusicTimestamp))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FMusicSeekTarget>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::SeekTarget))
				)
			);

			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FMusicTimestampReadRef InMusicTimestamp = InputData.GetOrCreateDefaultDataReadReference<FMusicTimestamp>(METASOUND_GET_PARAM_NAME(Inputs::MusicTimestamp), InParams.OperatorSettings);

			return MakeUnique<FMusicTimestampToSeekTargetOperator>(InParams, InMusicTimestamp);
		}

		FMusicTimestampToSeekTargetOperator(
			const FBuildOperatorParams& InParams,
			const FMusicTimestampReadRef& InMusicTimestamp
		)
			: MusicTimestampInPin(InMusicTimestamp)
			, SeekTargetOutPin(FMusicSeekTargetWriteRef::CreateNew())
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MusicTimestamp), MusicTimestampInPin);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::SeekTarget), SeekTargetOutPin);
		}

		void Reset(const FResetParams& ResetParams)
		{
			SeekTargetOutPin->Type = ESeekPointType::BarBeat;
			SeekTargetOutPin->BarBeat = FMusicTimestamp(1, 1.0f);
		}

		void Execute()
		{
			SeekTargetOutPin->Type = ESeekPointType::BarBeat;
			SeekTargetOutPin->BarBeat = *MusicTimestampInPin;
		}
	protected:

		FMusicTimestampReadRef MusicTimestampInPin;
		FMusicSeekTargetWriteRef SeekTargetOutPin;
	};
	
	using FMusicTimestampToSeekTargetNode = Metasound::TNodeFacade<FMusicTimestampToSeekTargetOperator>;
	METASOUND_REGISTER_NODE(FMusicTimestampToSeekTargetNode);
	
};

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"