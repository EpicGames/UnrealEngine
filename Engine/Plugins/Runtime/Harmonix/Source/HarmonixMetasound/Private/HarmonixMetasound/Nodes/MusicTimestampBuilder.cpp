// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "MetasoundTriggerCompareNode.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound
{
	using namespace Metasound;

	using FTriggerCompareNodeMusicTimestamp = TTriggerCompareNode<FMusicTimestamp>;
	METASOUND_REGISTER_NODE(FTriggerCompareNodeMusicTimestamp)

	class FMusicTimestampBuilderOperator : public TExecutableOperator<FMusicTimestampBuilderOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FMusicTimestampBuilderOperator(
			const FBuildOperatorParams& InParams,
			const FInt32ReadRef& InBar,
			const FFloatReadRef& InBeat);

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		void Reset(const FResetParams& ResetParams);

		void Execute();

	private:
		//** INPUTS
		FInt32ReadRef BarInPin;
		FFloatReadRef BeatInPin;

		//** OUTPUTS
		FMusicTimestampWriteRef MusicTimeStampOutPin;
	};

	using FMusicTimestampBuilderNode = Metasound::TNodeFacade<FMusicTimestampBuilderOperator>;
	METASOUND_REGISTER_NODE(FMusicTimestampBuilderNode)

	const FNodeClassMetadata& FMusicTimestampBuilderOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = { HarmonixNodeNamespace, TEXT("MusicTimestampBuilder"), TEXT("") };
			Info.MajorVersion     = 0;
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("MusicTimestampBuilderNode_DisplayName", "Music Timestamp Builder");
			Info.Description      = METASOUND_LOCTEXT("MusicTimestampBuilderNode_Description", "Combines the input Bar and Beat into a single Music Timestamp structure.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	const FVertexInterface& FMusicTimestampBuilderOperator::GetVertexInterface()
	{
		using namespace CommonPinNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Bar), 1),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::FloatBeat), 1.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FMusicTimestamp>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MusicTimestamp))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FMusicTimestampBuilderOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace CommonPinNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FInt32ReadRef InBar  = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(Inputs::Bar), InParams.OperatorSettings);
		FFloatReadRef InBeat = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::FloatBeat), InParams.OperatorSettings);

		return MakeUnique<FMusicTimestampBuilderOperator>(InParams, InBar, InBeat);
	}

	FMusicTimestampBuilderOperator::FMusicTimestampBuilderOperator(
		const FBuildOperatorParams& InParams,
		const FInt32ReadRef& InBar,
		const FFloatReadRef& InBeat)
	: BarInPin(InBar)
	, BeatInPin(InBeat)
	, MusicTimeStampOutPin(FMusicTimestampWriteRef::CreateNew())
	{
		Reset(InParams);
	}

	void FMusicTimestampBuilderOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Bar), BarInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::FloatBeat), BeatInPin);
	}

	void FMusicTimestampBuilderOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;

		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MusicTimestamp), MusicTimeStampOutPin);
	}

	void FMusicTimestampBuilderOperator::Reset(const FResetParams& ResetParams)
	{
		MusicTimeStampOutPin->Reset();
	}

	void FMusicTimestampBuilderOperator::Execute()
	{
		MusicTimeStampOutPin->Bar = *BarInPin;
		MusicTimeStampOutPin->Beat = *BeatInPin;
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
