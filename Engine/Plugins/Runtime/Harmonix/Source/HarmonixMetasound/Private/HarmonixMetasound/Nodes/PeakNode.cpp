// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/PeakNode.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "DSP/FloatArrayMath.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::Peak
{
	const Metasound::FNodeClassName& GetClassName()
	{
		static Metasound::FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
			"Peak",
			""
		};
		return ClassName;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable)
		DEFINE_METASOUND_PARAM_ALIAS(AudioMono, CommonPinNames::Inputs::AudioMono);
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(Peak, "Peak", "The peak for the latest block")
	}

	class FOp final : public Metasound::TExecutableOperator<FOp>
	{
	public:
		static const Metasound::FVertexInterface& GetVertexInterface()
		{
			auto InitVertexInterface = []() -> Metasound::FVertexInterface
			{
				using namespace Metasound;
				
				return {
					FInputVertexInterface
					{
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
						TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudioMono))
					},
					Metasound::FOutputVertexInterface
					{
						TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Peak))
					}
				};
			};
			
			static const Metasound::FVertexInterface Interface = InitVertexInterface();
			return Interface;
		}
		
		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> Metasound::FNodeClassMetadata
			{
				Metasound::FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("Peak_DisplayName", "Peak");
				Info.Description = METASOUND_LOCTEXT("Peak_Description", "Reports the peak for an audio signal");
				Info.Author = Metasound::PluginAuthor;
				Info.PromptIfMissing = Metasound::PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, MetasoundNodeCategories::Analysis };
				return Info;
			};

			static const Metasound::FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		struct FInputs
		{
			Metasound::FAudioBufferReadRef Audio;
			Metasound::FBoolReadRef Enable;
		};

		struct FOutputs
		{
			Metasound::FFloatWriteRef Peak;
		};

		static TUniquePtr<IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			FInputs Inputs
			{
				InParams.InputData.GetOrCreateDefaultDataReadReference<Metasound::FAudioBuffer>(
					Inputs::AudioMonoName,
					InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<bool>(
					Inputs::EnableName,
					InParams.OperatorSettings)
			};

			FOutputs Outputs
			{
				Metasound::FFloatWriteRef::CreateNew()
			};

			return MakeUnique<FOp>(InParams, MoveTemp(Inputs), MoveTemp(Outputs));
		}

		FOp(const Metasound::FBuildOperatorParams& Params, FInputs&& InInputs, FOutputs&& InOutputs)
		: Inputs(MoveTemp(InInputs))
		, Outputs(MoveTemp(InOutputs))
		{
			Reset(Params);
		}

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::AudioMonoName, Inputs.Audio);
			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enable);

			UpdatePeak();
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::PeakName, Outputs.Peak);
		}

		void Reset(const FResetParams&)
		{
			*Outputs.Peak = 0.0f;
		}

		void Execute()
		{
			UpdatePeak();
		}

	private:
		void UpdatePeak()
		{
			if (*Inputs.Enable)
			{
				*Outputs.Peak = Audio::ArrayMaxAbsValue(*Inputs.Audio);
			}
			else
			{
				*Outputs.Peak = 0.0f;
			}
		}
		
		FInputs Inputs;
		FOutputs Outputs;
	};

	using FPeakNode = Metasound::TNodeFacade<FOp>;
	METASOUND_REGISTER_NODE(FPeakNode);
}

#undef LOCTEXT_NAMESPACE
