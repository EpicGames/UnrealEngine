// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/SecsUntilMusicTimestampNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "MetasoundTrigger.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"
#include "MetasoundEnumRegistrationMacro.h"

#include <limits>

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::SecsUntilMusicTimestampNode
{
	using namespace Metasound;
	
	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
				TEXT("TimeUntilMusicTimestampNode"),
				""
		};
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 1;
	}
	
	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enable, CommonPinNames::Inputs::Enable);
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_METASOUND_PARAM_ALIAS(Timestamp, CommonPinNames::Inputs::Timestamp);
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(SecsUntilTimestamp, "Secs. Until Timestamp", "Looks at the MIDI clock and calculates how long it will be until the timestamp triggers. It DOES consider the speed of the clock as well!");
	}

	class FOp final : public TExecutableOperator<FOp>
	{
	public:
		struct FInputs
		{
			FBoolReadRef Enable;
			FMidiClockReadRef MidiClock;
			FMusicTimestampReadRef Timestamp;
		};

		struct FOutputs
		{
			FFloatWriteRef SecsUntilTimestamp;
		};

		static const FVertexInterface& GetVertexInterface()
		{
			const auto MakeInterface = []() -> FVertexInterface
			{
				using namespace Metasound;

				return
				{
					FInputVertexInterface
					{
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enable), true),
						TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
						TInputDataVertex<FMusicTimestamp>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Timestamp))
					},
					FOutputVertexInterface
					{
						TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::SecsUntilTimestamp))
					}
				};
			};

			static const FVertexInterface Interface = MakeInterface();

			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("SecsUntilMusicTimestampNode_DisplayName", "Seconds Until Music Timestamp");
				Info.Description = METASOUND_LOCTEXT("SecsUntilMusicTimestampNode_Description", "Given a MIDI Clock and a Music Timestamp, calculates the number of seconds until that timestamp is reached. NOTE: It DOES take the clock's speed into account.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FOperatorSettings& OperatorSettings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			bool bClockIsBound = InputData.IsVertexBound(Inputs::MidiClockName);

			FInputs Inputs
			{
				InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnableName,OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(Inputs::MidiClockName, OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FMusicTimestamp>(Inputs::TimestampName, OperatorSettings)
			};

			FOutputs Outputs
			{
				FFloatWriteRef::CreateNew(Never)
			};

			return MakeUnique<FOp>(InParams, MoveTemp(Inputs), bClockIsBound, MoveTemp(Outputs));
		}

		FOp(const FBuildOperatorParams& Params, FInputs&& InInputs, bool bClockIsBoundIn,  FOutputs&& InOutputs)
			: Inputs(MoveTemp(InInputs))
			, Outputs(MoveTemp(InOutputs))
			, bClockIsBound(bClockIsBoundIn)
		{
			Reset(Params);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			// See if the clock input is connected to anything...
			bClockIsBound = InVertexData.IsVertexBound(Inputs::MidiClockName);

			InVertexData.BindReadVertex(Inputs::EnableName, Inputs.Enable);
			InVertexData.BindReadVertex(Inputs::MidiClockName, Inputs.MidiClock);
			InVertexData.BindReadVertex(Inputs::TimestampName, Inputs.Timestamp);
			
			// Force a recompute of the time...
			MsOfTimestamp = Never;
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::SecsUntilTimestampName, Outputs.SecsUntilTimestamp);
		}

		void Reset(const FResetParams& ResetParams)
		{
			*Outputs.SecsUntilTimestamp = std::numeric_limits<float>::max();
		}

		void Execute()
		{
			if (!*Inputs.Enable || !bClockIsBound)
			{
				*Outputs.SecsUntilTimestamp = Never;
				return;
			}

			const ISongMapEvaluator& SongMaps = Inputs.MidiClock->GetSongMapEvaluator();

			if (MsOfTimestamp == Never || *Inputs.Timestamp != CurrentTimestamp || Inputs.MidiClock->GetSongMapsChangedInBlock())
			{
				int32 Tick = SongMaps.MusicTimestampToTick(*Inputs.Timestamp);
				MsOfTimestamp = SongMaps.TickToMs((float)Tick);
			}

			float Speed = Inputs.MidiClock->GetSpeedAtStartOfBlock();
			if (FMath::IsNearlyZero(Speed))
			{
				*Outputs.SecsUntilTimestamp = Never;
				return;
			}

			const int32 SongTick = Inputs.MidiClock->GetNextMidiTickToProcess();
			float SongPosMs = SongMaps.TickToMs((float)SongTick);

			*Outputs.SecsUntilTimestamp = ((MsOfTimestamp - SongPosMs) / 1000.0f) / Speed;
		}

	private:
		static constexpr float Never = std::numeric_limits<float>::max();

		FInputs Inputs;
		FOutputs Outputs;
		bool bClockIsBound = false;
		FMusicTimestamp CurrentTimestamp;
		float MsOfTimestamp = Never;
	};

	using FSecsUntilMusicTimestampNode = Metasound::TNodeFacade<FOp>;
	METASOUND_REGISTER_NODE(FSecsUntilMusicTimestampNode);
}

#undef LOCTEXT_NAMESPACE