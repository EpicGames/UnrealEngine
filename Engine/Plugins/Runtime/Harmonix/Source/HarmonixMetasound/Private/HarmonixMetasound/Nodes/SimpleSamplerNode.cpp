// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/SimpleSamplerNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixDsp/AudioBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleSamplerNode)

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::SimpleSamplerNode
{
	using namespace Metasound;

	/**
	 * Maximum duration in seconds for capture and playback
	 */
	static constexpr float kMaxSampleSeconds = 10.0f;

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enabled, CommonPinNames::Inputs::Enable);
		DEFINE_INPUT_METASOUND_PARAM(Capture, "Capture", "When triggered, starts capturing the input audio to a buffer.");
		DEFINE_INPUT_METASOUND_PARAM(CaptureDuration, "Capture Duration Sec", "How long a sample to capture.");
		DEFINE_INPUT_METASOUND_PARAM(Play, "Play", "Triggers playback of the captured buffer.");
		DEFINE_INPUT_METASOUND_PARAM(PlayDuration, "Play Duration Sec", "How much of the captured sample to play.");
		DEFINE_INPUT_METASOUND_PARAM(Reverse, "Reverse Playback", "If true the captured buffer is played in reverse.");
		DEFINE_INPUT_METASOUND_PARAM(Reset, "Reset", "Stop capture/playback and start passing audio from input to output.");
		static constexpr int32 kNumNonAudioInputs = 7; // NOTE: Keep in sync with the number of input vertecies defined above!
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(InputAudioInUse, "Input Audio In Use", "True when audio from the input is being used to generate the output. False when the input audio is unneeded and can be changed.");
	}

	const FLazyName FSimpleSamplerNodeOperatorData::OperatorDataTypeName = "SimpleSamplerOperatorData";

	static TArray<FVertexName> CreateVertexNames(const TCHAR* Prefix, int32 NumChannels)
	{
		TStringBuilder<32> NameStr;
		TArray<FVertexName> Names;
		Names.AddUninitialized(NumChannels);

		for (int ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			NameStr << Prefix << ChannelIndex;
			Names[ChannelIndex] = *NameStr;
			NameStr.Reset();
		}

		return Names;
	}

	static const TArray<FVertexName> AudioInputNames = CreateVertexNames(TEXT("In "), kMaxChannels);
	static const TArray<FVertexName> AudioOutputNames = CreateVertexNames(TEXT("Out "), kMaxChannels);

#if WITH_EDITOR
	static const FText GetAudioInputDisplayName(uint32 ChannelIndex)
	{
		return METASOUND_LOCTEXT_FORMAT("AudioInputDisplayName", "In {0}", ChannelIndex);
	}

	static const FText GetAudioOutputDisplayName(uint32 ChannelIndex)
	{
		return METASOUND_LOCTEXT_FORMAT("AudioOutputDisplayName", "Out {0}", ChannelIndex);
	}
#endif // WITH_EDITOR

	FVertexInterface GetVertexInterface(int32 NumChannels)
	{
		using namespace HarmonixMetasound::SimpleSamplerNode;

		FInputVertexInterface InputInterface;
		FOutputVertexInterface OutputInterface;

		InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enabled)));
		InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Capture)));
		InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::CaptureDuration)));
		InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Play)));
		InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::PlayDuration)));
		InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Reverse)));
		InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Reset)));

		OutputInterface.Add(TOutputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::InputAudioInUse)));

		check (InputInterface.Num() == Inputs::kNumNonAudioInputs);
		check (NumChannels <= kMaxChannels);

		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
#if WITH_EDITOR
			const FDataVertexMetadata AudioInputMetadata
			{
				METASOUND_LOCTEXT("AudioInputDescription", "Audio Input"),
				GetAudioInputDisplayName(ChannelIndex)
			};
#else 
			const FDataVertexMetadata AudioInputMetadata;
#endif // WITH_EDITOR
			InputInterface.Add(TInputDataVertex<FAudioBuffer>(AudioInputNames[ChannelIndex], AudioInputMetadata));

#if WITH_EDITOR
			const FDataVertexMetadata AudioOutputMetadata
			{
				METASOUND_LOCTEXT("AudioOutputDescription", "Audio Output"),
				GetAudioOutputDisplayName(ChannelIndex)
			};
#else 
			const FDataVertexMetadata AudioOutputMetadata;
#endif // WITH_EDITOR
			OutputInterface.Add(TOutputDataVertex<FAudioBuffer>(AudioOutputNames[ChannelIndex], AudioOutputMetadata));
		}

		return FVertexInterface
		{
			MoveTemp(InputInterface),
			MoveTemp(OutputInterface)
		};
	}

	class FSimpleSamplerOperator final : public TExecutableOperator<FSimpleSamplerOperator>
	{
	public:
		struct FInputs
		{
			FBoolReadRef        Enabled;
			FTriggerReadRef     Capture;
			FFloatReadRef       CaptureDuration;
			FTriggerReadRef     Play;
			FFloatReadRef       PlayDuration;
			FBoolReadRef        Reverse;
			FTriggerReadRef     Reset;
			TArray<FAudioBufferReadRef> AudioInputs;
		};

		struct FOutputs
		{
			FBoolWriteRef        InputAudioInUse;
			TArray<FAudioBufferWriteRef> AudioOutputs;
		};
		
		FSimpleSamplerOperator(const FBuildOperatorParams& Params, FInputs&& Inputs, FOutputs&& Outputs)
			: Inputs(MoveTemp(Inputs))
			, Outputs(MoveTemp(Outputs))
		{
			TSharedPtr<const FSimpleSamplerNodeOperatorData> InOperatorData = StaticCastSharedPtr<const FSimpleSamplerNodeOperatorData>(Params.Node.GetOperatorData());
			MaxCaptureSeconds = InOperatorData ? FMath::Min(InOperatorData->MaxCaptureSeconds, kMaxSampleSeconds) : 5.0f;
			Reset(Params);
		}
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = FNodeClassMetadata::GetEmpty();
				Info.ClassName = { HarmonixNodeNamespace, TEXT("SimpleSampler"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("SimpleSampler_DisplayName", "Simple Sampler");
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix };
				Info.Description = METASOUND_LOCTEXT("SimpleSampler_Description", "Allows you to capture a buffer of incoming audio and then trigger its playback.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface(1);
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FVertexInterface& NodeInterface = InParams.Node.GetVertexInterface();
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TArray<FAudioBufferReadRef> AudioInputs;
			TArray<FAudioBufferWriteRef> AudioOutputs;
			int32 NumAudioChannels = NodeInterface.GetInputInterface().Num() - Inputs::kNumNonAudioInputs;
			for (int32 ChannelIndex = 0; ChannelIndex < NumAudioChannels; ++ChannelIndex)
			{
				AudioInputs.Emplace(InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(AudioInputNames[ChannelIndex], InParams.OperatorSettings));
				AudioOutputs.Emplace(TDataWriteReferenceFactory<FAudioBuffer>::CreateExplicitArgs(InParams.OperatorSettings));
			}

			FInputs Inputs {
					InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnabledName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::CaptureName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<float>(Inputs::CaptureDurationName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::PlayName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<float>(Inputs::PlayDurationName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::ReverseName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::ResetName, InParams.OperatorSettings),
					MoveTemp(AudioInputs)
			};

			FOutputs Outputs {	
					FBoolWriteRef::CreateNew(),
					MoveTemp(AudioOutputs)
			};

			return MakeUnique<FSimpleSamplerOperator>(
				InParams,
				MoveTemp(Inputs),
				MoveTemp(Outputs)
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnabledName, Inputs.Enabled);
			InVertexData.BindReadVertex(Inputs::CaptureName, Inputs.Capture);
			InVertexData.BindReadVertex(Inputs::CaptureDurationName, Inputs.CaptureDuration);
			InVertexData.BindReadVertex(Inputs::PlayName, Inputs.Play);
			InVertexData.BindReadVertex(Inputs::PlayDurationName, Inputs.PlayDuration);
			InVertexData.BindReadVertex(Inputs::ReverseName, Inputs.Reverse);
			InVertexData.BindReadVertex(Inputs::ResetName, Inputs.Reset);
			for (int32 ChannelIndex = 0; ChannelIndex < Inputs.AudioInputs.Num(); ++ChannelIndex)
			{
				InVertexData.BindReadVertex(AudioInputNames[ChannelIndex], Inputs.AudioInputs[ChannelIndex]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::InputAudioInUseName, Outputs.InputAudioInUse);
			for (int32 ChannelIndex = 0; ChannelIndex < Outputs.AudioOutputs.Num(); ++ChannelIndex)
			{
				InVertexData.BindReadVertex(AudioOutputNames[ChannelIndex], Outputs.AudioOutputs[ChannelIndex]);
			}
		}
		
		void Reset(const FResetParams& Params)
		{
			int32 NumFrames = FMath::CeilToInt32(Params.OperatorSettings.GetSampleRate() * MaxCaptureSeconds);
			CaptureBuffer.Configure(Inputs.AudioInputs.Num(), NumFrames, EAudioBufferCleanupMode::Delete, Params.OperatorSettings.GetSampleRate(), false);
			CaptureBuffer.SetNumValidFrames(0);
			CaptureBuffer.SetNumValidChannels(Inputs.AudioInputs.Num());
			bCurrentlyPlaying = bCurrentlyCapturing = bFullyCaptured = false;
			bPlayingReverse = bHavePlayedThisCapture = false;
			CaptureLengthFrames = PlayLengthFrames = NextPlayFrame = 0;
			*Outputs.InputAudioInUse = false;
			for (FAudioBufferWriteRef& Out : Outputs.AudioOutputs)
			{
				Out->Zero();
			}
		}

		enum class EInputOrSilence
		{
			Input,
			Silence
		};

		void BulkOutput(EInputOrSilence Mode, int32 StartFrameIndexInBlock = 0, int32 NumFrames = -1)
		{
			StartFrameIndexInBlock = FMath::Max(StartFrameIndexInBlock, 0);

			if (NumFrames == -1)
			{
				NumFrames = Outputs.AudioOutputs[0]->Num() - StartFrameIndexInBlock;
			}
			else
			{
				NumFrames = FMath::Min(NumFrames, Outputs.AudioOutputs[0]->Num() - StartFrameIndexInBlock);
			}

			if (NumFrames <= 0)
			{
				return;
			}

			if (Mode == EInputOrSilence::Input)
			{
				*Outputs.InputAudioInUse = true;
				for (int32 ChannelIndex = 0; ChannelIndex < Outputs.AudioOutputs.Num(); ++ChannelIndex)
				{
					FMemory::Memcpy(Outputs.AudioOutputs[ChannelIndex]->GetData() + StartFrameIndexInBlock, Inputs.AudioInputs[ChannelIndex]->GetData() + StartFrameIndexInBlock, sizeof(float) * NumFrames);
				}
			}
			else
			{
				for (int32 ChannelIndex = 0; ChannelIndex < Outputs.AudioOutputs.Num(); ++ChannelIndex)
				{
					FMemory::Memset(Outputs.AudioOutputs[ChannelIndex]->GetData() + StartFrameIndexInBlock, 0, sizeof(float) * NumFrames);
				}
			}
		}

		void Execute()
		{
			bool Holding = Inputs.Capture->IsTriggeredInBlock() || CaptureBuffer.GetNumValidFrames() > 0;
			if (!*Inputs.Enabled || !Holding)
			{
				BulkOutput(EInputOrSilence::Input);
				return;
			}

			float CaptureDurationSec = FMath::Clamp(*Inputs.CaptureDuration, 0.0f, kMaxSampleSeconds);
			CaptureLengthFrames = FMath::Clamp(FMath::CeilToInt32(CaptureDurationSec * CaptureBuffer.GetMaxConfig().GetSampleRate()), 0, CaptureBuffer.GetMaxNumFrames());
			if (CaptureLengthFrames <= CaptureBuffer.GetNumValidFrames())
			{
				// crop our capture buffer...
				CaptureBuffer.SetNumValidFrames(CaptureLengthFrames);
				// we may still be capturing... so stop...
				bCurrentlyCapturing = false;
				bFullyCaptured = true;
				*Outputs.InputAudioInUse = false;
			}

			bool bRenderedOutput = false;

			if (Inputs.Capture->IsTriggeredInBlock() || bCurrentlyCapturing)
			{
				bFullyCaptured = false;

				// We pass in a where/when a play is ALSO triggered so the capture code knows
				// whether to pass input -> output or let the play call do it...
				bRenderedOutput = DoCapture(Inputs.Play->Last());
				
				// bFullyCaptured could have changed inside of DoCapture... so...
				*Outputs.InputAudioInUse = !bFullyCaptured; 
			}

			if (Inputs.Play->IsTriggeredInBlock() || bCurrentlyPlaying)
			{
				bHavePlayedThisCapture = true;

				if (Inputs.Play->IsTriggeredInBlock()) // a new trigger in this block...
				{
					int32 NewPlayPosition = Inputs.Play->Last();

					// Are we still playing a previous trigger?
					if (bCurrentlyPlaying)
					{
						NewPlayPosition = FMath::Max(Inputs.Play->Last(), kNumFadeFrames);
						DoPlay(0, NewPlayPosition, true);
						bRenderedOutput = true;
					}
					else
					{
						// No... we MAY want to write silence before this new trigger starts...
						if (bFullyCaptured && NewPlayPosition > 0)
						{
							BulkOutput(EInputOrSilence::Silence, 0, NewPlayPosition);
						}
					}

					// Since we are starting... capture any new length at our input...
					float PlayDurationSec = FMath::Clamp(*Inputs.PlayDuration, 0.0f, kMaxSampleSeconds);
					PlayLengthFrames = FMath::Clamp(FMath::CeilToInt32(PlayDurationSec * CaptureBuffer.GetMaxConfig().GetSampleRate()), 0, CaptureLengthFrames);
					// ... and whether we should be playing backward...
					if (*Inputs.Reverse)
					{
						// We can only play in reverse if we have captured enough samples!
						if (CaptureBuffer.GetNumValidFrames() >= PlayLengthFrames)
						{
							bPlayingReverse = true;
						}
						else
						{
							// We'll take accept anything within a 2 milliseconds...
							int32 ErrorFrames = PlayLengthFrames - CaptureBuffer.GetNumValidFrames();
							bPlayingReverse = ErrorFrames < (CaptureBuffer.GetMaxConfig().GetSampleRate() * 2 / 1000);
							if (bPlayingReverse)
							{
								PlayLengthFrames = CaptureBuffer.GetNumValidFrames();
							}
						}
					}
					else
					{
						bPlayingReverse = false;
					}

					// We will be starting (or restarting) so set up some things...
					bCurrentlyPlaying = false;
					NextPlayFrame = 0;
					int32 NumSamplesToRenderThisBlock = Outputs.AudioOutputs[0]->Num() - NewPlayPosition;

					// Now render...
					bRenderedOutput = DoPlay(NewPlayPosition, NumSamplesToRenderThisBlock) || bRenderedOutput ;
				}
				else // we must be currently playing... so continue...
				{
					bRenderedOutput = DoPlay(0, Outputs.AudioOutputs[0]->Num());
				}
			}

			// If we didn't render anything, and we are fully captured
			if (!bRenderedOutput && (bFullyCaptured || bHavePlayedThisCapture))
			{
				for (FAudioBufferWriteRef& Out : Outputs.AudioOutputs)
				{
					Out->Zero();
				}
			}

			if (Inputs.Reset->IsTriggeredInBlock())
			{
				ResetCaptureAndPlayback(Inputs.Reset->First());
			}
		}

		/**
		Blows out the capture buffer, stops current playback, and starts piping
		input -> output.
		@param FromFrame The position of the reset within the block.
		*/
		void ResetCaptureAndPlayback(int32 FromFrame)
		{
			bCurrentlyPlaying = bCurrentlyCapturing = bFullyCaptured = bHavePlayedThisCapture = false;
			NextPlayFrame = 0;
			CaptureBuffer.SetNumValidFrames(0);
			// Make sure we have at least enough frames to fade in...
			if (Outputs.AudioOutputs[0]->Num() - FromFrame < kNumFadeFrames)
			{
				FromFrame = Outputs.AudioOutputs[0]->Num() - kNumFadeFrames;
			}
			// Now write un-faded frames...
			BulkOutput(EInputOrSilence::Input, FromFrame);
			// Now fade...
			float Fade = 0.0f;
			for (int32 FadeFrame = 0; FadeFrame < kNumFadeFrames; ++FadeFrame)
			{
				Fade = float(FadeFrame) / float(kNumFadeFrames);
				for (int32 ChannelIndex = 0; ChannelIndex < Outputs.AudioOutputs.Num(); ++ChannelIndex)
				{
					*(Outputs.AudioOutputs[ChannelIndex]->GetData() + FromFrame + FadeFrame) *= Fade;
				}
			}
		}

		/**
		Plays the captured audio. Will do a little fade in at the beginning, and a little 
		fade out at the end.
		@param DestinationFrame Where in the output buffer to write samples.
		@param NumFrames Number of frames to write.
		@param bForceFade Whether to force the fade out of playback even
		                  if we are not at the end of the available sample data.
						  This typically only happens if we are being asked to start
						  playback again while we are already playing.
		@return True if this function ended up writing ANYTHING to the output buffer.
		*/
		bool DoPlay(int32 DestinationFrame, int32 NumFrames, bool bForceFade = false)
		{
			int32 FramesWriteableInOutput = NumFrames;
			int32 FramesAvailableToPlay = PlayLengthFrames - NextPlayFrame;
			int32 FramesToPlay = FMath::Min(FramesWriteableInOutput, FramesAvailableToPlay);

			if (FramesToPlay > 0)
			{
				bCurrentlyPlaying = true;
				if (!bPlayingReverse)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
					{
						FMemory::Memcpy(Outputs.AudioOutputs[ChannelIndex]->GetData() + DestinationFrame, CaptureBuffer.GetValidChannelData(ChannelIndex) + NextPlayFrame, FramesToPlay * sizeof(float));
					}
				}
				else
				{
					for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
					{
						float* CapturedData = CaptureBuffer.GetValidChannelData(ChannelIndex);
						float* DestinationAddress = Outputs.AudioOutputs[ChannelIndex]->GetData() + DestinationFrame;
						for (int32 SourceIndex = PlayLengthFrames - NextPlayFrame - 1, FrameCount = 0; FrameCount < FramesToPlay; ++FrameCount, --SourceIndex)
						{
							*(DestinationAddress + FrameCount) = CapturedData[SourceIndex];
						}
					}
				}
				// fade in the beginning to avoid click...
				if (NextPlayFrame < kNumFadeFrames)
				{
					int32 FadeFrames = FMath::Min(kNumFadeFrames, FramesToPlay);
					float CurrentFade = float(NextPlayFrame) / float(kNumFadeFrames);
					static float constexpr FadeInc = 1.0f/float(kNumFadeFrames);
					for (int32 FrameIndex = DestinationFrame; FrameIndex < DestinationFrame + FadeFrames; ++FrameIndex)
					{
						for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
						{
							Outputs.AudioOutputs[ChannelIndex]->GetData()[FrameIndex] *= CurrentFade;
						}
						CurrentFade += FadeInc;
					}
				}
				// fade out the end to avoid click...
				if ((NextPlayFrame + FramesToPlay) > (PlayLengthFrames - kNumFadeFrames) || bForceFade)
				{
					int32 RenderedLastFrame = NextPlayFrame + FramesToPlay;
					int32 FramesRemainingToRender = bForceFade ? 0 : PlayLengthFrames - RenderedLastFrame;
					int32 NumFadeFrames = bForceFade ? FMath::Min(FramesToPlay, kNumFadeFrames) : kNumFadeFrames;
					int32 MaxFramesToFade = NumFadeFrames - FramesRemainingToRender;
					float CurrentFade = FMath::Min(1.0f,  float(FramesRemainingToRender) / float(NumFadeFrames));
					float FadeInc = 1.0f / float(NumFadeFrames);
					int32 FadeAtIndex = DestinationFrame + FramesToPlay - 1;
					for (int32 FadeIndex = 0; FadeIndex < MaxFramesToFade; ++FadeIndex)
					{
						for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
						{
							Outputs.AudioOutputs[ChannelIndex]->GetData()[FadeAtIndex] *= CurrentFade;
						}
						--FadeAtIndex;
						CurrentFade = FMath::Clamp(CurrentFade + FadeInc, 0.0f, 1.0f);
					}
				}
				NextPlayFrame += FramesToPlay;
				bCurrentlyPlaying = NextPlayFrame < PlayLengthFrames;
			}
			else
			{
				NextPlayFrame = 0;
				bCurrentlyPlaying = false;
			}

			// we may have some silence to write at the end of playback...
			if (!bCurrentlyPlaying)
			{
				int32 SilentFrameIndex = DestinationFrame + FramesToPlay;
				for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
				{
					int32 NumSilentFrames = Outputs.AudioOutputs[ChannelIndex]->Num() - SilentFrameIndex;
					FMemory::Memset(Outputs.AudioOutputs[ChannelIndex]->GetData() + SilentFrameIndex, 0, NumSilentFrames * sizeof(float));
				}
			}

			return true;
		}

		/** 
		Captures the input audio to a buffer.
		@param FirstFrameOfPlay There MAY be a request to play WHILE we are capturing. If this is > -1 
		                        then there is a request to play the captured buffer in this same block.
		@return True if this function ended up writing ANYTHING to the output buffer.
		*/
		bool DoCapture(int32 FirstFrameOfPlay)
		{
			int32 LastCaptureTriggerBlockIndex = Inputs.Capture->Last();
			check (bCurrentlyCapturing || LastCaptureTriggerBlockIndex > -1);

			int32 FirstFrameInBlockToCapture = 0;
			if (LastCaptureTriggerBlockIndex != -1)
			{
				// start or restart capturing!
				CaptureBuffer.SetNumValidFrames(0);
				FirstFrameInBlockToCapture = LastCaptureTriggerBlockIndex;
				bCurrentlyCapturing = true;
			}

			int32 FramesAvailableAtInput = Inputs.AudioInputs[0]->Num() - FirstFrameInBlockToCapture;
			int32 FramesWriteableInBuffer = CaptureLengthFrames - CaptureBuffer.GetNumValidFrames();
			int32 NumFramesToCapture = FMath::Min(FramesAvailableAtInput, FramesWriteableInBuffer);

			if (NumFramesToCapture < 1)
			{
				bCurrentlyCapturing = false;
				bFullyCaptured = true;
				return false; // we did not render anything to the output.
			}

			for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
			{
				FMemory::Memcpy(CaptureBuffer.GetValidChannelData(ChannelIndex) + CaptureBuffer.GetNumValidFrames(), Inputs.AudioInputs[ChannelIndex]->GetData() + FirstFrameInBlockToCapture, NumFramesToCapture * sizeof(float));
			}

			CaptureBuffer.SetNumValidFrames(CaptureBuffer.GetNumValidFrames() + NumFramesToCapture);

			if (CaptureBuffer.GetNumValidFrames() == CaptureBuffer.GetMaxNumFrames())
			{
				bCurrentlyCapturing = false;
				bFullyCaptured = true;
			}

			// we still have input data to copy to the output...
			if (!bCurrentlyPlaying && !bHavePlayedThisCapture)
			{
				if (FirstFrameOfPlay > -1)
				{
					// We WILL be playing this block... so limit "capture output thru" up to that point...
					FirstFrameOfPlay = FMath::Max(FirstFrameOfPlay, kNumFadeFrames);
					int32 FrameToFinish = FMath::Min(FirstFrameOfPlay, FirstFrameInBlockToCapture + NumFramesToCapture);
					BulkOutput(EInputOrSilence::Input, 0, FrameToFinish);

					// fade out the input before the start of the next play
					float CurrentFade = 0.0f;
					float FadeInc = 1.0f/float(kNumFadeFrames);
					int32 FadeAtIndex = FrameToFinish - 1;
					for (int32 FadeIndex = 0; FadeIndex < kNumFadeFrames; ++FadeIndex)
					{
						for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
						{
							Outputs.AudioOutputs[ChannelIndex]->GetData()[FadeAtIndex] *= CurrentFade;
						}
						--FadeAtIndex;
						CurrentFade = FMath::Clamp(CurrentFade + FadeInc, 0.0f, 1.0f);
					}
				}
				else
				{
					// If we are capturing the last little bit, we faded out playback in the previous block,
					// so don't pass input to output here!
					if (NumFramesToCapture >= kNumFadeFrames)
					{
						BulkOutput(EInputOrSilence::Input, 0, FirstFrameInBlockToCapture + NumFramesToCapture);
						BulkOutput(EInputOrSilence::Silence, FirstFrameInBlockToCapture + NumFramesToCapture);
						// We may be done capturing and have to fade to the state where we are outputting silence.
						// NOTE: in order to prevent a click if the end of capturing is at the very beginning of the NEXT 
						// block, we will fade NOW, making the capture buffer a little shorter if we have to...
						if (CaptureBuffer.GetNumValidFrames() >= (CaptureLengthFrames - kNumFadeFrames))
						{
							float CurrentFade = 0.0f;
							float FadeInc = 1.0f/float(kNumFadeFrames);
							int32 FadeAtIndex = FirstFrameInBlockToCapture + NumFramesToCapture - 1;
							for (int32 FadeIndex = 0; FadeIndex < kNumFadeFrames; ++FadeIndex)
							{
								for (int32 ChannelIndex = 0; ChannelIndex < CaptureBuffer.GetNumValidChannels(); ++ChannelIndex)
								{
									Outputs.AudioOutputs[ChannelIndex]->GetData()[FadeAtIndex] *= CurrentFade;
								}
								--FadeAtIndex;
								CurrentFade = FMath::Clamp(CurrentFade + FadeInc, 0.0f, 1.0f);
							}
						}
					}
					else
					{
						BulkOutput(EInputOrSilence::Silence);
					}
				}
				return true; // we rendered something to the output.
			}
			return false; // we did not render anything to the output.
		}

	private:
		FInputs  Inputs;
		FOutputs Outputs;
		float    MaxCaptureSeconds             = 5.0f;
		bool     bCurrentlyCapturing           = false;
		bool     bFullyCaptured                = false;
		bool     bCurrentlyPlaying             = false;
		int32    CaptureLengthFrames           = 0;
		int32    PlayLengthFrames              = 0;
		bool     bPlayingReverse               = false;
		int32    NextPlayFrame                 = 0;
		bool     bHavePlayedThisCapture        = false;
		TAudioBuffer<float> CaptureBuffer;
	};

	using FSimpleSamplerNode = Metasound::TNodeFacade<FSimpleSamplerOperator>;
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FSimpleSamplerNode, FHarmonixSimpleSamplerNodeConfiguration);
}

TInstancedStruct<FMetasoundFrontendClassInterface> FHarmonixSimpleSamplerNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(HarmonixMetasound::SimpleSamplerNode::GetVertexInterface(NumChannels)));
}

TSharedPtr<const Metasound::IOperatorData> FHarmonixSimpleSamplerNodeConfiguration::GetOperatorData() const
{
	return MakeShared<HarmonixMetasound::SimpleSamplerNode::FSimpleSamplerNodeOperatorData>(MaxCaptureSeconds);
}

#undef LOCTEXT_NAMESPACE
