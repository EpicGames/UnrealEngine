// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/StepSequencePlayerNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "IAudioProxyInitializer.h"
#include "Harmonix/AudioRenderableProxy.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MidiStepSequence.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#include "HarmonixMidi/MidiVoiceId.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

DEFINE_LOG_CATEGORY_STATIC(LogStepSequencePlayer, Log, All);

namespace HarmonixMetasound::Nodes::StepSequencePlayer
{
	using namespace Metasound;
	using namespace Harmonix;

	constexpr uint8 MidiChannel = 0;
	
	const Metasound::FNodeClassName& GetClassName()
	{
		static Metasound::FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
			"StepSequencePlayer",
			""
		};
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	enum class EStepSequencePlayerState : uint8
	{
		NotPlaying,
		PlayingLooping,
		PlayingOneShot,
		Finished
	};

	class FStepSequencePlayerOperator : public TExecutableOperator<FStepSequencePlayerOperator>, public FMusicTransportControllable, public FMidiVoiceGeneratorBase
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface&   GetVertexInterface();
		static TUniquePtr<IOperator>     CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FStepSequencePlayerOperator(const FBuildOperatorParams& InParams, 
							        const FMidiStepSequenceAssetReadRef& InSequenceAsset,
							        const FMusicTransportEventStreamReadRef& InTransport,
							        const FMidiClockReadRef& InMidiClockSource,
							        const FFloatReadRef& InSpeedMultiplier,
									const FFloatReadRef& InVelocityMultiplier,
									const FFloatReadRef& InMaxColumns,
									const FFloatReadRef& InAdditionalOctaves,
									const FFloatReadRef& InStepSizeQuarterNotes,
									const FFloatReadRef& InActivePage,
									const FBoolReadRef& InAutoPage,
									const FBoolReadRef& InAutoPagePlaysBlankPages,
									const FBoolReadRef& InLoop,
									const FBoolReadRef& InEnabled);

		virtual ~FStepSequencePlayerOperator() override;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;
		void Reset(const FResetParams& ResetParams);
		void Execute();

	private:
		void Init();
		void InitSequenceTable();
		
		//** INPUTS
		FMidiStepSequenceAssetReadRef SequenceAssetInPin;
		FMusicTransportEventStreamReadRef TransportInPin;
		FFloatReadRef SpeedMultInPin;
		FFloatReadRef VelocityMultInPin;
		FMidiClockReadRef MidiClockInPin;
		FFloatReadRef MaxColumnsInPin;
		FFloatReadRef AdditionalOctavesInPin;
		FFloatReadRef StepSizeQuarterNotesInPin;
		FFloatReadRef ActivePageInPin;
		FBoolReadRef AutoPageInPin;
		FBoolReadRef AutoPagePlaysBlankPagesInPin;
		FBoolReadRef LoopInPin;
		FBoolReadRef EnabledInPin;

		//** OUTPUTS
		FMidiStreamWriteRef MidiOutPin;

		//** DATA
		TSharedAudioRenderableDataPtr<FStepSequenceTable, TRefCountedAudioRenderableWithQueuedChanges<FStepSequenceTable>> SequenceTable;
		FSampleCount BlockSize			    = 0;
		int32 CurrentBlockSpanStart		    = 0;
		int32 CurrentPageIndex			    = -1;
		int32 CurrentCellIndex			    = -1;
		int32 ProcessedThruTick			    = -1;
		int32 SequenceStartTick			    = -1;
		int32 CurrentStepSkipIndex		    = 0;
		bool  bAutoPage					    = false;
		bool  bPreviousAutoPage			    = false;
		bool  bAutoPagePlaysBlankPages	    = false;
		bool  bLoop						    = true;
		bool  bNeedsRebase				    = false;
		EStepSequencePlayerState PlayState  = EStepSequencePlayerState::NotPlaying;
		TArray<FMidiVoiceId> CurrentCellNotes;
		TArray<uint8> CurrentTransposedNotes;

		void CheckForUpdatedSequenceTable();
		void ResizeCellStatesForTable();
		void AllNotesOff(int32 AtFrameIndex, int32 AbsMidiTick, bool ResetCellIndex);
		int32 CalculateAutoPageIndex(int32 CurTick, int32 TableTickLength, bool bRound) const;
		int32 CalculatePagesProgressed(const int FromTick, const int ToTick, int32 TableTickLength, const bool bRound) const;
		void CalculatePageProperties(const FStepSequencePage& Page, int32 MaxColumns, float StepSizeQuarterNotes, int32& OutColumns, int32& OutTicksPerCell, int32& OutTableTickLength) const;
		void RebaseSequenceStartTickForLoop(const int32 CurTick, const int32 TableTickLength);
		void EnsureCurrentPageIndexIsValid();
		bool IsPlaying() const;

	protected:

		void SeekToTick(int32 BlockFrameIndex, int32 Tick);
		void SeekThruTick(int32 BlockFrameIndex, int32 Tick);
		void AdvanceThruTick(int32 BlockFrameIndex, int32 Tick);

	};

	using FStepSequencePlayerNode = TNodeFacade<FStepSequencePlayerOperator>;
	METASOUND_REGISTER_NODE(FStepSequencePlayerNode)

	const FNodeClassMetadata& FStepSequencePlayerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = GetClassName();
			Info.MajorVersion     = GetCurrentMajorVersion();
			Info.MinorVersion     = 1;
			Info.DisplayName      = METASOUND_LOCTEXT("StepSequencePlayerNode_DisplayName", "Step Sequence Player");
			Info.Description      = METASOUND_LOCTEXT("StepSequencePlayerNode_Description", "Plays a Step Sequence Asset.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace Inputs
	{
		DEFINE_INPUT_METASOUND_PARAM(SequenceAsset, "Step Sequence Asset", "Step sequence to play.");
		DEFINE_INPUT_METASOUND_PARAM(VelocityMultiplier, "Velocity Multiplier", "Multiplies the current note velocity by this number");
		DEFINE_INPUT_METASOUND_PARAM(MaxColumns, "Max Columns", "The Maximinum Number of cells to play per step sequence row.");
		DEFINE_INPUT_METASOUND_PARAM(AdditionalOctaves, "Additional Octaves", "The number of octaves to add to the authored step sequence note.");
		DEFINE_INPUT_METASOUND_PARAM(StepSizeQuarterNotes, "Step Size Quarter Notes", "The size, in quarter notes, of each step");
		DEFINE_INPUT_METASOUND_PARAM(ActivePage, "Active Page", "The page of the step sequence to play (1 indexed)");
		DEFINE_INPUT_METASOUND_PARAM(AutoPage, "Auto Page", "Whether to calculate the page of the step sequence based on current position");
		DEFINE_INPUT_METASOUND_PARAM(AutoPagePlaysBlankPages, "Auto Page Plays Blank Pages", "If autopaging, should blank pages be played?");
		DEFINE_METASOUND_PARAM_ALIAS(Transport, CommonPinNames::Inputs::Transport);
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_METASOUND_PARAM_ALIAS(Speed, CommonPinNames::Inputs::Speed);
		DEFINE_METASOUND_PARAM_ALIAS(Loop, CommonPinNames::Inputs::Loop);
		DEFINE_METASOUND_PARAM_ALIAS(Enabled, CommonPinNames::Inputs::Enable);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiStream, CommonPinNames::Outputs::MidiStream);
	}

	const FVertexInterface& FStepSequencePlayerOperator::GetVertexInterface()
	{

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FMidiStepSequenceAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::SequenceAsset)),
				TInputDataVertex<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Transport)),
				TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Speed), 1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::VelocityMultiplier), 1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MaxColumns), 64.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AdditionalOctaves), 0),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::StepSizeQuarterNotes), 0.25f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::ActivePage), 0),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AutoPage), false),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AutoPagePlaysBlankPages), true),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Loop), true),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enabled), true)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FMidiStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::MidiStream))
			)
		);
		return Interface;
	}

	TUniquePtr<IOperator> FStepSequencePlayerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FStepSequencePlayerNode& PlayerNode = static_cast<const FStepSequencePlayerNode&>(InParams.Node);

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FMidiStepSequenceAssetReadRef InSequenceAsset = InputData.GetOrCreateDefaultDataReadReference<FMidiStepSequenceAsset>(METASOUND_GET_PARAM_NAME(Inputs::SequenceAsset), InParams.OperatorSettings);
		FMusicTransportEventStreamReadRef InTransport = InputData.GetOrCreateDefaultDataReadReference<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME(Inputs::Transport), InParams.OperatorSettings);
		FFloatReadRef InSpeed = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::Speed), InParams.OperatorSettings);
		FFloatReadRef InVelocityMult = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::VelocityMultiplier), InParams.OperatorSettings);
		FMidiClockReadRef InMidiClock = InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), InParams.OperatorSettings);
		FFloatReadRef InMaxColumns = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::MaxColumns), InParams.OperatorSettings);
		FFloatReadRef InAdditionalOctaves = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::AdditionalOctaves), InParams.OperatorSettings);
		FFloatReadRef InStepSize = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::StepSizeQuarterNotes), InParams.OperatorSettings);
		FFloatReadRef InActivePage = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(Inputs::ActivePage), InParams.OperatorSettings);
		FBoolReadRef InAutoPage = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::AutoPage), InParams.OperatorSettings);
		FBoolReadRef InAutoPagePlaysBlankPages = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::AutoPagePlaysBlankPages), InParams.OperatorSettings);
		FBoolReadRef InLoop = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Loop), InParams.OperatorSettings);
		FBoolReadRef InEnabled = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(Inputs::Enabled), InParams.OperatorSettings);
		return MakeUnique<FStepSequencePlayerOperator>(InParams, InSequenceAsset, InTransport, InMidiClock, InSpeed, InVelocityMult, InMaxColumns, InAdditionalOctaves, InStepSize, InActivePage, InAutoPage, InAutoPagePlaysBlankPages, InLoop, InEnabled);
	}

	FStepSequencePlayerOperator::FStepSequencePlayerOperator(const FBuildOperatorParams& InParams,
															 const FMidiStepSequenceAssetReadRef& InSequenceAsset,
															 const FMusicTransportEventStreamReadRef& InTransport,
															 const FMidiClockReadRef& InMidiClockSource,
															 const FFloatReadRef& InSpeedMultiplier,
															 const FFloatReadRef& InVelocityMultiplier,
															 const FFloatReadRef& InMaxColumns,
															 const FFloatReadRef& InAdditionalOctaves,
															 const FFloatReadRef& InStepSizeQuarterNotes,
															 const FFloatReadRef& InActivePage,
															 const FBoolReadRef& InAutoPage,
															 const FBoolReadRef& InAutoPagePlaysBlankPages,
															 const FBoolReadRef& InLoop,
															 const FBoolReadRef& InEnabled)
		: FMusicTransportControllable(EMusicPlayerTransportState::Prepared)
		, SequenceAssetInPin(InSequenceAsset)
		, TransportInPin(InTransport)
		, SpeedMultInPin(InSpeedMultiplier)
		, VelocityMultInPin(InVelocityMultiplier)
		, MidiClockInPin(InMidiClockSource)
		, MaxColumnsInPin(InMaxColumns)
		, AdditionalOctavesInPin(InAdditionalOctaves)
		, StepSizeQuarterNotesInPin(InStepSizeQuarterNotes)
		, ActivePageInPin(InActivePage)
		, AutoPageInPin(InAutoPage)
		, AutoPagePlaysBlankPagesInPin(InAutoPagePlaysBlankPages)
		, LoopInPin(InLoop)
		, EnabledInPin(InEnabled)
		, MidiOutPin(FMidiStreamWriteRef::CreateNew())
	{
		Reset(InParams);
		Init();
	}

	FStepSequencePlayerOperator::~FStepSequencePlayerOperator()
	{
	}

	void FStepSequencePlayerOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::SequenceAsset), SequenceAssetInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Transport), TransportInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), MidiClockInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Speed), SpeedMultInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::VelocityMultiplier), VelocityMultInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MaxColumns), MaxColumnsInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::AdditionalOctaves), AdditionalOctavesInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::StepSizeQuarterNotes), StepSizeQuarterNotesInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::ActivePage), ActivePageInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::AutoPage), AutoPageInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::AutoPagePlaysBlankPages), AutoPagePlaysBlankPagesInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Loop), LoopInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Enabled), EnabledInPin);

		Init();
	}

	void FStepSequencePlayerOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		using namespace CommonPinNames;
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::MidiStream), MidiOutPin);
	}

	void FStepSequencePlayerOperator::Reset(const FResetParams& ResetParams)
	{
		BlockSize = ResetParams.OperatorSettings.GetNumFramesPerBlock();
		CurrentBlockSpanStart = 0;
		CurrentPageIndex = -1;
		CurrentCellIndex = -1;
		ProcessedThruTick = -1;
		SequenceStartTick = -1;
		bAutoPage = false;
		bPreviousAutoPage = false;
		bAutoPagePlaysBlankPages = false;
		bLoop = true;
		bNeedsRebase = false;
		PlayState = EStepSequencePlayerState::NotPlaying;
		CurrentCellNotes.Reset();
		CurrentTransposedNotes.Reset();
	}

	void FStepSequencePlayerOperator::Execute()
	{
		MidiOutPin->PrepareBlock();

		CheckForUpdatedSequenceTable();

		// if we have no sequence table there is nothing to do. 
		// Make sure the notes are all off and return.
		if (!SequenceTable)
		{
			if (CurrentCellNotes.Num() > 0)
			{
				AllNotesOff(0, MidiClockInPin->GetLastProcessedMidiTick(), true);
				CurrentCellNotes.Empty();
				CurrentTransposedNotes.Empty();
			}
			return;
		}

		// We need to cache this to avoid avoid a crash if the value in the 
		// sequence table asset changes while we are in the middle of rendering. 
		CurrentStepSkipIndex = SequenceTable->StepSkipIndex;

		TransportSpanPostProcessor ClockEventHandler = [&](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState State)
			{
				using namespace MidiClockMessageTypes;

				switch (State)
				{
				case EMusicPlayerTransportState::Playing:
				case EMusicPlayerTransportState::Continuing:
					for (const FMidiClockEvent& Event : MidiClockInPin->GetMidiClockEventsInBlock())
					{
						if (Event.BlockFrameIndex >= EndFrameIndex)
						{
							break;
						}

						if (Event.BlockFrameIndex >= StartFrameIndex)
						{
							if (const FAdvance* AsAdvance = Event.TryGet<FAdvance>())
							{
								AdvanceThruTick(Event.BlockFrameIndex, AsAdvance->LastTickToProcess());
							}
							else if (const FSeek* AsSeekTo = Event.TryGet<FSeek>())
							{
								SeekToTick(Event.BlockFrameIndex, AsSeekTo->NewNextTick);
							}
							else if (const FLoop* AsLoop = Event.TryGet<FLoop>())
							{
								SeekToTick(Event.BlockFrameIndex, AsLoop->FirstTickInLoop);
							}
						}
					}
				}
			};

		TransportSpanProcessor TransportHandler = [&](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
			{
				CurrentBlockSpanStart = StartFrameIndex;
				switch (CurrentState)
				{
				case EMusicPlayerTransportState::Invalid:
				case EMusicPlayerTransportState::Preparing:
					return EMusicPlayerTransportState::Prepared;
				
				case EMusicPlayerTransportState::Prepared:
					return CurrentState;
				
				case EMusicPlayerTransportState::Starting:
					return EMusicPlayerTransportState::Playing;

				case EMusicPlayerTransportState::Playing:
					return EMusicPlayerTransportState::Playing;

				case EMusicPlayerTransportState::Seeking:
					return GetTransportState();

				case EMusicPlayerTransportState::Continuing:
					return EMusicPlayerTransportState::Playing;

				case EMusicPlayerTransportState::Pausing:
					return EMusicPlayerTransportState::Paused;

				case EMusicPlayerTransportState::Paused:
					return EMusicPlayerTransportState::Paused;

				case EMusicPlayerTransportState::Stopping:
					AllNotesOff(StartFrameIndex, MidiClockInPin->GetLastProcessedMidiTick(), true);
					return EMusicPlayerTransportState::Prepared;

				case EMusicPlayerTransportState::Killing:
					AllNotesOff(StartFrameIndex, MidiClockInPin->GetLastProcessedMidiTick(), true);
					return EMusicPlayerTransportState::Prepared;

				default:
					checkNoEntry();
					return EMusicPlayerTransportState::Invalid;
				}
			};
		ExecuteTransportSpans(TransportInPin, BlockSize, TransportHandler, ClockEventHandler);
	}

	void FStepSequencePlayerOperator::Init()
	{
		MidiOutPin->SetClock(*MidiClockInPin);
		MidiOutPin->PrepareBlock();

		InitSequenceTable();

		FTransportInitFn InitFn = [this](EMusicPlayerTransportState CurrentState)
		{
			CurrentBlockSpanStart = 0;
			switch (CurrentState)
			{
			case EMusicPlayerTransportState::Invalid:
			case EMusicPlayerTransportState::Preparing:
				return EMusicPlayerTransportState::Prepared;
				
			case EMusicPlayerTransportState::Prepared:
				return CurrentState;
				
			case EMusicPlayerTransportState::Starting:
				return EMusicPlayerTransportState::Playing;

			case EMusicPlayerTransportState::Playing:
				return EMusicPlayerTransportState::Playing;

			case EMusicPlayerTransportState::Seeking:
				return GetTransportState();

			case EMusicPlayerTransportState::Continuing:
				return EMusicPlayerTransportState::Playing;

			case EMusicPlayerTransportState::Pausing:
				return EMusicPlayerTransportState::Paused;

			case EMusicPlayerTransportState::Paused:
				return EMusicPlayerTransportState::Paused;

			case EMusicPlayerTransportState::Stopping:
				AllNotesOff(0, MidiClockInPin->GetLastProcessedMidiTick(), true);
				return EMusicPlayerTransportState::Prepared;

			case EMusicPlayerTransportState::Killing:
				AllNotesOff(0, MidiClockInPin->GetLastProcessedMidiTick(), true);
				return EMusicPlayerTransportState::Prepared;

			default:
				checkNoEntry();
				return EMusicPlayerTransportState::Invalid;
			}
		};
		FMusicTransportControllable::Init(*TransportInPin, MoveTemp(InitFn));
	}

	void FStepSequencePlayerOperator::InitSequenceTable()
	{
		SequenceTable = SequenceAssetInPin->GetRenderable();
		if (SequenceTable)
		{
			ResizeCellStatesForTable();
			UE_LOGF(LogStepSequencePlayer, Verbose, "Got a Sequence: %d pages, %d rows, %d columns",
				SequenceTable->Pages.Num(),
				SequenceTable->Pages.IsEmpty() ? 0 : SequenceTable->Pages[0].Rows.Num(),
				SequenceTable->Pages.IsEmpty() || SequenceTable->Pages[0].Rows.IsEmpty() ? 0 : SequenceTable->Pages[0].Rows[0].Cells.Num());
		}
		else
		{
			UE_LOGF(LogStepSequencePlayer, Verbose, "No Sequence Provided!");
		}
	}

	void FStepSequencePlayerOperator::SeekToTick(int32 BlockFrameIndex, int32 Tick)
	{
		SeekThruTick(BlockFrameIndex, Tick-1);
	}

	void FStepSequencePlayerOperator::SeekThruTick(int32 BlockFrameIndex ,int32 Tick)
	{
		ProcessedThruTick = FMath::Max(-1, Tick);
		bNeedsRebase = true;
		AllNotesOff(BlockFrameIndex, Tick, true);
	}

	int32 FStepSequencePlayerOperator::CalculatePagesProgressed(const int FromTick, const int ToTick, int32 TableTickLength, const bool bRound) const
	{
		const float PagesProgressed = (static_cast<float>(ToTick - FromTick) / static_cast<float>(TableTickLength));
		return bRound ? FMath::RoundToInt32(PagesProgressed) : FMath::FloorToInt32(PagesProgressed);
	}

	int32 FStepSequencePlayerOperator::CalculateAutoPageIndex(int32 CurTick, int32 TableTickLength, bool bRound) const
	{
		const int32 PagesProgressed = CalculatePagesProgressed(SequenceStartTick, CurTick, TableTickLength, bRound);
		int32 TargetPageIndex = SequenceTable->CalculateAutoPageIndex(PagesProgressed, bAutoPagePlaysBlankPages, bLoop);

		if (TargetPageIndex != INDEX_NONE && !SequenceTable->Pages.IsValidIndex(TargetPageIndex))
		{
			TargetPageIndex = 0;
		}

		// Page Index is 1-based. If INDEX_NONE is returned from the above, this will be 0 and therefore signals to be nonplaying
		++TargetPageIndex;
		return TargetPageIndex;
	}

	void FStepSequencePlayerOperator::CalculatePageProperties(const FStepSequencePage& Page, int32 MaxColumns, float StepSizeQuarterNotes, int32& OutColumns, int32& OutTicksPerCell, int32& OutTableTickLength) const
	{		
		OutColumns = FMath::Min(Page.Rows[0].Cells.Num(), (MaxColumns < 1 ? 1 : MaxColumns));

		int32 LengthColumns = OutColumns;
		
		// Lower the total number of columns to reflect the number of steps skipped
		// This value is used only for calculating the tick length
		if (CurrentStepSkipIndex > 0)
		{
			LengthColumns -= LengthColumns / CurrentStepSkipIndex;
			LengthColumns = FMath::Max(LengthColumns, 1);
		}

		// Min float possible is 64th note triplets.
		OutTicksPerCell = FMath::Max(StepSizeQuarterNotes, 0.0416666f) * Midi::Constants::GTicksPerQuarterNote;
		OutTableTickLength = OutTicksPerCell * LengthColumns;
	}

	void FStepSequencePlayerOperator::RebaseSequenceStartTickForLoop(const int32 CurTick, const int32 TableTickLength)
	{
		if (CurTick == INDEX_NONE)
		{
			SequenceStartTick = 0;
			CurrentPageIndex = -1;
			bNeedsRebase = false;
			return;
		}

		const int32 PagesProgressed = CalculatePagesProgressed(0, CurTick, TableTickLength, false);
		const int32 TickInPage = TableTickLength == 0 ? 0 : CurTick % TableTickLength;
		const int32 NumValidPages = bAutoPage ? SequenceTable->CalculateNumValidPages(bAutoPagePlaysBlankPages) : 1;
		const int32 PagesSinceStart = NumValidPages == 0 ? 0 : PagesProgressed % NumValidPages;

		SequenceStartTick = CurTick - (PagesSinceStart * TableTickLength) - TickInPage;
		CurrentPageIndex = CalculateAutoPageIndex(CurTick, TableTickLength, false);
		bNeedsRebase = false;
	}

	void FStepSequencePlayerOperator::EnsureCurrentPageIndexIsValid()
	{
		if (bAutoPage)
		{
			// Ensure we don't process erroneous -1, conversion to 1-based first page
			CurrentPageIndex = FMath::Abs(CurrentPageIndex);
			if (CurrentPageIndex == 0)
			{
				// The loop ended in a previous tick, IsPlaying() should be false here.
				// Reset the current index, and we'll wait to start playing again.
				CurrentPageIndex = SequenceTable->GetFirstValidPage(bAutoPagePlaysBlankPages) + 1;
			}
		}
		else
		{
			CurrentPageIndex = FMath::Clamp((int32)*ActivePageInPin, 1, SequenceTable->Pages.Num());
		}
	}

	bool FStepSequencePlayerOperator::IsPlaying() const
	{
		return PlayState == EStepSequencePlayerState::PlayingLooping || PlayState == EStepSequencePlayerState::PlayingOneShot;
	}

	void FStepSequencePlayerOperator::AdvanceThruTick(int32 BlockFrameIndex, int32 Tick)
	{
		if (!SequenceTable || SequenceTable->Pages.IsEmpty())
		{
			return;
		}

		// Read input pins
		const int32 CurrentMaxColumns = *MaxColumnsInPin;
		const int32 AdditionalOctaveNotes = (int32)*AdditionalOctavesInPin * 12;
		const float CurrentStepSizeQuarterNotes = *StepSizeQuarterNotesInPin;
		const float CurrentVelocityMultiplierValue = *VelocityMultInPin;

		if (CurrentMaxColumns < 1)
		{
			// Nothing to play
			return;
		}

		if (!bAutoPage && *AutoPageInPin)
		{
			bNeedsRebase = true;
		}

		bAutoPage = *AutoPageInPin;
		bAutoPagePlaysBlankPages = *AutoPagePlaysBlankPagesInPin;

		EnsureCurrentPageIndexIsValid();
		if (!SequenceTable->Pages.IsValidIndex(CurrentPageIndex - 1))
		{
			// WE SHOULD NEVER HIT THIS
			return;
		}
		FStepSequencePage* CurrentPage = &(SequenceTable->Pages[CurrentPageIndex - 1]);

		// Do an initial calc with the page we think we're on
		int32 Columns;
		int32 TicksPerCell;
		int32 TableTickLength;
		CalculatePageProperties(*CurrentPage, CurrentMaxColumns, CurrentStepSizeQuarterNotes, Columns, TicksPerCell, TableTickLength);
		
		EStepSequencePlayerState LastPlayState = PlayState;

		// If this is enabled and not playing, transition it to playing
		if (*EnabledInPin && PlayState == EStepSequencePlayerState::NotPlaying)
		{
			PlayState = bLoop ? EStepSequencePlayerState::PlayingLooping : EStepSequencePlayerState::PlayingOneShot;
		}
		// If this is not enabled and not in the "not playing" state, set that state
		else if (!*EnabledInPin && PlayState != EStepSequencePlayerState::NotPlaying)
		{
			PlayState = EStepSequencePlayerState::NotPlaying;
		}

		// Looping changed - adjust state if needed
		if (*LoopInPin != bLoop)
		{
			bLoop = *LoopInPin;

			if (!bLoop && PlayState == EStepSequencePlayerState::PlayingLooping)
			{
				PlayState = EStepSequencePlayerState::PlayingOneShot;
			}
			else if (bLoop && (PlayState == EStepSequencePlayerState::PlayingLooping || PlayState == EStepSequencePlayerState::Finished))
			{
				PlayState = EStepSequencePlayerState::PlayingLooping;
			}
		}
		
		if (LastPlayState != PlayState)
		{
			switch (PlayState)
			{
			case EStepSequencePlayerState::PlayingLooping:
			{
				// Set that this needs a rebase - changing auto page may also need this same block to run
				bNeedsRebase = true;
				break;
			}
			case EStepSequencePlayerState::PlayingOneShot:
			{
				int32 TickInCell = (ProcessedThruTick % TicksPerCell);

				// If the tick in the cell is close enough to a current division, 
				// Move the ProcessedThruTick back to the previous division.
				// That will become the SequenceStartTick, and that sequence will start immediately.
				if (TickInCell < TicksPerCell / 16)
				{
					ProcessedThruTick -= TickInCell;
					TickInCell = 0;
				}

				// Begin the oneshot on the next beat subdivision
				SequenceStartTick = ProcessedThruTick + (TickInCell > 0 ? TicksPerCell - TickInCell : 0);
				if (bAutoPage)
				{
					// Nonlooping autopaging sequences start on the first valid page.
					CurrentPageIndex = SequenceTable->GetFirstValidPage(bAutoPagePlaysBlankPages) + 1;
				}
				break;
			}
			case EStepSequencePlayerState::Finished:
			case EStepSequencePlayerState::NotPlaying:
			default:
			{
				// These states behave the same, they just have different transitions
				// "Finished" cannot transition to "PlayingOneShot", and
				// it won't become "NotPlaying" until the device is disabled
				AllNotesOff(BlockFrameIndex, Tick, true);
				break;
			}
			}
		}

		if (bNeedsRebase)
		{
			if (PlayState == EStepSequencePlayerState::PlayingLooping)
			{
				RebaseSequenceStartTickForLoop(ProcessedThruTick, TableTickLength);
				// Our page index may have changed, let's set it again
				EnsureCurrentPageIndexIsValid();
				if (!SequenceTable->Pages.IsValidIndex(CurrentPageIndex - 1))
				{
					// WE SHOULD NEVER HIT THIS
					return;
				}
				CurrentPage = &(SequenceTable->Pages[CurrentPageIndex - 1]);
				// Uncomment if there is ever a time where pages can have different tick lengths
				// CalculatePageProperties(*CurrentPage, CurrentMaxColumns, CurrentStepSizeQuarterNotes, Columns, TicksPerCell, TableTickLength);
			}
			else
			{
				bNeedsRebase = false;
			}
		}

		ProcessedThruTick = FMath::Max(-1, ProcessedThruTick);
		
		while (ProcessedThruTick < Tick)
		{
			ProcessedThruTick++;

			if (!IsPlaying() || ProcessedThruTick < SequenceStartTick)
			{
				// Loop has ended or hasn't started yet
				continue;
			}

			int32 EffectiveTickForLoopPosition = ProcessedThruTick;

			if (!bLoop)
			{
				// If loop is off, we want to position ourselves relative to when the sequence was triggered
				EffectiveTickForLoopPosition -= SequenceStartTick;

				if ((bAutoPage && CurrentPageIndex == 0) || (!bAutoPage && EffectiveTickForLoopPosition >= TableTickLength))
				{
					PlayState = EStepSequencePlayerState::Finished;
					continue;
				}
			}

			const int32 TickInTable = (TableTickLength == 0) ? 0 : EffectiveTickForLoopPosition % TableTickLength;
			int32 CellInRow = TickInTable / TicksPerCell;
			const int32 TickInCell = TickInTable - (CellInRow * TicksPerCell);
			int32 SkippedIndex = CurrentStepSkipIndex - 1;
			const bool bFirstTickInCell = (TickInTable % TicksPerCell) == 0;

			if (CurrentStepSkipIndex >= 2)
			{
				// Skip past the unused cells
				CellInRow += CellInRow / SkippedIndex;
			}

			if (TickInCell >= TicksPerCell - 1)
			{
				int32 NextCellInRow = (CellInRow + 1) % Columns;

				if (CurrentStepSkipIndex >= 2 && NextCellInRow % SkippedIndex == 0)
				{
					// If the next cell is unused, instead check the cell past this
					NextCellInRow = (NextCellInRow + 1) % Columns;
				}

				if (bAutoPage && (NextCellInRow == 0 || bAutoPage != bPreviousAutoPage))
				{
					CurrentPageIndex = CalculateAutoPageIndex(ProcessedThruTick, TableTickLength, true);

					if (CurrentPageIndex == 0)
					{
						// Loop has ended
						PlayState = EStepSequencePlayerState::Finished;
						continue;
					}
					else if (!SequenceTable->Pages.IsValidIndex(CurrentPageIndex - 1))
					{
						// WE SHOULD NEVER HIT THIS
						return;
					}

					CurrentPage = &(SequenceTable->Pages[CurrentPageIndex - 1]);
					CalculatePageProperties(*CurrentPage, CurrentMaxColumns, CurrentStepSizeQuarterNotes, Columns, TicksPerCell, TableTickLength);
				}	
				bPreviousAutoPage = bAutoPage;

				for (int32 NoteIdx = 0; NoteIdx < CurrentPage->Rows.Num(); ++NoteIdx)
				{
					if (CurrentCellNotes[NoteIdx])
					{
						// If: 
						// 1. The note that would play is not enabled (it is not a new note);
						// 2. A note with this pitch would start up this tick;
						// 3. The note is marked as a continuation note
						// Then keep this note playing through the next cell
						if (!CurrentPage->Rows[NoteIdx].Cells[NextCellInRow].bEnabled && CurrentPage->Rows[NoteIdx].Cells[NextCellInRow].bContinuation)
						{
							continue;
						}

						// note off!
						FMidiStreamEvent MidiEvent(CurrentCellNotes[NoteIdx].GetGeneratorId(), FMidiMsg::CreateNoteOff(MidiChannel, NoteIdx));
						MidiEvent.MidiMessage.Data1 = CurrentTransposedNotes[NoteIdx];
						MidiEvent.BlockSampleFrameIndex = BlockFrameIndex;
						MidiEvent.AuthoredMidiTick = ProcessedThruTick;
						MidiEvent.CurrentMidiTick = ProcessedThruTick;
						MidiEvent.TrackIndex = 1;
						MidiOutPin->AddNoteOffEventOrCancelPendingNoteOn(MidiEvent);
						UE_LOGF(LogStepSequencePlayer, Verbose, "0x%x Note-Off %d at %d", (uint32)(size_t)this, SequenceTable->Notes[NoteIdx].NoteNumber + AdditionalOctaveNotes, BlockFrameIndex);
						CurrentCellNotes[NoteIdx] = FMidiVoiceId::None();
						CurrentTransposedNotes[NoteIdx] = 0u;
					}
				}
			}

			if (CellInRow != CurrentCellIndex || (CurrentMaxColumns == 1 && bFirstTickInCell))
			{	
				for (int32 NoteIdx = 0; NoteIdx < CurrentPage->Rows.Num(); ++NoteIdx)
				{
					if (CurrentPage->Rows[NoteIdx].bRowEnabled && CurrentPage->Rows[NoteIdx].Cells[CellInRow].bEnabled)
					{
						if (!CurrentCellNotes[NoteIdx])
						{
							// note on!
							// create the midi event with the original note to maintain voice ids.
							FMidiStreamEvent MidiEvent(this, FMidiMsg::CreateNoteOn(MidiChannel, NoteIdx, SequenceTable->Notes[NoteIdx].Velocity));
							// and then assign the note directly to the midi message after
							int32 OriginalNote = SequenceTable->Notes[NoteIdx].NoteNumber;
							int32 TransposedNote = FMath::Clamp(OriginalNote + AdditionalOctaveNotes, 0, 127);
							MidiEvent.MidiMessage.Data1 = TransposedNote;
							float NoteOnVelocity = FMath::Clamp(static_cast<float>(SequenceTable->Notes[NoteIdx].Velocity) * CurrentVelocityMultiplierValue, 0.0f, 127.0f);
							MidiEvent.MidiMessage.SetNoteOnVelocity(static_cast<uint8>(NoteOnVelocity));
							MidiEvent.BlockSampleFrameIndex = BlockFrameIndex;
							MidiEvent.AuthoredMidiTick = ProcessedThruTick;
							MidiEvent.CurrentMidiTick = ProcessedThruTick;
							MidiEvent.TrackIndex = 1;
							MidiOutPin->AddMidiEvent(MidiEvent);
							UE_LOGF(LogStepSequencePlayer, Verbose, "0x%x Note-On %d at %d", (uint32)(size_t)this, SequenceTable->Notes[NoteIdx].NoteNumber + AdditionalOctaveNotes, BlockFrameIndex);
							CurrentCellNotes[NoteIdx] = MidiEvent.GetVoiceId();
							CurrentTransposedNotes[NoteIdx] = TransposedNote;
						}
					}
				}
				CurrentCellIndex = CellInRow;
			}
		}

		if (PlayState == EStepSequencePlayerState::Finished && LastPlayState != PlayState)
		{
			// Loop ended above, make sure we turn everything off
			AllNotesOff(BlockFrameIndex, Tick, true);
		}
	}

	void FStepSequencePlayerOperator::CheckForUpdatedSequenceTable()
	{
		FStepSequenceTableProxy::NodePtr Tester = SequenceAssetInPin->GetRenderable();
		if (Tester != SequenceTable)
		{
			InitSequenceTable();
			return;
		}

		if (SequenceTable)
		{
			TRefCountedAudioRenderableWithQueuedChanges<FStepSequenceTable>* Table = SequenceTable;
			if (Table->HasUpdate())
			{
				SequenceTable = Table->GetUpdate();
				ResizeCellStatesForTable();
				UE_LOGF(LogStepSequencePlayer, Verbose, "Got a NEW Sequence: %d pages, %d rows, %d columns",
					SequenceTable->Pages.Num(),
					SequenceTable->Pages.IsEmpty() ? 0 : SequenceTable->Pages[0].Rows.Num(),
					SequenceTable->Pages.IsEmpty() || SequenceTable->Pages[0].Rows.IsEmpty() ? 0 : SequenceTable->Pages[0].Rows[0].Cells.Num());
			}
		}
	}

	void FStepSequencePlayerOperator::ResizeCellStatesForTable()
	{
		if (!SequenceTable || SequenceTable->Notes.Num() == 0)
		{
			return;
		}

		if (SequenceTable->Notes.Num() < CurrentCellNotes.Num())
		{
			// We may have existing notes that need to be stopped.
			for (int32 NoteIdx = SequenceTable->Notes.Num(); NoteIdx < CurrentCellNotes.Num(); ++NoteIdx)
			{
				if (CurrentCellNotes[NoteIdx])
				{
					FMidiStreamEvent MidiEvent(CurrentCellNotes[NoteIdx].GetGeneratorId(), FMidiMsg::CreateNoteOff(MidiChannel, NoteIdx));
					MidiEvent.BlockSampleFrameIndex  = CurrentBlockSpanStart;
					MidiEvent.AuthoredMidiTick       = 0;
					MidiEvent.CurrentMidiTick        = 0;
					MidiEvent.TrackIndex             = 1;
					UE_LOGF(LogStepSequencePlayer, Verbose, "0x%x Note-Off %d (during resize)", (uint32)(size_t)this, NoteIdx);
					MidiOutPin->AddMidiEvent(MidiEvent);
				}
			}
			CurrentCellNotes.SetNum(SequenceTable->Notes.Num());
			CurrentTransposedNotes.SetNum(SequenceTable->Notes.Num());
		}
		else
		{
			while (CurrentCellNotes.Num() < SequenceTable->Notes.Num())
			{
				CurrentCellNotes.Add(FMidiVoiceId());
				CurrentTransposedNotes.Add(0u);
			}
		}
	}

	void FStepSequencePlayerOperator::AllNotesOff(int32 AtFrameIndex, int32 AbsMidiTick, bool ResetCellIndex)
	{
		for (int NoteIdx = 0; NoteIdx < CurrentCellNotes.Num(); ++NoteIdx)
		{
			if (CurrentCellNotes[NoteIdx])
			{
				FMidiStreamEvent MidiEvent(CurrentCellNotes[NoteIdx].GetGeneratorId(), FMidiMsg::CreateNoteOff(MidiChannel, NoteIdx));
				MidiEvent.BlockSampleFrameIndex  = AtFrameIndex;
				MidiEvent.AuthoredMidiTick       = AbsMidiTick;
				MidiEvent.CurrentMidiTick        = AbsMidiTick;
				MidiEvent.TrackIndex             = 1;
				MidiOutPin->AddMidiEvent(MidiEvent);
				UE_LOGF(LogStepSequencePlayer, Verbose, "0x%x Note-Off %d (during all notes off)", (uint32)(size_t)this, NoteIdx);
				CurrentCellNotes[NoteIdx] = FMidiVoiceId::None();
				CurrentTransposedNotes[NoteIdx] = 0u;
			}
		}
		if (ResetCellIndex)
		{
			CurrentCellIndex = -1;
		}
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
