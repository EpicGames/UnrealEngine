// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/StutterControllerNode.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/MidiOps/PulseGenerator.h"
#include "HarmonixMidi/MusicTimeSpan.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_StutterControllerNode"

namespace HarmonixMetasound::StutterControllerNode
{
	using namespace Metasound;
	using namespace Harmonix;
	using namespace HarmonixMetasound;

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_INPUT_METASOUND_PARAM(Start, "Start", "Starts the stutter sequence.");
		DEFINE_INPUT_METASOUND_PARAM(Stop, "Stop", "Stops the stutter sequence, and resets the simple sampler.");
		DEFINE_INPUT_METASOUND_PARAM(CaptureDuration, "Capture Duration", "How long a sample to capture as a unit of musical time.");
		DEFINE_INPUT_METASOUND_PARAM(Spacing, "Spacing", "The spacing of each stutter as a unit of musical time.");
		DEFINE_INPUT_METASOUND_PARAM(AudibleDuration, "Audible Duration", "The duration of the audible portion of the stutter as a unit of music time.");
		DEFINE_INPUT_METASOUND_PARAM(Reverse, "Reverse", "Whether to play the audio backward.");
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(Capture, "Capture", "Trigger to start capturing the input audio.");
		DEFINE_OUTPUT_METASOUND_PARAM(CaptureDuration, "Capture Duration Sec", "How long a sample to capture in seconds.");
		DEFINE_OUTPUT_METASOUND_PARAM(Play, "Play", "Trigger to play back the captured buffer.");
		DEFINE_OUTPUT_METASOUND_PARAM(PlayDuration, "Play Duration Sec", "How much of the captured sample to play.");
		DEFINE_OUTPUT_METASOUND_PARAM(Reverse, "Reverse Playback", "If true the captured buffer should play in reverse.");
		DEFINE_OUTPUT_METASOUND_PARAM(Reset, "Reset", "Trigger to stop capture/playback and start passing audio from input to output.");
	}

	FVertexInterface GetVertexInterface()
	{
		FInputVertexInterface InputInterface;
		FOutputVertexInterface OutputInterface;

		InputInterface.Add(TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)));
		InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Start)));
		InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Stop)));
		InputInterface.Add(TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::CaptureDuration)));
		InputInterface.Add(TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Spacing)));
		InputInterface.Add(TInputDataVertex<FEnumMidiClockSubdivisionQuantizationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudibleDuration)));
		InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Reverse)));

		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Capture)));
		OutputInterface.Add(TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::CaptureDuration)));
		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Play)));
		OutputInterface.Add(TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::PlayDuration)));
		OutputInterface.Add(TOutputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Reverse)));
		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Reset)));

		return FVertexInterface
		{
			MoveTemp(InputInterface),
			MoveTemp(OutputInterface)
		};
	}

	class FStutterControllerOperator final : public TExecutableOperator<FStutterControllerOperator>
	{
	public:
		struct FInputs
		{
			FMidiClockReadRef            MidiClock;
			FTriggerReadRef              Start;
			FTriggerReadRef              Stop;
			FEnumMidiClockSubdivisionQuantizationReadRef CaptureDuration;
			FEnumMidiClockSubdivisionQuantizationReadRef Spacing;
			FEnumMidiClockSubdivisionQuantizationReadRef AudibleDuration;
			FBoolReadRef                 Reverse;
		};

		struct FOutputs
		{
			FTriggerWriteRef Capture;
			FFloatWriteRef   CaptureDuration;
			FTriggerWriteRef Play;
			FFloatWriteRef   PlayDuration;
			FBoolWriteRef    Reverse;
			FTriggerWriteRef Reset;
		};
		
		FStutterControllerOperator(const FBuildOperatorParams& Params, FInputs&& Inputs, FOutputs&& Outputs)
			: Inputs(MoveTemp(Inputs))
			, Outputs(MoveTemp(Outputs))
		{
			Reset(Params);
		}
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = FNodeClassMetadata::GetEmpty();
				Info.ClassName = { HarmonixNodeNamespace, TEXT("StutterController"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("StutterController_DisplayName", "Stutter Controller");
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix };
				Info.Description = METASOUND_LOCTEXT("StutterController_Description", "Generate triggers based on the given settings and uses it to control a Simple Sampler node to create a stutter edit effect.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FInputs Inputs {
					InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(Inputs::MidiClockName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::StartName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::StopName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FEnumMidiClockSubdivisionQuantizationType>(Inputs::CaptureDurationName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FEnumMidiClockSubdivisionQuantizationType>(Inputs::SpacingName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FEnumMidiClockSubdivisionQuantizationType>(Inputs::AudibleDurationName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::ReverseName, InParams.OperatorSettings)
			};

			FOutputs Outputs {	
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
					FFloatWriteRef::CreateNew(),
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
					FFloatWriteRef::CreateNew(),
					FBoolWriteRef::CreateNew(),
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
			};

			return MakeUnique<FStutterControllerOperator>(
				InParams,
				MoveTemp(Inputs),
				MoveTemp(Outputs)
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::MidiClockName, Inputs.MidiClock);
			InVertexData.BindReadVertex(Inputs::StartName, Inputs.Start);
			InVertexData.BindReadVertex(Inputs::StopName, Inputs.Stop);
			InVertexData.BindReadVertex(Inputs::CaptureDurationName, Inputs.CaptureDuration);
			InVertexData.BindReadVertex(Inputs::SpacingName, Inputs.Spacing);
			InVertexData.BindReadVertex(Inputs::AudibleDurationName, Inputs.AudibleDuration);
			InVertexData.BindReadVertex(Inputs::ReverseName, Inputs.Reverse);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::CaptureName, Outputs.Capture);
			InVertexData.BindReadVertex(Outputs::CaptureDurationName, Outputs.CaptureDuration);
			InVertexData.BindReadVertex(Outputs::PlayName, Outputs.Play);
			InVertexData.BindReadVertex(Outputs::PlayDurationName, Outputs.PlayDuration);
			InVertexData.BindReadVertex(Outputs::ReverseName, Outputs.Reverse);
			InVertexData.BindReadVertex(Outputs::ResetName, Outputs.Reset);
		}
		
		void Reset(const FResetParams& Params)
		{
			bRunning = false;
			Outputs.Capture->Reset();
			*Outputs.CaptureDuration = 0.0f;
			Outputs.Play->Reset();
			*Outputs.PlayDuration = 0.0f;
			*Outputs.Reverse = false;
			Outputs.Reset->Reset();
			PulseGenerator.Reset();
		}
		
		void Execute()
		{
			Outputs.Capture->AdvanceBlock();
			Outputs.Play->AdvanceBlock();
			Outputs.Reset->AdvanceBlock();
			
			PulseGenerator.SetInterval({
            	*Inputs.Spacing,
            	*Inputs.Spacing,
            	1,
            	0
            });

			if (!bRunning && Inputs.Start->IsTriggeredInBlock())
			{
				bTriggerStart = true;
			}
			
			PulseGenerator.Process(*Inputs.MidiClock,
			[this](const Midi::Ops::FPulseGenerator::FPulseInfo& Pulse)
			{
				if (bRunning)
				{
					const ISongMapEvaluator& SongMaps = Inputs.MidiClock->GetSongMapEvaluator();
					EMusicTimeSpanLengthUnits AudibleDuration = static_cast<EMusicTimeSpanLengthUnits>(Inputs.AudibleDuration->ToInt());
					int32 NumTicksToPlay = MusicalTimeSpan::CalculateMusicalSpanLengthTicks(AudibleDuration, SongMaps, Pulse.Tick);
					float MsOfPlayStart = SongMaps.TickToMs(Pulse.Tick);
					float MsOfPlayEnd = SongMaps.TickToMs(Pulse.Tick + NumTicksToPlay);
					float NewPlayDuration = ((MsOfPlayEnd - MsOfPlayStart) / 1000.0f) / Inputs.MidiClock->GetSpeedAtBlockSampleFrame(Pulse.BlockFrameIndex);
					*Outputs.PlayDuration = NewPlayDuration;
					*Outputs.Reverse = *Inputs.Reverse;
					Outputs.Play->TriggerFrame(Pulse.BlockFrameIndex);
				}

				if (!bRunning && bTriggerStart)
				{
					const ISongMapEvaluator& SongMaps = Inputs.MidiClock->GetSongMapEvaluator();
					EMusicTimeSpanLengthUnits CaptureDuration = static_cast<EMusicTimeSpanLengthUnits>(Inputs.CaptureDuration->ToInt());
					int32 NumTicksToCapture = MusicalTimeSpan::CalculateMusicalSpanLengthTicks(CaptureDuration, SongMaps, Pulse.Tick);
					float MsOfCaptureStart = SongMaps.TickToMs(Pulse.Tick);
					float MsOfCaptureEnd = SongMaps.TickToMs(Pulse.Tick + NumTicksToCapture);
					*Outputs.CaptureDuration = ((MsOfCaptureEnd - MsOfCaptureStart) / 1000.0f) / Inputs.MidiClock->GetSpeedAtBlockSampleFrame(Pulse.BlockFrameIndex);
					Outputs.Capture->TriggerFrame(Pulse.BlockFrameIndex);
					bRunning = true;
				}
				
				bTriggerStart = false;
			},
			[this](const Midi::Ops::FPulseGenerator::FPulseInfo& PulseOff)
			{
				
			});

			if (bRunning && Inputs.Start->Last() < Inputs.Stop->Last())
			{
				bTriggerStart = false;
				bRunning = false;
				Outputs.Reset->TriggerFrame(Inputs.Stop->Last());
			}
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;

		bool bTriggerStart = false;
		bool bRunning = false;

		//** DATA (current state)
		Midi::Ops::FPulseGenerator PulseGenerator;
	};

	using FStutterControllerNode = Metasound::TNodeFacade<FStutterControllerOperator>;
	METASOUND_REGISTER_NODE(FStutterControllerNode);
}

#undef LOCTEXT_NAMESPACE
