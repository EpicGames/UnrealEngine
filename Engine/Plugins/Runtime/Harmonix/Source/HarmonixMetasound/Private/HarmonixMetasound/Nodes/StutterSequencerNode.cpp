// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/StutterSequencerNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MidiStutterSequence.h"
#include "IAudioProxyInitializer.h"
#include "Harmonix/AudioRenderableProxy.h"
#include "Algo/FindLast.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_StutterSequencerNode"

namespace HarmonixMetasound::StutterSequencerNode
{
	using namespace Metasound;
	using namespace Harmonix;
	using namespace HarmonixMetasound;

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enabled, CommonPinNames::Inputs::Enable);
		DEFINE_INPUT_METASOUND_PARAM(StutterSequence, "Stutter Sequence", "The asset describing the sequence of stutters to play.");
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
		DEFINE_INPUT_METASOUND_PARAM(Start, "Start", "Starts the stutter sequence.");
		DEFINE_INPUT_METASOUND_PARAM(Stop, "Stop", "Stops the stutter sequence, and resets the simple sampler.");
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(Capture, "Capture", "Trigger to start capturing the input audio.");
		DEFINE_OUTPUT_METASOUND_PARAM(CaptureDuration, "Capture Duration Sec", "How long a sample to capture.");
		DEFINE_OUTPUT_METASOUND_PARAM(Play, "Play", "Trigger to play back the captured buffer.");
		DEFINE_OUTPUT_METASOUND_PARAM(PlayDuration, "Play Duration Sec", "How much of the captured sample to play.");
		DEFINE_OUTPUT_METASOUND_PARAM(Reverse, "Reverse Playback", "If true the captured buffer should play in reverse.");
		DEFINE_OUTPUT_METASOUND_PARAM(Reset, "Reset", "Trigger to stop capture/playback and start passing audio from input to output.");
	}

	FVertexInterface GetVertexInterface()
	{
		FInputVertexInterface InputInterface;
		FOutputVertexInterface OutputInterface;

		InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enabled)));
		InputInterface.Add(TInputDataVertex<FStutterSequenceAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::StutterSequence)));
		InputInterface.Add(TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock)));
		InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Start)));
		InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Stop)));

		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Capture)));
		OutputInterface.Add(TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::CaptureDuration)));
		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Play)));
		OutputInterface.Add(TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::PlayDuration)));
		OutputInterface.Add(TOutputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Reverse)));
		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Reset)));

		return FVertexInterface
		{
			MoveTemp(InputInterface),
			MoveTemp(OutputInterface)
		};
	}

	class FStutterSequencerOperator final : public TExecutableOperator<FStutterSequencerOperator>
	{
	public:
		struct FInputs
		{
			FBoolReadRef                 Enabled;
			FStutterSequenceAssetReadRef StutterSequence;
			FMidiClockReadRef            MidiClock;
			FTriggerReadRef              Start;
			FTriggerReadRef              Stop;
		};

		struct FOutputs
		{
			FTriggerWriteRef Capture;
			FFloatWriteRef   CaptureDuration;
			FTriggerWriteRef Play;
			FFloatWriteRef   PlayDuration;
			FBoolWriteRef    Reverse;
			FTriggerWriteRef Reset;
		};
		
		FStutterSequencerOperator(const FBuildOperatorParams& Params, FInputs&& Inputs, FOutputs&& Outputs)
			: Inputs(MoveTemp(Inputs))
			, Outputs(MoveTemp(Outputs))
		{
			Reset(Params);
		}
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = FNodeClassMetadata::GetEmpty();
				Info.ClassName = { HarmonixNodeNamespace, TEXT("StutterSequencer"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("StutterSequencer_DisplayName", "Stutter Sequencer");
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix };
				Info.Description = METASOUND_LOCTEXT("StutterSequencer_Description", "Takes a Stutter Sequence asset and uses it to control a Simple Sampler node to create a stutter edit effect.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FInputs Inputs {
					InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnabledName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FStutterSequenceAsset>(Inputs::StutterSequenceName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(Inputs::MidiClockName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::StartName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<FTrigger>(Inputs::StopName, InParams.OperatorSettings),
			};

			FOutputs Outputs {	
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
					FFloatWriteRef::CreateNew(),
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
					FFloatWriteRef::CreateNew(),
					FBoolWriteRef::CreateNew(),
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
			};

			return MakeUnique<FStutterSequencerOperator>(
				InParams,
				MoveTemp(Inputs),
				MoveTemp(Outputs)
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnabledName, Inputs.Enabled);
			InVertexData.BindReadVertex(Inputs::StutterSequenceName, Inputs.StutterSequence);
			InVertexData.BindReadVertex(Inputs::MidiClockName, Inputs.MidiClock);
			InVertexData.BindReadVertex(Inputs::StartName, Inputs.Start);
			InVertexData.BindReadVertex(Inputs::StopName, Inputs.Stop);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::CaptureName, Outputs.Capture);
			InVertexData.BindReadVertex(Outputs::CaptureDurationName, Outputs.CaptureDuration);
			InVertexData.BindReadVertex(Outputs::PlayName, Outputs.Play);
			InVertexData.BindReadVertex(Outputs::PlayDurationName, Outputs.PlayDuration);
			InVertexData.BindReadVertex(Outputs::ReverseName, Outputs.Reverse);
			InVertexData.BindReadVertex(Outputs::ResetName, Outputs.Reset);
		}
		
		void Reset(const FResetParams& Params)
		{
			StutterTable = Inputs.StutterSequence->GetRenderable();
			bRunning = false;
			StartTick = -1;
			NextStutterTick = -1;
			CurrentStutterEntryIndex = 0;
			NumStuttersPlayedThisEntry = 0;
			Outputs.Capture->Reset();
			*Outputs.CaptureDuration = 0.0f;
			Outputs.Play->Reset();
			*Outputs.PlayDuration = 0.0f;
			*Outputs.Reverse = false;
			Outputs.Reset->Reset();
		}
		
		void Execute()
		{
			Outputs.Capture->AdvanceBlock();
			Outputs.Play->AdvanceBlock();
			Outputs.Reset->AdvanceBlock();

			CheckForUpdatedStutterTable();

			// should we be running at all?
			if (!StutterTable || !*Inputs.Enabled || (!bRunning && !Inputs.Start->IsTriggeredInBlock()))
			{
				if (bRunning)
				{
					bRunning = false;
					if (StutterTable && StutterTable->ResetOnCompletion)
					{
						Outputs.Reset->TriggerFrame(0);
					}
				}
				return;
			}

			if (Inputs.MidiClock->GetTransportStateAtStartOfBlock() != EMusicPlayerTransportState::Playing &&
				Inputs.MidiClock->GetTransportStateAtEndOfBlock() != EMusicPlayerTransportState::Playing)
			{
				// nothing happening!
				return;
			}

			if (StutterTable->ResetOnCompletion)
			{
				for (int32 StopFrame : Inputs.Stop->GetTriggeredFrames())
				{
					Outputs.Reset->TriggerFrame(StopFrame);
				}
			}

			if (!bRunning)
			{
				// There must be a new start trigger this block or we wouldn't be here!
				int32 BlockFrame = Inputs.Start->Last();
				Outputs.Capture->TriggerFrame(BlockFrame);
				StartTick = Inputs.MidiClock->GetNextTickToProcessAtBlockFrame(BlockFrame);
				NextStutterTick = StartTick;
				CurrentStutterEntryIndex = 0;
				NumStuttersPlayedThisEntry = 0;
				const ISongMapEvaluator& SongMaps = Inputs.MidiClock->GetSongMapEvaluator();
				int32 NumTicksToCapture = MusicalTimeSpan::CalculateMusicalSpanLengthTicks(StutterTable->CaptureDuration, SongMaps, StartTick);
				float MsOfCaptureStart = SongMaps.TickToMs(StartTick);
				float MsOfCaptureEnd = SongMaps.TickToMs(StartTick + NumTicksToCapture);
				*Outputs.CaptureDuration = ((MsOfCaptureEnd - MsOfCaptureStart) / 1000.0f) / Inputs.MidiClock->GetSpeedAtBlockSampleFrame(BlockFrame);
				bRunning = true;
			}

			for (const FMidiClockEvent& Event : Inputs.MidiClock->GetMidiClockEventsInBlock())
			{
				using namespace MidiClockMessageTypes;

				if (const FAdvance* AsAdvance = Event.TryGet<FAdvance>())
				{
					if (AsAdvance->LastTickToProcess() >= NextStutterTick)
					{
						// We may actually be at the end of the last stutter in the table...
						if (CurrentStutterEntryIndex == StutterTable->Stutters.Num())
						{
							// yup... done...
							NextStutterTick = -1;
							bRunning = false;
							if (StutterTable->ResetOnCompletion)
							{
								Outputs.Reset->TriggerFrame(Event.BlockFrameIndex);
							}
							break;
						}

						// we have a stutter to play...
						Outputs.Play->TriggerFrame(Event.BlockFrameIndex);

						// Calculate the play duration and figure out when this stutter will be done
						// so we can come back here and trigger the next one...
						const ISongMapEvaluator& SongMaps = Inputs.MidiClock->GetSongMapEvaluator();
						int32 NumTicksToPlay = MusicalTimeSpan::CalculateMusicalSpanLengthTicks(StutterTable->Stutters[CurrentStutterEntryIndex].AudibleDuration, SongMaps, NextStutterTick);
						int32 NumTicksToNextStutter = MusicalTimeSpan::CalculateMusicalSpanLengthTicks(StutterTable->Stutters[CurrentStutterEntryIndex].Spacing, SongMaps, NextStutterTick);
						if (NumTicksToNextStutter < NumTicksToPlay)
						{
							NumTicksToPlay = NumTicksToNextStutter;
						}
						float MsOfPlayStart = SongMaps.TickToMs(NextStutterTick);
						float MsOfPlayEnd = SongMaps.TickToMs(NextStutterTick + NumTicksToPlay);
						float NewPlayDuration = ((MsOfPlayEnd - MsOfPlayStart) / 1000.0f) / Inputs.MidiClock->GetSpeedAtBlockSampleFrame(Event.BlockFrameIndex);
						*Outputs.PlayDuration = NewPlayDuration;

						*Outputs.Reverse = StutterTable->Stutters[CurrentStutterEntryIndex].Reverse;

						NextStutterTick += NumTicksToNextStutter;
						++NumStuttersPlayedThisEntry;
						if (NumStuttersPlayedThisEntry >= StutterTable->Stutters[CurrentStutterEntryIndex].Count)
						{
							++CurrentStutterEntryIndex;
							NumStuttersPlayedThisEntry = 0;
						}
					}
				}
			}

			// and finally... 
			if (Inputs.Start->Last() < Inputs.Stop->Last())
			{
				bRunning = false;
				StartTick = -1;
				NextStutterTick = -1;
				if (StutterTable->ResetOnCompletion)
				{
					Outputs.Reset->TriggerFrame(Inputs.Stop->Last());
				}
			}
		}

		void CheckForUpdatedStutterTable()
		{
			FStutterSequenceTableProxy::NodePtr Tester = Inputs.StutterSequence->GetRenderable();
			if (Tester != StutterTable)
			{
				StutterTable = Inputs.StutterSequence->GetRenderable();
				return;
			}
			if (StutterTable)
			{
				TRefCountedAudioRenderableWithQueuedChanges<FStutterSequenceTable>* AsTableWithPossibleChanges = StutterTable;
				if (AsTableWithPossibleChanges->HasUpdate())
				{
					StutterTable = AsTableWithPossibleChanges->GetUpdate();
				}
			}
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;

		TSharedAudioRenderableDataPtr<FStutterSequenceTable, TRefCountedAudioRenderableWithQueuedChanges<FStutterSequenceTable>> StutterTable;
		bool bRunning = false;
		int32 StartTick = -1;
		int32 NextStutterTick = -1;
		int32 CurrentStutterEntryIndex = 0;
		int32 NumStuttersPlayedThisEntry = 0;
	};

	using FStutterSequencerNode = Metasound::TNodeFacade<FStutterSequencerOperator>;
	METASOUND_REGISTER_NODE(FStutterSequencerNode);
}

#undef LOCTEXT_NAMESPACE
