// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Nodes/TapeSSNode.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
//#include "HarmonixDsp/StridePointer.h"
#include "HarmonixDsp/Effects/TapeSS.h"
#include "HarmonixMetasound/Common.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TapeSSNode)

#define LOCTEXT_NAMESPACE "HarmonixMetaSound_TapeSSNode"

namespace HarmonixMetasound::TapeSSNode
{
	using namespace Metasound;
	using namespace Harmonix::Dsp::Effects;

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Enabled, CommonPinNames::Inputs::Enable);
		DEFINE_INPUT_METASOUND_PARAM(Stop, "Stop", "When toggled from false to true, causes audio to slow down and stop.");
		DEFINE_INPUT_METASOUND_PARAM(RampDuration, "Duration", "How long it should take to slow down from full speed to stopped in seconds.");
		static constexpr int32 kNumNonAudioInputs = 3; // NOTE: Keep in sync with the number of input vertecies defined above!
	}

	namespace Outputs
	{
		DEFINE_OUTPUT_METASOUND_PARAM(InputAudioInUse, "Input Audio In Use", "True when audio from the input is being used to generate the output. False when the input audio is unneeded and can be changed.");
		DEFINE_OUTPUT_METASOUND_PARAM(OnStopped, "On Stopped", "Triggered when the audio has completely stopped.");
		DEFINE_OUTPUT_METASOUND_PARAM(OnFullSpeed, "On Full Speed", "Triggered when the audio has sped back up to full speed.");
		DEFINE_OUTPUT_METASOUND_PARAM(CurrentSpeed, "CurrentSpeed", "A value from 0 (stopped) to 1 (playing normally).");
	}

	const FLazyName FTapeSSNodeOperatorData::OperatorDataTypeName = "TapeStopStartOperatorData";

	static TArray<FVertexName> CreateVertexNames(const TCHAR* Prefix, int32 NumChannels)
	{
		TStringBuilder<32> NameStr;
		TArray<FVertexName> Names;
		Names.AddUninitialized(NumChannels);

		for (int ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			NameStr << Prefix << ChannelIndex;
			Names[ChannelIndex] = *NameStr;
			NameStr.Reset();
		}

		return Names;
	}

	static const TArray<FVertexName> AudioInputNames = CreateVertexNames(TEXT("In "), kMaxChannels);
	static const TArray<FVertexName> AudioOutputNames = CreateVertexNames(TEXT("Out "), kMaxChannels);

#if WITH_EDITOR
	static const FText GetAudioInputDisplayName(uint32 ChannelIndex)
	{
		return METASOUND_LOCTEXT_FORMAT("AudioInputDisplayName", "In {0}", ChannelIndex);
	}

	static const FText GetAudioOutputDisplayName(uint32 ChannelIndex)
	{
		return METASOUND_LOCTEXT_FORMAT("AudioOutputDisplayName", "Out {0}", ChannelIndex);
	}
#endif // WITH_EDITOR

	FVertexInterface GetVertexInterface(int32 NumChannels)
	{
		using namespace HarmonixMetasound::TapeSSNode;

		FInputVertexInterface InputInterface;
		FOutputVertexInterface OutputInterface;

		InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Enabled)));
		InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Stop)));
		InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::RampDuration)));

		OutputInterface.Add(TOutputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::InputAudioInUse)));
		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::OnStopped)));
		OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::OnFullSpeed)));
		OutputInterface.Add(TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::CurrentSpeed)));

		check (InputInterface.Num() == Inputs::kNumNonAudioInputs);
		check (NumChannels <= kMaxChannels);

		TStringBuilder<32> NameStr;
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
#if WITH_EDITOR
			const FDataVertexMetadata AudioInputMetadata
			{
				METASOUND_LOCTEXT("AudioInputDescription", "Audio Input"),
				GetAudioInputDisplayName(ChannelIndex)
			};
#else 
			const FDataVertexMetadata AudioInputMetadata;
#endif // WITH_EDITOR
			InputInterface.Add(TInputDataVertex<FAudioBuffer>(AudioInputNames[ChannelIndex], AudioInputMetadata));

#if WITH_EDITOR
			const FDataVertexMetadata AudioOutputMetadata
			{
				METASOUND_LOCTEXT("AudioOutputDescription", "Audio Output"),
				GetAudioOutputDisplayName(ChannelIndex)
			};
#else 
			const FDataVertexMetadata AudioOutputMetadata;
#endif // WITH_EDITOR
			OutputInterface.Add(TOutputDataVertex<FAudioBuffer>(AudioOutputNames[ChannelIndex], AudioOutputMetadata));
		}

		return FVertexInterface
		{
			MoveTemp(InputInterface),
			MoveTemp(OutputInterface)
		};
	}

	class FTapeSSOperator final : public TExecutableOperator<FTapeSSOperator>
	{
	public:
		struct FInputs
		{
			FBoolReadRef        Enabled;
			FBoolReadRef        Stop;
			FFloatReadRef       RampDuration;
			TArray<FAudioBufferReadRef> AudioInputs;
		};

		struct FOutputs
		{
			FBoolWriteRef        InputAudioInUse;
			FTriggerWriteRef     OnStopped;
			FTriggerWriteRef     OnFullSpeed;
			FFloatWriteRef       CurrentSpeed;
			TArray<FAudioBufferWriteRef> AudioOutputs;
		};
		
		FTapeSSOperator(const FBuildOperatorParams& Params, FInputs&& Inputs, FOutputs&& Outputs)
			: Inputs(MoveTemp(Inputs))
			, Outputs(MoveTemp(Outputs))
		{
			TSharedPtr<const FTapeSSNodeOperatorData> InOperatorData = StaticCastSharedPtr<const FTapeSSNodeOperatorData>(Params.Node.GetOperatorData());
			MaxRampSeconds = InOperatorData ? InOperatorData->MaxRampSeconds : 5.0f;
			Reset(Params);
		}
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = FNodeClassMetadata::GetEmpty();
				Info.ClassName = { HarmonixNodeNamespace, TEXT("TapeSS"), TEXT("") };
				Info.MajorVersion = 0;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("TapeSS_DisplayName", "Tape Stop/Start");
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix };
				Info.Description = METASOUND_LOCTEXT("TapeSS_Description", "Simulates the stopping and starting of a tape machine.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface(1);
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FVertexInterface& NodeInterface = InParams.Node.GetVertexInterface();
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TArray<FAudioBufferReadRef> AudioInputs;
			TArray<FAudioBufferWriteRef> AudioOutputs;
			int32 NumAudioChannels = NodeInterface.GetInputInterface().Num() - Inputs::kNumNonAudioInputs;
			for (int32 ChannelIndex = 0; ChannelIndex < NumAudioChannels; ++ChannelIndex)
			{
				AudioInputs.Emplace(InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(AudioInputNames[ChannelIndex], InParams.OperatorSettings));
				AudioOutputs.Emplace(TDataWriteReferenceFactory<FAudioBuffer>::CreateExplicitArgs(InParams.OperatorSettings));
			}

			FInputs Inputs {
					InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::EnabledName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<bool>(Inputs::StopName, InParams.OperatorSettings),
					InputData.GetOrCreateDefaultDataReadReference<float>(Inputs::RampDurationName, InParams.OperatorSettings),
					MoveTemp(AudioInputs)
			};

			FOutputs Outputs {	
					FBoolWriteRef::CreateNew(),
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
					FTriggerWriteRef::CreateNew(InParams.OperatorSettings),
					FFloatWriteRef::CreateNew(),
					MoveTemp(AudioOutputs)
			};

			return MakeUnique<FTapeSSOperator>(
				InParams,
				MoveTemp(Inputs),
				MoveTemp(Outputs)
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Inputs::EnabledName, Inputs.Enabled);
			InVertexData.BindReadVertex(Inputs::StopName, Inputs.Stop);
			InVertexData.BindReadVertex(Inputs::RampDurationName, Inputs.RampDuration);
			for (int32 ChannelIndex = 0; ChannelIndex < Inputs.AudioInputs.Num(); ++ChannelIndex)
			{
				InVertexData.BindReadVertex(AudioInputNames[ChannelIndex], Inputs.AudioInputs[ChannelIndex]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(Outputs::InputAudioInUseName, Outputs.InputAudioInUse);
			InVertexData.BindReadVertex(Outputs::OnStoppedName, Outputs.OnStopped);
			InVertexData.BindReadVertex(Outputs::OnFullSpeedName, Outputs.OnFullSpeed);
			InVertexData.BindReadVertex(Outputs::CurrentSpeedName, Outputs.CurrentSpeed);
			for (int32 ChannelIndex = 0; ChannelIndex < Outputs.AudioOutputs.Num(); ++ChannelIndex)
			{
				InVertexData.BindReadVertex(AudioOutputNames[ChannelIndex], Outputs.AudioOutputs[ChannelIndex]);
			}
		}
		
		void Reset(const FResetParams& Params)
		{
			TapeSS.Prepare(Params.OperatorSettings.GetSampleRate(), Inputs.AudioInputs.Num(), MaxRampSeconds);
			*Outputs.InputAudioInUse = true;
			Outputs.OnStopped->Reset();
			Outputs.OnFullSpeed->Reset();
			*Outputs.CurrentSpeed = 0.0f;
			ChannelInputPointers.SetNumUninitialized(Inputs.AudioInputs.Num());
			ChannelOutputPointers.SetNumUninitialized(Inputs.AudioInputs.Num());
		}

		void Execute()
		{
			Outputs.OnStopped->AdvanceBlock();
			Outputs.OnFullSpeed->AdvanceBlock();

			if (!*Inputs.Enabled)
			{
				*Outputs.InputAudioInUse = true;
				for (int32 ChannelIndex = 0; ChannelIndex < Outputs.AudioOutputs.Num(); ++ChannelIndex)
				{
					FMemory::Memcpy(Outputs.AudioOutputs[ChannelIndex]->GetData(), Inputs.AudioInputs[ChannelIndex]->GetData(), sizeof(float) * Outputs.AudioOutputs[ChannelIndex]->Num());
				}
				return;
			}

			if (*Inputs.Stop && TapeSS.GetState() != FTapeStopStart::EState::Stop)
			{
				TapeSS.SetState(FTapeStopStart::EState::Stop, FMath::Min(*Inputs.RampDuration, MaxRampSeconds));
			}
			else if (!*Inputs.Stop && TapeSS.GetState() == FTapeStopStart::EState::Stop)
			{
				TapeSS.SetState(FTapeStopStart::EState::Play, FMath::Min(*Inputs.RampDuration, MaxRampSeconds));
			}

			for (int32 ChannelIndex = 0; ChannelIndex < Inputs.AudioInputs.Num(); ++ChannelIndex)
			{
				ChannelInputPointers[ChannelIndex] = Inputs.AudioInputs[ChannelIndex]->GetData();
				ChannelOutputPointers[ChannelIndex] = Outputs.AudioOutputs[ChannelIndex]->GetData();
			}

			FTapeStopStart::EPhase CurrentPhase = TapeSS.Process(ChannelInputPointers, ChannelOutputPointers, Inputs.AudioInputs[0]->Num());
			if (CurrentPhase != LastPhase)
			{
				if (CurrentPhase == FTapeStopStart::EPhase::Playing)
				{
					Outputs.OnFullSpeed->TriggerFrame(0);
				}
				else if (CurrentPhase == FTapeStopStart::EPhase::Stopped)
				{
					Outputs.OnStopped->TriggerFrame(0);
				}
				LastPhase = CurrentPhase;
			}
			*Outputs.InputAudioInUse = true;
			if (CurrentPhase == FTapeStopStart::EPhase::Stopped || (CurrentPhase == FTapeStopStart::EPhase::SlowingDown && TapeSS.GetSpeed() < 0.5f))
			{
				*Outputs.InputAudioInUse = false;
			}
			*Outputs.CurrentSpeed = TapeSS.GetSpeed();
		}

	private:
		FInputs Inputs;
		FOutputs Outputs;
		FTapeStopStart TapeSS;
		FTapeStopStart::EPhase LastPhase = FTapeStopStart::EPhase::Playing;
		TArray<const float*> ChannelInputPointers;
		TArray<float*> ChannelOutputPointers;
		int32 BlockSamples;
		float MaxRampSeconds = 5.0f;
	};

	using FTapeSSNode = Metasound::TNodeFacade<FTapeSSOperator>;
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FTapeSSNode, FHarmonixTapeSSNodeConfiguration);
}

TInstancedStruct<FMetasoundFrontendClassInterface> FHarmonixTapeSSNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(HarmonixMetasound::TapeSSNode::GetVertexInterface(NumChannels)));
}

TSharedPtr<const Metasound::IOperatorData> FHarmonixTapeSSNodeConfiguration::GetOperatorData() const
{
	return MakeShared<HarmonixMetasound::TapeSSNode::FTapeSSNodeOperatorData>(MaxRampSeconds);
}

#undef LOCTEXT_NAMESPACE
