// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/Nodes/TransportToTriggerNode.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::TransportToTriggerNode
{
	using namespace Metasound;
	
	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("TransportToTrigger"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Transport, CommonPinNames::Inputs::Transport);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(TransportPrepare, CommonPinNames::Outputs::TransportPrepare);
		DEFINE_METASOUND_PARAM_ALIAS(TransportPlay, CommonPinNames::Outputs::TransportPlay);
		DEFINE_METASOUND_PARAM_ALIAS(TransportPause, CommonPinNames::Outputs::TransportPause);
		DEFINE_METASOUND_PARAM_ALIAS(TransportContinue, CommonPinNames::Outputs::TransportContinue);
		DEFINE_METASOUND_PARAM_ALIAS(TransportStop, CommonPinNames::Outputs::TransportStop);
		DEFINE_METASOUND_PARAM_ALIAS(TransportKill, CommonPinNames::Outputs::TransportKill);
		DEFINE_METASOUND_PARAM_ALIAS(TransportSeek, CommonPinNames::Outputs::TransportSeek);
	}

	class FTransportToTriggerOperator : public TExecutableOperator<FTransportToTriggerOperator>
	{
	public:
		FTransportToTriggerOperator(const FBuildOperatorParams& InParams,
									const FMusicTransportEventStreamReadRef& InTransport)
			: TransportInPin(InTransport)
			, PrepareOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, PlayOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, PauseOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, ContinueOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, StopOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, KillOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, SeekOutPin(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace CommonPinNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Transport))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportPrepare)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportPlay)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportPause)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportContinue)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportStop)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportKill)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportSeek))
				)
			);

			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = GetCurrentMajorVersion();
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("TransportToTriggerNode_DisplayName", "Music Transport to Trigger");
				Info.Description = METASOUND_LOCTEXT("TransportToTriggerNode_Description", "Breaks out music transport requests into individual triggers");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace CommonPinNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Transport), TransportInPin);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace CommonPinNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportPrepare), PrepareOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportPlay), PlayOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportPause), PauseOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportContinue), ContinueOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportStop), StopOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportKill), KillOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportSeek), SeekOutPin);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CommonPinNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FMusicTransportEventStreamReadRef InTransport = InputData.GetOrCreateDefaultDataReadReference<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME(Inputs::Transport), InParams.OperatorSettings);
			
			return MakeUnique<FTransportToTriggerOperator>(InParams, InTransport);
		}

		void Reset(const FResetParams& ResetParams)
		{
			PrepareOutPin->Reset();
			PlayOutPin->Reset();
			PauseOutPin->Reset();
			ContinueOutPin->Reset();
			StopOutPin->Reset();
			KillOutPin->Reset();
			SeekOutPin->Reset();
		}

		void Execute()
		{
			// advance the outputs
			PrepareOutPin->AdvanceBlock();
			PlayOutPin->AdvanceBlock();
			PauseOutPin->AdvanceBlock();
			ContinueOutPin->AdvanceBlock();
			StopOutPin->AdvanceBlock();
			KillOutPin->AdvanceBlock();
			SeekOutPin->AdvanceBlock();

			// if a transport event comes through, pass it to the appropriate output pin
			const FMusicTransportEventStream::FEventList& TransportEventsThisBlock = TransportInPin->GetTransportEventsInBlock();

			for (const FMusicTransportEventStream::FRequestEvent& Event : TransportEventsThisBlock)
			{
				switch (Event.Request)
				{
				case EMusicPlayerTransportRequest::Prepare:
					PrepareOutPin->TriggerFrame(Event.SampleIndex);
					break;
				case EMusicPlayerTransportRequest::Play:
					PlayOutPin->TriggerFrame(Event.SampleIndex);
					break;
				case EMusicPlayerTransportRequest::Pause:
					PauseOutPin->TriggerFrame(Event.SampleIndex);
					break;
				case EMusicPlayerTransportRequest::Continue:
					ContinueOutPin->TriggerFrame(Event.SampleIndex);
					break;
				case EMusicPlayerTransportRequest::Stop:
					StopOutPin->TriggerFrame(Event.SampleIndex);
					break;
				case EMusicPlayerTransportRequest::Kill:
					KillOutPin->TriggerFrame(Event.SampleIndex);
					break;
				case EMusicPlayerTransportRequest::Seek:
					SeekOutPin->TriggerFrame(Event.SampleIndex);
					break;
				default:
					break;
				}
			}
		}

	private:

		// Inputs
		FMusicTransportEventStreamReadRef TransportInPin;

		// Outputs
		FTriggerWriteRef PrepareOutPin;
		FTriggerWriteRef PlayOutPin;
		FTriggerWriteRef PauseOutPin;
		FTriggerWriteRef ContinueOutPin;
		FTriggerWriteRef StopOutPin;
		FTriggerWriteRef KillOutPin;
		FTriggerWriteRef SeekOutPin;
	};

	using FTransportToTriggerNode = Metasound::TNodeFacade<FTransportToTriggerOperator>;
	METASOUND_REGISTER_NODE(FTransportToTriggerNode);
}

#undef LOCTEXT_NAMESPACE
