// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/Nodes/TransportWavePlayerControllerNode.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::TransportWavePlayerControllerNode
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("TransportWavePlayerController"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Transport, CommonPinNames::Inputs::Transport);
		DEFINE_METASOUND_PARAM_ALIAS(MidiClock, CommonPinNames::Inputs::MidiClock);
	}

	namespace Outputs
	{
		DEFINE_INPUT_METASOUND_PARAM(StartTime, "Start Time", "Time into the wave asset to start (seek) the wave asset.")
		DEFINE_METASOUND_PARAM_ALIAS(TransportPlay, CommonPinNames::Outputs::TransportPlay)
		DEFINE_METASOUND_PARAM_ALIAS(TransportStop, CommonPinNames::Outputs::TransportStop)
	}

	class FTransportWavePlayerControllerOperator : public TExecutableOperator<FTransportWavePlayerControllerOperator>, public FMusicTransportControllable
	{
	public:
		FTransportWavePlayerControllerOperator(const FOperatorSettings& InSettings,
											   const FMusicTransportEventStreamReadRef& InTransport,
											   const FMidiClockReadRef& InMidiClock,
		                                       bool bInHasClockInput)
			: FMusicTransportControllable(EMusicPlayerTransportState::Prepared)
			, TransportInPin(InTransport)
			, MidiClockInPin(InMidiClock)
			, bHasClockInput(bInHasClockInput)
			, PlayOutPin(FTriggerWriteRef::CreateNew(InSettings))
			, StopOutPin(FTriggerWriteRef::CreateNew(InSettings))
			, StartTimeOutPin(FTimeWriteRef::CreateNew())
			, BlockSizeFrames(InSettings.GetNumFramesPerBlock())
			, SampleRate(InSettings.GetSampleRate())
		{
		}

		static const FVertexInterface& GetVertexInterface()
		{
			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::Transport)),
					TInputDataVertex<FMidiClock>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::MidiClock))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportPlay)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::TransportStop)),
					TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::StartTime))
				)
			);

			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = GetClassName();
				Info.MajorVersion = GetCurrentMajorVersion();
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("TransportWavePlayerControllerNode_DisplayName", "Music Transport Wave Player Controller");
				Info.Description = METASOUND_LOCTEXT("TransportWavePlayerControllerNode_Description", "An interface between a music transport and a wave player");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Generators };
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::Transport), TransportInPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), MidiClockInPin);
			bHasClockInput = InVertexData.IsVertexBound(METASOUND_GET_PARAM_NAME(Inputs::MidiClock));
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportPlay), PlayOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::TransportStop), StopOutPin);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::StartTime), StartTimeOutPin);

			bNeedsTransportInit = true;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CommonPinNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			FMusicTransportEventStreamReadRef InTransport = InputData.GetOrCreateDefaultDataReadReference<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME(Inputs::Transport), InParams.OperatorSettings);
			FMidiClockReadRef InMidiClock = InputData.GetOrCreateDefaultDataReadReference<FMidiClock>(METASOUND_GET_PARAM_NAME(Inputs::MidiClock), InParams.OperatorSettings);
			bool bHasClockInput = InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(Inputs::MidiClock));

			return MakeUnique<FTransportWavePlayerControllerOperator>(InParams.OperatorSettings, InTransport, InMidiClock, bHasClockInput);
		}

		void Execute()
		{
			// advance the outputs
			PlayOutPin->AdvanceBlock();
			StopOutPin->AdvanceBlock();

			InitTransportIfNeeded();

			TransportSpanProcessor TransportHandler = [this](int32 StartFrameIndex, int32 EndFrameIndex, EMusicPlayerTransportState CurrentState)
			{
				switch (CurrentState)
				{
				case EMusicPlayerTransportState::Invalid:
				case EMusicPlayerTransportState::Preparing:
					return EMusicPlayerTransportState::Prepared;

				case EMusicPlayerTransportState::Prepared:
					return EMusicPlayerTransportState::Prepared;

				case EMusicPlayerTransportState::Starting:
					if (!ReceivedSeekWhileStopped())
					{
						// Play from the beginning if we haven't received a seek call while we were stopped...
						*StartTimeOutPin = FTime();
					}
					PlayOutPin->TriggerFrame(StartFrameIndex);
					bPlaying = true;
					return EMusicPlayerTransportState::Playing;

				case EMusicPlayerTransportState::Playing:
					return EMusicPlayerTransportState::Playing;

				case EMusicPlayerTransportState::Seeking:
				{
					float NextSeekMs = bHasClockInput ? MidiClockInPin->GetCurrentSongPosMs() :
						TransportInPin->GetNextSeekDestination().Type == ESeekPointType::Millisecond ? TransportInPin->GetNextSeekDestination().Ms :
						0.0f;
					float NextSeekSeconds = NextSeekMs * 0.001f;
					if (ReceivedSeekWhileStopped())
					{
						// Assumes the MidiClock is stopped for the remainder of the block.
						*StartTimeOutPin = FTime(NextSeekSeconds);
					}
					else
					{
						StopOutPin->TriggerFrame(StartFrameIndex);
						int32 PlayFrameIndex = FMath::Min(StartFrameIndex + 1, EndFrameIndex);

						// Assumes the MidiClock is playing for the remainder of the block.
						*StartTimeOutPin = FTime(NextSeekSeconds - (BlockSizeFrames - PlayFrameIndex) / SampleRate);
						PlayOutPin->TriggerFrame(PlayFrameIndex);
					}
					// Here we will return that we want to be in the same state we were in before this request to 
					// seek since we can seek "instantaneously"...
					return GetTransportState();
				}
				case EMusicPlayerTransportState::Continuing:
					// Assumes the StartTimeOutPin won't change for the remainder of the block.
					PlayOutPin->TriggerFrame(StartFrameIndex);
					bPlaying = true;
					return EMusicPlayerTransportState::Playing;

				case EMusicPlayerTransportState::Pausing:
					bPlaying = false;
					StopOutPin->TriggerFrame(StartFrameIndex);

					// Assumes the MidiClock is paused for the remainder of the block.
					*StartTimeOutPin = FTime(MidiClockInPin->GetCurrentSongPosMs() * 0.001f);
					return EMusicPlayerTransportState::Paused;

				case EMusicPlayerTransportState::Paused:
					return EMusicPlayerTransportState::Paused;

				case EMusicPlayerTransportState::Stopping:
				case EMusicPlayerTransportState::Killing:
					if (bPlaying)
					{
						bPlaying = false;
						StopOutPin->TriggerFrame(StartFrameIndex);
					}
					*StartTimeOutPin = FTime();
					return EMusicPlayerTransportState::Prepared;

				default:
					checkNoEntry();
					return EMusicPlayerTransportState::Invalid;
				}
			};
			ExecuteTransportSpans(TransportInPin, BlockSizeFrames, TransportHandler);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			PlayOutPin->Reset();
			StopOutPin->Reset();
			*StartTimeOutPin = FTime();

			BlockSizeFrames = InParams.OperatorSettings.GetNumFramesPerBlock();
			SampleRate = InParams.OperatorSettings.GetSampleRate();

			bPlaying = false;

			bNeedsTransportInit = true;
		}

	private:

		// Inputs
		FMusicTransportEventStreamReadRef TransportInPin;
		FMidiClockReadRef MidiClockInPin;
		bool bHasClockInput;

		// Outputs
		FTriggerWriteRef PlayOutPin;
		FTriggerWriteRef StopOutPin;
		FTimeWriteRef StartTimeOutPin;

		int32 BlockSizeFrames;
		float SampleRate;

		bool bPlaying = false;
		bool bNeedsTransportInit = true;

		void InitTransportIfNeeded()
		{
			if (bNeedsTransportInit)
			{
				// Get the node caught up to its transport input
				FTransportInitFn InitFn = [this](EMusicPlayerTransportState CurrentState)
					{
						switch (CurrentState)
						{
						case EMusicPlayerTransportState::Invalid:
						case EMusicPlayerTransportState::Preparing:
						case EMusicPlayerTransportState::Prepared:
						case EMusicPlayerTransportState::Stopping:
						case EMusicPlayerTransportState::Killing:
							StopOutPin->TriggerFrame(0);
							return EMusicPlayerTransportState::Prepared;

						case EMusicPlayerTransportState::Starting:
						case EMusicPlayerTransportState::Playing:
						case EMusicPlayerTransportState::Continuing:
							// Catch up with our MidiClock
							*StartTimeOutPin = FTime(MidiClockInPin->GetCurrentSongPosMs() * 0.001f);
							PlayOutPin->TriggerFrame(0);
							bPlaying = true;
							return EMusicPlayerTransportState::Playing;

						case EMusicPlayerTransportState::Seeking: // seeking is omitted from init, shouldn't happen
							checkNoEntry();
							return EMusicPlayerTransportState::Invalid;

						case EMusicPlayerTransportState::Pausing:
						case EMusicPlayerTransportState::Paused:
							StopOutPin->TriggerFrame(0);
							return EMusicPlayerTransportState::Paused;

						default:
							checkNoEntry();
							return EMusicPlayerTransportState::Invalid;
						}
					};
				Init(*TransportInPin, MoveTemp(InitFn));

				bNeedsTransportInit = false;
			}
		}
	};

	using FTransportWavePlayerControllerNode = Metasound::TNodeFacade<FTransportWavePlayerControllerOperator>;
	METASOUND_REGISTER_NODE(FTransportWavePlayerControllerNode);
}

#undef LOCTEXT_NAMESPACE
