// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"
#include "HarmonixMetasound/DataTypes/MusicSeekRequest.h"
#include "HarmonixMetasound/Nodes/TriggerToTransportNode.h"

#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

namespace HarmonixMetasound::Nodes::TriggerToTransportNode
{
	using namespace Metasound;

	const FNodeClassName& GetClassName()
	{
		static const FNodeClassName ClassName { HarmonixNodeNamespace, TEXT("TriggerToTransport"), TEXT("") };
		return ClassName;
	}

	int32 GetCurrentMajorVersion()
	{
		return 0;
	}

	class FTriggerToTransportOperator : public TExecutableOperator<FTriggerToTransportOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FTriggerToTransportOperator(const FBuildOperatorParams& InParams,
									const FTriggerReadRef&         InTriggerPrepare,
									const FTriggerReadRef&         InTriggerPlay,
									const FTriggerReadRef&         InTriggerPause, 
									const FTriggerReadRef&         InTriggerContinue, 
									const FTriggerReadRef&         InTriggerStop, 
									const FTriggerReadRef&         InTriggerKill,
									const FTriggerReadRef&         InTriggerSeek,
									const FMusicSeekTargetReadRef& InSeekDestination);

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		void Reset(const FResetParams& ResetParams);
		
		void Execute();

	private:
		//** INPUTS
		FTriggerReadRef PrepareInPin;
		FTriggerReadRef PlayInPin;
		FTriggerReadRef PauseInPin;
		FTriggerReadRef ContinueInPin;
		FTriggerReadRef StopInPin;
		FTriggerReadRef KillInPin;
		FTriggerReadRef TriggerSeekInPin;
		FMusicSeekTargetReadRef SeekDestinationInPin;

		//** OUTPUTS
		FMusicTransportEventStreamWriteRef TransportOutPin;
	};

	using FTriggerToTransportNode = Metasound::TNodeFacade<FTriggerToTransportOperator>;
	METASOUND_REGISTER_NODE(FTriggerToTransportNode)

	const FNodeClassMetadata& FTriggerToTransportOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName        = GetClassName();
			Info.MajorVersion     = GetCurrentMajorVersion();
			Info.MinorVersion     = 2;
			Info.DisplayName      = METASOUND_LOCTEXT("TriggerToTransportNode_DisplayName", "Trigger To Music Transport");
			Info.Description      = METASOUND_LOCTEXT("TriggerToTransportNode_Description", "Combines input triggers into meaningful music transport requests.");
			Info.Author           = PluginAuthor;
			Info.PromptIfMissing  = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy = { MetasoundNodeCategories::Harmonix, NodeCategories::Music };
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	namespace Inputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(TransportPrepare, CommonPinNames::Inputs::TransportPrepare);
		DEFINE_METASOUND_PARAM_ALIAS(TransportPlay, CommonPinNames::Inputs::TransportPlay);
		DEFINE_METASOUND_PARAM_ALIAS(TransportPause, CommonPinNames::Inputs::TransportPause);
		DEFINE_METASOUND_PARAM_ALIAS(TransportContinue, CommonPinNames::Inputs::TransportContinue);
		DEFINE_METASOUND_PARAM_ALIAS(TransportStop, CommonPinNames::Inputs::TransportStop);
		DEFINE_METASOUND_PARAM_ALIAS(TransportKill, CommonPinNames::Inputs::TransportKill);
		DEFINE_METASOUND_PARAM_ALIAS(TriggerSeek, CommonPinNames::Inputs::TriggerSeek);
		DEFINE_METASOUND_PARAM_ALIAS(SeekDestination, CommonPinNames::Inputs::SeekDestination);
	}

	namespace Outputs
	{
		DEFINE_METASOUND_PARAM_ALIAS(Transport, CommonPinNames::Outputs::Transport);
	}

	const FVertexInterface& FTriggerToTransportOperator::GetVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TransportPrepare)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TransportPlay)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TransportPause)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TransportContinue)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TransportStop)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TransportKill)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::TriggerSeek)),
				TInputDataVertex<FMusicSeekTarget>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::SeekDestination))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FMusicTransportEventStream>(METASOUND_GET_PARAM_NAME_AND_METADATA(Outputs::Transport))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FTriggerToTransportOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FTriggerReadRef InTriggerPrepare          = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TransportPrepare), InParams.OperatorSettings);
		FTriggerReadRef InTriggerPlay             = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TransportPlay), InParams.OperatorSettings);
		FTriggerReadRef InTriggerPause            = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TransportPause), InParams.OperatorSettings);
		FTriggerReadRef InTriggerContinue         = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TransportContinue), InParams.OperatorSettings);
		FTriggerReadRef InTriggerStop             = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TransportStop), InParams.OperatorSettings);
		FTriggerReadRef InTriggerKill             = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TransportKill), InParams.OperatorSettings);
		FTriggerReadRef InTriggerSeek             = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(Inputs::TriggerSeek), InParams.OperatorSettings);
		FMusicSeekTargetReadRef InSeekDestination = InputData.GetOrCreateDefaultDataReadReference<FMusicSeekTarget>(METASOUND_GET_PARAM_NAME(Inputs::SeekDestination), InParams.OperatorSettings);
		return MakeUnique<FTriggerToTransportOperator>(InParams,
			InTriggerPrepare,
			InTriggerPlay,
			InTriggerPause,
			InTriggerContinue,
			InTriggerStop,
			InTriggerKill,
			InTriggerSeek,
			InSeekDestination);
	}

	FTriggerToTransportOperator::FTriggerToTransportOperator(const FBuildOperatorParams& InParams,
															 const FTriggerReadRef&   InTriggerPrepare,
															 const FTriggerReadRef&   InTriggerPlay,
															 const FTriggerReadRef&   InTriggerPause,
															 const FTriggerReadRef&   InTriggerContinue,
															 const FTriggerReadRef&   InTriggerStop,
															 const FTriggerReadRef&   InTriggerKill,
															 const FTriggerReadRef&   InTriggerSeek,
															 const FMusicSeekTargetReadRef& InSeekDestination)
		: PrepareInPin(InTriggerPrepare)
		, PlayInPin(InTriggerPlay)
		, PauseInPin(InTriggerPause)
		, ContinueInPin(InTriggerContinue)
		, StopInPin(InTriggerStop)
		, KillInPin(InTriggerKill)
		, TriggerSeekInPin(InTriggerSeek)
		, SeekDestinationInPin(InSeekDestination)
		, TransportOutPin(FMusicTransportEventStreamWriteRef::CreateNew(InParams.OperatorSettings))
	{
		Reset(InParams);
	}

	void FTriggerToTransportOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TransportPrepare), PrepareInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TransportPlay), PlayInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TransportPause), PauseInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TransportContinue), ContinueInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TransportStop),  StopInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TransportKill), KillInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::TriggerSeek), TriggerSeekInPin);
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::SeekDestination), SeekDestinationInPin);
	}

	void FTriggerToTransportOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
	{
		InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Outputs::Transport), TransportOutPin);
	}

	void FTriggerToTransportOperator::Reset(const FResetParams& ResetParams)
	{
		TransportOutPin->Reset();
	}

	void FTriggerToTransportOperator::Execute()
	{
		TransportOutPin->Reset();

		// early out if no transport changes are pending...
		if (!PrepareInPin->IsTriggeredInBlock() &&
			!PlayInPin->IsTriggeredInBlock() &&
			!PauseInPin->IsTriggeredInBlock() &&
			!ContinueInPin->IsTriggeredInBlock() &&
			!StopInPin->IsTriggeredInBlock() &&
			!KillInPin->IsTriggeredInBlock() &&
			!TriggerSeekInPin->IsTriggeredInBlock())
		{
			return;
		}

		auto AddEvents = [this](FTriggerReadRef& Pin, EMusicPlayerTransportRequest Request)
			{
				for (int32 SampleFrame : Pin->GetTriggeredFrames())
				{
					TransportOutPin->AddTransportRequest(Request, SampleFrame);
				}
			};

		// The order here is intentional. It assures that for requests on the exact same sample index...
		// 1 - Stops and Kills will be processed last. This is important to avoid "stuck notes".
		// 2 - Seeks happen before Plays so that we don't "pre-roll" for a play from the beginning 
		//     and then immediately "pre-roll" again to start from the seeked to position.
		for (int32 SampleFrame : TriggerSeekInPin->GetTriggeredFrames())
		{
			TransportOutPin->AddSeekRequest(SampleFrame, *SeekDestinationInPin);
		}
		AddEvents(PrepareInPin,  EMusicPlayerTransportRequest::Prepare);
		AddEvents(PlayInPin,     EMusicPlayerTransportRequest::Play);
		AddEvents(PauseInPin,    EMusicPlayerTransportRequest::Pause);
		AddEvents(ContinueInPin, EMusicPlayerTransportRequest::Continue);
		AddEvents(StopInPin,     EMusicPlayerTransportRequest::Stop);
		AddEvents(KillInPin,     EMusicPlayerTransportRequest::Kill);
		
	}
}

#undef LOCTEXT_NAMESPACE // "HarmonixMetaSound"
