// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"

#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/MusicClock/MusicClock.h"
#include "HarmonixMetasound/MusicSource/MusicSource.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiClockUpdateSubsystem)

namespace MidiClockUpdateSubsystem
{
	int32 kClockHistorySize = 100;
}

FCriticalSection UMidiClockUpdateSubsystem::ClockHistoryMapLocker;
TMap<uint32, TWeakPtr<HarmonixMetasound::Analysis::FMidiClockSongPositionHistory>> UMidiClockUpdateSubsystem::ClockHistories;

double UMidiClockUpdateSubsystem::UpdateTimeSeconds = 0.0;

void UMidiClockUpdateSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	EngineSamplingInputDelegate = FCoreDelegates::OnSamplingInput.AddUObject(this, &UMidiClockUpdateSubsystem::CoreDelegatesSamplingInput);
}

void UMidiClockUpdateSubsystem::Deinitialize()
{
	FCoreDelegates::OnSamplingInput.Remove(EngineSamplingInputDelegate);
}

void UMidiClockUpdateSubsystem::TrackMusicClockComponent(UMusicClockComponent* Clock)
{
	check(GEngine);

	UMidiClockUpdateSubsystem* UpdateSubsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>();

	check(UpdateSubsystem);

	UpdateSubsystem->TrackMusicClockComponentImpl(Clock);
}

void UMidiClockUpdateSubsystem::StopTrackingMusicClockComponent(UMusicClockComponent* Clock)
{
	if (GEngine)
	{
		if (UMidiClockUpdateSubsystem* UpdateSubsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>())
		{
			UpdateSubsystem->StopTrackingMusicClockComponentImpl(Clock);
		}
	}
}

void UMidiClockUpdateSubsystem::TrackMusicSource(UObject* Source)
{
	check(GEngine);
	UMidiClockUpdateSubsystem* UpdateSubsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>();
	check(UpdateSubsystem);
	UpdateSubsystem->TrackMusicSourceImpl(Source);
}

void UMidiClockUpdateSubsystem::StopTrackingMusicSource(UObject* Source)
{
	if (GEngine)
	{
		if (UMidiClockUpdateSubsystem* UpdateSubsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>())
		{
			UpdateSubsystem->StopTrackingMusicSourceImpl(Source);
		}
	}
}

void UMidiClockUpdateSubsystem::TrackMusicClock(UMusicClock* Clock)
{
	check(GEngine);
	UMidiClockUpdateSubsystem* UpdateSubsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>();
	check(UpdateSubsystem);
	UpdateSubsystem->TrackMusicClockImpl(Clock);
}

void UMidiClockUpdateSubsystem::StopTrackingMusicClock(UMusicClock* Clock)
{
	if (GEngine)
	{
		if (UMidiClockUpdateSubsystem* UpdateSubsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>())
		{
			UpdateSubsystem->StopTrackingMusicClockImpl(Clock);
		}
	}
}

double UMidiClockUpdateSubsystem::GetUpdateTimeSeconds()
{
	return UpdateTimeSeconds;
}

uint32 UMidiClockUpdateSubsystem::MakeMidiSongPosAnalyzerAddressHash(const Metasound::Frontend::FAnalyzerAddress& ForAddress)
{
	uint32 AddressHash = GetTypeHashHelper(ForAddress.AnalyzerMemberName);
	AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(ForAddress.AnalyzerName));
	AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(ForAddress.DataType));
	AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(ForAddress.InstanceID));
	AddressHash = HashCombineFast(AddressHash, ForAddress.NodeID.A);
	AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(ForAddress.OutputName));
	return AddressHash;
}

UMidiClockUpdateSubsystem::FClockHistoryPtr UMidiClockUpdateSubsystem::GetOrCreateClockHistory(const Metasound::Frontend::FAnalyzerAddress& ForAddress)
{
	FScopeLock Lock(&ClockHistoryMapLocker);

	uint32 AddressHash = MakeMidiSongPosAnalyzerAddressHash(ForAddress);

	if (ClockHistories.Contains(AddressHash))
	{
		if (FClockHistoryPtr HistoryPtr = ClockHistories[AddressHash].Pin())
		{
			return HistoryPtr;
		}
		FClockHistoryPtr NewHistory = MakeShared<HarmonixMetasound::Analysis::FMidiClockSongPositionHistory>(MidiClockUpdateSubsystem::kClockHistorySize);
		ClockHistories[AddressHash] = NewHistory;
		return NewHistory;
	}

	for (auto It = ClockHistories.CreateIterator(); It; ++It)
	{
		if (!It.Value().IsValid())
		{
			It.RemoveCurrent();
		}
	}

	FClockHistoryPtr NewHistory = MakeShared<HarmonixMetasound::Analysis::FMidiClockSongPositionHistory>(MidiClockUpdateSubsystem::kClockHistorySize);
	ClockHistories.Add(AddressHash, NewHistory);
	return NewHistory;
}

void UMidiClockUpdateSubsystem::TrackMusicClockComponentImpl(UMusicClockComponent* Clock)
{
	TrackedMusicClockComponents.AddUnique(Clock);

	OnTrackMusicClockComponentDelegate.Broadcast(Clock);
}

void UMidiClockUpdateSubsystem::StopTrackingMusicClockComponentImpl(UMusicClockComponent* Clock)
{
	OnStopTrackingMusicClockComponentDelegate.Broadcast(Clock);

	TrackedMusicClockComponents.Remove(Clock);
}

void UMidiClockUpdateSubsystem::TrackMusicSourceImpl(UObject* Source)
{
	check(Source && Source->GetClass()->ImplementsInterface(UMusicSource::StaticClass()));
	TrackedMusicSources.AddUnique(Source);
}

void UMidiClockUpdateSubsystem::StopTrackingMusicSourceImpl(UObject* Source)
{
	TrackedMusicSources.Remove(Source);
}

void UMidiClockUpdateSubsystem::TrackMusicClockImpl(UMusicClock* Clock)
{
	TrackedMusicClocks.AddUnique(Clock);
}

void UMidiClockUpdateSubsystem::StopTrackingMusicClockImpl(UMusicClock* Clock)
{
	TrackedMusicClocks.Remove(Clock);
}

void UMidiClockUpdateSubsystem::UpdateMusicSources()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateMusicSources);

	for (auto It = TrackedMusicSources.CreateIterator(); It; ++It)
	{
		if (UObject* SourceObj = It->Get())
		{
			IMusicSource* Source = Cast<IMusicSource>(SourceObj);
			if (Source)
			{
				Source->Update();
			}
		}
		else
		{
			It.RemoveCurrentSwap();
		}
	}
}

void UMidiClockUpdateSubsystem::UpdateMusicClocks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateMusicClocks);

	for (auto It = TrackedMusicClocks.CreateIterator(); It; ++It)
	{
		if (UMusicClock* Clock = It->Get())
		{
			Clock->UpdateForGameFrame();
		}
		else
		{
			It.RemoveCurrentSwap();
		}
	}
}

void UMidiClockUpdateSubsystem::CoreDelegatesSamplingInput()
{
	// Update order matters:
	// 1. Music sources produce new time data
	// 2. Music clocks read from sources, compute deltas, broadcast events
	// 3. Legacy MusicClockComponents (adapter path)
	UpdateMusicSources();
	UpdateMusicClocks();
	UpdateUMusicClockComponents();
}

void UMidiClockUpdateSubsystem::UpdateUMusicClockComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateUMusicClockComponents);

	UpdateTimeSeconds = FPlatformTime::Seconds();
	using IteratorType = TArray<TWeakObjectPtr<UMusicClockComponent>>::TIterator;
	for (IteratorType ClockIterator = TrackedMusicClockComponents.CreateIterator(); ClockIterator; ++ClockIterator)
	{
		if (UMusicClockComponent* MusicClockPtr = ClockIterator->Get())
		{
			MusicClockPtr->UpdateClock();
		}
		else
		{
			ClockIterator.RemoveCurrentSwap();
		}
	}

	OnMusicClockComponentsUpdatedDelegate.Broadcast(TrackedMusicClockComponents);

	for (auto It = ClockHistories.CreateIterator(); It; ++It)
	{
		if (!It.Value().IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

// Implement a "tick" method that can be used during automated testing so that
// the test code doesn't need knowledge of how the low-res clocks are being ticked...
void UMidiClockUpdateSubsystem::TickForTesting()
{
	UpdateMusicSources();
	UpdateMusicClocks();
	UpdateUMusicClockComponents();
}
