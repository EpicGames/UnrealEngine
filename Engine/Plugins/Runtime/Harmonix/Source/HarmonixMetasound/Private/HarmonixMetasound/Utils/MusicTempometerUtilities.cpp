// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/Utils/MusicTempometerUtilities.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicTempometerUtilities)

void MusicTempometerUtilities::UpdateMaterialParameterCollectionFromClock(const UObject* InWorldContextObject, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance, const TObjectPtr<UMaterialParameterCollection>& InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMCPParameters, const UMusicClockComponent* InClockComponent)
{
	if (InClockComponent)
	{
		const FMidiSongPos MidiSongPos = InClockComponent->GetCurrentVideoRenderSongPos();
		UpdateMaterialParameterCollectionFromSongPos(InWorldContextObject, MidiSongPos, MidiSongPos, InMaterialParameterCollection, InMCPParameters, InOutMaterialParameterCollectionInstance);
	}
}
	
void MusicTempometerUtilities::UpdateMaterialParameterCollectionFromSongPos(const UObject* InWorldContextObject, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance, const TObjectPtr<UMaterialParameterCollection>& InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMCPParameters, const FMidiSongPos& InMidiSongPos)
{
	UpdateMaterialParameterCollectionFromSongPos(InWorldContextObject, InMidiSongPos, InMidiSongPos, InMaterialParameterCollection, InMCPParameters, InOutMaterialParameterCollectionInstance);
}

FMidiSongPos MusicTempometerUtilities::UpdateMaterialParameterCollectionFromClock(const UMusicClockComponent* InClockComponent, const FMidiSongPos& InPreviousFrameMidiSongPos, const UMaterialParameterCollection* InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMPCParameters, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance)
{
	FMidiSongPos MidiSongPos;
	if (InClockComponent)
	{
		MidiSongPos = InClockComponent->GetCurrentVideoRenderSongPos();
		UpdateMaterialParameterCollectionFromSongPos(InClockComponent, MidiSongPos, InPreviousFrameMidiSongPos, InMaterialParameterCollection, InMPCParameters, InOutMaterialParameterCollectionInstance);
	}
	return MidiSongPos;
}

void MusicTempometerUtilities::UpdateMaterialParameterCollectionFromSongPos(const UObject* InWorldContextObject, const FMidiSongPos& InCurrentFrameMidiSongPos, const FMidiSongPos& InPreviousFrameMidiSongPos, const UMaterialParameterCollection* InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMPCParameters, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance)
{
	// Find a MaterialParameterCollectionInstance to update.
	UMaterialParameterCollectionInstance* MaterialParameterCollectionInstance = InOutMaterialParameterCollectionInstance.Get();
	if (!MaterialParameterCollectionInstance)
	{
		if (InMPCParameters.IsValid() && InWorldContextObject)
		{
			if (UWorld* World = InWorldContextObject->GetWorld())
			{
				MaterialParameterCollectionInstance = World->GetParameterCollectionInstance(InMaterialParameterCollection);
				InOutMaterialParameterCollectionInstance = MaterialParameterCollectionInstance;
			}
		}

		if (!MaterialParameterCollectionInstance)
		{
			return;
		}
	}

	// Update the parameters for each frame.
	TTuple<const FMidiSongPos*, const FMusicTempometerMPCParameterNames*> Frames[] = {
		{ &InCurrentFrameMidiSongPos, &InMPCParameters.CurrentFrameParameterNames },
		{ &InPreviousFrameMidiSongPos, &InMPCParameters.PreviousFrameParameterNames } };
	for (auto& Frame : Frames)
	{
		const FMidiSongPos& MidiSongPos = *get<0>(Frame);
		const FMusicTempometerMPCParameterNames& MPCParameterNames = *get<1>(Frame);

		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.SecondsIncludingCountInParameterName, MidiSongPos.SecondsIncludingCountIn);
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.BarsIncludingCountInParameterName, MidiSongPos.BarsIncludingCountIn);
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.BeatsIncludingCountInParameterName, MidiSongPos.BeatsIncludingCountIn);

		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.SecondsFromBarOneParameterName, MidiSongPos.SecondsFromBarOne);
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.TimestampBarParameterName, float(MidiSongPos.Timestamp.Bar));
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.TimestampBeatInBarParameterName, MidiSongPos.Timestamp.Beat);

		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.BarProgressParameterName, FMath::Fractional(MidiSongPos.BarsIncludingCountIn));
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.BeatProgressParameterName, FMath::Fractional(MidiSongPos.BeatsIncludingCountIn));

		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.TimeSignatureNumeratorParameterName, MidiSongPos.TimeSigNumerator);
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.TimeSignatureDenominatorParameterName, MidiSongPos.TimeSigDenominator);
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.TempoParameterName, MidiSongPos.Tempo);
		MaterialParameterCollectionInstance->SetScalarParameterValue(MPCParameterNames.TimestampValidParameterName, MidiSongPos.IsValid() ? 1.0f : 0.0f);
	}
}
