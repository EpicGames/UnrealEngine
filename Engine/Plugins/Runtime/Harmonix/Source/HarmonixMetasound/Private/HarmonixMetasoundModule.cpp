// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasoundModule.h"

#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundGeneratorHandle.h"
#include "MusicEnvironmentSubsystem.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"

#include "HarmonixMetasound/Analysis/MidiClockVertexAnalyzer.h"
#include "HarmonixMetasound/Analysis/MidiSongPosVertexAnalyzer.h"
#include "HarmonixMetasound/Analysis/MidiStreamVertexAnalyzer.h"
#include "HarmonixMetasound/Analysis/MusicTransportEventStreamVertexAnalyzer.h"
#include "HarmonixMetasound/Analysis/FFTAnalyzerResultVertexAnalyzer.h"
#include "HarmonixMetasound/DataTypes/FFTAnalyzerResult.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/Interfaces/HarmonixMusicInterfaces.h"
#include "HarmonixMetasound/MusicEnvironmentSupport/HarmonixMusicEnvironmentMetronome.h"

#include "Engine/Engine.h"
#include "Logging/LogMacros.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "UObject/CoreRedirects.h"

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixMetasoundModule, Log, Log)

void FHarmonixMetasoundModule::StartupModule()
{
	using namespace Metasound;

	METASOUND_REGISTER_ITEMS_IN_MODULE

	// Register passthrough analyzers for output watching
	UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
		GetMetasoundDataTypeName<HarmonixMetasound::FMidiStream>(),
		HarmonixMetasound::Analysis::FMidiStreamVertexAnalyzer::GetAnalyzerName(),
		HarmonixMetasound::Analysis::FMidiStreamVertexAnalyzer::FOutputs::GetValue().Name);
	UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
		GetMetasoundDataTypeName<HarmonixMetasound::FMidiClock>(),
		HarmonixMetasound::Analysis::FMidiClockVertexAnalyzer::GetAnalyzerName(),
		HarmonixMetasound::Analysis::FMidiClockVertexAnalyzer::FOutputs::GetValue().Name);
	UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
		GetMetasoundDataTypeName<HarmonixMetasound::FMusicTransportEventStream>(),
		HarmonixMetasound::Analysis::FMusicTransportEventStreamVertexAnalyzer::GetAnalyzerName(),
		HarmonixMetasound::Analysis::FMusicTransportEventStreamVertexAnalyzer::FOutputs::GetValue().Name);
	UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
		GetMetasoundDataTypeName<FHarmonixFFTAnalyzerResults>(),
		HarmonixMetasound::Analysis::FFFTAnalyzerResultVertexAnalyzer::GetAnalyzerName(),
		HarmonixMetasound::Analysis::FFFTAnalyzerResultVertexAnalyzer::FOutputs::GetValue().Name);

	// Register vertex analyzer factories
	METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(HarmonixMetasound::Analysis::FMidiStreamVertexAnalyzer)
	METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(HarmonixMetasound::Analysis::FMidiClockVertexAnalyzer)
	METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(HarmonixMetasound::Analysis::FMidiSongPosVertexAnalyzer)
	METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(HarmonixMetasound::Analysis::FMusicTransportEventStreamVertexAnalyzer)
	METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(HarmonixMetasound::Analysis::FFFTAnalyzerResultVertexAnalyzer)

	// The first redirect for the module
	TArray<FCoreRedirect> Redirects;
	Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MusicClockComponent.CreateMusicClockComponent"), TEXT("MusicClockComponent.CreateMetasoundDrivenMusicClock"));
	FCoreRedirects::AddRedirectList(Redirects, TEXT("HarmonixMetasoundModule"));

	// When the engine is done loading we want to register our metronome type with the Music Environment Subsystem so
	// it can spawn metronomes for things like the sequence player/editor.
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddLambda([]()
		{
			if (UMusicEnvironmentSubsystem* MusicEnvironment = GEngine->GetEngineSubsystem<UMusicEnvironmentSubsystem>())
			{
				MusicEnvironment->SetMetronomeClass(UHarmonixMusicEnvironmentMetronome::StaticClass());
			}
		});

	HarmonixMetasound::RegisterHarmonixMetasoundMusicInterfaces();
}

void FHarmonixMetasoundModule::ShutdownModule()
{
	METASOUND_UNREGISTER_ITEMS_IN_MODULE
}

METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
IMPLEMENT_MODULE(FHarmonixMetasoundModule, HarmonixMetasound);
