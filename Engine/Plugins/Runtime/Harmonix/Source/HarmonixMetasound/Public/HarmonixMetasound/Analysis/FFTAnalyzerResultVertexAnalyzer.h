// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"

#include "HarmonixMetasound/DataTypes/FFTAnalyzerResult.h"
#include "MetasoundDataReference.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound::Analysis
{
	class FFFTAnalyzerResultVertexAnalyzer final : public Metasound::Frontend::FVertexAnalyzerBase
	{
	public:
		static UE_API const FName& GetAnalyzerName();
		static UE_API const FName& GetDataType();

		struct FOutputs
		{
			static UE_API const Metasound::Frontend::FAnalyzerOutput& GetValue();
		};

		class FFactory final : public Metasound::Frontend::TVertexAnalyzerFactory<FFFTAnalyzerResultVertexAnalyzer>
		{
		public:
			UE_API virtual const TArray<Metasound::Frontend::FAnalyzerOutput>& GetAnalyzerOutputs() const override;
		};

		UE_API explicit FFFTAnalyzerResultVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams);
		virtual ~FFFTAnalyzerResultVertexAnalyzer() override = default;

		UE_API virtual void Execute() override;

	private:
		FHarmonixFFTAnalyzerResultsWriteRef LastAnalyzerResult;
	};
}

#undef UE_API
