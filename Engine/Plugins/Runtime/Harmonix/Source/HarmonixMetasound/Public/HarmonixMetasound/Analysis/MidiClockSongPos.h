// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MetasoundSampleCounter.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMetasound/Analysis/SpmcAnalysisResultQueue.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include <atomic>

namespace HarmonixMetasound::Analysis
{
	struct FMidiClockSongPosition
	{
		enum class EMarkerType : uint8
		{
			None,
			LastPositionBeforeSeekLoop,
			FirstPositionAfterSeekLoop
		};
		Metasound::FSampleCount SampleCount {0};
		int32 UpToTick;
		int32 TempoMapTick;
		float CurrentSpeed {1.0f};
		EMusicPlayerTransportState CurrentTransportState {EMusicPlayerTransportState::Invalid};
		EMarkerType MarkerType {EMarkerType::None};
	};

	struct FSongMapChain : TSharedFromThis<FSongMapChain>
	{
		FSongMapChain() {}
		FSongMapChain(TSharedPtr<const ISongMapEvaluator>& InMaps, int32 InFirstTickInLoop, int32 InLoopLengthTicks)
			: SongMaps(InMaps)
			, FirstTickInLoop(InFirstTickInLoop)
			, LoopLengthTicks(InLoopLengthTicks)
		{}

		TSharedPtr<const ISongMapEvaluator> SongMaps;
		int32 FirstTickInLoop {-1};
		int32 LoopLengthTicks {0};

		TSharedPtr<FSongMapChain> GetLatest()
		{
			FSongMapChain* Latest = this;
			while (Latest->NewSongMapsPtr.load() != nullptr)
			{
				Latest = Latest->NewSongMapsPtr;
			}
			return Latest->AsShared();
		}

		void Update(TSharedPtr<const ISongMapEvaluator>& InMaps, int32 InFirstTickInLoop, int32 InLoopLengthTicks)
		{
			FSongMapChain* Latest = this;
			while (Latest->NewSongMapsPtr.load() != nullptr)
			{
				Latest = Latest->NewSongMapsPtr;
			}
			Latest->NewSongMaps = MakeShared<FSongMapChain>(InMaps, InFirstTickInLoop, InLoopLengthTicks);
			Latest->NewSongMapsPtr = Latest->NewSongMaps.Get();
		}

		bool IsLatest() const
		{
			return NewSongMapsPtr.load() == nullptr;
		}

	private:
		std::atomic<FSongMapChain*> NewSongMapsPtr = nullptr;
		TSharedPtr<FSongMapChain> NewSongMaps;
	};

	// Declare aliases IN the namespace...
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FMidiClockSongPosition, FMidiClockSongPositionTypeInfo, FMidiClockSongPositionReadRef, FMidiClockSongPositionWriteRef)

	struct FMidiClockSongPositionHistory
	{
		using FPositionQueue = FSpmcAnalysisResultQueue<FMidiClockSongPosition>;
		using FScopedItemWriteRef = FPositionQueue::FScopedItemWriteRef;
		using FScopedItemConsumeRef = FPositionQueue::FScopedItemConsumeRef;
		using FScopedItemPeekRef = FPositionQueue::FScopedItemPeekRef;
		using FReadCursor = FPositionQueue::FReadCursor;

		FMidiClockSongPositionHistory(int32 NumItems)
			: Positions(NumItems)
			, SongMapChain(MakeShared<FSongMapChain>())
		{}

		FPositionQueue Positions;
		
		TSharedPtr<FSongMapChain> SongMapChain;

		float SampleRate {48000.0f};

		FReadCursor CreateReadCursor()
		{
			return FReadCursor(Positions);
		}

		TSharedPtr<const FSongMapChain> GetLatestMapsForConsumer()
		{
			SongMapChain = SongMapChain->GetLatest();
			return SongMapChain;
		}

		TSharedPtr<const FSongMapChain> GetLatestMapsForProducer() const
		{
			return SongMapChain->GetLatest();
		}

		void UpdateMaps(TSharedPtr<const ISongMapEvaluator> Maps, int32 InFirstTickInLoop, int32 InLoopLengthTicks)
		{
			SongMapChain->Update(Maps, InFirstTickInLoop, InLoopLengthTicks);
		}
	};
}

// Declare reference types OUT of the namespace...
DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(HarmonixMetasound::Analysis::FMidiClockSongPosition, HARMONIXMETASOUND_API)
