// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"

#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"
#include "HarmonixMetasound/DataTypes/TimeSignature.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound::Analysis
{
	class FMidiClockVertexAnalyzer final : public Metasound::Frontend::FVertexAnalyzerBase
	{
	public:
		struct FOutputs
		{
			/**
			 * @brief Get the default output for this analyzer
			 * @return The default output
			 */
			static UE_API const Metasound::Frontend::FAnalyzerOutput& GetValue();
			
			static UE_API const Metasound::Frontend::FAnalyzerOutput Timestamp;
			static UE_API const Metasound::Frontend::FAnalyzerOutput Tempo;
			static UE_API const Metasound::Frontend::FAnalyzerOutput TimeSignature;
			static UE_API const Metasound::Frontend::FAnalyzerOutput Speed;
		};

		class FFactory final : public Metasound::Frontend::TVertexAnalyzerFactory<FMidiClockVertexAnalyzer>
		{
		public:
			UE_API virtual const TArray<Metasound::Frontend::FAnalyzerOutput>& GetAnalyzerOutputs() const override;
		};

		static UE_API const FName& GetAnalyzerName();
		static UE_API const FName& GetDataType();

		UE_API explicit FMidiClockVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams);

		UE_API virtual void Execute() override;

	private:
		FMusicTimestampWriteRef Timestamp;
		Metasound::FFloatWriteRef Tempo;
		FTimeSignatureWriteRef TimeSignature;
		Metasound::FFloatWriteRef Speed;
	};
}

#undef UE_API
