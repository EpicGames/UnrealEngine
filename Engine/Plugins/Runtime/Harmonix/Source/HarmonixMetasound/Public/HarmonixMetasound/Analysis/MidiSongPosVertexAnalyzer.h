// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "MetasoundSampleCounter.h"

#include "SpmcAnalysisResultQueue.h"
#include "MidiClockSongPos.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"

#include <memory>

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound
{
	struct FMidiClockEvent;
}

namespace HarmonixMetasound::Analysis
{
	class FMidiSongPosVertexAnalyzer final : public Metasound::Frontend::FVertexAnalyzerBase
	{
	public:
		static UE_API const Metasound::Frontend::FAnalyzerOutput SongPosition;

 		class FFactory final : public Metasound::Frontend::TVertexAnalyzerFactory<FMidiSongPosVertexAnalyzer>
		{
		public:
			UE_API virtual const TArray<Metasound::Frontend::FAnalyzerOutput>& GetAnalyzerOutputs() const override;
		};

		static UE_API const FName& GetAnalyzerName();
		static UE_API const FName& GetDataType();

		UE_API explicit FMidiSongPosVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams);
		UE_API virtual ~FMidiSongPosVertexAnalyzer();

		UE_API virtual void Execute() override;

	private:
		TWeakPtr<const FMidiClock, ESPMode::NotThreadSafe> LastClock;
		FMidiClockSongPositionWriteRef LastMidiClockSongPos;
		TSharedPtr<FMidiClockSongPositionHistory> History;
		bool LastPosWasSeekOrLoop {false};
		float SampleRate {0.0f};
		int32 BlockSize {0};
		Metasound::FSampleCount SampleCount {0};
		int32 LastTickProcessed {-1};
		int32 LastTempoMapTickProcessed {-1};
		int32 LastAdvanceUpToTick {-1};
		float CurrentFramesPerTick {0.0f};
		const ISongMapEvaluator* CurrentSongMapEvaluator {nullptr};

		bool TryProcessAsAdvance(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop);
		bool TryProcessAsTempoChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop);
		bool TryProcessAsTimeSignatureChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop);
		bool TryProcessAsSpeedChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop);
		bool TryProcessAsTransportChange(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop);
		bool TryProcessAsLoop(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop);
		bool TryProcessAsSeek(const HarmonixMetasound::FMidiClockEvent& Event, int32& BlockFrameAdvance, bool FirstEventAfterSeekOrLoop);
	};
}

#undef UE_API
