// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundDataReference.h"
#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"

#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound::Analysis
{
	class FMusicTransportEventStreamVertexAnalyzer final : public Metasound::Frontend::FVertexAnalyzerBase
	{
	public:
		static UE_API const FName& GetAnalyzerName();
		static UE_API const FName& GetDataType();

		struct FOutputs
		{
			/**
			 * @brief Get the default output for this analyzer
			 * @return The default output
			 */
			static UE_API const Metasound::Frontend::FAnalyzerOutput& GetValue();

			static UE_API const Metasound::Frontend::FAnalyzerOutput SeekDestination;
			static UE_API const Metasound::Frontend::FAnalyzerOutput TransportEvent;
		};

		class FFactory final : public Metasound::Frontend::TVertexAnalyzerFactory<FMusicTransportEventStreamVertexAnalyzer>
		{
		public:
			UE_API virtual const TArray<Metasound::Frontend::FAnalyzerOutput>& GetAnalyzerOutputs() const override;

		private:
			static const TArray<Metasound::Frontend::FAnalyzerOutput> AnalyzerOutputs;
		};

		UE_API explicit FMusicTransportEventStreamVertexAnalyzer(const Metasound::Frontend::FCreateAnalyzerParams& InParams);
		virtual ~FMusicTransportEventStreamVertexAnalyzer() override = default;

		UE_API virtual void Execute() override;

	private:
		FMusicSeekTargetWriteRef  SeekDestination;
		FMusicTransportEventWriteRef LastMusicTransportEvent;
		int64 NumFrames{ 0 };
		int32 FramesPerBlock{ 0 };
		float SampleRate{ 0 };

		static UE_API const FName AnalyzerName;
	};
}

#undef UE_API
