// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PeakTamer.generated.h"

#define UE_API HARMONIXMETASOUND_API

USTRUCT(BlueprintType)
struct FHarmonixPeakTamerSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AudioAnalysis")
	float PeakAttackTimeSeconds = 0.01f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AudioAnalysis")
	float PeakReleaseTimeSeconds = 2.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AudioAnalysis")
	bool bEnableValueSmoothing = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AudioAnalysis")
	float ValueAttackTimeSeconds = 0.01f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AudioAnalysis")
	float ValueReleaseTimeSeconds = 0.01f;
};

namespace Harmonix::AudioReactivity
{
	namespace PeakTamerPrivate
	{
		// Smooth a value given a delta time, a smoothing time, and the last value.
		// This is pretty much a one-pole lowpass filter.
		HARMONIXMETASOUND_API float SmoothValue(float X0, float Y1, float DeltaTime, float SmoothTime);
	}
	
	/**
	 * Takes a raw peak value from an audio analyzer and outputs a smoothed, compressed value in range [0, 1]
	 */
	class FPeakTamer final
	{
	public:
		UE_API void Configure(const FHarmonixPeakTamerSettings& InSettings);
		
		UE_API void Update(float InputValue, float DeltaTimeSeconds);

		float GetPeak() const { return Peak; }
		
		float GetValue() const { return Value; }

	private:
		FHarmonixPeakTamerSettings Settings{};
		float Peak{ 0.0f };
		float Value{ 0.0f };
	};
}

/**
 * Takes a raw peak value from an audio analyzer and outputs a smoothed, compressed value in range [0, 1]
 */
UCLASS(MinimalAPI, BlueprintType)
class UHarmonixPeakTamer final : public UObject
{
	GENERATED_BODY()
	
public:
	UFUNCTION(BlueprintCallable, Category="AudioAnalysis")
	static UE_API UHarmonixPeakTamer* CreateHarmonixPeakTamer();
	
	UFUNCTION(BlueprintCallable, Category="AudioAnalysis")
	UE_API void Configure(const FHarmonixPeakTamerSettings& Settings);

	UFUNCTION(BlueprintCallable, Category="AudioAnalysis")
	UE_API void Update(float InputValue, float DeltaTimeSeconds);

	UFUNCTION(BlueprintCallable, Category="AudioAnalysis")
	UE_API float GetPeak() const;
	
	UFUNCTION(BlueprintCallable, Category="AudioAnalysis")
	UE_API float GetValue() const;

private:
	Harmonix::AudioReactivity::FPeakTamer PeakTamer;
};

#undef UE_API
