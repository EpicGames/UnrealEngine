// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"
#include "Containers/Array.h"

#include <atomic>

namespace HarmonixMetasound::Analysis
{
	template <typename T>
	class FSpmcAnalysisResultQueue
	{
	private:
		struct FEntry
		{
			int32 SerialNumber{ -1 };
			mutable std::atomic_flag Locked = ATOMIC_FLAG_INIT;
			T Item;
		};

	public:
		using ItemType = T;

		class FReadCursor;
		class FScopedItemConsumeRef;
		class FScopedItemPeekRef;
		class FScopedItemWriteRef;

		class FReadCursor
		{
		public:
			FReadCursor()
				: Queue(nullptr)
				, NextReadIndex(0)
			{

			}

			FReadCursor(FSpmcAnalysisResultQueue& InQueue)
				: Queue(&InQueue)
				, NextReadIndex(InQueue.GetNextWriteIndex())
			{

			}

			FScopedItemConsumeRef ConsumeNext()
			{
				if (DataAvailable())
				{
					return FScopedItemConsumeRef(Queue->GetEntry(NextReadIndex), this);
				}
				return FScopedItemConsumeRef();
			}

			FScopedItemPeekRef PeekNext()
			{
				const FEntry* TheEntry = nullptr;
				if (DataAvailable())
				{
					TheEntry = Queue->GetEntry(NextReadIndex);
				}
				return FScopedItemPeekRef(TheEntry, this);
			}

			FScopedItemPeekRef PeekAhead(int32 NumberAhead)
			{
				const FEntry* TheEntry = nullptr;
				if (NumDataAvailable() >= (1 + NumberAhead))
				{
					int32 Index = NextReadIndex;
					Queue->IncrementIndex(Index, NumberAhead);
					TheEntry = Queue->GetEntry(Index);
				}
				return FScopedItemPeekRef(TheEntry, this);
			}

			void PeekAhead(int32 NumberAhead, FScopedItemPeekRef& Target)
			{
				Target.Release();
				if (NumDataAvailable() < (1 + NumberAhead))
				{
					return;
				}

				int32 Index = NextReadIndex;
				Queue->IncrementIndex(Index, NumberAhead);
				Target.ConnectTo(Queue->GetEntry(Index), this);
			}

			bool DataAvailable() const 
			{
				return Queue && NextReadIndex != Queue->NextWritePos;
			}

			int32 NumDataAvailable() const
			{
				if (!Queue || NextReadIndex == Queue->NextWritePos)
				{
					return 0;
				}

				if (NextReadIndex > Queue->NextWritePos)
				{
					return Queue->NextWritePos + (Queue->Items.Num() - NextReadIndex);
				}

				return Queue->NextWritePos - NextReadIndex;
			}

			const FSpmcAnalysisResultQueue* Queue;
			int32 NextReadIndex{ 0 };
			int32 NextExpectedSerialNumber{ -1 };
		};

		class FScopedItemConsumeRef
		{
		public:
			FScopedItemConsumeRef()
				: Entry(nullptr)
				, ReadCursor(nullptr)
			{
			}

			FScopedItemConsumeRef(const FEntry* InEntry, FReadCursor* InReadCursor)
				: Entry(InEntry)
				, ReadCursor(InReadCursor)
			{
				if (!Entry)
				{
					return;
				}

				while (Entry->Locked.test_and_set())
				{}

				if (Entry->SerialNumber != InReadCursor->NextExpectedSerialNumber && InReadCursor->NextExpectedSerialNumber != -1)
				{
					DiscontinuityDetectedInLastRead = true;
				}
				else
				{
					DiscontinuityDetectedInLastRead = false;
				}
				InReadCursor->NextExpectedSerialNumber = Entry->SerialNumber + 1;
				if (InReadCursor->NextExpectedSerialNumber < 0)
				{
					InReadCursor->NextExpectedSerialNumber = 0;
				}
				InReadCursor->Queue->IncrementIndex(ReadCursor->NextReadIndex);
			}

			FScopedItemConsumeRef(const FScopedItemConsumeRef& Other) = delete;
			FScopedItemConsumeRef(FScopedItemConsumeRef&& Other)
				: DiscontinuityDetectedInLastRead(Other.DiscontinuityDetectedInLastRead)
				, Entry(Other.Entry)
				, ReadCursor(Other.ReadCursor)
			{
				Other.Entry = nullptr;
			}
			FScopedItemConsumeRef& operator=(const FScopedItemConsumeRef& Other) = delete;

			~FScopedItemConsumeRef()
			{
				if (Entry)
				{
					check(Entry->Locked.test_and_set());
					Entry->Locked.clear();
				}
			}

			const T* operator->()
			{
				return Entry ? &Entry->Item : nullptr;
			}

			const T& operator*()
			{
				check(Entry);
				return (Entry->Item);
			}

			operator bool() const
			{
				return Entry != nullptr;
			}

			bool DiscontinuityDetectedInLastRead{ false };
			const FEntry* Entry;
			FReadCursor* ReadCursor;
		};

		class FScopedItemPeekRef
		{
		public:
			FScopedItemPeekRef()
				: Entry(nullptr)
				, ReadCursor(nullptr)
			{
			}
			FScopedItemPeekRef(const FEntry* InEntry, FReadCursor* InReadCursor)
				: Entry(InEntry)
				, ReadCursor(InReadCursor)
			{
				if (!Entry)
				{
					return;
				}

				while (Entry->Locked.test_and_set())
				{
				}

				if (Entry->SerialNumber != InReadCursor->NextExpectedSerialNumber && InReadCursor->NextExpectedSerialNumber != -1)
				{
					DiscontinuityDetectedInLastRead = true;
				}
				else
				{
					DiscontinuityDetectedInLastRead = false;
				}
			}
			FScopedItemPeekRef(const FScopedItemConsumeRef& Other) = delete;
			FScopedItemPeekRef(FScopedItemPeekRef&& Other)
				: DiscontinuityDetectedInLastRead(Other.DiscontinuityDetectedInLastRead)
				, Entry(Other.Entry)
				, ReadCursor(Other.ReadCursor)
			{
				Other.Entry = nullptr;
			}
			FScopedItemPeekRef& operator=(const FScopedItemConsumeRef& Other) = delete;
			FScopedItemPeekRef& operator=(FScopedItemConsumeRef&& Other)
			{
				Entry = MoveTemp(Other.Entry);
				DiscontinuityDetectedInLastRead = Other.DiscontinuityDetectedInLastRead;
				ReadCursor = MoveTemp(Other.ReadCursor);
				return *this;
			}

			void Release()
			{
				if (!Entry)
				{
					return;
				}
				check(Entry->Locked.test_and_set());
				Entry->Locked.clear();
				Entry = nullptr;
			}

			void ConnectTo(const FEntry* InEntry, FReadCursor* InReadCursor)
			{
				Release();
				Entry = InEntry;
				ReadCursor = InReadCursor;

				while (Entry->Locked.test_and_set())
				{
				}

				if (Entry->SerialNumber != InReadCursor->NextExpectedSerialNumber && InReadCursor->NextExpectedSerialNumber != -1)
				{
					DiscontinuityDetectedInLastRead = true;
				}
				else
				{
					DiscontinuityDetectedInLastRead = false;
				}
			}

			~FScopedItemPeekRef()
			{
				if (Entry)
				{
					check(Entry->Locked.test_and_set());
					Entry->Locked.clear();
				}
			}

			const T* operator->()
			{
				return Entry ? &Entry->Item : nullptr;
			}

			const T& operator*()
			{
				check(Entry);
				return (Entry->Item);
			}

			operator bool() const
			{
				return Entry != nullptr;
			}

			bool operator==(const FEntry& Other)
			{
				return &Other == Entry;
			}

			bool operator!=(const FEntry& Other)
			{
				return &Other != Entry;
			}

			bool DiscontinuityDetectedInLastRead{ false };
			const FEntry* Entry;
			FReadCursor* ReadCursor;
		};

		class FScopedItemWriteRef
		{
		public:
			FScopedItemWriteRef() = delete;
			FScopedItemWriteRef(FEntry& InEntry, FSpmcAnalysisResultQueue& InQueue)
				: Entry(InEntry)
				, Queue(InQueue)
			{
				while (Entry.Locked.test_and_set())
				{}
			}

			~FScopedItemWriteRef()
			{
				check(Entry.Locked.test_and_set());
				Entry.SerialNumber = Queue.NextItemSerialNumber++;
				Queue.IncrementIndex(Queue.NextWritePos);
				Entry.Locked.clear();
			}

			T* operator->()
			{
				return &Entry.Item;
			}

			T& operator*()
			{
				return Entry.Item;
			}

		private:
			FEntry& Entry;
			FSpmcAnalysisResultQueue& Queue;
		};

		FSpmcAnalysisResultQueue()
		{}
		FSpmcAnalysisResultQueue(int32 InitialItemCount)
		{
			Items.SetNum(InitialItemCount + 1);
		}
			
		void SetNumItems(int32 NumItems)
		{
			Items.SetNum(NumItems + 1);
		}
		int32 NumItems() const { return Items.Num() - 1; }

		int32 GetNextWriteIndex() const { return NextWritePos; }
		int32 GetLastWriteIndex() const 
		{
			int32 LastWritePos = NextWritePos - 1;
			if (LastWritePos < 0)
			{
				LastWritePos = Items.Num() - 1;
			}
			return LastWritePos;
		;}

		const FEntry* GetEntry(int32 Index) const
		{
			check(Index < Items.Num() && Index >= 0);
			return &Items[Index];
		}

		SIZE_T GetAllocatedSize() const 
		{
			return sizeof(FSpmcAnalysisResultQueue<T>) + Items.GetAllocatedSize();
		}
		
		FScopedItemWriteRef GetNextAtomicWriteSlot()
		{
			return FScopedItemWriteRef(Items[NextWritePos], *this);
		}

		void IncrementIndex(int32& Index, int32 NumberAhead = 1) const
		{
			Index += NumberAhead;
			while (Index >= Items.Num())
			{
				Index -= Items.Num();
			}
		}

		void DecrementIndex(int32& Index) const
		{
			if (Index == 0)
			{
				Index = Items.Num();
			}
			Index--;
		}

		const FEntry& operator[](int32 Index)
		{
			check(Items.IsValidIndex(Index));
			return Items[Index];
		}

	private:
		int32 NextItemSerialNumber{ 0 };
		int32 NextWritePos{ 0 };
		TArray<FEntry> Items;
	};
}