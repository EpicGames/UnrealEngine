// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundParamHelper.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

namespace HarmonixMetasound
{
	HARMONIXMETASOUND_API extern const FName HarmonixNodeNamespace;

	namespace MetasoundNodeCategories
	{
		HARMONIXMETASOUND_API extern const FText Harmonix;
		HARMONIXMETASOUND_API extern const FText Modulation;
		HARMONIXMETASOUND_API extern const FText Analysis;
	}

#if WITH_EDITOR

#define DECLARE_METASOUND_PARAM_EXTERN(NAME)      \
	HARMONIXMETASOUND_API extern const TCHAR* NAME##Name;       \
	HARMONIXMETASOUND_API extern const FText NAME##Tooltip;     \
	HARMONIXMETASOUND_API extern const FText NAME##DisplayName;

#define DECLARE_METASOUND_PARAM_ALIAS(NAME)      \
	HARMONIXMETASOUND_API extern const TCHAR* NAME##Name;       \
	HARMONIXMETASOUND_API extern const FText& NAME##Tooltip;     \
	HARMONIXMETASOUND_API extern const FText& NAME##DisplayName;

#define DEFINE_INPUT_METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT)						\
	const TCHAR* NAME##Name = TEXT(NAME_TEXT);											\
	const FText NAME##Tooltip = LOCTEXT(#NAME "InputTooltip", TOOLTIP_TEXT);			\
	const FText NAME##DisplayName = LOCTEXT(#NAME "InputDisplayName", NAME_TEXT);
#define DEFINE_OUTPUT_METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT)					\
	const TCHAR* NAME##Name = TEXT(NAME_TEXT);											\
	const FText NAME##Tooltip = LOCTEXT(#NAME "OutputTooltip", TOOLTIP_TEXT);			\
    const FText NAME##DisplayName = LOCTEXT(#NAME "OutputDisplayName", NAME_TEXT);

// Alias an existing param definition. Helpful for re-using definitions while providing a public node API.
#define DEFINE_METASOUND_PARAM_ALIAS(ALIAS_NAME, TARGET_NAME) \
	const TCHAR* ALIAS_NAME##Name = TARGET_NAME##Name; \
	const FText& ALIAS_NAME##Tooltip = TARGET_NAME##Tooltip; \
	const FText& ALIAS_NAME##DisplayName = TARGET_NAME##DisplayName; 

#else // WITH_EDITOR

#define DECLARE_METASOUND_PARAM_EXTERN(NAME)      \
	HARMONIXMETASOUND_API extern const TCHAR* NAME##Name;

#define DECLARE_METASOUND_PARAM_ALIAS(NAME)      \
	HARMONIXMETASOUND_API extern const TCHAR* NAME##Name;

#define DEFINE_INPUT_METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT)		\
	const TCHAR* NAME##Name = TEXT(NAME_TEXT);
#define DEFINE_OUTPUT_METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT)	\
	const TCHAR* NAME##Name = TEXT(NAME_TEXT);

// Alias an existing param definition. Helpful for re-using definitions while providing a public node API.
#define DEFINE_METASOUND_PARAM_ALIAS(ALIAS_NAME, TARGET_NAME) \
	const TCHAR* ALIAS_NAME##Name = TARGET_NAME##Name;

#endif // WITH_EDITOR

	namespace CommonPinNames
	{
		namespace Inputs
		{
			DECLARE_METASOUND_PARAM_EXTERN(Enable)
			DECLARE_METASOUND_PARAM_EXTERN(MidiStream)
			DECLARE_METASOUND_PARAM_EXTERN(MidiChannelNumber)
			DECLARE_METASOUND_PARAM_EXTERN(MidiChannelFilterSpecifier)
			DECLARE_METASOUND_PARAM_EXTERN(MidiTrackNumber)
			DECLARE_METASOUND_PARAM_EXTERN(MidiTrackIndexFilterSpecifier)
			DECLARE_METASOUND_PARAM_EXTERN(MinMidiNote)
			DECLARE_METASOUND_PARAM_EXTERN(MaxMidiNote)
			DECLARE_METASOUND_PARAM_EXTERN(MinMidiVelocity)
			DECLARE_METASOUND_PARAM_EXTERN(MaxMidiVelocity)
			DECLARE_METASOUND_PARAM_EXTERN(MidiFileAsset)
			DECLARE_METASOUND_PARAM_EXTERN(Transport)
			DECLARE_METASOUND_PARAM_EXTERN(MidiClock)
			DECLARE_METASOUND_PARAM_EXTERN(Tempo)
			DECLARE_METASOUND_PARAM_EXTERN(TimeSigNumerator)
			DECLARE_METASOUND_PARAM_EXTERN(TimeSigDenominator)
			DECLARE_METASOUND_PARAM_EXTERN(Speed)
			DECLARE_METASOUND_PARAM_EXTERN(Loop)
			DECLARE_METASOUND_PARAM_EXTERN(LoopLengthBars)
			DECLARE_METASOUND_PARAM_EXTERN(TransportPrepare)
			DECLARE_METASOUND_PARAM_EXTERN(TransportPlay)
			DECLARE_METASOUND_PARAM_EXTERN(TransportPause)
			DECLARE_METASOUND_PARAM_EXTERN(TransportContinue)
			DECLARE_METASOUND_PARAM_EXTERN(TransportStop)
			DECLARE_METASOUND_PARAM_EXTERN(TransportKill)
			DECLARE_METASOUND_PARAM_EXTERN(TriggerSeek)
			DECLARE_METASOUND_PARAM_EXTERN(SeekDestination)
			DECLARE_METASOUND_PARAM_EXTERN(PrerollBars)
			DECLARE_METASOUND_PARAM_EXTERN(GridSizeUnits)
			DECLARE_METASOUND_PARAM_EXTERN(GridSizeMult)
			DECLARE_METASOUND_PARAM_EXTERN(OffsetUnits)
			DECLARE_METASOUND_PARAM_EXTERN(OffsetMult)
			DECLARE_METASOUND_PARAM_EXTERN(Bar)
			DECLARE_METASOUND_PARAM_EXTERN(FloatBeat)
			DECLARE_METASOUND_PARAM_EXTERN(SynthPatch)
			DECLARE_METASOUND_PARAM_EXTERN(Transposition)
			DECLARE_METASOUND_PARAM_EXTERN(ClockSpeedToPitch)
			DECLARE_METASOUND_PARAM_EXTERN(ClockSpeedToFrequency)
			DECLARE_METASOUND_PARAM_EXTERN(LFOSyncType);
			DECLARE_METASOUND_PARAM_EXTERN(LFOFrequency);
			DECLARE_METASOUND_PARAM_EXTERN(LFOInvert);
			DECLARE_METASOUND_PARAM_EXTERN(LFOShape);
			DECLARE_METASOUND_PARAM_EXTERN(AudioMono);
			DECLARE_METASOUND_PARAM_EXTERN(AudioLeft);
			DECLARE_METASOUND_PARAM_EXTERN(AudioRight);
			DECLARE_METASOUND_PARAM_EXTERN(AudioCenter)
			DECLARE_METASOUND_PARAM_EXTERN(AudioLFE)
			DECLARE_METASOUND_PARAM_EXTERN(AudioLeftBack)
			DECLARE_METASOUND_PARAM_EXTERN(AudioRightBack)
			DECLARE_METASOUND_PARAM_EXTERN(AudioLeftSide)
			DECLARE_METASOUND_PARAM_EXTERN(AudioRightSide)
			DECLARE_METASOUND_PARAM_EXTERN(Timestamp);
			DECLARE_METASOUND_PARAM_EXTERN(ShouldQuantizeTimestamp);
			DECLARE_METASOUND_PARAM_EXTERN(QuantizationUnit);
		}
		namespace Outputs
		{
			DECLARE_METASOUND_PARAM_EXTERN(Transport)
			DECLARE_METASOUND_PARAM_EXTERN(MidiClock)
			DECLARE_METASOUND_PARAM_EXTERN(MidiFileAsset)
			DECLARE_METASOUND_PARAM_EXTERN(MidiStream)
			DECLARE_METASOUND_PARAM_EXTERN(NoteOn)
			DECLARE_METASOUND_PARAM_EXTERN(NoteOff)
			DECLARE_METASOUND_PARAM_EXTERN(MidiNoteNumber)
			DECLARE_METASOUND_PARAM_EXTERN(Frequency)
			DECLARE_METASOUND_PARAM_EXTERN(MidiVelocity)
			DECLARE_METASOUND_PARAM_EXTERN(NormalizedVelocity)
			DECLARE_METASOUND_PARAM_EXTERN(TransportPrepare)
			DECLARE_METASOUND_PARAM_EXTERN(TransportPlay)
			DECLARE_METASOUND_PARAM_EXTERN(TransportPause)
			DECLARE_METASOUND_PARAM_EXTERN(TransportContinue)
			DECLARE_METASOUND_PARAM_EXTERN(TransportStop)
			DECLARE_METASOUND_PARAM_EXTERN(TransportKill)
			DECLARE_METASOUND_PARAM_EXTERN(TransportSeek)
			DECLARE_METASOUND_PARAM_EXTERN(MusicTimestamp)
			DECLARE_METASOUND_PARAM_EXTERN(SeekTarget)
			DECLARE_METASOUND_PARAM_EXTERN(AudioMono)
			DECLARE_METASOUND_PARAM_EXTERN(AudioLeft)
			DECLARE_METASOUND_PARAM_EXTERN(AudioRight)
			DECLARE_METASOUND_PARAM_EXTERN(AudioCenter)
			DECLARE_METASOUND_PARAM_EXTERN(AudioLFE)
			DECLARE_METASOUND_PARAM_EXTERN(AudioLeftBack)
			DECLARE_METASOUND_PARAM_EXTERN(AudioRightBack)
			DECLARE_METASOUND_PARAM_EXTERN(AudioLeftSide)
			DECLARE_METASOUND_PARAM_EXTERN(AudioRightSide)
			DECLARE_METASOUND_PARAM_EXTERN(MusicTimespanBar)
			DECLARE_METASOUND_PARAM_EXTERN(MusicTimespanBeat)
			DECLARE_METASOUND_PARAM_EXTERN(Tempo)
			DECLARE_METASOUND_PARAM_EXTERN(Speed)
			DECLARE_METASOUND_PARAM_EXTERN(TimeSigNumerator)
			DECLARE_METASOUND_PARAM_EXTERN(TimeSigDenominator)
			DECLARE_METASOUND_PARAM_EXTERN(SecsPerQuarter)
			DECLARE_METASOUND_PARAM_EXTERN(SecsPerBeat)
		}
	}
}
