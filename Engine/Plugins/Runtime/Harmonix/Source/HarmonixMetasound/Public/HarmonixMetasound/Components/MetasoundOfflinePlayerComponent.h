// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/ActorComponent.h"
#include "MetasoundSource.h"

#include "MetasoundOfflinePlayerComponent.generated.h"

#define UE_API HARMONIXMETASOUND_API

namespace Metasound
{
	class FMetasoundGeneratorHandle;
}
class UAudioComponent;
class UMetasoundGeneratorHandle;

UCLASS(MinimalAPI)
class UMetasoundOfflinePlayerComponent : public UActorComponent
{
	GENERATED_BODY()

public:	
	UE_API UMetasoundOfflinePlayerComponent();

	// To deprecate, only required to keep engine-only integrated backends happy
	UE_API UMetasoundGeneratorHandle* CreateGeneratorBasedOnAudioComponent(UAudioComponent* AudioComponent, int32 InSampleRate = 12000, int32 InBlockSize = 256);

	UE_API TSharedPtr<Metasound::FMetasoundGeneratorHandle> CreateSharedGeneratorBasedOnAudioComponent(UAudioComponent* AudioComponent, int32 InSampleRate = 12000, int32 InBlockSize = 256);
	UE_API void ReleaseGenerator();

	UE_API virtual void BeginPlay() override;
	UE_API virtual void BeginDestroy() override;

	UE_API TSharedPtr<Metasound::FMetasoundGenerator> GetGenerator();

public:	
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

private:
	ISoundGeneratorPtr Generator;

	UPROPERTY(Transient)
	TObjectPtr<UMetaSoundSource> MetasoundSource;

	int32 SampleRate;
	int32 BlockSize;
	int32 GeneratorBlockSize;

	double StartSecond = 0.0;
	int64  RenderedSamples = 0;
	TArray<float> ScratchBuffer;
};

#undef UE_API
