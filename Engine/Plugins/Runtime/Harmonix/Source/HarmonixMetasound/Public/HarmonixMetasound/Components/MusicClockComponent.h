// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/ActorComponent.h"
#include "Evaluation/IMovieSceneCustomClockSource.h"
#include "Harmonix/MusicalTimebase.h"
#include "HarmonixMetasound/Components/MusicTimerManager.h"
#include "HarmonixMetasound/MusicClock/MusicClockState.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/SongMaps.h"
#include "MusicEnvironmentClockSource.h"
#include "MusicClockTypes.h"
#include "StructUtils/InstancedStruct.h"

#include "MusicClockComponent.generated.h"

#define UE_API HARMONIXMETASOUND_API

DECLARE_LOG_CATEGORY_EXTERN(LogMusicClock, Log, All)

namespace Metasound { class FMetasoundGenerator; }
struct FMusicalTimeSpan;
class UAudioComponent;
class FMusicClockDriverBase;
class FMusicTimer;

UENUM(BlueprintType)
enum class EMusicClockDriveMethod : uint8
{
	WallClock,
	MetaSound,
	Custom
};

UENUM(BlueprintType)
enum class EMusicTimeDiscontinuityType : uint8
{
	Loop,
	Seek,
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FBeatEvent, int, BeatNumber, int, BeatInBar);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FBarEvent, int, BarNumber);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FSectionEvent, const FString&, SectionName, float, SectionStartMs, float, SectionLengthMs);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPlayStateEvent, EMusicClockState, State);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMusicTimeDiscontinuityEvent, EMusicTimeDiscontinuityType, Type, FMidiSongPos, PreviousPos, FMidiSongPos, NewPos);

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMusicClockConnected);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMusicClockDisconnected);

UCLASS(MinimalAPI, ClassGroup = (MetaSoundMusic), PrioritizeCategories = "MusicClock", meta = (BlueprintSpawnableComponent, DisplayName = "Music Clock", ScriptName = MusicClockComponent))
class UMusicClockComponent : public UActorComponent, public IMusicEnvironmentClockSource
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	UE_API UMusicClockComponent();

	/**
	 * Start this clock ticking and start tracking music time
	 * Same as "Start()"
	 *
	 * Setting bReset to true will restart (trigger "Stop" and then "Start") the clock is already Running
	 */
	UE_API void Activate(bool bReset) override;

	/**
	 * Stop this clock ticking and stop tracking music time
	 * Same as "Stop()"
	 */
	UE_API void Deactivate() override;
	
	UE_API void BeginPlay() override;
	UE_API void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API void BeginDestroy() override;
	
	UPROPERTY(EditAnywhere, Category = "MusicClock")
	EMusicClockDriveMethod DriveMethod = EMusicClockDriveMethod::MetaSound;

	UPROPERTY(EditAnywhere, Category = "MusicClock", meta = (EditCondition = "DriveMethod == EMusicClockDriveMethod::MetaSound", EditConditionHides))
	FName MetasoundOutputName = "MIDI Clock";

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "MusicClock", meta = (EditCondition = "DriveMethod == EMusicClockDriveMethod::MetaSound", EditConditionHides))
	TObjectPtr<UAudioComponent> MetasoundsAudioComponent;

	UPROPERTY(EditAnywhere, Category = "MusicClock", meta = (EditCondition = "DriveMethod == EMusicClockDriveMethod::WallClock", EditConditionHides))
	TObjectPtr<UMidiFile> TempoMap;

	UPROPERTY(EditAnywhere, Category = "MusicClock", NoClear, meta = (EditCondition = "DriveMethod == EMusicClockDriveMethod::Custom"))
	TInstancedStruct<FMusicClockSettingsBase> MusicClockSettings = TInstancedStruct<FMetasoundMusicClockSettings>::Make();

	/**
	 * Start the clock ticking and tracking music time
	 * Same as "Activate"
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void Start();

	/**
	 * Pause the clock at the current running time
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void Pause();

	/**
	 * Continue the clock from paused
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void Continue();

	/**
	 * Stop this clock ticking and no longer track music time
	 * Same as "Deactivate"
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void Stop();

	/**
	 * Get The Current Tempo of the running clock
	 * 0 if the clock is stopped
	 * This is Midi Tempo, which is QuarterNotesPerMinute. 
	 * This does not consider the advance rate,
	 * if you want that, use GetBeats|BarsPerSecond.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentTempo() const;

	/**
	 * Get The Current BPM of the running clock
	 * 0 if the clock is stopped
	 * This is in units of our Beats, not Quarter notes.
	 * This does not consider the advance rate,
	 * if you want that, use GetBeats|BarsPerSecond.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentBeatsPerMinute() const;

	/**
	 * Get The Current Time Signature
	 * 
	 * 0 if the clock is stopped
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API void GetCurrentTimeSignature(int& OutNumerator, int& OutDenominator) const;

	/**
	 * Get The Current Time Signature
	 *
	 * 0 if the clock is stopped
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API FTimeSignature GetCurrentTimeSignatureStruct() const;

	/**
	 * Get The Current Bars per Second.
	 * Based on the current tempo, advance rate and time signature
	 * 
	 * 0 if the clock is stopped.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentBarsPerSecond() const;

	/**
	 * Get The Current Seconds per Bar
	 * Based on the current tempo, advance rate and time signature
	 * 
	 * 0 if the clock is stopped
	 * Don't use this method, use GetCurrentBarsPerSecond instead if possible because this method
	 * returns zero when the clock is stopped rather than inf, which is what it should be.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentSecondsPerBar() const;
	
	/**
	 * Get The Current Beats per Second.
	 * Based on the current tempo, advance rate and time signature
	 * 
	 * 0 if the clock is stopped.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentBeatsPerSecond() const;

	/**
	 * Get The Current Seconds per Beat
	 * Based on the current tempo, advance rate and time signature
	 * 
	 * 0 if the clock is stopped.
	 * Don't use this method, use GetCurrentBeatsPerSecond instead if possible because this method
	 * returns zero when the clock is stopped rather than inf, which is what it should be.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentSecondsPerBeat() const;
	
	/**
     * Get The Current Clock advance rate
     * 
     * 0 if the clock is stopped
     */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentClockAdvanceRate() const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API EMusicClockState GetState() const;
	
	// Set the tempo map when running off of wall clock
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void SetTempoMapForWallClock(UMidiFile* InTempoMap);

	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void SetRunPastMusicEnd(bool bRunPastMusicEnd);

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API bool GetRunPastMusicEnd() const;

	// Getters for all of the fields in mCurrentSongPos...
	
	// Time from the beginning of the authored music content.
	// NOTE: INCLUDES time for count-in and pickup bars.
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetSecondsIncludingCountIn(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	// Time from Bar 1 Beat 1. The classic "start of the song".
	// NOTE: DOES NOT INCLUDE time for count-in and pickup bars.
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetSecondsFromBarOne(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	// Returns the fractional total bars from the beginning of the authored music content.
	// NOTE: INCLUDES time for count-in and pickup bars.
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetBarsIncludingCountIn(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	// Returns the fractional total beats from the beginning of the authored music content.
	// NOTE: INCLUDES time for count-in and pickup bars.
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetBeatsIncludingCountIn(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	// NOTE: Working in ticks is a little risky. Midi files have have different numbers of ticks per quarter note,
	// Ticks change duration with tempo changes, etc. So we don't expose ticks to blueprints and recommend using them
	// in c++ code. That said, sometimes there is a use for them... so... 
	UE_API float GetTicksFromBarOne(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;
	UE_API float GetTicksIncludingCountIn(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	// Returns the "classic" musical timestamp in the form Bar (int) & Beat (float). In this form...
	//    - Bar 1, Beat 1.0 is the "beginning of the song" AFTER count-in/pickups
	//    - Bar 0, Beat 1.0 would be one bar BEFORE the "beginning of the song"... eg. a bar of count-in or pickup.
	//    - While Bar can be positive or negative, Beat is always >= 1.0 and is read as "beat in the bar". Again... '1' based!
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API FMusicTimestamp GetCurrentTimestamp(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Returns the name of the section that we're currently in (intro, chorus, outro) */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API FString GetCurrentSectionName(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Returns the index of the current section for the provided time base. [0, Num-1] */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API int32 GetCurrentSectionIndex(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API const TArray<FSongSection>& GetSongSections() const;

	/** Returns the start time of the current section in milliseconds */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentSectionStartMs(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Returns the length of the current section in milliseconds */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentSectionLengthMs(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Gets a value expressed in beats between 0-1 that indicates how much progress we made in the current beat */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceFromCurrentBeat(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Gets a value expressed in beats between 0-1 that indicates how close we are to the next beat. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToNextBeat(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Gets a value expressed in beats between 0-1 that indicates how close we are to the closest beat (current beat or next beat). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToClosestBeat(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Gets a value expressed in bars between 0-1 that indicates how much progress we made towards the current bar to the next one. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceFromCurrentBar(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Gets a value expressed in bars between 0-1 that indicates how close we are to the next bar. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToNextBar(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Gets value expressed in bars between 0-1 that indicates how close we are to the closest bar (current bar or next bar). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToClosestBar(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDeltaBar(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDeltaBeat(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API const FMidiSongPos& GetSongPos(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API const FMidiSongPos& GetPreviousSongPos(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Returns the remaining time until the end of the MIDI in milliseconds based on the timestamp corresponding to the passed Timebase */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetSongRemainingMs(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Returns true if there was a seek in the specified timebase */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API bool SeekedThisFrame(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	/** Returns true if there was a seek in the specified timebase */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API bool LoopedThisFrame(ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API TOptional<FVector> TryGetAudioSourceLocation() const;

	UFUNCTION(BlueprintPure, Category = "Count In")
	UE_API float GetCountInSeconds() const;

	UFUNCTION(BlueprintPure, Category = "Tick")
	UE_API float TickToMs(float Tick) const;

	/* Convert zero indexed fractional beat to Ms */ 
	UFUNCTION(BlueprintPure, Category = "Beat")
	UE_API float BeatToMs(float Beat) const;

	/* Convert Ms to zero indexed fractional beat */
	UFUNCTION(BlueprintPure, Category = "Beat")
	UE_API float MsToBeat(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Beat")
	UE_API float GetMsPerBeatAtMs(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Beat")
	UE_API float GetNumBeatsInBarAtMs(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Beat")
	UE_API float GetBeatInBarAtMs(float Ms) const;
	
	UFUNCTION(BlueprintPure, Category = "Bar")
	UE_API float BarToMs(float Bar) const;

	UFUNCTION(BlueprintPure, Category = "Bar")
	UE_API float GetMsPerBarAtMs(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API FString GetSectionNameAtMs(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API float GetSectionLengthMsAtMs(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API float GetSectionStartMsAtMs(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API float GetSectionEndMsAtMs(float Ms) const;

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API int32 GetNumSections() const;

	UFUNCTION(BlueprintPure, Category = "Song Data")
	UE_API float GetSongLengthMs() const;

	UFUNCTION(BlueprintPure, Category = "Song Data")
	UE_API float GetSongLengthBeats() const;

	UFUNCTION(BlueprintPure, Category = "Song Data")
	UE_API float GetSongLengthBars() const;

	UE_API const ISongMapEvaluator& GetSongMaps() const;

	/**
	 * Adds a musical timer.
	 *
	 * @param TimerInterval  Interval configuration for the timer.
	 * @param StartTime      Optional start time. If unset, the current musical
	 *                       time (floored) is used.
	 * @param Timebase       Timebase used to evaluate the timer.
	 * @param bLooping       Whether the timer should loop.
	 * @param TimerDelegate  Delegate executed when the timer fires.
	 */
	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	FMusicTimerHandle BP_AddTimer(const FMusicTimeInterval& TimerInterval, TOptional<FMusicTimestamp> StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FOnMusicalTimerExecute TimerDelegate);

	/**
	 * Adds a musical timer.
	 *
	 * @param TimerInterval  Interval configuration for the timer.
	 * @param StartTime      Optional start time. If unset, the current musical
	 *                       time (floored) is used.
	 * @param Timebase       Timebase used to evaluate the timer.
	 * @param bLooping       Whether the timer should loop.
	 * @param TimerDelegate  Delegate executed when the timer fires.
	 */
	UE_API FMusicTimerHandle AddTimerNative(const FMusicTimeInterval& TimerInterval, TOptional<FMusicTimestamp> StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FSimpleDelegate TimerDelegate);

	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	UE_API void RemoveTimer(const FMusicTimerHandle& Handle);

	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	UE_API void PauseTimer(const FMusicTimerHandle& Handle, bool bPause);

	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	FMusicTimestamp Quantize(const FMusicTimestamp& Time, EMidiClockSubdivisionQuantization QuantizationInterval, EMidiFileQuantizeDirection InDirection) const;

#if !UE_BUILD_SHIPPING
	// A short string description of the clock, for debugging, like song name or animation name or whatever is driving it.
	UE_API FString GetDisplayName() const;

	// Sets the DisplayName. Will be used by GetDisplayName if set, otherwise falls back to drivers. 
	UE_API void SetDisplayName(FString Name);
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicClock")
	ECalibratedMusicTimebase TimebaseForBarAndBeatEvents = ECalibratedMusicTimebase::VideoRenderTime;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FPlayStateEvent PlayStateEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FBeatEvent BeatEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FBarEvent BarEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FSectionEvent SectionEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicClockConnected MusicClockConnectedEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicClockDisconnected MusicClockDisconnectedEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicTimeDiscontinuityEvent AudioRenderMusicTimeDiscontinuityEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicTimeDiscontinuityEvent PlayerExperienceMusicTimeDiscontinuityEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicTimeDiscontinuityEvent VideoRenderMusicTimeDiscontinuityEvent;

	// Set the default tempo, only call when there's no tempo map.
	UE_API void SetDefaultTempo(float TempoBpm);
	// Set the default time signature numerator, only call when there's no tempo map.
	UE_API void SetDefaultTimeSignatureNum(int Num);
	// Set the default time signature denominator, only call when there's no tempo map.
	UE_API void SetDefaultTimeSignatureDenom(int Denom);

protected:

	/**
	 * Default Tempo (BPM)
	 * used for Wall clock with no tempo map
	 *
	 * NOT THE CURRENT RUNNING TEMPO
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicClock|Defaults")
	float DefaultTempo = 120.0f;

	/**
	 * Default Time Signature (Numerator)
	 * used for the Wall Clock with no tempo map
	 *
	 * NOT THE CURRENT RUNNING TIME SIG
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicClock|Defaults")
	int DefaultTimeSignatureNum = 4;

	/**
	 * Default Time Signature (Denominator) 
	 * used for the Wall clock with no tempo map
	 *
	 * NOT THE CURRENT RUNNING TIME SIG
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicClock|Defaults")
	int DefaultTimeSignatureDenom = 4;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicClock")
	bool RunPastMusicEnd = false;

public:
	// Getter functions for the Blueprint properties exposed above...
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MusicClock")
	UE_API FMidiSongPos GetCurrentSmoothedAudioRenderSongPos() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MusicClock")
	UE_API FMidiSongPos GetPreviousSmoothedAudioRenderSongPos() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MusicClock")
	UE_API FMidiSongPos GetCurrentVideoRenderSongPos() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MusicClock")
	UE_API FMidiSongPos GetPreviousVideoRenderSongPos() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MusicClock")
	UE_API FMidiSongPos GetCurrentPlayerExperiencedSongPos() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MusicClock")
	UE_API FMidiSongPos GetPreviousPlayerExperiencedSongPos() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MusicClock")
	UE_API FMidiSongPos GetCurrentRawAudioRenderSongPos() const;

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float MeasureSpanProgress(const FMusicalTimeSpan& Span, ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UFUNCTION(BlueprintCallable, Category = "Audio|MusicClock", meta = (WorldContext = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API UMusicClockComponent* CreateMetasoundDrivenMusicClock(UObject* WorldContextObject, UAudioComponent* InAudioComponent, FName MetasoundOuputPinName = "MIDI Clock", bool Start = true, bool bPersistAcrossLevelTransition = false);

	UFUNCTION(BlueprintCallable, Category = "Audio|MusicClock", meta = (WorldContext = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API UMusicClockComponent* CreateWallClockDrivenMusicClock(UObject* WorldContextObject, UMidiFile* WithTempoMap, bool Start = true, bool bPersistAcrossLevelTransition = false);

	static UE_API UMusicClockComponent* CreateMusicClockWithSettings(UObject* WorldContextObject, TInstancedStruct<FMusicClockSettingsBase> MusicClockSettings, bool Start = true, bool bPersistAcrossLevelTransition = false);

	template<typename MusicClockSettingsType = FMusicClockSettingsBase>
	static UMusicClockComponent* CreateMusicClockWithSettings(UObject* WorldContextObject, const MusicClockSettingsType& InMusicClockSettings, bool Start = true, bool bPersistAcrossLevelTransition = false)
	{
		return CreateMusicClockWithSettings(WorldContextObject, TInstancedStruct<FMusicClockSettingsBase>::Make(InMusicClockSettings), Start, bPersistAcrossLevelTransition);
	}

	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API bool ConnectToMetasoundOnAudioComponent(UAudioComponent* InAudioComponent);

	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void ConnectToWallClockForMidi(UMidiFile* InTempoMap);

	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void ConnectToCustomClockWithSettings(TInstancedStruct<FMusicClockSettingsBase> InMusicClockSettings);
	
	template<typename MusicClockSettingsType = FMusicClockSettingsBase>
	void ConnectToCustomClockWithSettings(const MusicClockSettingsType& InMusicClockSettings)
	{
		ConnectToCustomClockWithSettings(TInstancedStruct<FMusicClockSettingsBase>::Make(InMusicClockSettings));
	}

	UE_API FMidiSongPos CalculateSongPosWithOffset(float MsOffset, ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime) const;

	UE_API const FMidiSongPos& GetRawUnsmoothedAudioRenderPos() const;

	// vvv BEGIN  IMusicEnvironmentClockSource Implementation vvv
	UE_API virtual float GetPositionSeconds() const override;
	UE_API virtual FMusicalTime GetPositionMusicalTime() const override;
	UE_API virtual FMusicalTime GetPositionMusicalTime(const FMusicalTime& SourceSpaceOOffset) const override;
	UE_API virtual int32 GetPositionAbsoluteTick() const override;
	UE_API virtual int32 GetPositionAbsoluteTick(const FMusicalTime& SourceSpaceOffset) const override;
	UE_API virtual FMusicalTime Quantize(const FMusicalTime& MusicalTime, int32 QuantizationInterval, UFrameBasedMusicMap::EQuantizeDirection Direction = UFrameBasedMusicMap::EQuantizeDirection::Nearest) const;
	virtual bool CanAuditionInEditor() const override { return false; }
	// ^^^ END IMusicEnvironmentClockSource Implementation ^^^
	
	bool HasValidClockDriver() const { return ClockDriver.IsValid(); }

private:
	friend class  UMidiClockUpdateSubsystem;

	TSharedPtr<FMusicClockDriverBase> ClockDriver;

	FSongMaps DefaultMaps;
	
	int32 LastBroadcastBar = -1;
	int32 LastBroadcastBeat = -1;
	FSongSection LastBroadcastSongSection;
#if !UE_BUILD_SHIPPING
	// Holds a name for display, ideally shows the original source for the clock.
	FString DisplayName;
#endif
	
	UPROPERTY(Transient)
	TObjectPtr<UMusicTimerManager> MusicTimerManager;

	static TNotNull<UMusicClockComponent*> CreateMusicClockShared(UObject* WorldContextObject, bool bPersistAcrossLevelTransition);
	
	UE_API void CreateClockDriver();
	UE_API void BroadcastSongPosChanges(ECalibratedMusicTimebase Timebase);
	UE_API void BroadcastSeekLoopDetections(ECalibratedMusicTimebase Timebase) const;
	UE_API const FMusicTimeDiscontinuityEvent* GetMusicTimeDiscontinuityEventInternal(ECalibratedMusicTimebase Timebase) const;
	UE_API void MakeDefaultSongMap();
	UE_API bool ConnectToMetasound();
	UE_API void ConnectToWallClock();
	UE_API void ConnectToCustomClock();
	UE_API void DisconnectFromClockDriver();
	
	UE_API void UpdateClock();
	UE_API const FMidiSongPos& GetCurrentSongPosInternal(ECalibratedMusicTimebase Timebase) const;
	UE_API const FMidiSongPos& GetPreviousSongPosInternal(ECalibratedMusicTimebase Timebase) const;
};

#undef UE_API
