// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/Components/MusicClockTypes.h"
#include "Harmonix/MusicalTimebase.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/SongMaps.h"

struct FMusicClockSettingsBase;
struct ISongMapEvaluator;

#define UE_API HARMONIXMETASOUND_API

class FMusicClockDriverBase : public TSharedFromThis<FMusicClockDriverBase>
{
public:
	UE_API FMusicClockDriverBase(UObject* WorldContextObject, const FMusicClockSettingsBase& Settings);
	UE_API virtual ~FMusicClockDriverBase();

	virtual TOptional<FVector> TryGetAudioSourceLocation() const = 0;

	UE_API void UpdateClockForGameFrame();

	UE_API virtual bool CalculateSongPosWithOffset(float MsOffset, ECalibratedMusicTimebase Timebase, FMidiSongPos& OutResult) const;

	UE_API double GetWallClockTime() const;

	EMusicClockState GetState() const { return MusicClockState; }
	UE_API const FMidiSongPos& GetCurrentSongPos(ECalibratedMusicTimebase Timebase) const;
	UE_API const FMidiSongPos& GetPreviousSongPos(ECalibratedMusicTimebase Timebase) const;
	UE_API float GetDeltaBarF(ECalibratedMusicTimebase Timebase) const;
	UE_API float GetDeltaBeatF(ECalibratedMusicTimebase Timebase) const;

	const FMidiSongPos& GetCurrentSmoothedAudioRenderSongPos() const { return CurrentSongPos[(int32)ECalibratedMusicTimebase::AudioRenderTime]; }
	const FMidiSongPos& GetCurrentPlayerExperiencedSongPos() const { return CurrentSongPos[(int32)ECalibratedMusicTimebase::ExperiencedTime]; }
	const FMidiSongPos& GetCurrentVideoRenderSongPos() const { return CurrentSongPos[(int32)ECalibratedMusicTimebase::VideoRenderTime]; }
	const FMidiSongPos& GetCurrentRawAudioRenderSongPos() const { return CurrentSongPos[(int32)ECalibratedMusicTimebase::RawAudioRenderTime]; }

	UE_API void Start();
	UE_API void Pause();
	UE_API void Continue();
	UE_API void Stop();
	
	virtual const ISongMapEvaluator* GetCurrentSongMapEvaluator() const = 0;

	virtual bool LoopedThisFrame(ECalibratedMusicTimebase Timebase) const = 0;
	virtual bool SeekedThisFrame(ECalibratedMusicTimebase Timebase) const = 0;

	virtual void Disconnect() = 0;

	UE_API void UpdateMusicPlaybackRate(float InTempo, float InClockAdvanceRate, int32 InTimeSigNum, int32 InTimeSigDenom);

	// This returns our BeatsPerMinute which is our true beats, not quarter notes.
	UE_API float GetBeatsPerMinute() const;

#if !UE_BUILD_SHIPPING
	// Short user readable description good for logging. Should include name of song or sound source or whatnot. 
	virtual FString GetDisplayName() const = 0;
#endif

	FSongMaps DefaultMaps;
	
	// Note that tempo is MidiTempo, quarter notes per minute.
	float Tempo = 0.0f;
	int TimeSignatureNum = 0;
	int TimeSignatureDenom = 0;

	float CurrentBeatsPerSecond = 0.0f;
	float CurrentBarsPerSecond = 0.0f;
	float CurrentClockAdvanceRate = 1.0f;

	TStaticArray<FMidiSongPos, (int32)ECalibratedMusicTimebase::Count> CurrentSongPos;
	TStaticArray<FMidiSongPos, (int32)ECalibratedMusicTimebase::Count> PrevSongPos;

	// TODO shouldn't these deltas be rolled into the MidiSongPos?
	float DeltaBarF[(int32)ECalibratedMusicTimebase::Count];
	float DeltaBeatF[(int32)ECalibratedMusicTimebase::Count];

protected:
	
	virtual void OnStart() = 0;
	virtual void OnPause() = 0;
	virtual void OnContinue() = 0;
	virtual void OnStop() = 0;
	
	UE_API int32 GetArrayIndexForTimebase(ECalibratedMusicTimebase Timebase) const;

	UE_API void SetCurrentSongPos(ECalibratedMusicTimebase Timebase, const FMidiSongPos& InSongPos);
	UE_API void SetCurrentSongPosByTime(ECalibratedMusicTimebase Timebase, float InElapsedMs, const ISongMapEvaluator& Map);
	UE_API void SetCurrentSongPosByTick(ECalibratedMusicTimebase Timebase, float InTick, const ISongMapEvaluator& Map);

private:

	UE_API void ResetDefaultSongMaps(float DefaultTempo, int32 DefaultTimeSigNum, int32 DefaultTimeSigDenom);
	virtual bool RefreshCurrentSongPos() = 0;
	UE_API void UpdateBeatsAndBarsPerSecond();

	uint64 LastUpdateFrame = 0;

	EMusicClockState MusicClockState = EMusicClockState::Stopped;

	TWeakObjectPtr<UObject> ContextObj;
};

#undef UE_API
