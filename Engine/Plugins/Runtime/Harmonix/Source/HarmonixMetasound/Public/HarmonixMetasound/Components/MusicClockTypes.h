// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/NameTypes.h"

#include "MusicClockTypes.generated.h"

class UAudioComponent;
class UMidiFile;
class FMusicClockDriverBase;

USTRUCT(MinimalAPI)
struct FMusicClockSettingsBase
{
	GENERATED_BODY()

	UPROPERTY()
	float DefaultTempo = 120.f;

	UPROPERTY()
	int32 DefaultTimeSigNumerator = 4;

	UPROPERTY()
	int32 DefaultTimeSigDenominator = 4;

	virtual TSharedPtr<FMusicClockDriverBase> MakeInstance(UObject* WorldContextObj) const
	{
		unimplemented();
		return nullptr;
	}

	virtual ~FMusicClockSettingsBase() = default;
};

USTRUCT(DisplayName = "Wall Clock")
struct FWallClockMusicClockSettings final : public FMusicClockSettingsBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "MusicClock")
	TObjectPtr<UMidiFile> TempoMap;

	virtual TSharedPtr<FMusicClockDriverBase> MakeInstance(UObject* WorldContextObj) const override;
};

USTRUCT(DisplayName = "MetaSound Clock")
struct FMetasoundMusicClockSettings final : public FMusicClockSettingsBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "MusicClock")
	FName MetasoundOutputName = "MIDI Clock";

	UPROPERTY(VisibleAnywhere, Category = "MusicClock")
	TObjectPtr<UAudioComponent> AudioComponent;

	virtual TSharedPtr<FMusicClockDriverBase> MakeInstance(UObject* WorldContextObj) const override;
};
