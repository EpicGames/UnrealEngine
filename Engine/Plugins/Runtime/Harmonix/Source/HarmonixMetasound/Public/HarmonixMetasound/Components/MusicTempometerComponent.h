// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/Utils/MusicTempometerUtilities.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "Components/ActorComponent.h"

#include "MusicTempometerComponent.generated.h"

#define UE_API HARMONIXMETASOUND_API

class UMaterialParameterCollection;
class UMaterialParameterCollectionInstance;

/**
 * UMusicTempometerComponent provides playback properties of a UMusicClockComponent on its actor and optionally updates a UMaterialParameterCollection.
 */
UCLASS(MinimalAPI, ClassGroup = (MetaSoundMusic), PrioritizeCategories = "MusicClock", meta = (BlueprintSpawnableComponent, DisplayName = "Music Tempometer", ScriptName = MusicTempometerComponent))
class UMusicTempometerComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UE_API UMusicTempometerComponent();

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName SecondsIncludingCountInParameterName_DEPRECATED = "MusicSecondsIncludingCountIn";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName BarsIncludingCountInParameterName_DEPRECATED = "MusicBarsIncludingCountIn";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName BeatsIncludingCountInParameterName_DEPRECATED = "MusicBeatsIncludingCountIn";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName SecondsFromBarOneParameterName_DEPRECATED = "MusicSecondsFromBarOne";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName TimestampBarParameterName_DEPRECATED = "MusicTimestampBar";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName TimestampBeatInBarParameterName_DEPRECATED = "MusicTimestampBeatInBar";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName BarProgressParameterName_DEPRECATED = "MusicBarProgress";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName BeatProgressParameterName_DEPRECATED = "MusicBeatProgress";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName TimeSignatureNumeratorParameterName_DEPRECATED = "MusicTimeSignatureNumerator";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName TimeSignatureDenominatorParameterName_DEPRECATED = "MusicTimeSignatureDenominator";

	/**
	 * Deprecated, has moved into structure within MPCParameters
	 */
	UPROPERTY()
	FName TempoParameterName_DEPRECATED = "MusicTempo";

	/**
	 * Parameter names to use for the values in the material parameter collection.
	 */
	UPROPERTY(EditAnywhere, Category = "MusicClock")
	FMusicTempometerMPCParameters MPCParameters;

	/**
	 * The FSongPos for which the game thread is currently issuing graphics rendering commands, according to calibration data.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	const FMidiSongPos& GetSongPos() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos;
	}

	/**
	 * The FSongPos for which the game thread previously issued graphics rendering commands.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	const FMidiSongPos& GetPreviousFrameSongPos() const
	{
		UpdateCachedSongPosIfNeeded();
		return PreviousFrameSongPos;
	}

	/**
	* Seconds from the beginning of the entire music authoring.
	* Includes all count-in and pickup bars (ie. won't be negative when
	* the music starts, and bar 1 beat 1 may not be at 0.0 seconds!
	*/
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetSecondsIncludingCountIn() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.SecondsIncludingCountIn;
	}

	/**
	* Bars from the beginning of the music. Includes all
	* count-in and pickup bars (ie. won't be negative when the music starts,
	* and bar 1 beat 1 of the music may not be equal to elapsed bar 0.0!
	*/
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetBarsIncludingCountIn() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.BarsIncludingCountIn;
	}

	/**
	 * Beats from the beginning of the music. Includes all
	 * count-in and pickup bars/beats (ie. won't be negative when the music starts,
	 * and elapsed beat 0.0 may not equal a timestamp of bar 1 beat 1!
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetBeatsIncludingCountIn() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.BeatsIncludingCountIn;
	}

	/**
	 * Seconds from Bar 1 Beat 1 of the music. If the music has a count-in
	 * or pickup bars this number may be negative when the music starts!
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetSecondsFromBarOne() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.SecondsFromBarOne;
	}

	/**
	 * Current bar & beat in the traditional format, where...
	 *     - bar 1 beat 1 is the beginning of the song.
	 *     - bars BEFORE bar 1 are count-in or "pickup" bars.
	 *     - beat is always 1 or greater.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	FMusicTimestamp GetTimestamp() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.Timestamp;
	}

	/**
	 * Progress of the current bar [0, 1).
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetBarProgress() const
	{
		UpdateCachedSongPosIfNeeded();
		return FMath::Fractional(SongPos.BarsIncludingCountIn);
	}

	/**
	 * Progress of the current beat [0, 1).
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetBeatProgress() const
	{
		UpdateCachedSongPosIfNeeded();
		return FMath::Fractional(SongPos.BeatsIncludingCountIn);
	}

	/**
	 * Current time signature numerator (beats per bar for a simple meter).
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetTimeSignatureNumerator() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.TimeSigNumerator;
	}

	/**
	 * Current time signature denominator (scale from note duration to beat fraction for a simple meter).
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetTimeSignatureDenominator() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.TimeSigDenominator;
	}

	/**
	 * Current tempo (beats per minute).
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	float GetTempo() const
	{
		UpdateCachedSongPosIfNeeded();
		return SongPos.Tempo;
	}

	/**
	 * Set the MaterialParameterCollection whose parameters will be updated.
	 * If any of the named parameters are missing they will be ignored.
	 */
	UFUNCTION(BlueprintSetter, Category = "MusicClock")
	void SetMaterialParameterCollection(UMaterialParameterCollection* InMaterialParameterCollection)
	{
		FScopeLock lock(&SongPosUpdateMutex);
		MaterialParameterCollection = InMaterialParameterCollection;
		SetComponentTickEnabledAsync(MaterialParameterCollection != nullptr);
	}

	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UMaterialParameterCollection* GetMaterialParameterCollection() const
	{
		FScopeLock lock(&SongPosUpdateMutex);
		return MaterialParameterCollection;
	}

	/**
	 * SetSongPosInterface allows setting any UObject implementing the ISongPosInterface as the attribute source.
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	void SetClock(UMusicClockComponent* InClockComponent)
	{
		FScopeLock lock(&SongPosUpdateMutex);
		MusicClock = MoveTemp(InClockComponent);
	}

	/**
	 * SetSongPosInterfaceFromActor sets the actor or the first of its owned components that implements ISongPosInterface as the attribute source.
	 * BeginPlay calls this on the owning actor when the source ISongPosInterface is not set.
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	void SetClockFromActor(AActor* Actor)
	{
		FScopeLock lock(&SongPosUpdateMutex);
		MusicClock = FindClock(Actor);
	}

	UFUNCTION(BlueprintGetter, Category = "MusicClock")
	const UMusicClockComponent* GetClock() const
	{
		FScopeLock lock(&SongPosUpdateMutex);
		return GetClockNoMutex();
	}

	//~ Begin UObject Interface.
	UE_API virtual void PostLoad() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

private:
	UE_API UMusicClockComponent* FindClock(AActor* Actor) const;
	UE_API void SetOwnerClock() const;

	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	const UMusicClockComponent* GetClockNoMutex() const
	{
		if (!IsValid(MusicClock.Get()))
		{
			SetOwnerClock();
		}
		return MusicClock;
	}

	UMusicClockComponent* GetMutableClockNoMutex() const
	{
		if (!IsValid(MusicClock.Get()))
		{
			SetOwnerClock();
		}
		return MusicClock;
	}

	UE_API void UpdateCachedSongPos() const;
	UE_API void UpdateCachedSongPosIfNeeded() const;

	UPROPERTY(BlueprintGetter = GetSongPos, Transient, Category = "MusicClock")
	mutable FMidiSongPos SongPos;

	UPROPERTY(BlueprintGetter = GetPreviousFrameSongPos, Transient, Category = "MusicClock")
	mutable FMidiSongPos PreviousFrameSongPos;

	mutable FCriticalSection SongPosUpdateMutex;
	mutable uint64 LastFrameCounter;

	/**
	 * Music whose tempo to detect.
	 */
	UPROPERTY(BlueprintGetter = GetClock, BlueprintSetter = SetClock, Category = "MusicClock")
	mutable TObjectPtr<UMusicClockComponent> MusicClock;

	UPROPERTY(EditAnywhere, BlueprintGetter = GetMaterialParameterCollection, BlueprintSetter = SetMaterialParameterCollection, Category = "MusicClock")
	TObjectPtr<UMaterialParameterCollection> MaterialParameterCollection;

	/**
	 * Pinned material parameter collection instance
	 */
	UPROPERTY(Transient)
	mutable TWeakObjectPtr<UMaterialParameterCollectionInstance> MaterialParameterCollectionInstance;
};

#undef UE_API
