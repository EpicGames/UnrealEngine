// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MusicalTime.h"
#include "Harmonix/MusicalTimebase.h"
#include "HarmonixMetasound/DataTypes/MusicTimeInterval.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMusicTimer, Warning, All);

#define UE_API HARMONIXMETASOUND_API

class FMusicTimer
{
public:
	friend class UMusicTimerManager;
	UE_API FMusicTimer(uint64 InUniqueIdentifier, const FMusicTimeInterval& InTimeInterval, const FMusicTimestamp& InStartTime, ECalibratedMusicTimebase InTimebase, bool InbLooping, FSimpleDelegate InDelegate);

	ECalibratedMusicTimebase GetTimebase() const { return Timebase; }
	uint64 GetUniqueIdentifier() const;

	bool IsLooping() const { return bLooping; }
	bool IsPaused() const { return PausedTime.IsSet(); }

	// Returns whether or not the timer is finished
	bool Update(const FMusicTimestamp& CurrentMusicalTimestamp, const FTimeSignature& TimeSignature);
	void Reset();
private:

	UPROPERTY(EditAnywhere, Category = "Music|Timer")
	ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::VideoRenderTime;

	FSimpleDelegate Delegate;
	FMusicTimeInterval TimeInterval;
	FMusicTimestamp StartTime;
	TOptional<FMusicTimestamp> PausedTime;
	uint64 UniqueIdentifier = 0;
	bool bLooping = false;
};

#undef UE_API