// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MusicTimerHandle.generated.h"

#define UE_API HARMONIXMETASOUND_API

/** Unique handle that can be used to distinguish timers that have identical delegates. */
USTRUCT(BlueprintType)
struct FMusicTimerHandle
{
	GENERATED_BODY()

	friend class UMusicTimerManager;

	UE_API FMusicTimerHandle(uint64 InHandle = 0);

	UE_API bool IsValid() const;
	UE_API void Invalidate();

	UE_API FString ToString() const;
	UE_API uint64 GetUniqueIdentifier() const;
	UE_API friend uint32 GetTypeHash(const FMusicTimerHandle& InHandle);

	bool operator==(const FMusicTimerHandle& Other) const;
	bool operator!=(const FMusicTimerHandle& Other) const;

private:

	UPROPERTY(Transient)
	uint64 Handle;
};

#undef UE_API