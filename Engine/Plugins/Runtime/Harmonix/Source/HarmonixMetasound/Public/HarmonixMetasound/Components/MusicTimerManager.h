// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "MusicTimer.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "MusicTimerManager.generated.h"

#define UE_API HARMONIXMETASOUND_API

DECLARE_LOG_CATEGORY_EXTERN(LogMusicTimerManager, Warning, All);
DECLARE_DYNAMIC_DELEGATE(FOnMusicalTimerExecute);

enum class EMidiClockSubdivisionQuantization : uint8;
enum class EMidiFileQuantizeDirection : uint8;

struct FMusicTimerHandle;
class UMusicClockComponent;

UCLASS(Blueprintable, BlueprintType)
class UMusicTimerManager : public UObject
{
	GENERATED_BODY()

public:

	// We need the MidiSongPosition in each calibrated time, because each timer can be operating on their own timebase.
	void UpdateForGameFrame(const TStaticArray<FMidiSongPos, (int32)ECalibratedMusicTimebase::Count>& CurrentSongPositions);

	/**
	 * Adds a musical timer.
	 *
	 * @param TimerInterval  Interval configuration for the timer.
	 * @param StartTime      Start time. You can use the Quantize function below to snap to interval boundaries.
	 * @param Timebase       Timebase used to evaluate the timer.
	 * @param bLooping       Whether the timer should loop.
	 * @param TimerDelegate  Delegate executed when the timer fires.
	 */
	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	FMusicTimerHandle BP_AddTimer(const FMusicTimeInterval& TimerInterval, FMusicTimestamp StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FOnMusicalTimerExecute TimerDelegate);

	/**
	 * Adds a musical timer.
	 *
	 * @param TimerInterval  Interval configuration for the timer.
	 * @param StartTime      Start time. You can use the Quantize function below to snap to interval boundaries.
	 * @param Timebase       Timebase used to evaluate the timer.
	 * @param bLooping       Whether the timer should loop.
	 * @param TimerDelegate  Delegate executed when the timer fires.
	 */
	UE_API FMusicTimerHandle AddTimerNative(const FMusicTimeInterval& TimerInterval, FMusicTimestamp StartTime, ECalibratedMusicTimebase Timebase, bool bLooping, FSimpleDelegate TimerDelegate);

	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	UE_API void RemoveTimer(const FMusicTimerHandle& Handle);

	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	UE_API void PauseTimer(const FMusicTimerHandle& Handle, bool bPause);

	UFUNCTION(BlueprintCallable, Category = "Music|Timer")
	FMusicTimestamp Quantize(const UMusicClockComponent* MusicClock, const FMusicTimestamp& Time, EMidiClockSubdivisionQuantization QuantizationInterval, EMidiFileQuantizeDirection InDirection) const;

	uint64 GetUniqueTimerId() { return ++NextTimerId; }
private:

	// Used so pause doesn't need to pass in this info
	TStaticArray<FMidiSongPos, (int32)ECalibratedMusicTimebase::Count> LastUpdateFrameSongPositions;

	TArray<TUniquePtr<FMusicTimer>> ActiveTimers; // Current Timers being processed
	TArray<TUniquePtr<FMusicTimer>> PendingTimers; // New timers added during execution
	uint64 LastUpdateFrame = 0;
	static std::atomic<uint64> NextTimerId;
};

#undef UE_API
