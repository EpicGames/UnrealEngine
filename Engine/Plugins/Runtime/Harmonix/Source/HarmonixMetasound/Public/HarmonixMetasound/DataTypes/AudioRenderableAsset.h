// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace HarmonixMetasound
{
	// This next macro declares the class that will be registered with Metasound,
	// creating the TypeInfo, ReadRef, and WriteRef types needed for that.
	// You must use this macro in the header where your USTRUCT is defined, and you 
	// must use the corresponding "DEFINE_AUDIORENDERABLE_ASSET" macro in the cpp
	// file.
#define DECLARE_AUDIORENDERABLE_ASSET(InNamespace, AssetStructName, ProxyName, ModuleApi)																	\
	namespace InNamespace	                                                                                                                                \
	{                                                                                                                                                       \
		class AssetStructName                                                                                                                               \
		{                                                                                                                                                   \
			TSharedPtr<ProxyName, ESPMode::ThreadSafe> ProxyName ## Instance;                                                                               \
		public:                                                                                                                                             \
			AssetStructName() = default;                                                                                                                    \
			AssetStructName(const AssetStructName&) = default;                                                                                              \
			AssetStructName& operator=(const AssetStructName& Other) = default;                                                                             \
			AssetStructName(const TSharedPtr<Audio::IProxyData>& InInitData)                                                                                \
			{                                                                                                                                               \
				if (InInitData.IsValid())                                                                                                                   \
				{                                                                                                                                           \
					if (InInitData->CheckTypeCast<ProxyName>())                                                                                             \
					{                                                                                                                                       \
						ProxyName ## Instance = MakeShared<ProxyName, ESPMode::ThreadSafe>(InInitData->GetAs<ProxyName>());                                 \
					}                                                                                                                                       \
				}                                                                                                                                           \
			}                                                                                                                                               \
			ProxyName::NodePtr GetRenderable() const                                                                                                        \
			{                                                                                                                                               \
				if (ProxyName ## Instance.IsValid())                                                                                                        \
				{                                                                                                                                           \
					return ProxyName ## Instance->GetRenderable();                                                                                          \
				}                                                                                                                                           \
				return nullptr;                                                                                                                             \
			}                                                                                                                                               \
		};                                                                                                                                                  \
		/* Declare aliases IN the namespace... */                                                                                                           \
		DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(AssetStructName, AssetStructName ## TypeInfo, AssetStructName ## ReadRef, AssetStructName ## WriteRef) \
	}                                                                                                                                                       \
	/* Declare reference types OUT of the namespace... */                                                                                                   \
	DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(InNamespace::AssetStructName, ModuleApi)

	// This next macro defines the "audio renderable asset" declared with the macro above.
	// Use this one in your cpp file.
#define DEFINE_AUDIORENDERABLE_ASSET(InNamespace, AssetStructName, AssetPlainName, AssetUObjectClass) \
	REGISTER_METASOUND_DATATYPE(InNamespace::AssetStructName, #AssetPlainName, Metasound::ELiteralType::UObjectProxy, AssetUObjectClass);

}
