// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundEnumRegistrationMacro.h"
#include "HarmonixDsp/Effects/Settings/DelaySettings.h"

namespace Metasound
{
	DECLARE_METASOUND_ENUM(
		EDelayStereoType,
		EDelayStereoType::Default,
		HARMONIXMETASOUND_API,
		FEnumDelayStereoType,
		FEnumDelayStereoTypeInfo,
		FEnumDelayStereoTypeReadReference,
		FEnumDelayStereoTypeWriteReference);
}
