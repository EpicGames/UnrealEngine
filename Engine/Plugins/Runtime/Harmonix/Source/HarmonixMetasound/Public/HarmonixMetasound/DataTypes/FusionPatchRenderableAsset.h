// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioRenderableAsset.h"
#include "HarmonixDsp/FusionSampler/FusionPatch.h"
#include "MetasoundDataReferenceMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"

// This next macro does a few things. 
// - It says, "I want a Metasound exposed asset called 'FFusionPatchAsset' with 
//   corresponding TypeInfo, ReadRef, and WriteRef classes."
// - That asset is a wrapper around a proxy class that acts as the go-between from the 
//   UObject (GC'able) side to  the audio render thread side. 
//	 This proxy class should have already been defined 
//NOTE: This macro has a corresponding "DEFINE_AUDIORENDERABLE_ASSET" that must be added to the cpp file. 
DECLARE_AUDIORENDERABLE_ASSET(HarmonixMetasound, FFusionPatchAsset, FFusionPatchDataProxy, HARMONIXMETASOUND_API)