// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "HarmonixMusicAsset.h"
#include "MusicTypes/MusicalAsset.h"
#include "MusicTypes/MusicHandle.h"

#include "HarmonixMetasoundMusicAsset.generated.h"

#define UE_API HARMONIXMETASOUND_API

/**
 * This file defines a UHarmonixMetasoundMusicAsset, an implementation of the UHarmonixMusicAsset
 * class. 
 */

class UMetaSoundSource;
class UMidiFile;

/**
 * UHarmonixMetasoundMusicAsset:
 * In this case, the music asset is comprised of a MetaSound that uses the Harmonix MetaSound nodes to
 * generate musical time, and a midi file which maps the tempo and time signatures that define the music's
 * form.
 */
UCLASS(MinimalAPI, BlueprintType, Category = "Music", Meta = (DisplayName = "Harmonix MetaSound Music"))
class UHarmonixMetasoundMusicAsset : public UHarmonixMusicAsset
{
GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Music", meta = (GetAssetFilter = "AssetIsMissingHarmoixMusicInterface"))
	TObjectPtr<UMetaSoundSource> MetaSoundSource;

	UFUNCTION()
	static UE_API bool AssetIsMissingHarmoixMusicInterface(const FAssetData& AssetData);

protected:
	virtual UMetaSoundSource* GetMetaSoundSource() { return MetaSoundSource; }
};

#undef UE_API
