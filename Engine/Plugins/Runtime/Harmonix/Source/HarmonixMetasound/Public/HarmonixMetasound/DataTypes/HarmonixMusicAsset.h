// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MusicTypes/MusicalAsset.h"
#include "MusicTypes/MusicHandle.h"

#include "HarmonixMusicAsset.generated.h"

#define UE_API HARMONIXMETASOUND_API

/**
 * This file defines a UHarmonixMusicAsset, an implementation of the pure virtual IMusicalAsset 
 * interface that will act as a base class for UHarmonixMetasoundMusicAsset and UHarmonixWaveMusicAsset.
 * 
 * As such, it can return an FMusicHandle when it is asked to PrepareToPlay or Play.
 */

class UMetaSoundSource;
class UMidiFile;

/**
 * UHarmonixMusicAsset
 * An implementation of the pure virtual IMusicalAsset interface that will act as a base class
 * for UHarmonixMetasoundMusicAsset and UHarmonixWaveMusicAsset. As such, it can return an
 * FMusicHandle when it is asked to PrepareToPlay or Play.
 */
UCLASS(MinimalAPI, BlueprintType, Category = "Music", Meta = (DisplayName = "Harmonix MetaSound Music"))
class UHarmonixMusicAsset : public UObject, public IMusicalAsset
{
GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Music")
	TObjectPtr<UMidiFile> MidiSongMap;

	virtual UMetaSoundSource* GetMetaSoundSource() { return nullptr; }

	//** BEGIN IMusicalAsset
public:
	UE_API virtual void CreateFrameBasedMusicMap(UFrameBasedMusicMap* Map) const override;
	UE_API virtual FMarkerProviderResults GatherMarkers(const UFrameBasedMusicMap* Map) const override;
protected:
	UE_API virtual FMusicHandle PrepareToPlay_Internal(UObject* PlaybackContext, UAudioComponent* OnComponennt = nullptr, float FromSeconds = 0.0f, bool IsAudition = false) override;
	UE_API virtual FMusicHandle Play_Internal(UObject* PlaybackContext, UAudioComponent* OnComponennt = nullptr, float FromSeconds = 0.0f, bool IsAudition = false) override;
	//** END IMusicalAsset
};	

#undef UE_API
