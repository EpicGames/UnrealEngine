// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MusicTypes/MusicalAsset.h"
#include "MusicTypes/MusicHandle.h"
#include "MusicEnvironmentClockSource.h"
#include "UObject/Interface.h"
#include "HarmonixMusicHandle.generated.h"

#define UE_API HARMONIXMETASOUND_API

class UAudioComponent;
class UHarmonixMusicAsset;
class UMusicClockComponent;
class USoundBase;

/**
 * UHarmonixMusicHandle
 * 
 * This defines a concrete implementation of the IMusicHandle interface. It is a wrapper around
 * a UAudioComponent and a UMusicClockComponent. Using those two components this class can
 * control the transport of the playing music (play, pause, continue, seek stop, kill) and
 * provide the holder with a source of musical time.
 *
 * This is the 'flavor' of IMusicHandle returned by the UHarmonixMetasoundMusicAsset class (among others
 * in the future) when it is asked to instantiate itself (play).
 */

UCLASS(MinimalAPI)
class UHarmonixMusicHandle : public UObject, public IMusicHandle
{
public:
	GENERATED_BODY()
public:
	static UE_API UHarmonixMusicHandle* Instantiate(UHarmonixMusicAsset* Asset, UObject* PlaybackContext, UAudioComponent* OnComponent, float FromSeconds = 0.0f, bool bIsAudition = false);

	UE_API virtual bool IsValid() const override;
	UE_API virtual void Tick(float DeltaSeconds) override;

	//** Transport
	UE_API virtual bool Play(float FromSeconds = 0.0f) override;
	UE_API virtual void Pause() override;
	UE_API virtual void Continue() override;
protected:
	UE_API virtual void Stop_Internal() override;
	UE_API virtual void Kill_Internal() override;
public:

	UE_API virtual bool IsUsingAsset(const UObject* Asset) const override;

	UE_API virtual TScriptInterface<IMusicEnvironmentClockSource> GetMusicClockSource() override;

private:
	UPROPERTY(Transient)
	TObjectPtr<UAudioComponent> AudioComponent;

	UPROPERTY(Transient)
	TObjectPtr<UMusicClockComponent> MusicClockComponent;

	UPROPERTY(Transient)
	TObjectPtr<UHarmonixMusicAsset> PlayingAsset;

	bool bHasStarted = false;
};

#undef UE_API
