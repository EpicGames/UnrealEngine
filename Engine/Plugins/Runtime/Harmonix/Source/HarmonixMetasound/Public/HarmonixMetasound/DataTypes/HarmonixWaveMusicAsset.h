// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MusicTypes/MusicalAsset.h"
#include "MusicTypes/MusicHandle.h"
#include "HarmonixMusicAsset.h"

#include "HarmonixWaveMusicAsset.generated.h"

#define UE_API HARMONIXMETASOUND_API

/**
 * This file defines a UHarmonixMetasoundMusicAsset, an implementation of the UHarmonixMusicAsset
 * class. 
 * 
 * In this case, the music asset is comprised of a MetaSound that uses the Harmonix MetaSound nodes to 
 * generate musical time, and a midi file which maps the tempo and time signatures that define the music's
 * form. 
 */

class USoundWave;
class UMidiFile;

UCLASS(MinimalAPI, BlueprintType, Category = "Music", Meta = (DisplayName = "Harmonix Wave Music"))
class UHarmonixWaveMusicAsset : public UHarmonixMusicAsset
{
GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Music")
	TObjectPtr<USoundWave> WaveFile;

	UE_API virtual float GetSongLengthSeconds() const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	UE_API virtual UMetaSoundSource* GetMetaSoundSource();

private:
	UPROPERTY(Transient)
	TObjectPtr<UMetaSoundSource> CachedMetasound;
};

#undef UE_API
