// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "IAudioProxyInitializer.h"

#include "HarmonixMidi/MidiFile.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound
{
	class FMidiAsset
	{
		FMidiFileProxyPtr MidiFileProxy;

	public:

		FMidiAsset() = default;
		FMidiAsset(const FMidiAsset&) = default;
		FMidiAsset& operator=(const FMidiAsset& Other) = default;

		UE_API FMidiAsset(const TSharedPtr<Audio::IProxyData>& InInitData);

		UE_API bool IsMidiValid() const;

		const FMidiFileProxyPtr& GetMidiProxy() const
		{
			return MidiFileProxy;
		}

		const FMidiFileProxy* operator->() const
		{
			return MidiFileProxy.Get();
		}

		FMidiFileProxy* operator->()
		{
			return MidiFileProxy.Get();
		}
	};

	// Declare aliases IN the namespace...
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FMidiAsset, FMidiAssetTypeInfo, FMidiAssetReadRef, FMidiAssetWriteRef)
}

// Declare reference types OUT of the namespace...
DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(HarmonixMetasound::FMidiAsset, HARMONIXMETASOUND_API)

#undef UE_API
