// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"

#include "MetasoundDataReference.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVariable.h"
#include "MetasoundSampleCounter.h"

#include "HarmonixMidi/MidiFile.h"
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/DataTypes/MidiClockEvent.h"
#include "Sound/QuartzQuantizationUtilities.h"

#define UE_API HARMONIXMETASOUND_API

namespace Metasound
{
	DECLARE_METASOUND_ENUM(
		EMidiClockSubdivisionQuantization,
		EMidiClockSubdivisionQuantization::None,
		HARMONIXMETASOUND_API,
		FEnumMidiClockSubdivisionQuantizationType,
		FEnumMidiClockSubdivisionQuantizationTypeInfo,
		FEnumMidiClockSubdivisionQuantizationReadRef,
		FEnumMidiClockSubdivisionQuantizationWriteRef
	);
}

namespace HarmonixMetasound
{
	class FMidiClock;
	using FConstSharedMidiClockPtr = TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe>;
	using FMidiClockEvents = TArray<FMidiClockEvent>;

	class FMidiClock : public TSharedFromThis<FMidiClock, ESPMode::NotThreadSafe>
	{
	public:
		static constexpr int32 kMidiGranularity = 128;
		
		UE_API explicit FMidiClock(const Metasound::FOperatorSettings& InSettings);
		UE_API FMidiClock(const FMidiClock& Other);
		UE_API virtual ~FMidiClock();
		UE_API FMidiClock& operator=(const FMidiClock& Other);

		UE_API void AttachToSongMapEvaluator(TSharedPtr<ISongMapEvaluator> SongMaps, bool ResetToStart = true);
		UE_API void SongMapsChanged();
		UE_API void DetachFromSongMaps();
		UE_API void Reset(const Metasound::FOperatorSettings& InSettings);

		const ISongMapEvaluator& GetSongMapEvaluator() const { return *SongMapEvaluator; }

		UE_API void SetDrivingClock(FConstSharedMidiClockPtr NewExternalClockDriver);
		FConstSharedMidiClockPtr GetDrivingClock() const { return ExternalClockDriver; }

		UE_API void PrepareBlock();

		UE_API void SetTransportState(int32 BlockFrameIndex, EMusicPlayerTransportState TransportState);
		UE_API void SetSpeed(int32 BlockFrameIndex, float Speed);
		UE_API void SetTempo(int32 BlockFrameIndex, int32 Tick, float Bpm, int32 TempoMapTick);
		UE_API void SetTimeSignature(int32 BlockFrameIndex, int32 Tick, const FTimeSignature& TimeSignature, int32 TempoMapTick);

		// directly seek this clock with a musical seek target or a specific tick
		UE_API void SeekTo(int32 BlockFrameIndex, const FMusicSeekTarget& InTarget);
		UE_API void SeekTo(int32 BlockFrameIndex, int32 Tick, int32 TempoMapTick);

		// This will add a loop event to the clock event stream WITHOUT having 
		// to set this clock to looping. This is used when this clock is being 
		// driven by an external clock and THAT clock's looping setup causes this
		// clock to loop. 
		UE_API void AddTransientLoop(int32 BlockFrameIndex, int32 NewFirstTiokInLoop, int32 NewLoopLengthTicks);
		UE_API void SetupPersistentLoop(int32 NewFirstTickInLoop, int32 NewLoopLengthTicks);
		UE_API void ClearPersistentLoop();
		UE_API bool HasPersistentLoop() const;
		int32 GetFirstTickInLoop() const { return FirstTickInLoop; }
		int32 GetLoopLengthTicks() const { return LoopLengthTicks; }
		UE_API float GetLoopStartMs() const;
		UE_API float GetLoopEndMs() const;
		UE_API float GetLoopLengthMs() const;

		// process and advance the clock based on the driving clock given sample frames
		// will handle the driving clock events based on the frame range
		UE_API void Advance(const FMidiClock& DrivingClock, int32 StartFrame, int32 NumFrames);

		// process and advance the clock normally based on the given sample frames
		UE_API void Advance(int32 StartFrame, int32 NumFrames);

		UE_API bool AdvanceToTick(int32 BlockFrameIndex, int32 UpToTick, int32 TempoMapTick);
		UE_API bool AdvanceToMs(int32 BlockFrameIndex, float Ms);

		bool  HasTransportStateChangesInBlock() const    { return NumTransportChangeInBlock > 0; }
		int32 GetNumTransportStateChangesInBlock() const { return NumTransportChangeInBlock;     }
		bool  HasSpeedChangesInBlock() const             { return NumSpeedChangeInBlock > 0;     }
		int32 GetNumSpeedChangesInBlock() const          { return NumSpeedChangeInBlock;         }
		bool  HasTempoChangesInBlock() const             { return NumTempoChangeInBlock > 0;     }
		int32 GetNumTempoChangesInBlock() const          { return NumTempoChangeInBlock;         }

		const FMidiClockEvents& GetMidiClockEventsInBlock() const { return MidiClockEventsInBlock; }

		EMusicPlayerTransportState GetTransportStateAtStartOfBlock() const { return TransportAtBlockStart; }
		EMusicPlayerTransportState GetTransportStateAtEndOfBlock() const { return TransportAtBlockEnd; }

		float GetSpeedAtStartOfBlock() const { return SpeedAtBlockStart; }
		UE_API float GetSpeedAtBlockSampleFrame(int32 FrameIndex) const;
		float GetSpeedAtEndOfBlock() const { return SpeedAtBlockEnd; }

		float GetTempoAtStartOfBlock() const { return ExternalClockDriver ? ExternalClockDriver->GetTempoAtStartOfBlock() : TempoAtBlockStart; }
		UE_API float GetTempoAtBlockSampleFrame(int32 FrameIndex) const;
		float GetTempoAtEndOfBlock() const { return ExternalClockDriver ? ExternalClockDriver->GetTempoAtEndOfBlock() : TempoAtBlockEnd; }

		int32 GetLastProcessedMidiTick() const { return LastProcessedMidiTick; }
		int32 GetNextMidiTickToProcess() const { return NextMidiTickToProcess; }

		UE_API float GetCurrentSongPosMs() const;

		/**
		 * @brief Get the timestamp after the most recent clock update
		 * @return The current timestamp
		 */
		UE_API FMusicTimestamp GetMusicTimestampAtBlockEnd() const;
		
		/**
		 * @brief Get the music timestamp at a given frame offset from the last processed audio block.
		 * @param Offset - The frame index from the beginning of the last processed audio block
		 * @return The music timestamp
		 */
		UE_API FMusicTimestamp GetMusicTimestampAtBlockOffset(int32 Offset) const;

		/**
		 * @brief Get the absolute "music time" in ms for a frame within the last audio block. This is the 
		 * time in the musical content that the clock has advanced "up to". Note: This time will not be sample
		 * accurate as midi processing advances by ticks, and the time is calculated by turning the "current tick"
		 * at the offset provided into a time in ms.
		 * @param Offset - The frame index from the beginning of the last processed audio block
		 * @return The absolute time in ms
		 */
		UE_API float GetSongPosMsAtBlockOffset(int32 Offset) const;

		/**
		 * Given an input tick, outputs a looped tick if the input tick is > the StartTick of the Loop Region
		 * If the clock is not looping, or loop region length is 0, then the output will be unchanged.
		 *
		 * The output tick will be in range [Min(Tick, LoopStartTick), LoopEndTick).
		 * Example:
		 * LoopRegion: (0, 100):
		 * 10 -> 10
		 * 100 -> 0
		 * 110 -> 10
		 * -10 -> 90
		 *
		 * LoopRegion: (40, 100):
		 * 0 -> 0
		 * 10 -> 10
		 * -10 -> -10
		 * 99  -> 99
		 * 100 -> 40
		 * 110 -> 50
		 *
		 * @param		Tick - Absolute Tick
		 * @return		Looped Tick if Tick > LoopEnd: LoopedTick = LoopStart + (Tick - LoopStart) % (LoopEnd - LoopStart) 
		 */
		UE_API int32 WrapTickIfLooping(int32 Tick) const;
		
		bool GetSongMapsChangedInBlock() const { return MidiDataChangedInBlock; }

		UE_API int32 GetNextTickToProcessAtBlockFrame(int32 BlockFrame) const;

	private:
		UE_API void AddEvent(const FMidiClockEvent& InEvent, bool bRequireSequential = true);
		UE_API void HandleClockEvent(const FMidiClock& DrivingClock, const FMidiClockEvent& Event);
		UE_API void PostTempoOrTimeSignatureEventsIfNeeded();
		
		template <typename MSGTYPE>
		MSGTYPE* LookForEventOnMidiTick(int32 Tick);
		template <typename MSGTYPE>
		MSGTYPE* LookForEventOnBlockFrameIndex(int32 BlockFrameIndex);

		UE_API void AddTransportStateChangeToBlock(int32 BlockFrameIndex, EMusicPlayerTransportState TransportState);
		UE_API void AddTimeSignatureChangeToBlock(int32 BlockFrameIndex, int32 Tick, const FTimeSignature& TimeSignature, int32 TempoMapTick);
		UE_API void AddTempoChangeToBlock(int32 BlockFrameIndex, int32 Tick, float Tempo, int32 TempoMapTick);
		UE_API void AddSpeedChangeToBlock(int32 BlockFrameIndex, float Speed, bool bIsNewLocalSpeed);
		UE_API void AddLoopToBlock(int32 BlockFrameIndex, int32 FirstTick, int32 LoopLength, int32 TempoMapTick);
		UE_API void AddSeekToBlock(int32 BlockFrameIndex, int32 ToTick, int32 TempoMapTick);
		UE_API void AddAdvanceToBlock(int32 BlockFrameIndex, int32 FirstTick, int32 NumTicks, int32 TempoMapTick);
		UE_API void RebuildSongMapEvaluator(const TSharedPtr<const ISongMapEvaluator>& MidiWithTempo, const TSharedPtr<const ISongMapEvaluator>& MidiWithOtherMaps);

		TSharedPtr<FSongMapsWithAlternateTempoSource> SongMapEvaluator;
		int32 CurrentTempoInfoPointIndex;
		int32 CurrentTimeSignaturePointIndex;

		FConstSharedMidiClockPtr ExternalClockDriver;
		float TickResidualWhenDriven;

		int32 BlockSize;
		int32 CurrentBlockFrameIndex;
		int32 FirstTickProcessedThisBlock;
		int32 LastProcessedMidiTick;
		int32 NextMidiTickToProcess;
		int32 NextTempoMapTickToProcess;
		float SampleRate;
		Metasound::FSampleCount SampleCount;
		int32 FramesUntilNextProcess;
		EMusicPlayerTransportState TransportAtBlockStart;
		EMusicPlayerTransportState TransportAtBlockEnd;
		float SpeedAtBlockStart;
		float SpeedAtBlockEnd;
		float CurrentLocalSpeed;
		float TempoAtBlockStart;
		float TempoAtBlockEnd;
		FTimeSignature TimeSignatureAtBlockStart;
		FTimeSignature TimeSignatureAtBlockEnd;

		int32 NumTransportChangeInBlock;
		int32 NumSpeedChangeInBlock;
		int32 NumTempoChangeInBlock;
		int32 NumTimeSignatureChangeInBlock;
		
		int32 NextTempoChangeTick;
		int32 NextTimeSigChangeTick;
		int32 NextTempoOrTimeSigChangeTick;

		int32 FirstTickInLoop;
		int32 LoopLengthTicks;

		bool MidiDataChangedInBlock;
		bool NeedsSeekToDrivingClock;

		FMidiClockEvents MidiClockEventsInBlock;
	};

	// Declare aliases IN the namespace...
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FMidiClock, FMidiClockTypeInfo, FMidiClockReadRef, FMidiClockWriteRef)

	template <typename MSGTYPE>
	MSGTYPE* FMidiClock::LookForEventOnMidiTick(int32 Tick)
	{
		for (auto It = MidiClockEventsInBlock.rbegin(); It != MidiClockEventsInBlock.rend(); ++It)
		{
			FMidiClockEvent& AsClockEvent = *It;
			if (MSGTYPE* AsDesiredMsgType = AsClockEvent.TryGet<MSGTYPE>())
			{
				if (AsDesiredMsgType->ContainsTick(Tick))
				{
					return AsDesiredMsgType;
				}
			}
		}
		return nullptr;
	}

	template <typename MSGTYPE>
	MSGTYPE* FMidiClock::LookForEventOnBlockFrameIndex(int32 BlockFrameIndex)
	{
		for (auto It = MidiClockEventsInBlock.rbegin(); It != MidiClockEventsInBlock.rend(); ++It)
		{
			FMidiClockEvent& AsClockEvent = *It;
			if (AsClockEvent.BlockFrameIndex > BlockFrameIndex)
			{
				continue;
			}
			if (AsClockEvent.BlockFrameIndex < BlockFrameIndex)
			{
				return nullptr;
			}
			if (MSGTYPE* AsDesiredMsgType = AsClockEvent.TryGet<MSGTYPE>())
			{
				return AsDesiredMsgType;
			}
		}
		return nullptr;
	}
}

// Declare reference types OUT of the namespace...
DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(HarmonixMetasound::FMidiClock, HARMONIXMETASOUND_API)

#undef UE_API
