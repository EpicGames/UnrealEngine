// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

namespace HarmonixMetasound
{
	namespace MidiClockMessageTypes
	{
		struct FLoop
		{
			int32 FirstTickInLoop;
			int32 LengthInTicks;
			int32 TempoMapTick;

			FLoop(const int32 FirstTickInLoop, const int32 LengthInTicks, const int32 TempoMapTick)
				: FirstTickInLoop(FirstTickInLoop)
				, LengthInTicks(LengthInTicks)
				, TempoMapTick(TempoMapTick)
			{}
		};

		struct FSeek
		{
			int32 LastTickProcessedBeforeSeek;
			int32 NewNextTick;
			int32 TempoMapTick;

			FSeek(const int32 LastTickProcessedBeforeSeek, const int32 NewNextTick, const int32 TempoMapTick)
				: LastTickProcessedBeforeSeek(LastTickProcessedBeforeSeek)
				, NewNextTick(NewNextTick)
				, TempoMapTick(TempoMapTick)
			{}
		};

		struct FAdvance
		{
			int32 FirstTickToProcess;
			int32 NumberOfTicksToProcess;
			int32 TempoMapTick;

			FAdvance(const int32 FirstTickToProcess, const int32 NumberOfTicksToProcess, const int32 TempoMapTick)
				: FirstTickToProcess(FirstTickToProcess)
				, NumberOfTicksToProcess(NumberOfTicksToProcess)
				, TempoMapTick(TempoMapTick)
			{}

			int32 LastTickToProcess() const { return FirstTickToProcess + NumberOfTicksToProcess - 1; }

			bool ContainsTick(int32 InTick) const { return InTick >= FirstTickToProcess && InTick < (FirstTickToProcess + NumberOfTicksToProcess); }
		};

		struct FTempoChange
		{
			int32 Tick;
			float Tempo;
			int32 TempoMapTick;

			FTempoChange(const int32 Tick, const float Tempo, const int32 TempoMapTick)
				: Tick(Tick)
				, Tempo(Tempo)
				, TempoMapTick(TempoMapTick)
			{}

			bool ContainsTick(int32 InTick) const { return InTick == Tick; }
		};

		struct FTimeSignatureChange
		{
			int32 Tick;
			FTimeSignature TimeSignature;
			int32 TempoMapTick;

			FTimeSignatureChange(const int32 Tick, FTimeSignature&& TimeSignature, const int32 TempoMapTick)
				: Tick(Tick)
				, TimeSignature(MoveTemp(TimeSignature))
				, TempoMapTick(TempoMapTick)
			{}

			bool ContainsTick(int32 InTick) const { return InTick == Tick; }
		};

		struct FTransportChange
		{
			EMusicPlayerTransportState TransportState;

			FTransportChange(EMusicPlayerTransportState NewTransportState)
				: TransportState(NewTransportState)
			{}
		};

		struct FSpeedChange
		{
			float Speed;

			FSpeedChange(float NewSpeed)
				: Speed(NewSpeed)
			{}
		};
	}

	using FMidiClockMsg = TVariant<
		MidiClockMessageTypes::FLoop,
		MidiClockMessageTypes::FSeek,
		MidiClockMessageTypes::FAdvance,
		MidiClockMessageTypes::FTempoChange,
		MidiClockMessageTypes::FTimeSignatureChange,
		MidiClockMessageTypes::FTransportChange,
		MidiClockMessageTypes::FSpeedChange
	>;
	
	struct FMidiClockEvent
	{
		const int32 BlockFrameIndex;
		FMidiClockMsg Msg;

		template<typename T>
		FMidiClockEvent(const int32 InBlockFrameIndex, T&& Msg)
			: BlockFrameIndex(InBlockFrameIndex)
			, Msg(TInPlaceType<T>(), MoveTemp(Msg))
		{
		}

		template<typename T>
		bool IsType() const	{ return Msg.IsType<T>(); }

		template<typename T>
		const T* TryGet() const { return Msg.TryGet<T>(); }

		template<typename T>
		T* TryGet() { return Msg.TryGet<T>(); }
	};
};
