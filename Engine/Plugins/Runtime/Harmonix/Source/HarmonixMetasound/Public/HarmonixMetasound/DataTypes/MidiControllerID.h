// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MidiControllerID.generated.h"

/**
 * standard Midi Controller ID according to the MIDI Association 
 * reference: https://www.midi.org/specifications-old/item/table-3-control-change-messages-data-bytes-2
 */

UENUM()
enum class EStdMidiControllerID: uint8
{
	BankSelection = 0,
	ModWheel = 1,
	Breath = 2,
	
	Undefined3,
	
	FootController = 4,
	PortamentoTime = 5,
	DataCoarse = 6,
	Volume = 7,
	Balance = 8,
	Undefined9 = 9,
	Pan = 10,
	Expression = 11,
	EffectControl1 = 12,
	EffectControl2 = 13,
	
	Undefined14,
	Undefined15,
	
	GeneralPurposeController1 = 16,
	GeneralPurposeController2 = 17,
	GeneralPurposeController3 = 18,
	GeneralPurposeController4 = 19,
	
	Undefined20,
	Undefined21,
	Undefined22,
	Undefined23,
	Undefined24,
	Undefined25,
	Undefined26,
	Undefined27,
	Undefined28,
	Undefined29,
	Undefined30,
	Undefined31,

	//LSB of controller IDs 0-31 (0-31 are MSB)
	LSBControl0 = 32,
	LSBControl1 = 33,
	LSBControl2 = 34,
	LSBControl3 = 35,
	LSBControl4 = 36,
	LSBControl5 = 37,
	LSBControl6 = 38,
	LSBControl7 = 39,
	LSBControl8 = 40,
	LSBControl9 = 41,
	LSBControl10 = 42,
	LSBControl11 = 43,
	LSBControl12 = 44,
	LSBControl13 = 45,
	LSBControl14 = 46,
	LSBControl15 = 47,
	LSBControl16 = 48,
	LSBControl17 = 49,
	LSBControl18 = 50,
	LSBControl19 = 51,
	LSBControl20 = 52,
	LSBControl21 = 53,
	LSBControl22 = 54,
	LSBControl23 = 55,
	LSBControl24 = 56,
	LSBControl25 = 57,
	LSBControl26 = 58,
	LSBControl27 = 59,
	LSBControl28 = 60,
	LSBControl29 = 61,
	LSBControl30 = 62,
	LSBControl31 = 63,

	Hold = 64,
	PortamentoSwitch = 65,
	Sustenuto = 66,
	SoftPedal = 67,
	Legato = 68,
	Hold2 = 69,
	
	SoundController1 = 70,
	SoundController2 = 71,
	SoundController3 = 72,
	SoundController4 = 73,
	SoundController5 = 74,
	SoundController6 = 75,
	SoundController7 = 76,
	SoundController8 = 77,
	SoundController9 = 78,
	SoundController10 = 79,

	GeneralPurposeController5 = 80,
	GeneralPurposeController6 = 81,
	GeneralPurposeController7 = 82,
	GeneralPurposeController8 = 83,

	PortamentoControl = 84,

	Undefined85,
	Undefined86,
	Undefined87,

	HighResolutionVelocityPrefix = 88,

	Undefined89,
	Undefined90,

	Effects1Depth = 91,
	Effects2Depth = 92,
	Effects3Depth = 93,
	Effects4Depth = 94,
	Effects5Depth = 95,
	DataIncrement = 96,
	DataDecrement = 97,
	NRPNFine = 98,
	NRPNCoarse = 99,
	RPNFine = 100,
	RPNCoarse = 101,

	Undefined102,
	Undefined103,
	Undefined104,
	Undefined105,
	Undefined106,
	Undefined107,
	Undefined108,
	Undefined109,
	Undefined110,
	Undefined111,
	Undefined112,
	Undefined113,
	Undefined114,
	Undefined115,
	Undefined116,
	Undefined117,
	Undefined118,
	Undefined119,

	AllSoundOff = 120,
	Reset = 121,
	LocalKeyboardSwitch = 122,
	AllNotesOff = 123,
	OmniModeOff = 124,
	OmniModeOn = 125,
	MonoMode = 126,
	PolyMode = 127,

	NUM UMETA(Hidden)
	
};

namespace Metasound
{
	DECLARE_METASOUND_ENUM(
		EStdMidiControllerID,
		EStdMidiControllerID::BankSelection,
		HARMONIXMETASOUND_API,
		FEnumStdMidiControllerID,
		FEnumStdMidiControllerIDTypeInfo,
		FEnumStdMidiControllerIDReadRef,
		FEnumStdMidiControllerIDWriteRef
	);
}
