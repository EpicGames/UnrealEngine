// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReferenceMacro.h"
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/MidiMsg.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "MidiEventInfo.generated.h"

#define UE_API HARMONIXMETASOUND_API

USTRUCT(BlueprintType, Meta = (DisplayName = "MIDI Event Info"))
struct FMidiEventInfo
{
	GENERATED_BODY()

	UPROPERTY(Category = "Music", EditAnywhere, BlueprintReadWrite)
	FMusicTimestamp Timestamp{ 0, 0 };

	UPROPERTY(Category = "Music", EditAnywhere, BlueprintReadWrite)
	int32 TrackIndex{ 0 };

	FMidiMsg MidiMessage{ 0, 0, 0 };

	UE_API uint8 GetChannel() const;
	
	UE_API bool IsNote() const;
	UE_API bool IsNoteOn() const;
	UE_API bool IsNoteOff() const;
	UE_API uint8 GetNoteNumber() const;
	UE_API uint8 GetVelocity() const;

	friend FORCEINLINE uint32 GetTypeHash(const FMidiEventInfo& InMidiEventInfo)
	{
		return HashCombineFast(
			HashCombineFast(GetTypeHash(InMidiEventInfo.Timestamp), GetTypeHash(InMidiEventInfo.TrackIndex)),
			GetTypeHash(InMidiEventInfo.MidiMessage));
	}
};

UCLASS()
class UMidiEventInfoBlueprintLibrary final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsMidiEventInfo(const FMetaSoundOutput& Output);
	
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static FMidiEventInfo GetMidiEventInfo(const FMetaSoundOutput& Output, bool& Success);

	UFUNCTION(BlueprintCallable, Category = "Music")
	static int32 GetChannel(const FMidiEventInfo& Event);
	
	UFUNCTION(BlueprintCallable, Category = "Music")
	static bool IsNote(const FMidiEventInfo& Event);

	UFUNCTION(BlueprintCallable, Category = "Music")
	static bool IsNoteOn(const FMidiEventInfo& Event);

	UFUNCTION(BlueprintCallable, Category = "Music")
	static bool IsNoteOff(const FMidiEventInfo& Event);

	UFUNCTION(BlueprintCallable, Category = "Music")
	static int32 GetNoteNumber(const FMidiEventInfo& Event);

	UFUNCTION(BlueprintCallable, Category = "Music")
	static int32 GetVelocity(const FMidiEventInfo& Event);
};

DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMidiEventInfo, HARMONIXMETASOUND_API, FMidiEventInfoTypeInfo, FMidiEventInfoReadRef, FMidiEventInfoWriteRef);

#undef UE_API
