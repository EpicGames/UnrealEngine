// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioProxyInitializer.h"
#include "Harmonix/AudioRenderableProxy.h"
#include "AudioRenderableAsset.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "HarmonixMidi/MidiConstants.h"

#include "MidiStepSequence.generated.h"

#define UE_API HARMONIXMETASOUND_API

// First, the USTRUCT(s) that make up the user facing data 
// for this asset...

USTRUCT(BlueprintType)
struct FStepSequenceCell
{
	GENERATED_BODY()

public:
	FStepSequenceCell() = default;
	FStepSequenceCell(const FStepSequenceCell&) = default;
	FStepSequenceCell& operator=(const FStepSequenceCell&) = default;
	FStepSequenceCell(FStepSequenceCell&&) = default;
	FStepSequenceCell(bool bEnabled, bool bContinuation)
		: bEnabled(bEnabled)
		, bContinuation(bContinuation)
	{ }

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, editfixedsize, Category = "MidiStepSequence", Meta = (PostEditType = "Trivial"))
	bool bEnabled = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, editfixedsize, Category = "MidiStepSequence", Meta = (PostEditType = "Trivial"))
	bool bContinuation = false;
};

USTRUCT(BlueprintType)
struct FStepSequenceRow
{
	GENERATED_BODY()

public:
	FStepSequenceRow() = default;
	FStepSequenceRow(const FStepSequenceRow&) = default;
	FStepSequenceRow& operator=(const FStepSequenceRow&) = default;
	FStepSequenceRow(FStepSequenceRow&&) = default;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, editfixedsize, Category = "MidiStepSequence")
	TArray<FStepSequenceCell> Cells;

	// If the row is disabled, no notes in that row will play
	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, Category = "MidiStepSequence")
	bool bRowEnabled = true;
};

USTRUCT(BlueprintType)
struct FStepSequenceNote
{
	GENERATED_BODY()

public:
	FStepSequenceNote() = default;
	FStepSequenceNote(const FStepSequenceNote&) = default;
	FStepSequenceNote& operator=(const FStepSequenceNote&) = default;
	FStepSequenceNote(FStepSequenceNote&&) = default;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, meta = (PostEditType = "Trivial", ClampMin = "0", ClampMax = "127"), Category = "MidiStepSequence")
	int32 NoteNumber = 60;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, meta = (PostEditType = "Trivial", ClampMin = "0", ClampMax = "127"), Category = "MidiStepSequence")
	int32 Velocity = 120;

	friend bool operator==(const FStepSequenceNote& A, const FStepSequenceNote& B)
	{
		return A.NoteNumber == B.NoteNumber && A.Velocity == B.Velocity;
	}

	//if additional fields are added to this struct, we need to handle versioning in NetSerialize
	HARMONIXMETASOUND_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits< FStepSequenceNote > : public TStructOpsTypeTraitsBase2< FStepSequenceNote >
{
	enum
	{
		WithNetSerializer = true,
	};
};

USTRUCT(BlueprintType)
struct FStepSequencePage
{
	GENERATED_BODY()

public:
	FStepSequencePage()
		: Rows()
	{}

	FStepSequencePage(const FStepSequencePage& Other)
		: Rows(Other.Rows)
	{}
	FStepSequencePage& operator=(const FStepSequencePage& Other)
	{
		Rows = Other.Rows;
		return *this;
	}
	FStepSequencePage(FStepSequencePage&& Other) noexcept
		: Rows(MoveTemp(Other.Rows))
	{}
	FStepSequencePage& operator=(FStepSequencePage&& Other)
	{
		Rows = MoveTemp(Other.Rows);
		return *this;
	}
	~FStepSequencePage() = default;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, editfixedsize, Category = "MidiStepSequence")
	TArray<FStepSequenceRow> Rows;
};

USTRUCT(BlueprintType)
struct FStepSequenceTable
{
	GENERATED_BODY()

public:
	// This struct is going to be the "root" of the data we want to share between
	// the asset's UObject instance and one or more Metasound nodes running on
	// the audio thread. For that reason, we include this macro so other
	// templates and macros work.
	IMPL_AUDIORENDERABLE_PROXYABLE(FStepSequenceTable)

	FStepSequenceTable()
		: Pages()
		, Notes()
	{}
	FStepSequenceTable(const FStepSequenceTable& Other)
		: Pages(Other.Pages)
		, Notes(Other.Notes)
		, StepSkipIndex(Other.StepSkipIndex)
	{}
	FStepSequenceTable& operator=(const FStepSequenceTable& Other)
	{
		Pages = Other.Pages;
		Notes = Other.Notes;
		StepSkipIndex = Other.StepSkipIndex;
		return *this;
	}
	FStepSequenceTable(FStepSequenceTable&& Other) noexcept
		: Pages(MoveTemp(Other.Pages))
		, Notes(MoveTemp(Other.Notes))
		, StepSkipIndex(Other.StepSkipIndex)
	{}
	FStepSequenceTable& operator=(FStepSequenceTable&& Other)
	{
		Pages = MoveTemp(Other.Pages);
		Notes = MoveTemp(Other.Notes);
		StepSkipIndex = Other.StepSkipIndex;
		return *this;
	}
	~FStepSequenceTable() = default;

	HARMONIXMETASOUND_API bool IsPageBlank(int32 PageIdx) const;

	HARMONIXMETASOUND_API int32 GetFirstValidPage(bool bPlayBlankPages) const;
	HARMONIXMETASOUND_API int32 CalculateNumValidPages(bool bPlayBlankPages) const;

	HARMONIXMETASOUND_API int32 CalculateAutoPageIndex(int32 TotalPagesProgressed, bool bPlayBlankPages, bool bLoop) const;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, editfixedsize, Category = "MidiStepSequence")
	TArray<FStepSequencePage> Pages;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, editfixedsize, Category = "MidiStepSequence")
	TArray<FStepSequenceNote> Notes;

	// If >0, any step at index X will be skipped if X % N == N - 1
	UPROPERTY(EditAnywhere, BlueprintReadOnly, SaveGame, editfixedsize, Category = "MidiStepSequence")
	int32 StepSkipIndex = 0;

private:
	void GetUnusedPages(bool bPlayBlankPages, TBitArray<FDefaultBitArrayAllocator>& OutBlankPages) const;
};

USING_AUDIORENDERABLE_PROXY(FStepSequenceTable, FStepSequenceTableProxy)

//using FStepSequenceTableProxy = Harmonix::TAudioRenderableProxy<FStepSequenceTable, Harmonix::TGameThreadToAudioRenderThreadSettingQueue<FStepSequenceTable>>;

// This next macro does a few things. 
// - It says, "I want a Metasound exposed asset called 'FMidiStepSequenceAsset' with 
//   corresponding TypeInfo, ReadRef, and WriteRef classes."
// - That asset is a wrapper around a proxy class that acts as the go-between from the 
//   UObject (GC'able) side to  the audio render thread side. So here I tell the macro to 
//   wrap "FStepSequenceTable" in a proxy named "FStepSequenceTableProxy" and use that 
//   as the "guts" of the FMidiStepSequenceAsset asset.
//NOTE: This macro has a corresponding "DEFINE_AUDIORENDERABLE_ASSET" that must be added to the cpp file. 
DECLARE_AUDIORENDERABLE_ASSET(HarmonixMetasound, FMidiStepSequenceAsset, FStepSequenceTableProxy, HARMONIXMETASOUND_API)

// Now I can define the UCLASS that is the UObject side of the asset. Notice it is an 
// IAudioProxyDataFactory. In the code we will see that the CreateNewProxyData override
// returns an instance of a proxy class defined with the macro above.

// This class represents a step sequence table. It is used by the MetasSound Step Sequence node
// to generate midi note on/off messages.
UCLASS(MinimalAPI, BlueprintType, Category = "Music", Meta = (DisplayName = "MIDI Step Sequence"))
class UMidiStepSequence : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:

	static const int32 kMinPages = 1;
	static const int32 kMaxPages = 8;
	static const int32 kMinRows = 1;
	static const int32 kMaxRows = 16;
	static const int32 kMinColumns = 4;
	static const int32 kMaxColumns = 64;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetNumPages, meta = (ClampMin = "1", ClampMax = "8"), Category = "MidiStepSequence")
	int32 Pages = 4;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetNumRows, meta = (ClampMin = "1", ClampMax = "16"), Category = "MidiStepSequence")
	int32 Rows = 8;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetNumColumns, meta = (ClampMin = "4", ClampMax = "64"), Category = "MidiStepSequence")
	int32 Columns = 8;

	UE_API UMidiStepSequence(const FObjectInitializer& Initializer);

	UE_API virtual void PostReinitProperties() override;

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API void SetNumPages(int32 Count);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API void SetNumColumns(int32 Count);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API void SetNumRows(int32 Count);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API void DisableRowsAbove(int32 FirstDisabledRow);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API bool SetRowNoteNumber(int32 RowIndex, int32 MidiNoteNumber);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API bool SetRowVelocity(int32 RowIndex, int32 MidiVelocoty);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API bool SetStepSkipIndex(int32 StepIndex);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API FStepSequenceCell SetCell(int32 Row, int32 Column, bool State);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API FStepSequenceCell SetCellOnPage(int32 Page, int32 Row, int32 Column, bool State);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API FStepSequenceCell SetCellContinuation(int32 Row, int32 Column, bool bState);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API FStepSequenceCell SetCellContinuationOnPage(int32 Page, int32 Row, int32 Column, bool bState);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MidiStepSequence")
	UE_API FStepSequenceCell GetCell(int32 Row, int32 Column);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API FStepSequenceCell GetCellOnPage(int32 Page, int32 Row, int32 Column);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API FStepSequenceCell ToggleCell(int32 Row, int32 Column);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API FStepSequenceCell ToggleCellOnPage(int32 Page, int32 Row, int32 Column);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	UE_API void SetStepTable(const FStepSequenceTable& NewStepTable);

	UFUNCTION(BlueprintCallable, Category = "MidiStepSequence")
	const FStepSequenceTable& GetStepTable() const { return StepTable; }

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR

	UE_API TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Transient, DuplicateTransient,Category = "MidiStepSequence")
	int32 CurrentPageNumber = 1;
	UE_API const int32 GetClampedCurrentPageNumber();
	UE_API void SetClampedCurrentPageNumber(int32 InPageNumber);
#endif 

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStepSequence")
	FStepSequenceTable StepTable;
private:

	// Notice here that we cache a pointer the Proxy's "Queue" so we can...
	// 1 - Supply it to all instances of Metasound nodes rendering this data. How?
	//     CreateNewProxyData instantiates a NEW unique ptr to an FStepSequenceTableProxy
	//     every time it is called. All of those unique proxy instances refer to the same
	//     queue... this one that we have cached.
	// 2 - Modify that data in response to changes to this class's UPROPERTIES
	//     so that we can hear data changes reflected in the rendered audio.
	TSharedPtr<FStepSequenceTableProxy::QueueType> RenderableSequenceTable;

	UE_API void UpdateRenderableForNonTrivialChange();
};

#undef UE_API
