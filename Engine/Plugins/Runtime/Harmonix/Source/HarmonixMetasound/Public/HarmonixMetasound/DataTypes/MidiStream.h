// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVariable.h"
#include "MidiClock.h"
#include "HarmonixMidi/MidiFile.h"
#include "HarmonixMidi/MidiMsg.h"
#include "HarmonixMidi/MidiVoiceId.h"
#include "Logging/LogMacros.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound
{
	HARMONIXMETASOUND_API DECLARE_LOG_CATEGORY_EXTERN(LogMidiStreamDataType, Log, All)

	struct FMidiStreamEvent
	{
		int32        BlockSampleFrameIndex  = 0;
		int32        TrackIndex             = 0;
		int32        AuthoredMidiTick       = 0;
		int32        CurrentMidiTick        = 0;
		FMidiMsg     MidiMessage;

		UE_API FMidiStreamEvent(const FMidiVoiceGeneratorBase* Owner, const FMidiMsg& Message);

		UE_API FMidiStreamEvent(const uint32 OwnerId, const FMidiMsg& Message);

		FMidiVoiceId GetVoiceId() const { return VoiceId; }

	private:
		friend class FMidiStream;
		
		FMidiVoiceId VoiceId;
	};

	class FMidiStream
	{
	public:
		HARMONIXMETASOUND_API void SetClock(const FMidiClock& InClock);
		HARMONIXMETASOUND_API void ResetClock();
		HARMONIXMETASOUND_API TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> GetClock() const;

		HARMONIXMETASOUND_API void PrepareBlock();

		HARMONIXMETASOUND_API void AddMidiEvent(const FMidiStreamEvent& Event);
		HARMONIXMETASOUND_API void InsertMidiEvent(const FMidiStreamEvent& Event);
		HARMONIXMETASOUND_API void AddNoteOffEventOrCancelPendingNoteOn(const FMidiStreamEvent& Event);
		HARMONIXMETASOUND_API void InsertNoteOffEventOrCancelPendingNoteOn(const FMidiStreamEvent& Event);

		void SetMidiFile(const FMidiFileProxyPtr& MidiFile) { MidiFileSourceOfEvents = MidiFile; }
		HARMONIXMETASOUND_API const FString* GetMidiTrackText(int32 TrackNumber, int32 TextIndex) const;


		const TArray<FMidiStreamEvent>& GetEventsInBlock() const
		{
			return EventsInBlock;
		}

		int32 GetTicksPerQuarterNote() const { return TicksPerQuarterNote; }
		void SetTicksPerQuarterNote(int32 InTicksPerQuarterNote) { TicksPerQuarterNote = InTicksPerQuarterNote; }

		using FEventFilter = TFunction<bool(const FMidiStreamEvent&)>;
		inline static const FEventFilter NoOpFilter = [](const FMidiStreamEvent&) { return true; };
		
		using FEventTransformer = TFunction<FMidiStreamEvent(const FMidiStreamEvent&)>;
		inline static const FEventTransformer NoOpTransformer = [](const FMidiStreamEvent& Event) { return Event; };
		
		static HARMONIXMETASOUND_API void Copy(
			const FMidiStream& From,
			FMidiStream& To,
			const FEventFilter& Filter = NoOpFilter,
			const FEventTransformer& Transformer = NoOpTransformer);

		static HARMONIXMETASOUND_API void Merge(
			const FMidiStream& From,
			FMidiStream& To,
			const FEventFilter& Filter = NoOpFilter,
			const FEventTransformer& Transformer = NoOpTransformer);
		
		static HARMONIXMETASOUND_API void Merge(
			const FMidiStream& FromA,
			const FMidiStream& FromB,
			FMidiStream& To,
			const FEventFilter& Filter = NoOpFilter,
			const FEventTransformer& Transformer = NoOpTransformer);

		HARMONIXMETASOUND_API bool NoteIsActive(const FMidiStreamEvent& Event) const;

	private:
		FMidiFileProxyPtr MidiFileSourceOfEvents;
		int32 TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt;

		TArray<FMidiStreamEvent> EventsInBlock;

		TWeakPtr<const FMidiClock, ESPMode::NotThreadSafe> Clock;

		// Map to handle re-mapping merged MIDI events, which helps to disambiguate split/transposed notes
		TMap<uint32, FMidiVoiceGeneratorBase> GeneratorMap;

		void TrackNote(const FMidiStreamEvent& Event);
		TArray<FMidiStreamEvent> ActiveNotes;
	};

	// Declare aliases IN the namespace...
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FMidiStream, FMidiStreamTypeInfo, FMidiStreamReadRef, FMidiStreamWriteRef)
}

// Declare reference types OUT of the namespace...
DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(HarmonixMetasound::FMidiStream, HARMONIXMETASOUND_API)

#undef UE_API
