// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioProxyInitializer.h"
#include "Harmonix/AudioRenderableProxy.h"
#include "AudioRenderableAsset.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "HarmonixMidi/MusicTimeSpan.h"

#include "MidiStutterSequence.generated.h"

#define UE_API HARMONIXMETASOUND_API

USTRUCT(BlueprintType)
struct FStutterSequenceEntry
{
	GENERATED_BODY()

public:
	FStutterSequenceEntry() = default;
	FStutterSequenceEntry(const FStutterSequenceEntry&) = default;
	FStutterSequenceEntry& operator=(const FStutterSequenceEntry&) = default;
	FStutterSequenceEntry(FStutterSequenceEntry&&) = default;
	FStutterSequenceEntry(EMusicTimeSpanLengthUnits Spacing, EMusicTimeSpanLengthUnits AudibleDuration,
		bool Reverse, int32 Count)
		: Spacing(Spacing)
		, AudibleDuration(AudibleDuration)
		, Reverse(Reverse)
		, Count(Count)
	{ }

	/** Space between stutters */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStutterSequence", Meta = (PostEditType = "Trivial"))
	EMusicTimeSpanLengthUnits Spacing = EMusicTimeSpanLengthUnits::QuarterNotes;

	/** Duration of the audible portion of the stutter */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStutterSequence", Meta = (PostEditType = "Trivial"))
	EMusicTimeSpanLengthUnits AudibleDuration = EMusicTimeSpanLengthUnits::EighthNotes;

	/** Play the audio backwards? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStutterSequence", Meta = (PostEditType = "Trivial"))
	bool Reverse = false;

	/** How many stutters to play with these settings */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStutterSequence", Meta = (PostEditType = "Trivial", ClampMin = "1", ClampMax = "64"))
	int32 Count = 1;
};

USTRUCT(BlueprintType)
struct FStutterSequenceTable
{
	GENERATED_BODY()

public:
	// This struct is going to be the "root" of the data we want to share between
	// the asset's UObject instance and one or more Metasound nodes running on
	// the audio thread. For that reason, we include this macro so other
	// templates and macros work.
	IMPL_AUDIORENDERABLE_PROXYABLE(FStutterSequenceTable)

	FStutterSequenceTable()
		: Stutters()
	{}
	FStutterSequenceTable(const FStutterSequenceTable& Other)
		: CaptureDuration(Other.CaptureDuration)
		, ResetOnCompletion(Other.ResetOnCompletion)
		, Stutters(Other.Stutters)
	{}
	FStutterSequenceTable& operator=(const FStutterSequenceTable& Other)
	{
		CaptureDuration = Other.CaptureDuration;
		ResetOnCompletion = Other.ResetOnCompletion;
		Stutters = Other.Stutters;
		return *this;
	}
	FStutterSequenceTable(FStutterSequenceTable&& Other) noexcept
		: CaptureDuration(Other.CaptureDuration)
		, ResetOnCompletion(Other.ResetOnCompletion)
		, Stutters(MoveTemp(Other.Stutters))
	{}

	FStutterSequenceTable& operator=(FStutterSequenceTable&& Other)
	{
		CaptureDuration = Other.CaptureDuration;
		ResetOnCompletion = Other.ResetOnCompletion;
		Stutters = MoveTemp(Other.Stutters);
		return *this;
	}
	~FStutterSequenceTable() = default;

	/** The amount of audio to tell the sampler node to capture */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStutterSequence", Meta = (PostEditType = "Trivial"))
	EMusicTimeSpanLengthUnits CaptureDuration = EMusicTimeSpanLengthUnits::HalfNotes;

	/** Reset the Sampler node after the last stutter plays? This will cause the sampler node to start passing its input to its output again. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStutterSequence", Meta = (PostEditType = "Trivial"))
	bool ResetOnCompletion = true;

	/** A sequence of stutter 'sections'... */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStutterSequence")
	TArray<FStutterSequenceEntry> Stutters;
};

USING_AUDIORENDERABLE_PROXY(FStutterSequenceTable, FStutterSequenceTableProxy)

// This next macro does a few things. 
// - It says, "I want a Metasound exposed asset called 'FStutterSequenceAsset' with 
//   corresponding TypeInfo, ReadRef, and WriteRef classes."
// - That asset is a wrapper around a proxy class that acts as the go-between from the 
//   UObject (GC'able) side to  the audio render thread side. So here I tell the macro to 
//   wrap "FStutterSequenceTable" in a proxy named "FStutterSequenceTableProxy" and use that 
//   as the "guts" of the FMidiStepSequenceAsset asset.
//NOTE: This macro has a corresponding "DEFINE_AUDIORENDERABLE_ASSET" that must be added to the cpp file. 
DECLARE_AUDIORENDERABLE_ASSET(HarmonixMetasound, FStutterSequenceAsset, FStutterSequenceTableProxy, HARMONIXMETASOUND_API)

// Now I can define the UCLASS that is the UObject side of the asset. Notice it is an 
// IAudioProxyDataFactory. In the code we will see that the CreateNewProxyData override
// returns an instance of a proxy class defined with the macro above.

// This class represents a stutter sequence. It is used by the MetasSound Stutter Sequencer node
// to generate triggers for a Simple Sampler node to create a classic Stutter Edit effect.
UCLASS(MinimalAPI, BlueprintType, Category = "Music", Meta = (DisplayName = "MIDI Stutter Sequence"))
class UMidiStutterSequence : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR

	UE_API TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;


protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MidiStepSequence")
	FStutterSequenceTable StutterTable;

private:

	// Notice here that we cache a pointer the Proxy's "Queue" so we can...
	// 1 - Supply it to all instances of Metasound nodes rendering this data. How?
	//     CreateNewProxyData instantiates a NEW unique ptr to an FStepSequenceTableProxy
	//     every time it is called. All of those unique proxy instances refer to the same
	//     queue... this one that we have cached.
	// 2 - Modify that data in response to changes to this class's UPROPERTIES
	//     so that we can hear data changes reflected in the rendered audio.
	TSharedPtr<FStutterSequenceTableProxy::QueueType> RenderableStutterTable;

	UE_API void UpdateRenderableForNonTrivialChange();
};

#undef UE_API
