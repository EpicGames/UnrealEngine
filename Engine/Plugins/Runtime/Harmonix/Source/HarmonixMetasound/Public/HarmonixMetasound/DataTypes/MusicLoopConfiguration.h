// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "HarmonixMidi/BarMap.h"

#include "MusicLoopConfiguration.generated.h"

class UMetasoundParameterPack;

USTRUCT(BlueprintType)
struct FMusicLoopConfiguration
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = "MusicLoopConfiguration")
	bool Enabled = false;

	UPROPERTY(BlueprintReadWrite, Category = "MusicLoopConfiguration")
	FMusicTimestamp RegionStart;

	UPROPERTY(BlueprintReadWrite, Category = "MusicLoopConfiguration")
	FMusicTimestamp RegionEnd;

	FMusicLoopConfiguration() = default;
	FMusicLoopConfiguration(const Metasound::FOperatorSettings& InSettings, bool InEnabled = false, const FMusicTimestamp& InRegionStart = FMusicTimestamp(), const FMusicTimestamp& InRegionEnd = FMusicTimestamp())
		: FMusicLoopConfiguration(InEnabled, InRegionStart, InRegionEnd)
	{}
	FMusicLoopConfiguration(bool InEnabled, const FMusicTimestamp& InRegionStart, const FMusicTimestamp& InRegionEnd)
		: Enabled(InEnabled)
		, RegionStart(InRegionStart)
		, RegionEnd(InRegionEnd)
	{}
	
	friend FORCEINLINE uint32 GetTypeHash(const FMusicLoopConfiguration& InConfiguration)
	{
		return HashCombineFast(HashCombineFast(GetTypeHash(InConfiguration.RegionStart), GetTypeHash(InConfiguration.RegionEnd)), InConfiguration.Enabled);
	}
};

// NOTE: Since there is no corresponding cpp file, the corresponding REGISTER_METASOUND_DATATYPE is in MidiClock.cpp
DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMusicLoopConfiguration, HARMONIXMETASOUND_API, FMusicLoopConfigurationTypeInfo, FMusicLoopConfigurationReadRef, FMusicLoopConfigurationWriteRef)

