// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "MusicLoopConfiguration.h"
#include "MusicSeekRequest.h"
#include "HarmonixMidi/BarMap.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "MusicParameterBlueprintLibrary.generated.h"

class UMetasoundParameterPack;

UCLASS(MinimalAPI)
class UMusicParameterBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	//*************************************************************************************
	//** FMusicTimestamp

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	static void SetMusicTimestamp(UMetasoundParameterPack* Target, FName ParameterName, UPARAM(ref) FMusicTimestamp& Value, ESetParamResult& Result, bool OnlyIfExists = true);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	FMusicTimestamp GetMusicTimestamp(UMetasoundParameterPack* Target, FName ParameterName, ESetParamResult& Result);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	bool HasMusicTimestamp(UMetasoundParameterPack* Target, FName Name);

	//*************************************************************************************
	//** FMusicLoopConfiguration             

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	static void SetMusicLoopConfiguration(UMetasoundParameterPack* Target, FName ParameterName, UPARAM(ref) FMusicLoopConfiguration& Value, ESetParamResult& Result, bool OnlyIfExists = true);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	FMusicLoopConfiguration GetMusicLoopConfiguration(UMetasoundParameterPack* Target, FName ParameterName, ESetParamResult& Result);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	bool HasMusicLoopConfiguration(UMetasoundParameterPack* Target, FName Name);

	//*************************************************************************************
	//** FMusicSeekRequest 

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	static void SetMusicSeekRequest(UMetasoundParameterPack* Target, FName ParameterName, UPARAM(ref) FMusicSeekRequest& Value, ESetParamResult& Result, bool OnlyIfExists = true);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	FMusicSeekRequest GetMusicSeekRequest(UMetasoundParameterPack* Target, FName ParameterName, ESetParamResult& Result);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	bool HasMusicSeekRequest(UMetasoundParameterPack* Target, FName Name);
};
