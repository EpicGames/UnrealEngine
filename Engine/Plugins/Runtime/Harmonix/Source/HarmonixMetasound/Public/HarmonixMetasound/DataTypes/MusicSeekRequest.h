// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "HarmonixMidi/BarMap.h"

#include "MusicSeekRequest.generated.h"

UENUM(BlueprintType)
enum class ESeekPointType : uint8
{
	BarBeat,
	Millisecond
};

USTRUCT(BlueprintType)
struct FMusicSeekTarget
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = "MusicTransport")
	ESeekPointType Type = ESeekPointType::BarBeat;	

	UPROPERTY(BlueprintReadWrite, Category = "MusicTransport", meta = (EditCondition = "Type == ESeekPointType::BarBeat", EditConditionHides))
	FMusicTimestamp BarBeat;

	UPROPERTY(BlueprintReadWrite, Category = "MusicTransport", meta = (EditCondition = "Type == ESeekPointType::Millisecond", EditConditionHides))
	float Ms = 0.0f;
	
	friend FORCEINLINE uint32 GetTypeHash(const FMusicSeekTarget& InSeekTarget)
	{
		return HashCombineFast(HashCombineFast(GetTypeHash(InSeekTarget.Ms), GetTypeHash(InSeekTarget.BarBeat)), GetTypeHash(InSeekTarget.Type));
	}
};

USTRUCT(BlueprintType)
struct FMusicSeekRequest
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = "MusicTransport")
	bool EmmediateIfPastFromPoint = true;

	UPROPERTY(BlueprintReadWrite, Category = "MusicTransport")
	FMusicTimestamp FromPoint;

	UPROPERTY(BlueprintReadWrite, Category = "MusicTransport")
	FMusicTimestamp ToPoint;

	FMusicSeekRequest() = default;
	FMusicSeekRequest(const Metasound::FOperatorSettings& InSettings, bool InEmmediateIfPastFromPoint = true, const FMusicTimestamp& InFromPoint = FMusicTimestamp(), const FMusicTimestamp& InToPoint = FMusicTimestamp())
		: FMusicSeekRequest(InEmmediateIfPastFromPoint, InFromPoint, InToPoint)
	{}
	FMusicSeekRequest(bool InEmmediateIfPastFromPoint, const FMusicTimestamp& InFromPoint, const FMusicTimestamp& InToPoint)
		: EmmediateIfPastFromPoint(InEmmediateIfPastFromPoint)
		, FromPoint(InFromPoint)
		, ToPoint(InToPoint)
	{}

	void OneshotSubmitted()
	{
		EmmediateIfPastFromPoint = false;
		FromPoint  = ToPoint  = FMusicTimestamp();
	}
	
	friend FORCEINLINE uint32 GetTypeHash(const FMusicSeekRequest& InSeekRequest)
	{
		return HashCombineFast(HashCombineFast(GetTypeHash(InSeekRequest.FromPoint), GetTypeHash(InSeekRequest.ToPoint)), InSeekRequest.EmmediateIfPastFromPoint);
	}
};

// NOTE: Since there is no corresponding cpp file, the corresponding REGISTER_METASOUND_DATATYPE is in MidiClock.cpp
DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMusicSeekRequest, HARMONIXMETASOUND_API, FMusicSeekRequestTypeInfo, FMusicSeekRequestReadRef, FMusicSeekRequestWriteRef)

DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMusicSeekTarget, HARMONIXMETASOUND_API, FMusicSeekTargetTypeInfo, FMusicSeekTargetReadRef, FMusicSeekTargetWriteRef)
