// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MidiClock.h"
#include "MusicTimeInterval.generated.h"

#define UE_API HARMONIXMETASOUND_API

USTRUCT(MinimalAPI, BlueprintType)
struct FMusicTimeInterval
{
	GENERATED_BODY()

	UE_API FMusicTimeInterval();
	UE_API FMusicTimeInterval(EMidiClockSubdivisionQuantization InInterval, EMidiClockSubdivisionQuantization InOffset, int32 InIntervalMultiplier, int32 InOffsetMultiplier);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Time Interval Settings")
	EMidiClockSubdivisionQuantization Interval = EMidiClockSubdivisionQuantization::Beat;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Time Interval Settings")
	EMidiClockSubdivisionQuantization Offset = EMidiClockSubdivisionQuantization::Beat;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Time Interval Settings")
	int32 IntervalMultiplier = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Time Interval Settings")
	int32 OffsetMultiplier = 0;

	UE_API float GetIntervalBeats(const FTimeSignature& TimeSignature) const;
	UE_API float GetOffsetBeats(const FTimeSignature& TimeSignature) const;
};

namespace Harmonix
{
	HARMONIXMETASOUND_API void IncrementTimestampByInterval(
			FMusicTimestamp& Timestamp,
			const FMusicTimeInterval& Interval,
			const FTimeSignature& TimeSignature);

	HARMONIXMETASOUND_API void IncrementTimestampByOffset(
			FMusicTimestamp& Timestamp,
			const FMusicTimeInterval& Interval,
			const FTimeSignature& TimeSignature);

	HARMONIXMETASOUND_API void IncrementTimestampByBeats(
		FMusicTimestamp& Timestamp,
		float Beats,
		const FTimeSignature& TimeSignature);
}

#undef UE_API
