// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOutput.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MusicTimestamp.generated.h"

struct FMusicTimestamp;

UCLASS()
class UMusicTimestampBlueprintLibrary final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsMusicTimestamp(const FMetaSoundOutput& Output);
	
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static FMusicTimestamp GetMusicTimestamp(const FMetaSoundOutput& Output, bool& Success);
};

// NOTE: Since there is no corresponding cpp file, the corresponding REGISTER_METASOUND_DATATYPE is in MidiClock.cpp
DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMusicTimestamp, HARMONIXMETASOUND_API, FMusicTimestampTypeInfo, FMusicTimestampReadRef, FMusicTimestampWriteRef)
