// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundVariable.h"
#include "MusicSeekRequest.h"
#include "Templates/Function.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound
{
	enum class EMusicPlayerTransportRequest : uint8
	{
		None,
		Prepare,
		Play,
		Pause,
		Continue,
		Stop,
		Kill,
		Seek,
		Count
	};

	struct FMusicTransportEvent
	{
		Metasound::FTime Time;
		EMusicPlayerTransportRequest Request;
	};

	// Declare aliases IN the namespace...
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FMusicTransportEvent, FMusicTransportEventTypeInfo, FMusicTransportEventReadRef, FMusicTransportEventWriteRef)


	class FMusicTransportEventStream
	{
	public:
		UE_API FMusicTransportEventStream(const Metasound::FOperatorSettings& InSettings);

		UE_API void AddTransportRequest(EMusicPlayerTransportRequest InRequest, int32 AtSampleIndex);
		UE_API void AddSeekRequest(int32 AtSampleIndex, const FMusicSeekTarget& Target);

		/** Advance internal frame counters by block size. */
		UE_API void AdvanceBlock();
	
		UE_API void Reset();

		struct FRequestEvent
		{
			int32 SampleIndex;
			EMusicPlayerTransportRequest Request;
			bool operator<(const FRequestEvent& other) { return SampleIndex < other.SampleIndex; }
		};
		using FEventList = TArray<FRequestEvent>;

		const FEventList& GetTransportEventsInBlock() const { return TransportEventsThisBlock; }

		const FMusicSeekTarget& GetNextSeekDestination() const { return NextSeekDestination; }

		static constexpr EMusicPlayerTransportRequest InitialTransportStateRequest = EMusicPlayerTransportRequest::None;
		
		EMusicPlayerTransportRequest GetLastTransportStateRequest() const { return LastTransportStateRequest; }

	private:
		FEventList TransportEventsThisBlock;
		FMusicSeekTarget NextSeekDestination;
		// The last transport request that would result in a state change (so, *not* Seek)
		EMusicPlayerTransportRequest LastTransportStateRequest = InitialTransportStateRequest;
	};

	// Declare aliases IN the namespace...
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FMusicTransportEventStream, FMusicTransportEventStreamTypeInfo, FMusicTransportEventStreamReadRef, FMusicTransportEventStreamWriteRef)


	enum class EMusicPlayerTransportState : uint8
	{
		Invalid,	// --> Preparing
		Preparing,  // --> Prepared
		Prepared,	// --> Starting or Invalid 
		Starting,   // --> Playing or Paused
		Playing,	// --> Pausing or Stopping or Killing
		Seeking,    // --> Invalid, Preparing, Prepared, Playing, Paused
		Pausing,    // --> Paused
		Paused,		// --> Continuing or Stopping or Killing
		Continuing, // --> Playing
		Stopping,   // --> Preparing or Prepared or Invalid
		Killing,    // --> Preparing or Prepared or Invalid
		NumStates
	};

	FString HARMONIXMETASOUND_API MusicPlayerTransportStateToString(EMusicPlayerTransportState State);

	class FMusicTransportControllable
	{
	public:
		UE_API FMusicTransportControllable(EMusicPlayerTransportState InitialState);

		using FTransportInitFn = TUniqueFunction<EMusicPlayerTransportState(EMusicPlayerTransportState)>;
		UE_API void Init(const FMusicTransportEventStream& TransportEventStream, FTransportInitFn&& InitFn);

		EMusicPlayerTransportState GetTransportState() const { return TransportState; }
		void SetTransportState(EMusicPlayerTransportState NewState) { TransportState = NewState; }
		UE_API EMusicPlayerTransportState GetNextTransportState(EMusicPlayerTransportState DesiredState) const;
		
		using TransportSpanProcessor = TUniqueFunction<EMusicPlayerTransportState(int32,int32,EMusicPlayerTransportState)>;
		// post processor gets called immediately after the SpanProcessor so that the TransportState has been updated
		using TransportSpanPostProcessor = TUniqueFunction<void(int32, int32, EMusicPlayerTransportState)>;
		UE_API void ExecuteTransportSpans(FMusicTransportEventStreamReadRef& InTransportPin, int32 InBlockSize, TransportSpanProcessor& Callback);
		UE_API void ExecuteTransportSpans(FMusicTransportEventStreamReadRef& InTransportPin, int32 InBlockSize, TransportSpanProcessor& Callback, TransportSpanPostProcessor& PostProcessor);

		static UE_API FString StateToString(EMusicPlayerTransportState S);

	protected:
		UE_API EMusicPlayerTransportState GetDesiredState(EMusicPlayerTransportRequest Request) const;
		
		UE_API bool IsEffectivelyStopped() const;

		bool ReceivedSeekWhileStopped() const { return bReceivedSeekWhileStopped; }

	private:
		EMusicPlayerTransportState TransportState = EMusicPlayerTransportState::Invalid;
		bool bReceivedSeekWhileStopped = false;
	};
}

// Declare reference types OUT of the namespace...
DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(HarmonixMetasound::FMusicTransportEvent, HARMONIXMETASOUND_API)

// Declare reference types OUT of the namespace...
DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(HarmonixMetasound::FMusicTransportEventStream, HARMONIXMETASOUND_API)

#undef UE_API
