// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReferenceMacro.h"
#include "MetasoundOutput.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "TimeSignature.generated.h"

struct FTimeSignature;

UCLASS()
class UTimeSignatureBlueprintLibrary final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsTimeSignature(const FMetaSoundOutput& Output);
	
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static FTimeSignature GetTimeSignature(const FMetaSoundOutput& Output, bool& Success);
};

DECLARE_METASOUND_DATA_REFERENCE_TYPES(
	FTimeSignature,
	HARMONIXMETASOUND_API,
	FTimeSignatureTypeInfo,
	FTimeSignatureReadRef,
	FTimeSignatureWriteRef);
