// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundEnumRegistrationMacro.h"
#include "HarmonixDsp/TimeSyncOption.h"

namespace Metasound
{
	DECLARE_METASOUND_ENUM(
		ETimeSyncOption,
		ETimeSyncOption::None,
		HARMONIXMETASOUND_API,
		FEnumTimeSyncOption,
		FEnumTimeSyncOptionInfo,
		FEnumTimeSyncOptionReadReference,
		FEnumTimeSyncOptionWriteReference);
}
