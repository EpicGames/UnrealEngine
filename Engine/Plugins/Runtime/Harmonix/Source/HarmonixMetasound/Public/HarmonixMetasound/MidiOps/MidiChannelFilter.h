// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MidiOps.h"
#include "StuckNoteGuard.h"

#include "HarmonixMetasound/DataTypes/MidiStream.h"

#define UE_API HARMONIXMETASOUND_API

namespace Harmonix::Midi::Ops
{
	/**
	 * Includes MIDI events from enabled channels only
	 */
	class FMidiChannelFilter
	{
	public:
		/**
		 * Pass the filtered events from the input stream to the output stream
		 * @param InStream The MIDI stream to filter
		 * @param OutStream The filtered MIDI stream
		 */
		UE_API void Process(const HarmonixMetasound::FMidiStream& InStream, HarmonixMetasound::FMidiStream& OutStream);

		/**
		 * Enable or disable a channel (or all channels)
		 * @param Channel The MIDI channel (1-16, 0 is all channels)
		 * @param Enabled Whether the channel(s) should be enabled
		 */
		UE_API void SetChannelEnabled(uint8 Channel, bool Enabled);

		/**
		 * Find out if a channel is enabled
		 * @param Channel The MIDI channel (1-16, 0 is all channels)
		 * @return true if the channel is (or all channels are) enabled
		 */
		UE_API bool GetChannelEnabled(uint8 Channel) const;

	private:
		// bitfield to store which channels are enabled
		uint16 EnabledChannels = AllChannelsOff;

		FStuckNoteGuard StuckNoteGuard;
	};
}

#undef UE_API
