// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StuckNoteGuard.h"

#include "HarmonixDsp/Parameters/Parameter.h"

#include "HarmonixMetasound/DataTypes/MidiStream.h"

#define UE_API HARMONIXMETASOUND_API

namespace Harmonix::Midi::Ops
{
	/**
	 * Filters MIDI note events based on note number and velocity
	 */
	class FMidiNoteFilter
	{
	public:
		/**
		 * Pass the filtered events from the input stream to the output stream
		 * @param InStream The MIDI stream to filter
		 * @param OutStream The filtered MIDI stream
		 */
		UE_API void Process(const HarmonixMetasound::FMidiStream& InStream, HarmonixMetasound::FMidiStream& OutStream);

		/**
		 * Includes non-note events from the input into the output
		 */
		Dsp::Parameters::FBoolParameter IncludeOtherEvents{ true };

		using FMidiDataParam = Dsp::Parameters::TParameter<uint8>;

		/**
		 * The minimum note number the note can have to be included (inclusive)
		 */
		FMidiDataParam MinNoteNumber{ 0, 127, 0 };

		/**
		 * The maximum note number the note can have to be included (inclusive)
		 */
		FMidiDataParam MaxNoteNumber{ 0, 127, 127 };

		/**
		 * The minimum velocity the note can have to be included (inclusive)
		 */
		FMidiDataParam MinVelocity{ 0, 127, 0 };

		/**
		 * The maximum velocity the note can have to be included (inclusive)
		 */
		FMidiDataParam MaxVelocity{ 0, 127, 127 };

	private:
		FStuckNoteGuard StuckNoteGuard;
	};
}

#undef UE_API
