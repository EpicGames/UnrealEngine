// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixMidiOps, Log, All)

namespace Harmonix::Midi::Ops
{
	using FMidiTrackIndex = uint16;
	
	constexpr uint16 AllChannelsOn = 0xffff;
	constexpr uint16 AllChannelsOff = 0;
	
	/**
	 * Get a bitmask from a MIDI channel number (e.g. for use when creating a bitfield for disabling/enabling channels)
	 * @param MidiChannel The MIDI channel (1-16, or 0 for all)
	 * @return The bitmask for the channel(s)
	 */
	uint16 MidiChannelToBitmask(uint8 MidiChannel);
}
