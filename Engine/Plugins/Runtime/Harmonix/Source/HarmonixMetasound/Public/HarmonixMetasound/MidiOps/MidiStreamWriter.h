// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMidi/MidiTrack.h"

#define UE_API HARMONIXMETASOUND_API
 
 namespace HarmonixMetasound
 {
 	class FMidiStream;
 	class FMidiClock;
 }

namespace Harmonix::Midi::Ops
{
	/**
	 * Write the midi events from a midi stream into a midi file
	 */
	class FMidiStreamWriter
	{
	public:
		
		UE_API FMidiStreamWriter(TUniquePtr<FArchive>&& Archive);

		/**
		 * Writes the midi events in the midi stream
		 * Midi events are appended to the existing midi events
		 * 
		 * @param InStream The MIDI stream to read from
		 */
		UE_API void Process(const HarmonixMetasound::FMidiStream& InStream);
		
		/**
		 * Writes the midi events in the midi stream
		 * Midi events are appended to the existing midi events
		 * 
		 * @param InStream The MIDI stream to read from
		 */
		UE_API void Process(const HarmonixMetasound::FMidiStream& InStream, int32 FirstTickToProcess, int32 LastTickToProcess);
		

	private:
		TUniquePtr<FArchive> Archive;
		TMap<int32, FMidiTrack> MidiTracks;
		int32 NextWriteTick = 0;
	
	};
};

#undef UE_API
