// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StuckNoteGuard.h"

#include "HarmonixMetasound/DataTypes/MidiStream.h"

#define UE_API HARMONIXMETASOUND_API

namespace Harmonix::Midi::Ops
{
	/**
	 * Filters MIDI events based on which track they're on
	 */
	class FMidiTrackFilter
	{
	public:
		/**
		 * Pass the filtered events from the input stream to the output stream
		 * @param InStream The MIDI stream to filter
		 * @param OutStream The filtered MIDI stream
		 */
		UE_API void Process(const HarmonixMetasound::FMidiStream& InStream, HarmonixMetasound::FMidiStream& OutStream);

		/**
		 * Set the range of tracks to include. Track 0 is reserved as the "conductor" track, which only contains timing information.
		 * The first track with notes and other messages on it will be track 1. 
		 * @param InMinTrackIdx The first track to include in the range of tracks (inclusive).
		 * @param InMaxTrackIdx The last track to include in th range of tracks (inclusive).
		 * @param InIncludeConductorTrack Whether or not to include the conductor track (track 0)
		 */
		UE_API void SetTrackRange(uint16 InMinTrackIdx, uint16 InMaxTrackIdx, bool InIncludeConductorTrack);

	private:
		uint16 MinTrackIdx = 0;
		uint16 MaxTrackIdx = 0;
		bool IncludeConductorTrack = false;
		
		FStuckNoteGuard StuckNoteGuard;
	};
}

#undef UE_API
