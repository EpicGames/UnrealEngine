// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixDsp/Parameters/Parameter.h"

#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTimeInterval.h"

#include "HarmonixMidi/MidiVoiceId.h"

#define UE_API HARMONIXMETASOUND_API

namespace Harmonix::Midi::Ops
{
	class FPulseGenerator
	{
	public:
		virtual ~FPulseGenerator() = default;
		
		UE_API void Enable(bool bEnable);

		/**
		 * Set the interval for each pulse in Musical Time.
		 */
		UE_API void SetInterval(const FMusicTimeInterval& NewInterval);
		FMusicTimeInterval GetInterval() const { return Interval; }

		/**
		 * Set the Duty Cycle of the pulse
		 *
		 * The Duty Cycle defines the duration of the pulse as a percentage of the Interval.
		 *
		 * for example:
		 * - Duty Cycle = 1.0: pulse will span the entire length of the interval
		 * - Duty Cycle = 0.5: pulse will span half the length of the interval
		 */
		UE_API void SetDutyCycle(float InDutyCycle);
		float GetDutyCycle() const { return DutyCycle; }

		UE_API virtual void Reset();
		
		struct FPulseInfo
		{
			int32 BlockFrameIndex;
			int32 Tick;
		};
		
		UE_API void Process(
			const HarmonixMetasound::FMidiClock& MidiClock,
			const TFunctionRef<void(const FPulseInfo&)>& TriggerPulseOn,
			const TFunctionRef<void(const FPulseInfo&)>& TriggerPulseOff);

	protected:
		bool Enabled{ true };

		// the interval of the pulse
		FMusicTimeInterval Interval{};

		// cache the time signature from the clock to calculate seconds from musical time
		FTimeSignature CurrentTimeSignature{};

		// pulse width as a percentage of the current interval
		float DutyCycle = 1.0f;

		struct FPulse
		{
			enum class EType 
			{
				Off = 0,
				On = 1,
			};
			
			FMusicTimestamp Timestamp;
			EType Type = EType::On;

			static FPulse On(const FMusicTimestamp& Timestamp)
			{
				return FPulse{ Timestamp, EType::On };
			}

			static FPulse Off(const FMusicTimestamp& Timestamp)
			{
				return FPulse{ Timestamp, EType::Off };
			}
		};

		TArray<FPulse> NextPulses;
	};
	
	class FMidiPulseGenerator : public FPulseGenerator
	{
	public:
		virtual ~FMidiPulseGenerator() override = default;
		
		Dsp::Parameters::TParameter<uint8> Channel{ 1, 16, 1 };

		Dsp::Parameters::TParameter<uint16> Track{ 1, UINT16_MAX, 1 };
		
		Dsp::Parameters::TParameter<uint8> NoteNumber{ 0, 127, 60 };
		
		Dsp::Parameters::TParameter<uint8> Velocity{ 0, 127, 127 };

		UE_API virtual void Reset() override;

		UE_API void Process(const HarmonixMetasound::FMidiClock& MidiClock, HarmonixMetasound::FMidiStream& OutStream);

	private:
		UE_API void PulseNoteOn(const int32 BlockFrameIndex, const int32 PulseTick, HarmonixMetasound::FMidiStream& OutStream);
		UE_API void PulseNoteOff(const int32 BlockFrameIndex, const int32 PulseTick, HarmonixMetasound::FMidiStream& OutStream);

		FMidiVoiceGeneratorBase VoiceGenerator{};
		TOptional<HarmonixMetasound::FMidiStreamEvent> LastNoteOn;
	};
}

#undef UE_API
