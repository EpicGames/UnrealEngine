// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMetasound/DataTypes/MidiStream.h"

#define UE_API HARMONIXMETASOUND_API

namespace Harmonix::Midi::Ops
{
	/**
	 * Tracks incoming note messages and sends note offs when a note would become "stuck" because of a filter change
	 */
	class FStuckNoteGuard
	{
	public:
		using FIncludeNotePredicate = TFunctionRef<bool(const HarmonixMetasound::FMidiStreamEvent& Event)>;

		/**
		 * Track incoming note messages and send note offs for notes that would have become "stuck" because the caller's filter changed.
		 * @param InStream The input MIDI stream
		 * @param OutStream The output MIDI stream
		 * @param Predicate The predicate which indicates whether an event is still included in the caller's filter
		 */
		UE_API void Process(const HarmonixMetasound::FMidiStream& InStream, HarmonixMetasound::FMidiStream& OutStream, const FIncludeNotePredicate& Predicate);

		using FUnstickNoteFn = TFunction<void(const HarmonixMetasound::FMidiStreamEvent&)>;

		/**
		 * If any active notes we're tracking are no longer present in the input stream, trigger a function so the caller can turn them off.
		 * @param StreamToCompare The stream to use to check whether an active note has disappeared
		 * @param UnstickNoteFn The function to call when an active note has disappeared from the incoming stream
		 */
		UE_API void UnstickNotes(const HarmonixMetasound::FMidiStream& StreamToCompare, const FUnstickNoteFn& UnstickNoteFn);

	private:
		void TrackNotes(const HarmonixMetasound::FMidiStream& InStream, const FIncludeNotePredicate& Predicate);
		
		TArray<HarmonixMetasound::FMidiStreamEvent> ActiveNotes;
	};
}

#undef UE_API
