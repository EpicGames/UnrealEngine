// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixMetasound/MusicClock/MusicClockState.h"
#include "HarmonixMetasound/MusicSource/MusicSource.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/SongMaps.h"

#include "MusicClock.generated.h"

#define UE_API HARMONIXMETASOUND_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMusicClockBeatEvent, int, BeatNumber, int, BeatInBar);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMusicClockBarEvent, int, BarNumber);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMusicClockSectionEvent, const FString&, SectionName, float, SectionStartMs, float, SectionLengthMs);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMusicClockPlayStateEvent, EMusicClockState, State);

/**
 * A read-only view into a music source's time.
 *
 * UMusicClock does not own or control playback. It observes an IMusicSource
 * and provides:
 *   - Current/previous musical position (FMidiSongPos)
 *   - Tempo, time signature, BPM queries
 *   - Beat/bar/section event broadcasting
 *   - Song map delegation for time conversions
 *
 * A single UMusicClock represents a single time stream with no
 * ECalibratedMusicTimebase parameter. For calibrated offsets (experienced,
 * video render, etc.) use UOffsetMusicSource feeding into a separate UMusicClock.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMusicClock : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMusicClock();
	UE_API virtual void BeginDestroy() override;

	/** Bind this clock to a music source. The clock becomes a read-only view into that source. */
	UFUNCTION(BlueprintCallable, Category = "MusicClock")
	UE_API void SetSource(const TScriptInterface<IMusicSource>& InSource);

	/** Get the source this clock is bound to. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API TScriptInterface<IMusicSource> GetSource() const;

	/**
	 * Update the clock for the current game frame.
	 * Called by the update subsystem. Pulls position from the source,
	 * computes deltas, and broadcasts events.
	 */
	UE_API virtual void UpdateForGameFrame();

	// ---- State ----

	/** Get the current clock state (Stopped, Paused, Running). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API virtual EMusicClockState GetState() const;

	// ---- Position ----

	/** Get the current musical position for this frame. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API const FMidiSongPos& GetCurrentSongPos() const;

	/** Get the previous frame's musical position. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API const FMidiSongPos& GetPreviousSongPos() const;

	/** Time from the beginning of the authored music content, including count-in/pickup bars. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetSecondsIncludingCountIn() const;

	/** Time from Bar 1 Beat 1 (the classic "start of the song"), excluding count-in/pickup. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetSecondsFromBarOne() const;

	/** Fractional total bars from the beginning, including count-in. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetBarsIncludingCountIn() const;

	/** Fractional total beats from the beginning, including count-in. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetBeatsIncludingCountIn() const;

	/**
	 * Get the current musical timestamp (Bar + Beat).
	 * Bar 1, Beat 1.0 is the beginning of the song after count-in.
	 */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API FMusicTimestamp GetCurrentTimestamp() const;

	// ---- Tempo & Time Signature ----

	/** Current tempo in MIDI quarter notes per minute. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentTempo() const;

	/** Current BPM in true beats (not quarter notes). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentBeatsPerMinute() const;

	/** Current beats per second, accounting for tempo, time signature, and speed. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentBeatsPerSecond() const;

	/** Current bars per second, accounting for tempo, time signature, and speed. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentBarsPerSecond() const;

	/** Current playback speed multiplier from the source. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentClockAdvanceRate() const;

	/** Get the current time signature as numerator and denominator. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API void GetCurrentTimeSignature(int& OutNumerator, int& OutDenominator) const;

	// ---- Sections ----

	/** Name of the current song section (from MIDI section markers). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API FString GetCurrentSectionName() const;

	/** Index of the current section in the section map. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API int32 GetCurrentSectionIndex() const;

	/** Start time of the current section in milliseconds. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentSectionStartMs() const;

	/** Length of the current section in milliseconds. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetCurrentSectionLengthMs() const;

	// ---- Beat/Bar Distance ----
	// All distance values are fractional: 0.0 = on the beat/bar, 0.5 = halfway to the next.

	/** Fractional distance past the most recent beat (0.0 = exactly on beat). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceFromCurrentBeat() const;

	/** Fractional distance until the next beat (0.0 = exactly on next beat). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToNextBeat() const;

	/** Fractional distance to the closest beat (past or future), always <= 0.5. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToClosestBeat() const;

	/** Fractional distance past the most recent bar (0.0 = exactly on bar). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceFromCurrentBar() const;

	/** Fractional distance until the next bar. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToNextBar() const;

	/** Fractional distance to the closest bar (past or future), always <= 0.5. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDistanceToClosestBar() const;

	// ---- Deltas ----

	/** Fractional bars advanced this frame. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDeltaBar() const;

	/** Fractional beats advanced this frame. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetDeltaBeat() const;

	// ---- Discontinuities ----

	/** Whether the source crossed a loop boundary this frame. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API bool LoopedThisFrame() const;

	/** Whether the source performed a non-contiguous time jump this frame. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API bool SeekedThisFrame() const;

	// ---- Loop Region ----

	/** Whether the source is looping. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API virtual bool IsLooping() const;

	/** Start of loop region in ms. Only meaningful if IsLooping(). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API virtual float GetLoopStartMs() const;

	/** Length of loop region in ms. Only meaningful if IsLooping(). */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API virtual float GetLoopLengthMs() const;

	// ---- Song Map Delegation ----

	/** Get the song maps from the bound source, for time<->beat conversions. */
	UE_API virtual const ISongMapEvaluator* GetSongMaps() const;

	// ---- Song Data ----

	/** Total length of the song in milliseconds. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetSongLengthMs() const;

	/** Remaining time until the end of the song in milliseconds. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API float GetSongRemainingMs() const;

	// ---- Spatial ----

	/** Get the world-space location of the audio source, if available. */
	UFUNCTION(BlueprintPure, Category = "MusicClock")
	UE_API TOptional<FVector> TryGetAudioSourceLocation() const;

	// ---- Events ----

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicClockBeatEvent BeatEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicClockBarEvent BarEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicClockSectionEvent SectionEvent;

	UPROPERTY(BlueprintAssignable, Category = "MusicClock")
	FMusicClockPlayStateEvent PlayStateEvent;

#if !UE_BUILD_SHIPPING
	UE_API FString GetDisplayName() const;
#endif

protected:

	/** Weak reference to the source UObject. Does not prevent GC. */
	TWeakObjectPtr<UObject> SourceObject;

	/** Cached interface pointer — valid only when SourceObject is valid. */
	IMusicSource* SourceInterfaceCache = nullptr;

	/** Get the source if still alive, or nullptr. */
	IMusicSource* GetValidSource() const;


	FMidiSongPos CurrentPos;
	FMidiSongPos PrevPos;

	float DeltaBarF = 0.f;
	float DeltaBeatF = 0.f;

	float Tempo = 0.f;
	int32 TimeSignatureNum = 0;
	int32 TimeSignatureDenom = 0;
	float CurrentBeatsPerSecondCached = 0.f;
	float CurrentBarsPerSecondCached = 0.f;
	float ClockAdvanceRate = 1.f;

	// Event broadcasting state
	int32 LastBroadcastBar = -1;
	int32 LastBroadcastBeat = -1;
	FSongSection LastBroadcastSection;

	uint64 LastUpdateFrame = 0;

	EMusicClockState CachedClockState = EMusicClockState::Stopped;

	bool bLoopedThisFrame = false;
	bool bSeekedThisFrame = false;

	/** Update cached tempo, time signature, and derived rates when the song position changes. */
	UE_API void UpdatePlaybackRate(const FMidiSongPos& SongPos, float InClockAdvanceRate);

	/** Broadcast beat, bar, and section events if thresholds were crossed this frame. */
	UE_API void BroadcastSongPosChanges();
};

#undef UE_API
