// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MusicClockState.generated.h"

UENUM(BlueprintType)
enum class EMusicClockState : uint8
{
	Stopped,
	Paused,
	Running,
};
