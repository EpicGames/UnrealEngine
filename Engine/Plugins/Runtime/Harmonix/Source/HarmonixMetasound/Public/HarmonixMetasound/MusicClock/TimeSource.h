// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreTypes.h"
#include "Math/Vector.h"

#define UE_API HARMONIXMETASOUND_API

namespace Harmonix
{
	enum class ESourceState : uint8
	{
		Stopped,
		Preparing, // Intent to run, but no data available yet (e.g., Metasound not yet connected)
		Paused,
		Running
	};

	enum class ESourceEvent : uint8
	{
		None,
		Start,
		Advance,
		Stop,
		Pause,
		Continue,
		Seek,
		Loop
	};

	/**
	 * Base class for Time Sources used by the Runtime Music Clock
	 */
	class ITimeSource
	{
	public:
		virtual ~ITimeSource() {}

		/**
		 * Determine the source event from a given state transition
		 *
		 * e.g. Stopped => Running, return ESourceEvent::Start
		 * 
		 * @param PrevState 
		 * @param NewState 
		 * @return The source event representing the state transition
		 */
		UE_API static ESourceEvent SourceEventFromStateTransition(ESourceState PrevState, ESourceState NewState);
		
		/** @return DisplayName of this time source for debugging */
		virtual FString GetDisplayName() const = 0;

		/** @return The location of the Audio Source for this Time Source */
		virtual TOptional<FVector> TryGetAudioSourceLocation() const = 0;

		/**
		 * Used to update the time source once per frame
		 *
		 * CurrentTime, CurrentSourceState, and LatestSourceEvent
		 * Should all be updated & valid for this frame when Update is called
		 */
		virtual void Update() = 0;

		/**
		 * Get the current tme of this time source
		 * Is updated every frame via Update
		 * 
		 * @return Current Time in Seconds
		 */
		virtual double GetCurrentTime() const = 0;

		/**
		 * Get the speed at which time is advancing
		 * 
		 * @return Speed
		 */
		virtual float GetSpeed() const = 0;

		/**
		 * Get whether the source is Running, Stopped, or Paused
		 * 
		 * @return The Current State of the Source
		 */
		virtual ESourceState GetCurrentSourceState() const = 0;

		/**
		 * Retrieve the latest "Event" from this source
		 * Whether it "Advanced", "Stopped", "Continued", etc.
		 * Also whether it "Seeked" or "Looped"
		 *
		 * @return The Latest Event of the Source
		 */
		virtual ESourceEvent GetLatestSourceEvent() const = 0;

		// ---- Optional Transport ----
		// Default implementations are no-ops. Override in time sources
		// that support direct transport control (e.g., FWorldTimeSourceController).

		virtual void RequestStart(float StartTime = 0.f) {}
		virtual void RequestStop() {}
		virtual void RequestPause() {}
		virtual void RequestContinue() {}

		/** Seek to an absolute time position in seconds. Default is no-op. */
		virtual void RequestSeek(float TimeInSeconds) {}

		/** Set the playback speed multiplier. Default is no-op. */
		virtual void RequestSetSpeed(float InSpeed) {}
	};
}

#undef UE_API
