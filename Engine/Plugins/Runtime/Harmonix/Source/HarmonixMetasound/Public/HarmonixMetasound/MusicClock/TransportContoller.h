// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreTypes.h"

namespace Harmonix
{
	enum class ETransportRequest : uint8
	{
		None,
		Play,
		Pause,
		Continue,
		Stop,
		Seek
	};
}
