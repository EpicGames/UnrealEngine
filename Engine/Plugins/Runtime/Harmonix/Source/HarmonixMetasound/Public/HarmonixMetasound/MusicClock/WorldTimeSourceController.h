// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TimeSource.h"
#include "TransportContoller.h"
#include "UObject/WeakObjectPtr.h"

namespace Harmonix
{

/**
 * A time source driven by UWorld delta time.
 *
 * Provides a simple wall-clock time stream suitable for non-audio-driven
 * use cases. Supports transport control (Start/Stop/Pause/Continue/Seek)
 * and variable speed playback. Time advances by the world's delta time
 * each frame, scaled by the speed multiplier and time dilation.
 */
class FWorldTimeSourceController : public ITimeSource
{
public:

	FWorldTimeSourceController(UObject* InContextObj);

	virtual FString GetDisplayName() const override;

	virtual TOptional<FVector> TryGetAudioSourceLocation() const override;

	virtual void Update() override;

	virtual ESourceState GetCurrentSourceState() const override;

	virtual ESourceEvent GetLatestSourceEvent() const override;

	virtual double GetCurrentTime() const override;

	virtual float GetSpeed() const override;

	/** Set the playback speed multiplier. Clamped to [0.01, 10.0]. */
	void SetSpeed(float InSpeed);

	/** Start playback from the given time in seconds. */
	void Start(float StartTime = 0.0f);
	void Stop();
	void Pause();
	void Continue();

	// ITimeSource optional transport
	virtual void RequestStart(float StartTime = 0.f) override { Start(StartTime); }
	virtual void RequestStop() override { Stop(); }
	virtual void RequestPause() override { Pause(); }
	virtual void RequestContinue() override { Continue(); }
	virtual void RequestSeek(float TimeInSeconds) override { Seek(TimeInSeconds); }
	virtual void RequestSetSpeed(float InSpeed) override { SetSpeed(InSpeed); }

	/** Seek to an absolute time position in seconds. */
	void Seek(float InSeekTime);

private:

	double GetDeltaTime() const;

	float Speed = 1.0f;
	double RunTimeSeconds = 0.0;
	double SeekRequestTime = 0.0;
	ETransportRequest TransportRequest = ETransportRequest::None;
	ESourceEvent LatestSourceEvent = ESourceEvent::None;
	ESourceState CurrentSourceState = ESourceState::Stopped;

	FWeakObjectPtr ContextObj;
};

}
