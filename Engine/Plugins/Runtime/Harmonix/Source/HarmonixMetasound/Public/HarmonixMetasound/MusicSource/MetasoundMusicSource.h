// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Harmonix/LocalMinimumMagnitudeTracker.h"
#include "HarmonixMetasound/MusicSource/MusicSource.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"
#include "HarmonixMetasound/Analysis/MidiClockSongPos.h"
#include "MetasoundGeneratorHandle.h"
#include "MetasoundSampleCounter.h"
#include "UObject/StrongObjectPtr.h"

#include "MetasoundMusicSource.generated.h"

#define UE_API HARMONIXMETASOUND_API

class UAudioComponent;

namespace HarmonixMetasound::Analysis
{
	struct FSongMapChain;
}

namespace Metasound
{
	class FMetasoundGenerator;
}

/**
 * A music source driven by a Metasound's MIDI clock output.
 *
 * Connects to a Metasound via UAudioComponent, reads the MIDI clock
 * history ring buffer produced by the audio render thread, and smooths
 * it into a game-thread-rate position suitable for gameplay.
 *
 * Produces a single time stream: the smoothed audio render position.
 * Calibration offsets (experienced time, video render time) should be
 * applied externally via UOffsetMusicSource.
 *
 * State is derived from the Metasound's actual transport state, not from
 * explicit Start/Stop calls. If the AudioComponent starts playing externally,
 * the source detects it and reports Running. Transport methods on IMusicSource
 * are no-ops — control the Metasound through its AudioComponent or Metasound
 * interfaces directly.
 *
 * Reports Preparing when connected to an AudioComponent but the generator
 * is not yet attached or no clock history is available.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMetasoundMusicSource : public UObject, public IMusicSource
{
	GENERATED_BODY()

public:

	UE_API virtual void BeginDestroy() override;

	/**
	 * Connect to a Metasound on an audio component.
	 *
	 * @param InAudioComponent   The audio component playing a MetaSound.
	 * @param OutputPinName      The name of the MIDI Clock output pin on the MetaSound.
	 * @return true if connection was initiated (may complete asynchronously).
	 */
	UE_API bool ConnectToAudioComponent(
		UAudioComponent* InAudioComponent,
		FName OutputPinName = "MIDI Clock");

	/** Disconnect from the Metasound and release all handles. */
	UE_API void Disconnect();

	// ---- IMusicSource ----

	virtual void Start() override;
	virtual void Pause() override;
	virtual void Continue() override;
	virtual void Stop() override;

	virtual Harmonix::ESourceState GetSourceState() const override;
	virtual Harmonix::ESourceEvent GetLatestSourceEvent() const override;

	virtual const FMidiSongPos& GetCurrentSongPos() const override;
	virtual const FMidiSongPos& GetPreviousSongPos() const override;
	virtual const ISongMapEvaluator* GetCurrentSongMapEvaluator() const override;
	virtual float GetCurrentClockAdvanceRate() const override;

	virtual bool LoopedThisFrame() const override;
	virtual bool SeekedThisFrame() const override;

	virtual bool IsLooping() const override;
	virtual float GetLoopStartMs() const override;
	virtual float GetLoopLengthMs() const override;

	virtual TOptional<FVector> TryGetAudioSourceLocation() const override;

	virtual void Update() override;

#if !UE_BUILD_SHIPPING
	virtual FString GetDisplayName() const override;
#endif

private:
	// ---- Connection State ----

	FName MetasoundOutputName;
	TWeakObjectPtr<UAudioComponent> AudioComponentToWatch;
	TStrongObjectPtr<UMetasoundGeneratorHandle> CurrentGeneratorHandle;
	TWeakPtr<Metasound::FMetasoundGenerator> UnderlyingGenerator;

	Metasound::Frontend::FAnalyzerAddress MidiSongPosAnalyzerAddress;

	UMetasoundGeneratorHandle::FOnAttached::FDelegate OnAttachedDelegate;
	UMetasoundGeneratorHandle::FOnDetached::FDelegate OnDetachedDelegate;

	FDelegateHandle GeneratorAttachedCallbackHandle;
	FDelegateHandle GeneratorDetachedCallbackHandle;
	FDelegateHandle GeneratorIOUpdatedCallbackHandle;
	FDelegateHandle GraphChangedCallbackHandle;

	// ---- Clock History (read from audio thread) ----

	UMidiClockUpdateSubsystem::FClockHistoryPtr ClockHistory;
	HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor SmoothedClockHistoryCursor;
	TSharedPtr<const HarmonixMetasound::Analysis::FSongMapChain> CurrentMapChain;

	// ---- Smoothing / PLL State ----

	struct FSmoothedClockState
	{
		float TempoMapMs = 0.f;
		float TempoMapTick = 0.f;
		float LocalTick = 0.f;
	};

	FSmoothedClockState SmoothedState;

	double RenderStartWallClockTimeSeconds = 0.0;
	double LastRefreshWallClockTimeSeconds = 0.0;
	double DeltaSecondsBetweenRefreshes = 0.0;
	Metasound::FSampleCount RenderStartSampleCount {0};

	static const int kFramesOfErrorHistory = 10;
	FLocalMinimumMagnitudeTracker<double, kFramesOfErrorHistory> ErrorTracker;
	double SyncSpeed = 1.0;

	float RenderSmoothingLagSeconds = 0.030f;

	// ---- Position State ----

	FMidiSongPos CurrentPos;
	FMidiSongPos PrevPos;

	float CurrentAdvanceRate = 1.f;
	int32 LastTickSeen = 0;

	/** Cached source state, derived from clock history transport each frame. */
	Harmonix::ESourceState CachedSourceState = Harmonix::ESourceState::Stopped;

	bool bSeekDetected = false;
	bool bLoopDetected = false;

	// ---- Internal Methods ----

	/** Try to create a generator handle and wire up analyzer callbacks for the watched audio component. */
	UE_API bool AttemptToConnect(UAudioComponent* InAudioComponent);

	/** Remove all generator handle callbacks. */
	UE_API void DetachAllCallbacks();

	// Generator lifecycle callbacks
	void OnGeneratorAttached();
	void OnGeneratorDetached();
	void OnGraphSet();
	void OnGeneratorIOUpdatedWithChanges(const TArray<Metasound::FVertexInterfaceChange>& VertexInterfaceChanges);

	/** Set (or clear) the shared clock history ring buffer used for position smoothing. */
	UE_API void SetClockHistory(const UMidiClockUpdateSubsystem::FClockHistoryPtr& NewHistory);

	/** Read the latest position from the clock history ring and apply PLL smoothing. */
	UE_API void RefreshFromHistory();

	/** Apply the smoothed tick to CurrentPos for non-offset (normal or looping) playback. */
	UE_API void UpdateSmoothedPosition(float SmoothedTick, float SmoothedTempoMapTick);

	/** Apply the smoothed tick to CurrentPos for offset clock playback (clock tick != tempo map tick). */
	UE_API void UpdateSmoothedPositionForOffsetClock(float SmoothedTick, float SmoothedTempoMapTick);

	/** Calculate song position with loop wrapping or monotonic advancement. */
	UE_API FMidiSongPos CalculateSongPosForLoopingOrMonotonic(float AbsoluteMs, float& InOutPositionTick, bool& OutSeekDetected, bool& OutLoopDetected) const;

	/** Calculate song position for an offset clock with a tick delta from the tempo map. */
	UE_API FMidiSongPos CalculateSongPosForOffsetClock(float PositionMs, float ClockTickOffset, float& InOutPositionTick, bool& OutSeekDetected) const;

	/** Detect a seek by comparing actual tick delta to expected delta based on tempo and frame time. */
	UE_API bool CheckForSeek(float FirstTick, float NextTick, float CurrentTempo, int32 TicksPerQuarter) const;

	enum class EHistoryFailureType : uint8
	{
		None,
		NotEnoughDataInTheHistoryRing,
		NotEnoughHistory,
		LookingForTimeInTheFutureOfWhatHasEvenRendered,
		CaughtUpToRenderPosition
	};

	/** Interpolate the clock history ring buffer to find the smoothed tick at the expected render position. */
	UE_API EHistoryFailureType CalculateSmoothedTick(
		Metasound::FSampleCount ExpectedRenderPosSampleCount,
		Metasound::FSampleCount LastRenderPosSampleCount,
		float& SmoothedLocalTick,
		float& SmoothedTempoMapTick,
		float& CurrentSpeed,
		HarmonixMetasound::Analysis::FMidiClockSongPositionHistory::FReadCursor& ReadCursor,
		float LookBehindSeconds);

	static FString HistoryFailureTypeToString(EHistoryFailureType Error);
};

#undef UE_API
