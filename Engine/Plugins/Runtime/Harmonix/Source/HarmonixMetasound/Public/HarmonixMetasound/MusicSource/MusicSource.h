// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Interface.h"
#include "HarmonixMetasound/MusicClock/TimeSource.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/SongMaps.h"

#include "MusicSource.generated.h"

#define UE_API HARMONIXMETASOUND_API

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UMusicSource : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for objects that produce musical time.
 *
 * An IMusicSource is the owner of a music playback stream. It provides:
 *   - Transport controls (Start/Stop/Pause/Continue)
 *   - Current musical position (FMidiSongPos)
 *   - Song map access for time<->beat conversions
 *   - Discontinuity detection (loop/seek)
 *
 * A single IMusicSource produces a single time stream. Calibrated time
 * offsets (experienced, video render, etc.) are handled externally by
 * UOffsetMusicSource feeding into a UMusicClock.
 */
class IMusicSource
{
	GENERATED_BODY()

public:

	// ---- Transport Controls ----
	// The source owns playback. Consumers (UMusicClock) are read-only views.

	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void Start() = 0;

	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void Pause() = 0;

	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void Continue() = 0;

	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void Stop() = 0;

	/**
	 * Seek to a musical position.
	 * For RuntimeMusicSource: converts the timestamp to seconds via song maps and seeks the time source.
	 * For MetasoundMusicSource: requires Metasound interface support (future work).
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void Seek(const FMusicTimestamp& Timestamp) {}

	// ---- Source State ----

	/** Get the current transport state of the source (Stopped, Preparing, Paused, Running). */
	virtual Harmonix::ESourceState GetSourceState() const = 0;

	/** Get the most recent source event for this frame (Start, Advance, Stop, Pause, Continue, Seek, Loop). */
	virtual Harmonix::ESourceEvent GetLatestSourceEvent() const = 0;

	/** Whether the source is currently producing musical time. */
	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual bool IsPlaying() const { return GetSourceState() == Harmonix::ESourceState::Running; }

	/** Whether the source is stopped (not producing time, position at zero). */
	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual bool IsStopped() const { return GetSourceState() == Harmonix::ESourceState::Stopped; }

	/** Whether the source intends to run but is waiting for data (e.g., Metasound not yet connected). */
	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual bool IsPreparing() const { return GetSourceState() == Harmonix::ESourceState::Preparing; }

	// ---- Time Production ----

	/** The current musical position, updated once per frame via Update(). */
	virtual const FMidiSongPos& GetCurrentSongPos() const = 0;

	/** The previous frame's musical position. */
	virtual const FMidiSongPos& GetPreviousSongPos() const = 0;

	/** The song maps used to convert between time and musical position. */
	virtual const ISongMapEvaluator* GetCurrentSongMapEvaluator() const = 0;

	/** The current playback speed multiplier. */
	virtual float GetCurrentClockAdvanceRate() const = 0;

	// ---- Discontinuity Detection ----

	/** Whether the source crossed a loop boundary this frame. */
	virtual bool LoopedThisFrame() const = 0;

	/** Whether the source performed a non-contiguous time jump (seek) this frame. */
	virtual bool SeekedThisFrame() const = 0;

	// ---- Loop Region ----

	/** Whether this source is currently looping. */
	virtual bool IsLooping() const = 0;

	/** Start of the loop region in milliseconds. Only valid if IsLooping() is true. */
	virtual float GetLoopStartMs() const = 0;

	/** Length of the loop region in milliseconds. Only valid if IsLooping() is true. */
	virtual float GetLoopLengthMs() const = 0;

	// ---- Tempo & Time Signature Control ----

	/**
	 * Set the tempo in BPM.
	 * For RuntimeMusicSource: modifies the underlying song maps directly.
	 * For MetasoundMusicSource: requires Metasound interface support (future work).
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void SetTempo(float BPM) {}

	/**
	 * Set the time signature.
	 * For RuntimeMusicSource: modifies the underlying song maps directly.
	 * For MetasoundMusicSource: requires Metasound interface support (future work).
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void SetTimeSignature(int32 Numerator, int32 Denominator) {}

	/**
	 * Set the playback speed multiplier (1.0 = normal speed).
	 * For RuntimeMusicSource: sets the speed on the underlying ITimeSource.
	 * For MetasoundMusicSource: requires Metasound interface support (future work).
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicSource")
	virtual void SetSpeed(float Speed) {}

	// ---- Spatial ----

	/** Get the world-space location of the audio source, if available. */
	virtual TOptional<FVector> TryGetAudioSourceLocation() const = 0;

	// ---- Update ----

	/**
	 * Called once per frame by the update subsystem.
	 * Implementations should refresh their internal position state.
	 */
	virtual void Update() = 0;

	// ---- Debug ----

#if !UE_BUILD_SHIPPING
	virtual FString GetDisplayName() const = 0;
#endif
};

#undef UE_API
