// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "HarmonixMetasound/MusicSource/MusicSourceSettings.h"

#include "MusicSourceBlueprintLibrary.generated.h"

#define UE_API HARMONIXMETASOUND_API

class UMusicClock;

/**
 * Blueprint function library for creating and managing music sources and clocks.
 *
 * These factory functions handle creation, connection, and subsystem registration
 * in one call. The source type is determined by the settings struct — new source
 * types can be added by game plugins without modifying this library.
 */
UCLASS(MinimalAPI)
class UMusicSourceBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Create a music source from settings.
	 *
	 * The settings struct type determines what kind of source is created.
	 * The source is automatically registered with the update subsystem.
	 *
	 * @param Outer             Outer object for the new source (determines GC lifetime).
	 * @param Settings          Configuration determining source type and parameters.
	 * @return The created music source, or invalid if creation failed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Music|Source",
		Meta = (DefaultToSelf = "Outer"))
	static UE_API TScriptInterface<IMusicSource> CreateMusicSource(
		UObject* Outer,
		const TInstancedStruct<FMusicSourceSettings>& Settings);

	/**
	 * Create a read-only music clock bound to a source.
	 *
	 * The clock holds a weak reference to the source — if the source is
	 * destroyed, the clock gracefully reports Stopped.
	 * The clock is automatically registered with the update subsystem.
	 *
	 * @param Outer             Outer object for the new clock (determines GC lifetime).
	 * @param Source            The music source to observe.
	 * @return The created music clock, or null if Source is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Music|Clock",
		Meta = (DefaultToSelf = "Outer"))
	static UE_API UMusicClock* CreateMusicClock(
		UObject* Outer,
		const TScriptInterface<IMusicSource>& Source);
};

#undef UE_API
