// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixMetasound/MusicSource/MusicSource.h"
#include "StructUtils/InstancedStruct.h"

#include "MusicSourceSettings.generated.h"

#define UE_API HARMONIXMETASOUND_API

class UAudioComponent;
class UMidiFile;

/**
 * Base struct for music source configuration.
 *
 * Subclass this to define new music source types. The CreateSource() factory
 * method is called by UMusicSourceBlueprintLibrary::CreateMusicSource() —
 * Blueprint users pick the settings type from a TInstancedStruct dropdown,
 * fill in the type-specific fields, and get back an IMusicSource.
 *
 * Game plugins can add new source types without modifying engine code:
 * just subclass FMusicSourceSettings and implement CreateSource().
 */
USTRUCT(MinimalAPI, BlueprintType)
struct FMusicSourceSettings
{
	GENERATED_BODY()

	virtual ~FMusicSourceSettings() = default;

	/** Create the music source described by these settings and register it with the update subsystem. */
	UE_API virtual TScriptInterface<IMusicSource> CreateSource(UObject* Outer) const;
};

/**
 * Creates a UMetasoundMusicSource connected to an AudioComponent.
 */
USTRUCT(BlueprintType, DisplayName = "Metasound Music Source Settings")
struct FMetasoundMusicSourceSettings : public FMusicSourceSettings
{
	GENERATED_BODY()

	/** The audio component playing a MetaSound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource")
	TObjectPtr<UAudioComponent> AudioComponent;

	/** The name of the MIDI Clock output pin on the MetaSound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource")
	FName OutputPinName = "MIDI Clock";

	UE_API virtual TScriptInterface<IMusicSource> CreateSource(UObject* Outer) const override;
};

/**
 * Creates a URuntimeMusicSource driven by wall clock with tempo/time-sig from a MIDI file.
 */
USTRUCT(BlueprintType, DisplayName = "MIDI Music Source Settings")
struct FMidiMusicSourceSettings : public FMusicSourceSettings
{
	GENERATED_BODY()

	/** MIDI file providing tempo map and time signature data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource")
	TObjectPtr<UMidiFile> MidiFile;

	UE_API virtual TScriptInterface<IMusicSource> CreateSource(UObject* Outer) const override;
};

/**
 * Creates a URuntimeMusicSource driven by wall clock with manually specified tempo and time signature.
 * Tempo and time signature can be changed at runtime via SetTempo() / SetTimeSignature() on the source.
 */
USTRUCT(BlueprintType, DisplayName = "Manual Music Source Settings")
struct FManualMusicSourceSettings : public FMusicSourceSettings
{
	GENERATED_BODY()

	/** Initial tempo in BPM. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource", Meta = (ClampMin = "1.0", ClampMax = "999.0"))
	float Tempo = 120.f;

	/** Initial time signature numerator (beats per bar). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource", Meta = (ClampMin = "1", ClampMax = "32"))
	int32 TimeSigNumerator = 4;

	/** Initial time signature denominator (beat unit). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource", Meta = (ClampMin = "1", ClampMax = "32"))
	int32 TimeSigDenominator = 4;

	UE_API virtual TScriptInterface<IMusicSource> CreateSource(UObject* Outer) const override;
};

/**
 * Creates a UOffsetMusicSource that applies a millisecond offset to a parent source.
 * Used for calibrated time streams (e.g., video render, experienced time).
 */
USTRUCT(BlueprintType, DisplayName = "Offset Music Source Settings")
struct FOffsetMusicSourceSettings : public FMusicSourceSettings
{
	GENERATED_BODY()

	/** The parent music source to offset from. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource")
	TScriptInterface<IMusicSource> ParentSource;

	/** Offset in milliseconds. Positive = ahead, negative = behind. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MusicSource")
	float OffsetMs = 0.f;

	UE_API virtual TScriptInterface<IMusicSource> CreateSource(UObject* Outer) const override;
};

#undef UE_API
