// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixMetasound/MusicSource/MusicSource.h"

#include "OffsetMusicSource.generated.h"

#define UE_API HARMONIXMETASOUND_API

/**
 * An IMusicSource that reads from a parent IMusicSource and applies a
 * millisecond offset. This is how calibrated time clocks (experienced,
 * video render) are created:
 *
 *   UMetasoundMusicSource (audio render)
 *       -> UOffsetMusicSource (-30ms) -> UMusicClock (video render)
 *       -> UOffsetMusicSource (-50ms) -> UMusicClock (experienced)
 *
 * Offset sources can also chain: SourceA -> OffsetB -> OffsetC, where
 * each offset reads directly from the previous IMusicSource.
 *
 * Loop handling: when the parent source is looping, the offset time is
 * wrapped around the loop boundary. On the first pass through the loop
 * (before the parent has ever looped), negative offsets are NOT wrapped —
 * the offset time stays negative until the parent advances past the threshold.
 */
UCLASS(MinimalAPI, BlueprintType)
class UOffsetMusicSource : public UObject, public IMusicSource
{
	GENERATED_BODY()

public:

	/** Set the parent source this offset reads from. */
	UFUNCTION(BlueprintCallable, Category = "MusicSource|Offset")
	UE_API void SetParentSource(const TScriptInterface<IMusicSource>& InParentSource);

	/** Get the parent source. */
	UFUNCTION(BlueprintPure, Category = "MusicSource|Offset")
	UE_API TScriptInterface<IMusicSource> GetParentSource() const;

	/** Set the offset in milliseconds. Positive = ahead, negative = behind. */
	UFUNCTION(BlueprintCallable, Category = "MusicSource|Offset")
	UE_API void SetOffsetMs(float InOffsetMs);

	/** Get the current offset. */
	UFUNCTION(BlueprintPure, Category = "MusicSource|Offset")
	UE_API float GetOffsetMs() const;

	// ---- IMusicSource ----

	// Transport is a no-op — the parent source owns playback.
	virtual void Start() override {}
	virtual void Pause() override {}
	virtual void Continue() override {}
	virtual void Stop() override {}

	virtual Harmonix::ESourceState GetSourceState() const override;
	virtual Harmonix::ESourceEvent GetLatestSourceEvent() const override;

	virtual const FMidiSongPos& GetCurrentSongPos() const override;
	virtual const FMidiSongPos& GetPreviousSongPos() const override;
	virtual const ISongMapEvaluator* GetCurrentSongMapEvaluator() const override;
	virtual float GetCurrentClockAdvanceRate() const override;

	virtual bool LoopedThisFrame() const override;
	virtual bool SeekedThisFrame() const override;

	virtual bool IsLooping() const override;
	virtual float GetLoopStartMs() const override;
	virtual float GetLoopLengthMs() const override;

	virtual TOptional<FVector> TryGetAudioSourceLocation() const override;

	virtual void Update() override;

#if !UE_BUILD_SHIPPING
	virtual FString GetDisplayName() const override;
#endif

private:

	/** Weak reference to the parent source UObject. */
	TWeakObjectPtr<UObject> ParentSourceObject;

	/** Cached interface pointer — valid only when ParentSourceObject is valid. */
	IMusicSource* ParentSourceCache = nullptr;

	/** Get the parent source if still alive, or nullptr. */
	IMusicSource* GetValidParentSource() const;

	float OffsetMs = 0.f;

	FMidiSongPos CurrentPos;
	FMidiSongPos PrevPos;
	bool bLoopedThisFrame = false;
	bool bSeekedThisFrame = false;

	/** True once the parent source has completed at least one loop pass.
	 *  Prevents negative offsets from wrapping to the loop end on the first pass. */
	bool bParentHasLooped = false;
};

#undef UE_API
