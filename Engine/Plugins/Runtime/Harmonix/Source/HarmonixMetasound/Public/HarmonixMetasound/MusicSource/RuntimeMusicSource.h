// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixMetasound/MusicSource/MusicSource.h"
#include "HarmonixMetasound/MusicClock/TimeSource.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMidi/MidiFile.h"

#include "RuntimeMusicSource.generated.h"

#define UE_API HARMONIXMETASOUND_API

/**
 * A music source that combines an ITimeSource (providing raw time in seconds)
 * with an ISongMapEvaluator (providing time-to-musical-position mapping).
 *
 * This is the "simple" music source for non-Metasound use cases:
 * wall clock, animation-driven, media-driven, etc. The specific time
 * behavior is determined by which ITimeSource implementation is provided.
 *
 * Transport controls delegate to the ITimeSource if it supports them
 * (e.g., FWorldTimeSourceController has Start/Stop/Pause/Continue).
 */
UCLASS(MinimalAPI, BlueprintType)
class URuntimeMusicSource : public UObject, public IMusicSource
{
	GENERATED_BODY()

public:

	UE_API virtual void BeginDestroy() override;

	/**
	 * Initialize this source with a time source and song maps.
	 *
	 * @param InTimeSource   Provides raw elapsed time in seconds.
	 * @param InSongMaps     Provides time<->musical position mapping. Can be null for default maps.
	 */
	UE_API void Initialize(TSharedPtr<Harmonix::ITimeSource> InTimeSource, const ISongMapEvaluator* InSongMaps);

	/**
	 * Initialize from a MIDI file's song maps.
	 */
	UE_API void InitializeWithMidi(TSharedPtr<Harmonix::ITimeSource> InTimeSource, UMidiFile* InMidiFile);

	/** Get the underlying time source. */
	TSharedPtr<Harmonix::ITimeSource> GetTimeSource() const { return TimeSource; }

	// ---- IMusicSource ----

	virtual void Start() override;
	virtual void Pause() override;
	virtual void Continue() override;
	virtual void Stop() override;

	virtual Harmonix::ESourceState GetSourceState() const override;
	virtual Harmonix::ESourceEvent GetLatestSourceEvent() const override;

	virtual const FMidiSongPos& GetCurrentSongPos() const override;
	virtual const FMidiSongPos& GetPreviousSongPos() const override;
	virtual const ISongMapEvaluator* GetCurrentSongMapEvaluator() const override;
	virtual float GetCurrentClockAdvanceRate() const override;

	virtual bool LoopedThisFrame() const override;
	virtual bool SeekedThisFrame() const override;

	virtual bool IsLooping() const override;
	virtual float GetLoopStartMs() const override;
	virtual float GetLoopLengthMs() const override;

	/**
	 * Set a loop region by musical time.
	 * @param StartBar   The bar to start looping at (1-based, where Bar 1 = start of song).
	 * @param NumBars    Number of bars in the loop. Pass 0 to disable looping.
	 */
	UFUNCTION(BlueprintCallable, Category = "MusicSource|Loop")
	UE_API void SetLoopRegionByBars(int32 StartBar, int32 NumBars);

	/** Clear the loop region, disabling looping. */
	UFUNCTION(BlueprintCallable, Category = "MusicSource|Loop")
	UE_API void ClearLoopRegion();

	virtual void Seek(const FMusicTimestamp& Timestamp) override;
	virtual void SetTempo(float BPM) override;
	virtual void SetTimeSignature(int32 Numerator, int32 Denominator) override;
	virtual void SetSpeed(float Speed) override;

	virtual TOptional<FVector> TryGetAudioSourceLocation() const override;

	virtual void Update() override;

#if !UE_BUILD_SHIPPING
	virtual FString GetDisplayName() const override;
#endif

private:

	TSharedPtr<Harmonix::ITimeSource> TimeSource;

	/** External song maps (from MIDI file). */
	TWeakObjectPtr<UMidiFile> MidiFile;

	/** Default maps used when no external maps are provided. */
	FSongMaps DefaultMaps;

	FMidiSongPos CurrentPos;
	FMidiSongPos PrevPos;

	// Loop region defined in musical time
	int32 LoopStartBar = 0;
	int32 LoopNumBars = 0;

	// Cached ms values, recomputed from bar values + song maps
	float CachedLoopStartMs = 0.f;
	float CachedLoopLengthMs = 0.f;

	/** Recompute cached loop start/length in ms from bar values + current song maps. */
	UE_API void RecomputeLoopRegionMs();

	/** Get the active song map evaluator (MIDI file maps or DefaultMaps). */
	UE_API const ISongMapEvaluator* GetMaps() const;
};

#undef UE_API
