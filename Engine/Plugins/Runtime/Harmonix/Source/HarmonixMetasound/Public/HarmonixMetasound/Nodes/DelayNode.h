// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::DelayNode
{
	namespace Constants
	{
		inline constexpr int32 NumChannels = 2;
		inline constexpr float MaxDelayTime = 2500.0f;
	}

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(AudioLeft);
		DECLARE_METASOUND_PARAM_ALIAS(AudioRight);
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock)
		DECLARE_METASOUND_PARAM_EXTERN(DelayTimeType);
		DECLARE_METASOUND_PARAM_EXTERN(DelayTime);
		DECLARE_METASOUND_PARAM_EXTERN(Feedback);
		DECLARE_METASOUND_PARAM_EXTERN(DryLevel);
		DECLARE_METASOUND_PARAM_EXTERN(WetLevel);
		DECLARE_METASOUND_PARAM_EXTERN(WetFilterEnabled);
		DECLARE_METASOUND_PARAM_EXTERN(FeedbackFilterEnabled);
		DECLARE_METASOUND_PARAM_EXTERN(FilterType);
		DECLARE_METASOUND_PARAM_EXTERN(FilterCutoff);
		DECLARE_METASOUND_PARAM_EXTERN(FilterQ);
		DECLARE_METASOUND_PARAM_EXTERN(LFOEnabled);
		DECLARE_METASOUND_PARAM_EXTERN(LFOTimeType);
		DECLARE_METASOUND_PARAM_ALIAS(LFOFrequency);
		DECLARE_METASOUND_PARAM_EXTERN(LFODepth);
		DECLARE_METASOUND_PARAM_EXTERN(StereoType);
		DECLARE_METASOUND_PARAM_EXTERN(StereoSpreadLeft);
		DECLARE_METASOUND_PARAM_EXTERN(StereoSpreadRight);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(AudioLeft);
		DECLARE_METASOUND_PARAM_ALIAS(AudioRight);
	}
}
