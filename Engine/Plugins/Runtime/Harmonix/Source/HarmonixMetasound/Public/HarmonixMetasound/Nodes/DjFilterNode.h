// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::DjFilter
{
	const HARMONIXMETASOUND_API Metasound::FNodeClassName& GetClassName();
	
	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(AudioMono);
		DECLARE_METASOUND_PARAM_EXTERN(Amount);
		DECLARE_METASOUND_PARAM_EXTERN(Resonance);
		DECLARE_METASOUND_PARAM_EXTERN(LowPassMinFrequency);
		DECLARE_METASOUND_PARAM_EXTERN(LowPassMaxFrequency);
		DECLARE_METASOUND_PARAM_EXTERN(HighPassMinFrequency);
		DECLARE_METASOUND_PARAM_EXTERN(HighPassMaxFrequency);
		DECLARE_METASOUND_PARAM_EXTERN(DeadZoneSize);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(AudioMono);
	}
}
