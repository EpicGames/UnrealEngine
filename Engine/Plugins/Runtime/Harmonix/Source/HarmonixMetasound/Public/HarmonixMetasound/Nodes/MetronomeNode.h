// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound::Nodes::MetronomeNode
{
	using namespace Metasound;

	HARMONIXMETASOUND_API const Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	constexpr float kMinTempoBpm = 1.0f;
	constexpr float kMaxTempoBpm = 999.0f;

	constexpr int32 kMinTimeSigNumerator = 1;
	constexpr int32 kMaxTimeSigNumerator = 64;

	constexpr int32 kMinTimeSigDenominator = 1;
	constexpr int32 kMaxTimeSigDenominator = 64;

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Transport);
		DECLARE_METASOUND_PARAM_ALIAS(Loop);
		DECLARE_METASOUND_PARAM_ALIAS(LoopLengthBars);
		DECLARE_METASOUND_PARAM_ALIAS(TimeSigNumerator);
		DECLARE_METASOUND_PARAM_ALIAS(TimeSigDenominator);
		DECLARE_METASOUND_PARAM_ALIAS(Tempo);
		DECLARE_METASOUND_PARAM_ALIAS(Speed);
		DECLARE_METASOUND_PARAM_ALIAS(PrerollBars);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
	}
 
	class FMetronomeOperatorBase : public TExecutableOperator<FMetronomeOperatorBase>, public FMusicTransportControllable
	{
	public:

		UE_API FMetronomeOperatorBase(const FBuildOperatorParams& InParams,
			const FMusicTransportEventStreamReadRef& InTransport,
			const bool  InLoop,
			const int32 InLoopLengthBars,
			const FInt32ReadRef& InTimSigNumerator,
			const FInt32ReadRef& InTimeSigDenominator,
			const FFloatReadRef& InTempo,
			const FFloatReadRef& InSpeedMultiplier,
			const int32 InSeekPrerollBars);

		UE_API virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
		UE_API virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

		UE_API void Reset(const FResetParams& Params);
		UE_API void Execute();

	protected:
		UE_API void Init();

		//** INPUTS
		FMusicTransportEventStreamReadRef TransportInPin;
		const bool  LoopInPin;
		const int32 LoopLengthBarsInPin;
		FInt32ReadRef TimeSigNumInPin;
		FInt32ReadRef TimeSigDenomInPin;
		FFloatReadRef TempoInPin;
		FFloatReadRef SpeedMultInPin;
		const int32 SeekPreRollBarsInPin;

		//** OUTPUTS
		FMidiClockWriteRef MidiClockOutPin;

		//** DATA
		TSharedPtr<FMidiClock, ESPMode::NotThreadSafe> MonotonicallyIncreasingClock;
		TSharedPtr<FSongMaps> SongMaps;
		FSampleCount BlockSize;
		float        SampleRate;
		float        CurrentTempo = 0.0f;
		int32        CurrentTimeSigNum = 0;
		int32        CurrentTimeSigDenom = 0;
		int32		 LastProcessedClockTick = -1;
		int32		 NextClockTickToProcess = 0;
		bool		 bClocksArePreparedForExecute = true;

		UE_API void BuildSongMaps(bool ResetToStart = true);
		UE_API float GetInputTempoClamped() const;
		UE_API int32 GetInputTimeSigNumeratorClamped() const;
		UE_API int32 GetInputTimeSigDenominatorClamped() const;
		UE_API void UpdateMidi();
		UE_API void AddTempoChangeForMidi(float TempoBPM);
		UE_API virtual void HandleTimeSigChangeForMidi(int32 TimeSigNum, int32 TimeSigDenom);
		UE_API void HandleTransportChange(int32 StartFrameIndex, EMusicPlayerTransportState NewTransportState);
		UE_API void PrepareClocksForExecute();
		UE_API void MarkClocksAsExecuted();

		FMidiClock& GetDrivingMidiClock() { return LoopInPin ? *MonotonicallyIncreasingClock : (*MidiClockOutPin); }
	};
}

#undef UE_API
