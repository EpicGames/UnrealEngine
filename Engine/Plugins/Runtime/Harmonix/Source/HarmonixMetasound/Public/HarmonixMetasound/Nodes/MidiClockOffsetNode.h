// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::MidiClockOffset
{
	HARMONIXMETASOUND_API Metasound::FNodeClassName GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(OffsetMs);
		DECLARE_METASOUND_PARAM_EXTERN(OffsetBars);
		DECLARE_METASOUND_PARAM_EXTERN(OffsetBeats);
		DECLARE_METASOUND_PARAM_EXTERN(MidiClock);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(MidiClock);
	}
}