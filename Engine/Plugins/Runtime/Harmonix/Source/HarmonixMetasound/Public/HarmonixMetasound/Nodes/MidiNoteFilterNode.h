// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::MidiNoteFilter
{
	HARMONIXMETASOUND_API Metasound::FNodeClassName GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enable);
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
		DECLARE_METASOUND_PARAM_ALIAS(MinNoteNumber);
		DECLARE_METASOUND_PARAM_ALIAS(MaxNoteNumber);
		DECLARE_METASOUND_PARAM_ALIAS(MinVelocity);
		DECLARE_METASOUND_PARAM_ALIAS(MaxVelocity);
		DECLARE_METASOUND_PARAM_EXTERN(IncludeOtherEvents);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
	}
}
