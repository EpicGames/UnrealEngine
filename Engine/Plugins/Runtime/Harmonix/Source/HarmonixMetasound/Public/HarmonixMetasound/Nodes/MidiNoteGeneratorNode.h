// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "HarmonixMetasound/Common.h"

#define UE_API HARMONIXMETASOUND_API

#define LOCTEXT_NAMESPACE "HarmonixMetasound_MidiNoteGeneratorNode"

namespace HarmonixMetasound::Nodes::MidiNoteGeneratorNode
{
	const UE_API Metasound::FNodeClassName& GetClassName();
	UE_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_ALIAS(MidiChannelNumber);
		
		METASOUND_PARAM(TriggerNote, "Trigger", "Trigger a MIDI Note On Event with the provided MIDI Note Number & Velocity")
		METASOUND_PARAM(TrackNumber, "Track Number", "The Track Number for the output MIDI events");
		METASOUND_PARAM(MidiNote, "MIDI Note", "The MIDI Note to trigger")
		METASOUND_PARAM(MidiVelocity, "MIDI Velocity", "The Velocity to trigger the midi note with (0-127")
		METASOUND_PARAM(NoteOffDelay, "Note Off Delay (Seconds)", "Delay time before triggering the note off for the triggered note")
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
	}
}

#undef LOCTEXT_NAMESPACE

#undef UE_API