// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "MetasoundParamHelper.h"
#include "HarmonixMetasound/Common.h"
#include "MetasoundNodeInterface.h"

namespace HarmonixMetasound::Nodes::MidiPlayerNode
{
	using namespace Metasound;

	const HARMONIXMETASOUND_API FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	static constexpr float kMinClockSpeed = 0.01f;
	static constexpr float kMaxClockSpeed = 10.0f;

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiFileAsset);
		DECLARE_METASOUND_PARAM_ALIAS(Transport);
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_ALIAS(Loop);
		DECLARE_METASOUND_PARAM_ALIAS(Speed);
		DECLARE_METASOUND_PARAM_ALIAS(PrerollBars);
		DECLARE_METASOUND_PARAM_EXTERN(KillVoicesOnSeek);
		DECLARE_METASOUND_PARAM_EXTERN(KillVoicesOnMidiChange);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
	}
}