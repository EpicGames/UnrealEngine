// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::MidiPulseGeneratorNode
{
	const HARMONIXMETASOUND_API Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_EXTERN(DutyCycle);
		DECLARE_METASOUND_PARAM_EXTERN(Interval);
		DECLARE_METASOUND_PARAM_EXTERN(IntervalMultiplier);
		DECLARE_METASOUND_PARAM_EXTERN(Offset);
		DECLARE_METASOUND_PARAM_EXTERN(OffsetMultiplier);
		DECLARE_METASOUND_PARAM_ALIAS(MidiTrack);
		DECLARE_METASOUND_PARAM_ALIAS(MidiChannel);
		DECLARE_METASOUND_PARAM_EXTERN(MidiNoteNumber);
		DECLARE_METASOUND_PARAM_EXTERN(MidiVelocity);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
	}
}