// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

#define UE_API HARMONIXMETASOUND_API

namespace HarmonixMetasound::Nodes::MidiQuantizeTriggerNode
{
	const UE_API Metasound::FNodeClassName& GetClassName();
	UE_API int32 GetCurrentMajorVersion();
	
	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_EXTERN(TriggerInput);
		DECLARE_METASOUND_PARAM_EXTERN(Quantization)
		DECLARE_METASOUND_PARAM_EXTERN(Tolerance)
	}

	namespace  Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(TriggerOutput);
	}
}

#undef UE_API