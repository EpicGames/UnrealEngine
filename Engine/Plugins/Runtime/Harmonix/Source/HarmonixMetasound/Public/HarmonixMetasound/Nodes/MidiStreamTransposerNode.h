// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::MidiNoteTranspose
{
	HARMONIXMETASOUND_API const Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enable);
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
		DECLARE_METASOUND_PARAM_ALIAS(Transposition);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
	}
}
