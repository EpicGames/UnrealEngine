// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundNodeInterface.h"
#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::MorphingLFO
{
	template<typename OutputDataType>
	const Metasound::FNodeClassName& GetClassName()
	{
		static const Metasound::FNodeClassName ClassName
		{
			HarmonixNodeNamespace,
			TEXT("MorphingLFO"),
			Metasound::GetMetasoundDataTypeName<OutputDataType>()
		};
		return ClassName;
	}
	
	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_ALIAS(LFOSyncType);
		DECLARE_METASOUND_PARAM_ALIAS(LFOFrequency);
		DECLARE_METASOUND_PARAM_ALIAS(LFOInvert);
		DECLARE_METASOUND_PARAM_ALIAS(LFOShape)
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(LFO);
	}
}