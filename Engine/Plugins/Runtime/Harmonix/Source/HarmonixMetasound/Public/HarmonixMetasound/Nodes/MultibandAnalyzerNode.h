// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundEnvelopeFollowerTypes.h"
#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::MultibandAnalyzer
{
	const HARMONIXMETASOUND_API Metasound::FNodeClassName& GetClassName();

	struct FSettings
	{
		bool Enable{ true };
		TArray<float> CrossoverFrequencies{ 200, 400, 800 };
		bool ApplySmoothing{ true };
		Metasound::FTime AttackTime = Metasound::FTime::FromMilliseconds(10);
		Metasound::FTime ReleaseTime = Metasound::FTime::FromMilliseconds(10);
		Metasound::EEnvelopePeakMode PeakMode = Metasound::EEnvelopePeakMode::Peak;
	};
	
	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enable);
		DECLARE_METASOUND_PARAM_ALIAS(AudioMono);
		DECLARE_METASOUND_PARAM_EXTERN(CrossoverFrequencies);
		DECLARE_METASOUND_PARAM_EXTERN(ApplySmoothing);
		DECLARE_METASOUND_PARAM_EXTERN(AttackTime);
		DECLARE_METASOUND_PARAM_EXTERN(ReleaseTime);
		DECLARE_METASOUND_PARAM_EXTERN(PeakMode);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(BandLevels);
	}
}
