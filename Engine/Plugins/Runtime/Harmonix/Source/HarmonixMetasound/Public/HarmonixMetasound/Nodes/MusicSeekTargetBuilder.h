// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::BarBeatToSeekTarget
{
	HARMONIXMETASOUND_API const Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();
	
	namespace  Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Bar);
		DECLARE_METASOUND_PARAM_ALIAS(Beat);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(SeekTarget);
	}
}

namespace HarmonixMetasound::Nodes::MusicTimeStampToSeekTarget
{
	HARMONIXMETASOUND_API const Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();
	
	namespace  Inputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(MusicTimestamp);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(SeekTarget);
	}
}

namespace HarmonixMetasound::Nodes::TimeMsToSeekTarget
{
	HARMONIXMETASOUND_API const Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();
	
	namespace  Inputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(TimeMs);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(SeekTarget);
	}
}