// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::Peak
{
	const HARMONIXMETASOUND_API Metasound::FNodeClassName& GetClassName();
	
	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enable);
		DECLARE_METASOUND_PARAM_ALIAS(AudioMono);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(Peak);
	}
}
