// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::SecsUntilMusicTimestampNode
{
	const HARMONIXMETASOUND_API Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enable);
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_ALIAS(Timestamp);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(SecsUntilTimestamp)
	}
}