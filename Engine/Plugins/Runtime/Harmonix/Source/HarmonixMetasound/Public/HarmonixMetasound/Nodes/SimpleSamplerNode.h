// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMetasound/Common.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundOperatorData.h"

#include "SimpleSamplerNode.generated.h"

namespace HarmonixMetasound::SimpleSamplerNode
{
	class FSimpleSamplerNodeOperatorData;
}

USTRUCT()
struct FHarmonixSimpleSamplerNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", ClampMax = "8"))
	uint32 NumChannels = 1;

	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", ClampMax = "10"))
	float MaxCaptureSeconds = 5.0f;

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const;

	TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};

namespace HarmonixMetasound::SimpleSamplerNode
{
	class FSimpleSamplerNodeOperatorData : public Metasound::TOperatorData<FSimpleSamplerNodeOperatorData>
	{
	public:
		static const FLazyName OperatorDataTypeName;
		FSimpleSamplerNodeOperatorData(const float& InMaxCaptureSeconds)
			: MaxCaptureSeconds(InMaxCaptureSeconds)
		{}
		float MaxCaptureSeconds = 5.0f;
	};

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enabled);
		DECLARE_METASOUND_PARAM_EXTERN(Capture);
		DECLARE_METASOUND_PARAM_EXTERN(CaptureDuration);
		DECLARE_METASOUND_PARAM_EXTERN(Play);
		DECLARE_METASOUND_PARAM_EXTERN(PlayDuration);
		DECLARE_METASOUND_PARAM_EXTERN(Reverse);
		DECLARE_METASOUND_PARAM_EXTERN(Reset);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(InputAudioInUse);
	}

	static constexpr int32 kMaxChannels = 8;
	static constexpr int32 kNumFadeFrames = 32;
}
