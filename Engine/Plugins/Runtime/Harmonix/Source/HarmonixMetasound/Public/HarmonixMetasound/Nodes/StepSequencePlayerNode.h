// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::StepSequencePlayer
{
	HARMONIXMETASOUND_API const Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(SequenceAsset);
		DECLARE_METASOUND_PARAM_EXTERN(VelocityMultiplier);
		DECLARE_METASOUND_PARAM_EXTERN(MaxColumns);
		DECLARE_METASOUND_PARAM_EXTERN(AdditionalOctaves);
		DECLARE_METASOUND_PARAM_EXTERN(StepSizeQuarterNotes);
		DECLARE_METASOUND_PARAM_EXTERN(ActivePage);
		DECLARE_METASOUND_PARAM_EXTERN(AutoPage);
		DECLARE_METASOUND_PARAM_EXTERN(AutoPagePlaysBlankPages);
		DECLARE_METASOUND_PARAM_ALIAS(Transport);
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_ALIAS(Speed);
		DECLARE_METASOUND_PARAM_ALIAS(Loop);
		DECLARE_METASOUND_PARAM_ALIAS(Enabled);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(MidiStream);
	}
}
