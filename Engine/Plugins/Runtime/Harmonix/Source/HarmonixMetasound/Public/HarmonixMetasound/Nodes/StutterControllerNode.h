// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMetasound/Common.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundOperatorData.h"

namespace HarmonixMetasound::StutterSequencerNode
{
	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enabled);
		DECLARE_METASOUND_PARAM_EXTERN(StutterSequence);
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
		DECLARE_METASOUND_PARAM_EXTERN(Start);
		DECLARE_METASOUND_PARAM_EXTERN(Stop);
		DECLARE_METASOUND_PARAM_EXTERN(CaptureDuration);
		DECLARE_METASOUND_PARAM_EXTERN(Spacing);
		DECLARE_METASOUND_PARAM_EXTERN(AudibleDuration);
		DECLARE_METASOUND_PARAM_EXTERN(Reverse);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(Capture);
		DECLARE_METASOUND_PARAM_EXTERN(CaptureDuration);
		DECLARE_METASOUND_PARAM_EXTERN(Play);
		DECLARE_METASOUND_PARAM_EXTERN(PlayDuration);
		DECLARE_METASOUND_PARAM_EXTERN(Reverse);
		DECLARE_METASOUND_PARAM_EXTERN(Reset);
	}
}
