// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMetasound/Common.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundOperatorData.h"

#include "TapeSSNode.generated.h"

namespace HarmonixMetasound::TapeSSNode
{
	class FTapeSSNodeOperatorData;
}

USTRUCT()
struct FHarmonixTapeSSNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", ClampMax = "8"))
	uint32 NumChannels = 1;

	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", ClampMax = "10"))
	float MaxRampSeconds = 5.0f;

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const;

	TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};

namespace HarmonixMetasound::TapeSSNode
{
	class FTapeSSNodeOperatorData : public Metasound::TOperatorData<FTapeSSNodeOperatorData>
	{
	public:
		static const FLazyName OperatorDataTypeName;
		FTapeSSNodeOperatorData(const float& InMaxRampSeconds)
			: MaxRampSeconds(InMaxRampSeconds)
		{}
		float MaxRampSeconds = 5.0f;
	};

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Enabled);
		DECLARE_METASOUND_PARAM_EXTERN(Stop);
		DECLARE_METASOUND_PARAM_EXTERN(RampDuration);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_EXTERN(InputAudioInUse);
		DECLARE_METASOUND_PARAM_EXTERN(OnStopped);
		DECLARE_METASOUND_PARAM_EXTERN(OnFullSpeed);
		DECLARE_METASOUND_PARAM_EXTERN(CurrentSpeed);
	}

	static constexpr int32 kMaxChannels = 8;
}
