// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::TransportWavePlayerControllerNode
{
	const HARMONIXMETASOUND_API Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Transport);
		DECLARE_METASOUND_PARAM_ALIAS(MidiClock);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(TransportPlay);
		DECLARE_METASOUND_PARAM_ALIAS(TransportStop);
		DECLARE_METASOUND_PARAM_EXTERN(StartTime);
	}
}