// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"

#include "HarmonixMetasound/Common.h"

namespace HarmonixMetasound::Nodes::TriggerToTransportNode
{
	const HARMONIXMETASOUND_API Metasound::FNodeClassName& GetClassName();
	HARMONIXMETASOUND_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(TransportPrepare);
		DECLARE_METASOUND_PARAM_ALIAS(TransportPlay);
		DECLARE_METASOUND_PARAM_ALIAS(TransportPause);
		DECLARE_METASOUND_PARAM_ALIAS(TransportContinue);
		DECLARE_METASOUND_PARAM_ALIAS(TransportStop);
		DECLARE_METASOUND_PARAM_ALIAS(TransportKill);
		DECLARE_METASOUND_PARAM_ALIAS(TriggerSeek);
		DECLARE_METASOUND_PARAM_ALIAS(SeekDestination);
	}

	namespace Outputs
	{
		DECLARE_METASOUND_PARAM_ALIAS(Transport);
	}
}