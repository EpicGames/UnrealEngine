// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"
#include "Delegates/IDelegateInstance.h"
#include "Subsystems/EngineSubsystem.h"

#include "HarmonixMetasound/Analysis/SpmcAnalysisResultQueue.h"
#include "HarmonixMetasound/Analysis/MidiClockSongPos.h"
#include "Analysis/MetasoundFrontendAnalyzerAddress.h"

#include "MidiClockUpdateSubsystem.generated.h"

#define UE_API HARMONIXMETASOUND_API

class UMusicClockComponent;
class UMusicClock;
class IMusicSource;

namespace HarmonixMetasound
{
	class FMidiClock;
}

/**
 * Handles updating low-resolution play cursors associated with a MIDI clock. This now includes both FMidiClock
 * and UMusicClockComponents.
 * MidiClock: Because FMidiClock instances do not have their lifecycle managed by the garbage collector,
 * we need a way to tick them on the game thread while avoiding races. This gives us a way to register clocks to be
 * ticked from within their constructors/destructor, and provides no user-facing API.
 * MusicClockComponent: Because the "current music time" is typically of interest to many game systems, and some of
 * those systems run in parallel in different threads (e.g. TickComponent functions and animation jobs), it is 
 * important that the current music time is updated at the beginning of the game frame, and then that same time
 * can be used for all systems for the frame. 
 * This subsystem solves for those problems. 
 */
UCLASS(MinimalAPI)
class UMidiClockUpdateSubsystem final : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	// Begin USubsystem
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	// End USubsystem

	static UE_API void TrackMusicClockComponent(UMusicClockComponent* Clock);
	static UE_API void StopTrackingMusicClockComponent(UMusicClockComponent* Clock);

	/** Track a music source for per-frame updates. The source's Update() will be called each frame. */
	static UE_API void TrackMusicSource(UObject* Source);
	static UE_API void StopTrackingMusicSource(UObject* Source);

	/** Track a music clock for per-frame updates. The clock's UpdateForGameFrame() will be called each frame. */
	static UE_API void TrackMusicClock(UMusicClock* Clock);
	static UE_API void StopTrackingMusicClock(UMusicClock* Clock);

	static UE_API double GetUpdateTimeSeconds();

	using FClockHistoryPtr = TSharedPtr<HarmonixMetasound::Analysis::FMidiClockSongPositionHistory>;
	static UE_API FClockHistoryPtr GetOrCreateClockHistory(const Metasound::Frontend::FAnalyzerAddress& ForAddress);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMusicClockComponentsUpdated, const TArray<TWeakObjectPtr<UMusicClockComponent>>&);
	FOnMusicClockComponentsUpdated& OnMusicClockComponentsUpdated()
	{
		check(IsInGameThread());
		return OnMusicClockComponentsUpdatedDelegate;
	}

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTrackMusicClockComponentChange, UMusicClockComponent*);
	FOnTrackMusicClockComponentChange& OnTrackMusicClockComponent()
	{
		check(IsInGameThread());
		return OnTrackMusicClockComponentDelegate;
	}

	FOnTrackMusicClockComponentChange& OnStopTrackingMusicClockComponent()
	{
		check(IsInGameThread());
		return OnStopTrackingMusicClockComponentDelegate;
	}
	
	const TArray<TWeakObjectPtr<UMusicClockComponent>>& GetTrackedMusicClockComponents() const
	{
		return TrackedMusicClockComponents;
	}

private:

	mutable FCriticalSection TrackedMidiClocksMutex;
	static double UpdateTimeSeconds;

	TArray<TWeakObjectPtr<UMusicClockComponent>> TrackedMusicClockComponents;
	UE_API void TrackMusicClockComponentImpl(UMusicClockComponent* Clock);
	UE_API void StopTrackingMusicClockComponentImpl(UMusicClockComponent* Clock);
	UE_API void UpdateUMusicClockComponents();

	/** Music sources tracked for per-frame Update() calls. */
	TArray<TWeakObjectPtr<UObject>> TrackedMusicSources;
	UE_API void TrackMusicSourceImpl(UObject* Source);
	UE_API void StopTrackingMusicSourceImpl(UObject* Source);
	UE_API void UpdateMusicSources();

	/** Music clocks tracked for per-frame UpdateForGameFrame() calls. */
	TArray<TWeakObjectPtr<UMusicClock>> TrackedMusicClocks;
	UE_API void TrackMusicClockImpl(UMusicClock* Clock);
	UE_API void StopTrackingMusicClockImpl(UMusicClock* Clock);
	UE_API void UpdateMusicClocks();

	FDelegateHandle EngineSamplingInputDelegate;
	UE_API void CoreDelegatesSamplingInput();

	static UE_API FCriticalSection ClockHistoryMapLocker;
	static UE_API TMap<uint32, TWeakPtr<HarmonixMetasound::Analysis::FMidiClockSongPositionHistory>> ClockHistories;

	static UE_API uint32 MakeMidiSongPosAnalyzerAddressHash(const Metasound::Frontend::FAnalyzerAddress& ForAddress);

	FOnTrackMusicClockComponentChange OnTrackMusicClockComponentDelegate;
	FOnTrackMusicClockComponentChange OnStopTrackingMusicClockComponentDelegate;
	FOnMusicClockComponentsUpdated OnMusicClockComponentsUpdatedDelegate;

public:
	// Declare a "tick" method that can be used during automated testing so that
	// the test code doesn't need knowledge of how the low-res clocks are being ticked...
	UE_API void TickForTesting();
};

#undef UE_API
