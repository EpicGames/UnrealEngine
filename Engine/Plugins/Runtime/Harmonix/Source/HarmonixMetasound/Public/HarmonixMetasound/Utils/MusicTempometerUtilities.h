// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MusicTempometerUtilities.generated.h"

class UMaterialParameterCollection;
class UMaterialParameterCollectionInstance;
class UMusicClockComponent;

struct FMidiSongPos;

USTRUCT(BlueprintType)
struct FMusicTempometerMPCParameterNames
{
	GENERATED_BODY()

	/**
	 * Seconds from the beginning of the entire music authoring.
	 * Includes all count-in and pickup bars (ie. won't be negative when 
	 * the music starts, and bar 1 beat 1 may not be at 0.0 seconds!
	 */
	UPROPERTY(EditDefaultsOnly, Category="MusicClock")
	FName SecondsIncludingCountInParameterName = "MusicSecondsIncludingCountIn";

	/**
	 * Bars from the beginning of the music. Includes all
	 * count-in and pickup bars (ie. won't be negative when the music starts,
	 * and bar 1 beat 1 of the music may not be equal to elapsed bar 0.0!
	 * NOTE: This Bar, unlike the Bar in the MusicTimestamp structure, is
	 * floating point and will include a fractional portion.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName BarsIncludingCountInParameterName = "MusicBarsIncludingCountIn";

	/**
	 * Total Beats from the beginning of the music. Includes all
	 * count-in and pickup bars/beats (ie. won't be negative when the music starts,
	 * and elapsed beat 0.0 may not equal a timestamp of bar 1 beat 1!
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName BeatsIncludingCountInParameterName = "MusicBeatsIncludingCountIn";

	/**
	 * Seconds from Bar 1 Beat 1 of the music. If the music has a count-in
	 * or pickup bars this number may be negative when the music starts!
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName SecondsFromBarOneParameterName = "MusicSecondsFromBarOne";

	/**
	 * This is the bar member of the MusicTimestamp structure which represents a classic
	 * "music time" where bar 1 beat 1 is the beginning of the music AFTER count-in and pickups.
	 * If the music has a count-in or pickup bars this number may be negative (or zero) until
	 * bar 1 beat 1 is reached!
	 * NOTE: Unlike "BarsFromCountIn", Bar here will be an integer! This because it is the bar
	 * from the MusicTimestamp structure which also provides a "beat in bar" member to denote
	 * a "distance into the bar".
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName TimestampBarParameterName = "MusicTimestampBar";

	/**
	 * This is the beat member of the MusicTimestamp structure which represents a classic
	 * "music time" where bar 1 beat 1 is the beginning of the music AFTER count-in and pickups.
	 * If the music has a count-in or pickup bars this number may be negative (or zero) until
	 * bar 1 beat 1 is reached! NOTE: beat in this context is 1 based! The first beat in a bar
	 * is beat 1!
	 * Note: This is a floating point number.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName TimestampBeatInBarParameterName = "MusicTimestampBeatInBar";

	/**
	 * Progress of the current bar [0, 1]. This is the same as the
	 * fractional part of BarsFromCountIn.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName BarProgressParameterName = "MusicBarProgress";

	/**
	 * Progress of the current beat [0, 1]. This is the same as the
	 * fractional part of BeatsFromCountIn;
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName BeatProgressParameterName = "MusicBeatProgress";

	/**
	 * Current time signature numerator (beats per bar).
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName TimeSignatureNumeratorParameterName = "MusicTimeSignatureNumerator";

	/**
	 * Current time signature denominator (scale from note duration to beat).
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName TimeSignatureDenominatorParameterName = "MusicTimeSignatureDenominator";

	/**
	 * Current tempo (beats per minute).
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName TempoParameterName = "MusicTempo";

	/**
	 * 0 or 1, represents whether the rest of the parameters are defaults (0) or the current state of a clock (1).
	 */
	UPROPERTY(EditDefaultsOnly, Category = "MusicClock")
	FName TimestampValidParameterName = "MusicTimestampValid";

	bool IsValid() const
	{
		// Valid if there is an MPC collection assigned and at least one named parameter
		return	!SecondsIncludingCountInParameterName.IsNone() ||
				!BarsIncludingCountInParameterName.IsNone() ||
				!BeatsIncludingCountInParameterName.IsNone() ||
				!SecondsFromBarOneParameterName.IsNone() ||
				!TimestampBarParameterName.IsNone() ||
				!TimestampBeatInBarParameterName.IsNone() ||
				!BarProgressParameterName.IsNone() ||
				!BeatProgressParameterName.IsNone() ||
				!TimeSignatureNumeratorParameterName.IsNone() ||
				!TimeSignatureDenominatorParameterName.IsNone() ||
				!TempoParameterName.IsNone() ||
				!TimestampValidParameterName.IsNone();
	}

	friend bool operator ==(const FMusicTempometerMPCParameterNames& Lhs, const FMusicTempometerMPCParameterNames& Rhs) = default;
	friend bool operator !=(const FMusicTempometerMPCParameterNames& Lhs, const FMusicTempometerMPCParameterNames& Rhs) = default;
};
/**
 * Parameters used when creating/updating the corresponding Material Parameter Collection
  */
USTRUCT(BlueprintType)
struct FMusicTempometerMPCParameters
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category = "MusicClock", meta = (ShowOnlyInnerProperties))
	FMusicTempometerMPCParameterNames CurrentFrameParameterNames;

	UPROPERTY(EditDefaultsOnly, Category = "MusicClock", meta = (ShowOnlyInnerProperties))
	FMusicTempometerMPCParameterNames PreviousFrameParameterNames{
		"PreviousFrameMusicSecondsIncludingCountIn",
		"PreviousFrameMusicBarsIncludingCountIn",
		"PreviousFrameMusicBeatsIncludingCountIn",
		"PreviousFrameMusicSecondsFromBarOne",
		"PreviousFrameMusicTimestampBar",
		"PreviousFrameMusicTimestampBeatInBar",
		"PreviousFrameMusicBarProgress",
		"PreviousFrameMusicBeatProgress",
		"PreviousFrameMusicTimeSignatureNumerator",
		"PreviousFrameMusicTimeSignatureDenominator",
		"PreviousFrameMusicTempo",
		"PreviousFrameMusicTimestampValid" };

	bool IsValid() const
	{
		return CurrentFrameParameterNames.IsValid() || PreviousFrameParameterNames.IsValid();
	}
};

/**
 * Utility methods used by the UMusicTempometerComponent, also publically exposed for other systems to use.
 */
namespace MusicTempometerUtilities
{
	/**
	 * Update the supplied material parameter collection instance (create if null) with song position data extracted from the supplied clock
	 */
	UE_DEPRECATED(5.6, "This implementation of UpdateMaterialParameterCollectionFromClock() is deprecated.  Please use the implementation with an 'InPreviousFrameMidiSongPos' argument instead.")
	void HARMONIXMETASOUND_API UpdateMaterialParameterCollectionFromClock(const UObject* InWorldContextObject, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance, const TObjectPtr<UMaterialParameterCollection>& InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMCPParameters, const UMusicClockComponent* InClockComponent);
	
	/**
	 * Update the supplied material parameter collection instance (create if null) with data from the supplied song position
	 */
	UE_DEPRECATED(5.6, "This implementation of UpdateMaterialParameterCollectionFromSongPos() is deprecated.  Please use the implementation with an 'InPreviousFrameMidiSongPos' argument instead.")
	void HARMONIXMETASOUND_API UpdateMaterialParameterCollectionFromSongPos(const UObject* InWorldContextObject, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance, const TObjectPtr<UMaterialParameterCollection>& InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMCPParameters, const FMidiSongPos& InMidiSongPos);

	/**
	 * Update the supplied material parameter collection instance (create from the supplied material parameter collection if null) from the supplied clock and song position, and return the current song position according to the supplied clock.
	 */
	FMidiSongPos HARMONIXMETASOUND_API UpdateMaterialParameterCollectionFromClock(const UMusicClockComponent* InClockComponent, const FMidiSongPos& InPreviousFrameMidiSongPos, const UMaterialParameterCollection* InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMPCParameters, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance);

	/**
	 * Update the supplied material parameter collection instance (create from the supplied material parameter collection if null) from the supplied song positions.
	 */
	void HARMONIXMETASOUND_API UpdateMaterialParameterCollectionFromSongPos(const UObject* InWorldContextObject, const FMidiSongPos& InCurrentFrameMidiSongPos, const FMidiSongPos& InPreviousFrameMidiSongPos, const UMaterialParameterCollection* InMaterialParameterCollection, const FMusicTempometerMPCParameters& InMPCParameters, TWeakObjectPtr<UMaterialParameterCollectionInstance>& InOutMaterialParameterCollectionInstance);
};
