// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixMetasoundEditor : ModuleRules
{
	public HarmonixMetasoundEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		//OptimizeCode = CodeOptimization.Never;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore",
				"PropertyEditor"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"UnrealEd",
				"HarmonixMidi",
				"UnrealEd",
				"AssetDefinition",
				"HarmonixMetasound",
				"MetasoundEditor",
				"MetasoundFrontend",
				"MetasoundGraphCore",
				"DetailCustomizations",
				"Projects"
			}
		);
	}
}
