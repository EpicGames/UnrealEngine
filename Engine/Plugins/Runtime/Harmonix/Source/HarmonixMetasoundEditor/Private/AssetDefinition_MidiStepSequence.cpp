// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MidiStepSequence.h"
#include "HarmonixMetasound/DataTypes/MidiStepSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MidiStepSequence)

#define LOCTEXT_NAMESPACE "AssetDefinition_MidiStepSequence"

TSoftClassPtr<UObject> UAssetDefinition_MidiStepSequence::GetAssetClass() const
{
	return UMidiStepSequence::StaticClass();
}

FText UAssetDefinition_MidiStepSequence::GetAssetDisplayName() const
{
	return LOCTEXT("MIDIStepSequenceDefinition", "MIDI Step Sequence");
}

FLinearColor  UAssetDefinition_MidiStepSequence::GetAssetColor() const
{
	return FLinearColor(1.0f, 0.5f, 0.0f);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MidiStepSequence::GetAssetCategories() const
{
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio, 
				LOCTEXT("AssetDefinition_MidiStepSequenceSubMenu", "Advanced"), 
				FCategoryPath(LOCTEXT("AssetDefinition_MidiStepSequenceSubMenuSection", "Harmonix"), ECategoryMenuType::Section))
		};
	return Categories;
}

bool UAssetDefinition_MidiStepSequence::CanImport() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
