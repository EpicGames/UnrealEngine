// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MidiStutterSequence.h"
#include "HarmonixMetasound/DataTypes/MidiStutterSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MidiStutterSequence)

#define LOCTEXT_NAMESPACE "AssetDefinition_MidiStutterSequence"

TSoftClassPtr<UObject> UAssetDefinition_MidiStutterSequence::GetAssetClass() const
{
	return UMidiStutterSequence::StaticClass();
}

FText UAssetDefinition_MidiStutterSequence::GetAssetDisplayName() const
{
	return LOCTEXT("MIDIStutterSequenceDefinition", "MIDI Stutter Sequence");
}

FLinearColor  UAssetDefinition_MidiStutterSequence::GetAssetColor() const
{
	return FLinearColor(1.0f, 0.5f, 0.0f);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MidiStutterSequence::GetAssetCategories() const
{
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio, 
				LOCTEXT("AssetDefinition_MidiStutterSequenceSubMenu", "Advanced"), 
				FCategoryPath(LOCTEXT("AssetDefinition_MidiStutterSequenceSubMenuSection", "Harmonix"), ECategoryMenuType::Section))
		};
	return Categories;
}

bool UAssetDefinition_MidiStutterSequence::CanImport() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
