// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "AssetDefinitionDefault.h"

#include "AssetDefinition_MidiStutterSequence.generated.h"

#define ASSET_MENU_ITEM_GROUPING_CHEKED_IN 0

UCLASS()
class UAssetDefinition_MidiStutterSequence : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override;

};
