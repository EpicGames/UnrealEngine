// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MusicAssets.h"
#include "HarmonixMetasound/DataTypes/HarmonixMetasoundMusicAsset.h"
#include "HarmonixMetasound/DataTypes/HarmonixWaveMusicAsset.h"
#include "HarmonixMetasoundSlateStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MusicAssets)

#define LOCTEXT_NAMESPACE "AssetDefinition_MusicAssets"

TSoftClassPtr<UObject> UAssetDefinition_MetasoundMusic::GetAssetClass() const
{
	return UHarmonixMetasoundMusicAsset::StaticClass();
}

FText UAssetDefinition_MetasoundMusic::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "HarmonixMetasoundMusicDefinition", "Harmonix MetaSound Music Asset");
}

FLinearColor  UAssetDefinition_MetasoundMusic::GetAssetColor() const
{
	return FLinearColor(1.0f, 0.5f, 0.0f);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MetasoundMusic::GetAssetCategories() const
{
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio, 
				LOCTEXT("AssetDefinition_MetasoundMusicSubMenu", "Advanced"), 
				FCategoryPath(LOCTEXT("AssetDefinition_MetasoundMusicSubMenuSection", "Harmonix"), ECategoryMenuType::Section))
		};
	return Categories;
}

bool UAssetDefinition_MetasoundMusic::CanImport() const
{
	return false;
}

const FSlateBrush* UAssetDefinition_MetasoundMusic::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return HarmonixMetasoundEditor::FSlateStyle::Get().GetBrush("HarmonixMetasoundEditor.MetasoundMusic.Icon");
}

const FSlateBrush* UAssetDefinition_MetasoundMusic::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return HarmonixMetasoundEditor::FSlateStyle::Get().GetBrush("HarmonixMetasoundEditor.MetasoundMusic.Thumbnail");
}

TSoftClassPtr<UObject> UAssetDefinition_WaveMusic::GetAssetClass() const
{
	return UHarmonixWaveMusicAsset::StaticClass();
}

FText UAssetDefinition_WaveMusic::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "HarmonixWaveMusicDefinition", "Harmonix Wave Music Asset");
}

FLinearColor  UAssetDefinition_WaveMusic::GetAssetColor() const
{
	return FLinearColor(1.0f, 0.5f, 0.0f);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_WaveMusic::GetAssetCategories() const
{
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio, 
				LOCTEXT("AssetDefinition_WaveMusicSubMenu", "Advanced"), 
				FCategoryPath(LOCTEXT("AssetDefinition_WaveMusicSubMenuSection", "Harmonix"), ECategoryMenuType::Section))
		};
	return Categories;
}

bool UAssetDefinition_WaveMusic::CanImport() const
{
	return false;
}

const FSlateBrush* UAssetDefinition_WaveMusic::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return HarmonixMetasoundEditor::FSlateStyle::Get().GetBrush("HarmonixMetasoundEditor.WaveMusic.Icon");
}

const FSlateBrush* UAssetDefinition_WaveMusic::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return HarmonixMetasoundEditor::FSlateStyle::Get().GetBrush("HarmonixMetasoundEditor.WaveMusic.Thumbnail");
}

#undef LOCTEXT_NAMESPACE
