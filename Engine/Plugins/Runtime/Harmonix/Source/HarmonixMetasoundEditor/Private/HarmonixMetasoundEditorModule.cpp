// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasoundEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"
#include "IMetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorModule.h"
#include "HarmonixMetasoundSlateStyle.h"
#include "MidiStepSequenceDetailCustomization.h"
#include "HarmonixMetasound/DataTypes/MidiStepSequence.h"
#include "AssetDefinition_MidiStepSequence.h"
#include "MetasoundDataReference.h"


#define LOCTEXT_NAMESPACE "HarmonixMetaSound"

DEFINE_LOG_CATEGORY(LogHarmonixMetasoundEditor)

void FHarmonixMetasoundEditorModule::StartupModule()
{
	using namespace Metasound::Editor;

	const HarmonixMetasoundEditor::FSlateStyle& Style = HarmonixMetasoundEditor::FSlateStyle::Get();
	IMetasoundEditorModule& MetasoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
	TSharedRef<IMetaSoundGraphPanelPinFactory> PinFactory = MetasoundEditorModule.GetGraphPanelPinFactory();
	PinFactory->RegisterPin("MIDIAsset");
	PinFactory->RegisterPin("MIDIStepSequenceAsset");
	PinFactory->RegisterPin("StutterSequenceAsset");
	PinFactory->RegisterPin("FusionPatchAsset");
	PinFactory->RegisterPin("Enum:SubdivisionQuantizationType", FGraphPinParams { .PinCategory = "Int32" });
	PinFactory->RegisterPin("Enum:DelayFilterType", FGraphPinParams { .PinCategory = "Int32" });
	PinFactory->RegisterPin("Enum:DelayStereoType", FGraphPinParams { .PinCategory = "Int32" });
	PinFactory->RegisterPin("Enum:TimeSyncOption", FGraphPinParams { .PinCategory = "Int32" });
	PinFactory->RegisterPin("Enum:DistortionType", FGraphPinParams { .PinCategory = "Int32" });
	PinFactory->RegisterPin("Enum:Harmonix:BiquadFilterType", FGraphPinParams { .PinCategory = "Int32" });
	PinFactory->RegisterPin("Enum:Distortion:FilterPasses", FGraphPinParams { .PinCategory = "Int32" });
	PinFactory->RegisterPin("Enum:StdMIDIControllerID", FGraphPinParams { .PinCategory = "Int32" });

	auto RegisterCustomPinType = [&Style, &PinFactory](FName PinType)
	{
		FGraphPinParams PinParams;
		PinParams.PinCategory = PinType;
		PinParams.PinColor = &Style.GetPinColor(PinType);
		PinParams.PinConnectedIcon = Style.GetConnectedIcon(PinType);
		PinParams.PinDisconnectedIcon = Style.GetDisconnectedIcon(PinType);
		PinFactory->RegisterPin(PinType, PinParams);

		PinParams.PinConnectedIcon = nullptr;
		PinParams.PinDisconnectedIcon = nullptr;
		PinFactory->RegisterPin(Metasound::CreateArrayTypeNameFromElementTypeName(PinType), PinParams);
	};

	RegisterCustomPinType("MIDIStream");
	RegisterCustomPinType("MIDIClock");
	RegisterCustomPinType("MusicTransport");

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(
		UMidiStepSequence::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMidiStepSequenceDetailCustomization::MakeInstance)
	);

	PropertyModule.NotifyCustomizationModuleChanged();
}

void FHarmonixMetasoundEditorModule::ShutdownModule()
{
	using namespace Metasound::Editor;
	if (FModuleManager::Get().IsModuleLoaded(IMetasoundEditorModule::ModuleName))
	{
		IMetasoundEditorModule& Module = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
		TSharedRef<IMetaSoundGraphPanelPinFactory> PinFactory = Module.GetGraphPanelPinFactory();
		PinFactory->UnregisterPin("MIDIAsset");
		PinFactory->UnregisterPin("MIDIStepSequenceAsset");
		PinFactory->UnregisterPin("StutterSequenceAsset");
		PinFactory->UnregisterPin("FusionPatchAsset");
		PinFactory->UnregisterPin("Enum:SubdivisionQuantizationType");
		PinFactory->UnregisterPin("Enum:DelayFilterType");
		PinFactory->UnregisterPin("Enum:DelayStereoType");
		PinFactory->UnregisterPin("Enum:TimeSyncOption");
		PinFactory->UnregisterPin("Enum:DistortionType");
		PinFactory->UnregisterPin("Enum:Harmonix:BiquadFilterType");
		PinFactory->UnregisterPin("Enum:Distortion:FilterPasses");
		PinFactory->UnregisterPin("Enum:StdMIDIControllerID");

		auto UnregisterCustomPinType = [&PinFactory](FName PinType)
		{
			PinFactory->UnregisterPin(PinType);
			PinFactory->UnregisterPin(Metasound::CreateArrayTypeNameFromElementTypeName(PinType));
		};
		UnregisterCustomPinType("MIDIStream");
		UnregisterCustomPinType("MIDIClock");
		UnregisterCustomPinType("MusicTransport");
	}
}

IMPLEMENT_MODULE(FHarmonixMetasoundEditorModule, HarmonixMetasoundEditor);

#undef LOCTEXT_NAMESPACE
