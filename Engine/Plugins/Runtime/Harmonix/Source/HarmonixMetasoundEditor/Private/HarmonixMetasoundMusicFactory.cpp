// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasoundMusicFactory.h"
#include "HarmonixMetasound/DataTypes/HarmonixMetasoundMusicAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixMetasoundMusicFactory)

UHarmonixMetasoundMusicFactory::UHarmonixMetasoundMusicFactory()
{
	SupportedClass = UHarmonixMetasoundMusicAsset::StaticClass();
	bCreateNew = true;
}

FText UHarmonixMetasoundMusicFactory::GetDisplayName() const
{
	return NSLOCTEXT("Harmonix", "MetasoundMusicAssetDisplayName", "Harmonix MetaSound Music Asset");
}

UObject* UHarmonixMetasoundMusicFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UHarmonixMetasoundMusicAsset>(InParent, Class, Name, Flags, Context);
}
