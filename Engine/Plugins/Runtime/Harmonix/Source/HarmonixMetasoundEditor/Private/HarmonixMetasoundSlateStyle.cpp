// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasoundSlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"

namespace HarmonixMetasoundEditor
{
	static const FString ModuleName = TEXT("HarmonixMetasound");
	static const FString PinColorName = TEXT("PinColor");
	static const FString ConnectedIconName = TEXT("ConnectedIcon");
	static const FString DisconnectedIconName = TEXT("DisconnectedIcon");
	
	FSlateStyle::FSlateStyle()
		: FSlateStyleSet("HarmonixMetasoundSlateStyle")
	{
		const FLinearColor MidiStreamColor = FColor(117, 106, 182); // pastel purple
		const FLinearColor MidiClockColor = FColor(172, 135, 197); // pastel light purple
		const FLinearColor TransportColor = FColor(255, 229, 229); // pastel cream
		const FVector2D Icon22x22(22.0f, 22.0f);
		const FVector2D Icon18x10(18.0f, 10.0f);
		const FVector2D Icon18x18(18.0f, 18.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("Harmonix"));
		check(Plugin);
		SetContentRoot(Plugin->GetContentDir() / TEXT("Editor/Slate"));

		SetCustomPinStyle("MIDIStream", MidiStreamColor, new IMAGE_BRUSH(TEXT("Icons/MidiConnectedPin"), Icon22x22), new IMAGE_BRUSH(TEXT("Icons/MidiDisconnectedPin"), Icon22x22));
		SetCustomPinStyle("MIDIClock", MidiClockColor, new IMAGE_BRUSH(TEXT("Icons/ClockConnectedPin"), Icon22x22), new IMAGE_BRUSH(TEXT("Icons/ClockDisconnectedPin"), Icon22x22));
		SetCustomPinStyle("MusicTransport", TransportColor, new IMAGE_BRUSH(TEXT("Icons/TransportConnectedPin"), Icon18x10), new IMAGE_BRUSH(TEXT("Icons/TransportDisconnectedPin"), Icon18x10));

		Set("HarmonixMetasoundEditor.MetasoundMusic.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/harmonixmetasoundmusic_icon"), Icon20x20));
		Set("HarmonixMetasoundEditor.WaveMusic.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/harmonixwavemusic_icon"), Icon20x20));
		Set("HarmonixMetasoundEditor.MetasoundMusic.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/harmonixmetasoundmusic_thumbnail"), Icon20x20));
		Set("HarmonixMetasoundEditor.WaveMusic.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/harmonixwavemusic_thumbnail"), Icon20x20));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	const FSlateStyle& FSlateStyle::Get()
	{
		static FSlateStyle SlateStyle;
		return SlateStyle;
	}

	void FSlateStyle::SetCustomPinStyle(const FName PinType, const FLinearColor& PinColor, FSlateBrush* ConnectedIcon, FSlateBrush* DisconnectedIcon)
	{
		FString ColorProperty = FString::Printf(TEXT("%s.%s.%s"), *ModuleName, *PinType.ToString(), *PinColorName);
		FString ConnectedIconProperty = FString::Printf(TEXT("%s.%s.%s"), *ModuleName, *PinType.ToString(), *ConnectedIconName);
		FString DisconnectedIconProperty = FString::Printf(TEXT("%s.%s.%s"), *ModuleName, *PinType.ToString(), *DisconnectedIconName);
		Set(FName(ColorProperty), PinColor);
		Set(FName(ConnectedIconProperty), ConnectedIcon);
		Set(FName(DisconnectedIconProperty), DisconnectedIcon);
	}

	const FLinearColor& FSlateStyle::GetPinColor(FName PinType) const
	{
		FString PropertyName = FString::Printf(TEXT("%s.%s.%s"), *ModuleName, *PinType.ToString(), *PinColorName);
		return GetColor(FName(PropertyName));
	}

	const FSlateBrush* FSlateStyle::GetConnectedIcon(FName PinType) const
	{
		FString PropertyName = FString::Printf(TEXT("%s.%s.%s"), *ModuleName, *PinType.ToString(), *ConnectedIconName);
        return GetBrush(FName(PropertyName));
	}
	
	const FSlateBrush* FSlateStyle::GetDisconnectedIcon(FName PinType) const
	{
		FString PropertyName = FString::Printf(TEXT("%s.%s.%s"), *ModuleName, *PinType.ToString(), *DisconnectedIconName);
		return GetBrush(FName(PropertyName));
	}
	
	FSlateStyle::~FSlateStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}
