// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Styling/SlateStyle.h"

namespace HarmonixMetasoundEditor
{
	class FSlateStyle final
		: public FSlateStyleSet
	{
	public:
		FSlateStyle();
		static const FSlateStyle& Get();

		void SetCustomPinStyle(const FName PinType, const FLinearColor& PinColor, FSlateBrush* ConnectedIcon, FSlateBrush* DisconnectedIcon);

		const FLinearColor& GetPinColor(FName PinType) const;
		const FSlateBrush* GetConnectedIcon(FName PinType) const;
		const FSlateBrush* GetDisconnectedIcon(FName PinType) const;

		~FSlateStyle();
	};
}
