// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixWaveMusicFactory.h"
#include "HarmonixMetasound/DataTypes/HarmonixWaveMusicAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixWaveMusicFactory)

UHarmonixWaveMusicFactory::UHarmonixWaveMusicFactory()
{
	SupportedClass = UHarmonixWaveMusicAsset::StaticClass();
	bCreateNew = true;
}

FText UHarmonixWaveMusicFactory::GetDisplayName() const
{
	return NSLOCTEXT("Harmonix", "WaveMusicAssetDisplayName", "Harmonix Wave Music Asset");
}

UObject* UHarmonixWaveMusicFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UHarmonixWaveMusicAsset>(InParent, Class, Name, Flags, Context);
}
