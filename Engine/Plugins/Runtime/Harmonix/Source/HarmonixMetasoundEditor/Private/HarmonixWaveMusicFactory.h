// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "HarmonixWaveMusicFactory.generated.h"

UCLASS()
class UHarmonixWaveMusicFactory : public UFactory
{
	GENERATED_BODY()
public:
	UHarmonixWaveMusicFactory();
	//~ BEGIN UFactory interface
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ END UFactory interface
};

