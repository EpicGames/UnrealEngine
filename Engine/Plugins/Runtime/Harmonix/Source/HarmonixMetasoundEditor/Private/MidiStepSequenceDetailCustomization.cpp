// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiStepSequenceDetailCustomization.h"

void FMidiStepSequenceDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	//get the properties that do not need customization so that they can be placed on the top of the customized ones
	IDetailCategoryBuilder& MidiStepSequenceCategory = DetailLayout.EditCategory(TEXT("MidiStepSequence"));
	TSharedPtr<IPropertyHandle> PagesHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMidiStepSequence, Pages));
	MidiStepSequenceCategory.AddProperty(PagesHandle);

	TSharedPtr<IPropertyHandle> RowsHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMidiStepSequence, Rows));
	MidiStepSequenceCategory.AddProperty(RowsHandle);

	TSharedPtr<IPropertyHandle> ColumnsHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMidiStepSequence, Columns));
	MidiStepSequenceCategory.AddProperty(ColumnsHandle);
	
	//get a handle for the Step Table
	TSharedPtr<IPropertyHandle> StepTableHandle = DetailLayout.GetProperty("StepTable");

	//get the current step sequence being edited
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		return;
	}
	MidiStepSequence = Objects.Last();
	TWeakObjectPtr<UMidiStepSequence> MidiStepSequenceBeingEdited = Cast<UMidiStepSequence>(MidiStepSequence);

	//update number of pages and the current page number from the step sequence
	NumPages = MidiStepSequenceBeingEdited->Pages;
	const int32 CurrentPageNumberFromAsset = MidiStepSequenceBeingEdited->GetClampedCurrentPageNumber();
	CurrentPageNumber = CurrentPageNumberFromAsset;

	//callback for changes in number of pages 
	PagesHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, &MidiStepSequenceCategory, MidiStepSequenceBeingEdited, PagesHandle]()
		{
			int32 NewNumPages;
			PagesHandle->GetValue(NewNumPages);
			//if the current page number is greater than the newer number of pages,
			//set the current page number to the last page after setting the new number of pages
			if (CurrentPageNumber > NewNumPages)
			{
				MidiStepSequenceBeingEdited->SetClampedCurrentPageNumber(NewNumPages);
			}
			MidiStepSequenceCategory.GetParentLayout().ForceRefreshDetails();
		}));

	auto OnPreviousPageClicked = [this, StepTableHandle, &MidiStepSequenceCategory, MidiStepSequenceBeingEdited]
	{
		int32 NewCurrentPageNumber = MidiStepSequenceBeingEdited->GetClampedCurrentPageNumber() - 1;
		MidiStepSequenceBeingEdited->SetClampedCurrentPageNumber(NewCurrentPageNumber);
		MidiStepSequenceCategory.GetParentLayout().ForceRefreshDetails();
		return FReply::Handled();
	};

	auto OnNextPageClicked = [this, StepTableHandle, &MidiStepSequenceCategory, MidiStepSequenceBeingEdited]
	{
		int32 NewCurrentPageNumber = MidiStepSequenceBeingEdited->GetClampedCurrentPageNumber() + 1;
		MidiStepSequenceBeingEdited->SetClampedCurrentPageNumber(NewCurrentPageNumber);
		MidiStepSequenceCategory.GetParentLayout().ForceRefreshDetails();
		return FReply::Handled();
	};

	TSharedPtr<IPropertyHandle> CurrentPageNumberHandle =DetailLayout.GetProperty("CurrentPageNumber");
	CurrentPageNumberHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, &MidiStepSequenceCategory, MidiStepSequenceBeingEdited,CurrentPageNumberHandle]()
		{
			MidiStepSequenceCategory.GetParentLayout().ForceRefreshDetails();
		}));

	//add a step table title row
	MidiStepSequenceCategory.AddCustomRow(FText::FromString("Step Table"))
		.WholeRowContent()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString("Step Table"))
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		];

	//the actual customizations for the properties
	DrawCurrentPage(StepTableHandle, MidiStepSequenceCategory);

	//page navigation customization
	MidiStepSequenceCategory.AddCustomRow(FText::FromString("Step Table"))
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Text(FText::FromString("<"))
				.OnClicked_Lambda(OnPreviousPageClicked)
				.IsEnabled(CurrentPageNumber > 1)
				.ToolTipText(FText::FromString("Previous Page"))
			]
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString("Page"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5, 0, 5, 0)
				[
					CurrentPageNumberHandle->CreatePropertyValueWidget()
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Text(FText::FromString(">"))
					.OnClicked_Lambda(OnNextPageClicked)
					.IsEnabled(CurrentPageNumber != NumPages)
					.ToolTipText(FText::FromString("Next Page"))
				]
		];

	TSharedPtr<IPropertyHandle> SkipIndexHandle = StepTableHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceTable, StepSkipIndex));
	MidiStepSequenceCategory.AddProperty(SkipIndexHandle);

	//hide the non-customized versions of these properties
	DetailLayout.HideProperty(StepTableHandle);
	DetailLayout.HideProperty(CurrentPageNumberHandle);
}

void FMidiStepSequenceDetailCustomization::DrawCurrentPage(TSharedPtr<IPropertyHandle> StepTableHandle, IDetailCategoryBuilder& MidiStepSequenceCategory)
{
	//get handles for properties to be customized
	TSharedPtr<IPropertyHandleArray> NotesHandle = StepTableHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceTable, Notes))->AsArray();
	check(NotesHandle.IsValid());
	TSharedPtr<IPropertyHandleArray> PagesHandle = StepTableHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceTable, Pages))->AsArray();
	check(PagesHandle.IsValid());
	
	uint32 NumPagesInAsset;
	PagesHandle->GetNumElements(NumPagesInAsset);
	PagesHandle->SetOnNumElementsChanged(FSimpleDelegate::CreateLambda([this, &MidiStepSequenceCategory, NotesHandle]()
		{
			MidiStepSequenceCategory.GetParentLayout().ForceRefreshDetails();
		}));
	
	//customization for each page
	for (int32 PageIndex = 0; PageIndex < static_cast<int32>(NumPagesInAsset); ++PageIndex)
	{
		TSharedPtr<IPropertyHandle> CurrentPageHandle = PagesHandle->GetElement(PageIndex);
		check(CurrentPageHandle.IsValid());

		TSharedPtr<IPropertyHandleArray> RowsHandle = CurrentPageHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequencePage, Rows))->AsArray();
		check(RowsHandle.IsValid());

		uint32 NumRows;
		NotesHandle->GetNumElements(NumRows);

		//customize each row
		for (int32 RowIndex = 0; RowIndex < static_cast<int32>(NumRows); ++RowIndex)
		{
			TSharedPtr<IPropertyHandle> NoteHandle = NotesHandle->GetElement(RowIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceNote, NoteNumber));
			check(NoteHandle.IsValid());

			TSharedPtr<IPropertyHandle> VelocityHandle = NotesHandle->GetElement(RowIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceNote, Velocity));
			check(VelocityHandle.IsValid());

			TSharedPtr<IPropertyHandleArray> CellsHandle = RowsHandle->GetElement(RowIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceRow, Cells))->AsArray();
			check(CellsHandle.IsValid());

			uint32 NumCells;
			CellsHandle->GetNumElements(NumCells);
					
			FDetailWidgetRow& CustomRow = MidiStepSequenceCategory.AddCustomRow(FText::FromString("Step Table"));
			//note number and velocity
			TSharedPtr<SHorizontalBox> CustomRowWidget;
			SAssignNew(CustomRowWidget, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(0, 5)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(5,0,5,0)
						[
							NoteHandle->CreatePropertyNameWidget()
						]
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(5,0,5,0)
						[
							NoteHandle->CreatePropertyValueWidget()
						]
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(5, 0)
					.AutoWidth()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.Padding(10,0,15,0)
						[
							VelocityHandle->CreatePropertyNameWidget()
						]
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)					
						.Padding(10,0,15,0)
						[
							VelocityHandle->CreatePropertyValueWidget()
						]
					]
				];
			//customize cells by putting them into a container
			TSharedPtr<SScrollBox> CellWidgetContainer;
			SAssignNew(CellWidgetContainer, SScrollBox)
				.ScrollBarVisibility(RowIndex == static_cast<int32>(NumRows) - 1 ? EVisibility::Visible : EVisibility::Hidden)
				.OnUserScrolled_Lambda([this, RowIndex, NumRows](float Offset)
				{
						if (RowIndex == static_cast<int32>(NumRows) - 1)
						{
							for (int32 ScrollBoxIndex = 0; ScrollBoxIndex < CellWidgetContainerScrollBoxes.Num(); ++ScrollBoxIndex)
							{
								CellWidgetContainerScrollBoxes[ScrollBoxIndex]->SetScrollOffset(Offset);
							}
						}
				})
				.Orientation(Orient_Horizontal);
			for (int32 CellIndex = 0; CellIndex < static_cast<int32>(NumCells); ++CellIndex)
			{
				TSharedPtr<IPropertyHandle> EnabledHandle = CellsHandle->GetElement(CellIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceCell, bEnabled));
				check(EnabledHandle.IsValid());

				TSharedPtr<IPropertyHandle> ContinuationHandle = CellsHandle->GetElement(CellIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceCell, bContinuation));
				check(ContinuationHandle.IsValid());

				TSharedPtr<SButton> CellButton;
					CellWidgetContainer->AddSlot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(5)
					[
						SAssignNew(CellButton, SButton)
							.Text(FText::FromString(TEXT(" ")))
							.HAlign(HAlign_Left)
							.DesiredSizeScale(FVector2D(1.0f,1.0f))
							.ToolTipText_Lambda([this, EnabledHandle, ContinuationHandle, CellsHandle, NumCells, CellIndex]()
							{
									FText CellButtonToolTipText;
									bool bIsEnabledNow;
									EnabledHandle->GetValue(bIsEnabledNow);
									bool bContinuationEnabledNow;
									ContinuationHandle->GetValue(bContinuationEnabledNow);

									bool bIsContinuable = IsCellContinuable(EnabledHandle, ContinuationHandle, CellsHandle, static_cast<int32>(NumCells), CellIndex);
									
									if (bContinuationEnabledNow)
									{
										CellButtonToolTipText  = FText::FromString("continuation is enabled, click to disable this cell");
									}
									else {
										if (bIsContinuable)
										{
											CellButtonToolTipText = bIsEnabledNow ? FText::FromString("current cell is enabled, click once to enable continuation, click twice to disable it") : FText::FromString("current cell is disabled, click once to enable it, click twice to enable continuation");
										}
										else {
											CellButtonToolTipText = bIsEnabledNow ? FText::FromString("current cell is enabled, click to disable it") : FText::FromString("current cell is disabled, click to enable it");
										}
									}

									return CellButtonToolTipText;
							})
							.ButtonColorAndOpacity_Lambda([this, EnabledHandle, ContinuationHandle]()-> FSlateColor 
							{
								bool bIsEnabledNow;
								EnabledHandle->GetValue(bIsEnabledNow);
								bool bContinuationEnabledNow;
								ContinuationHandle->GetValue(bContinuationEnabledNow);

								//if cell is disabled, button color is gray
								FSlateColor CellButtonColor = FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f));
								//if cell is enabled, change button color to green
								if (bIsEnabledNow) CellButtonColor = FSlateColor(FLinearColor(124 / 255.f, 252 / 255.f, 0.f));
								//if cell's continuation is enabled, change button color to yellow
								if (bContinuationEnabledNow) CellButtonColor = FSlateColor(FLinearColor(1.0, 1.0, 0.f));
								return CellButtonColor;
							})
							.OnClicked_Lambda([this, EnabledHandle, ContinuationHandle, CellsHandle, NumCells, CellIndex]
							{
								bool bIsEnabledNow;
								bool bContinuationEnabled;
								bool bSholudEnableContinuation = false;
								EnabledHandle->GetValue(bIsEnabledNow);
								ContinuationHandle->GetValue(bContinuationEnabled);
								//if the cell is currently enabled, its continuation can be enabled if:
								//the previous cell is enabled, 
								//OR, the previous cell has continuation enabled, which chains to an enabled cell
								if (bIsEnabledNow)
								{
									bSholudEnableContinuation = true;
									HandleContinuationChange(bSholudEnableContinuation, EnabledHandle, ContinuationHandle, CellsHandle, static_cast<int32>(NumCells), CellIndex);
									EnabledHandle->SetValue(!bIsEnabledNow);
									bool bIsContinuationEnabledNow;
									ContinuationHandle->GetValue(bIsContinuationEnabledNow);
									//if continuation cannot be enabled (but the cell is still enabled),
									//clicking the Cell button will disable the cell and disable continuation for the following cells
									if (!bIsContinuationEnabledNow)
									{
										bSholudEnableContinuation = false;
										HandleContinuationChange(bSholudEnableContinuation, EnabledHandle, ContinuationHandle, CellsHandle, static_cast<int32>(NumCells), CellIndex);
									}
								}
								else if (bContinuationEnabled)
								{
									ContinuationHandle->SetValue(false);
									HandleContinuationChange(bSholudEnableContinuation, EnabledHandle, ContinuationHandle, CellsHandle, static_cast<int32>(NumCells), CellIndex);
								}
								else {
									EnabledHandle->SetValue(!bIsEnabledNow);
								}

								return FReply::Handled();
							})
					];
			}

			CellWidgetContainerScrollBoxes.Add(CellWidgetContainer);

			//add cell widgets to the current row
			CustomRowWidget->AddSlot().HAlign(HAlign_Left).FillWidth(1.5f).VAlign(VAlign_Fill)[CellWidgetContainer->AsShared()];
			CustomRow.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::FromString(FString::Printf(TEXT("Row %d"), RowIndex + 1)))
				];

			CustomRow.ValueContent()
				[
					CustomRowWidget->AsShared()
				];
			
			CellsHandle->SetOnNumElementsChanged(FSimpleDelegate::CreateLambda([this, &MidiStepSequenceCategory, NotesHandle]()
				{
					MidiStepSequenceCategory.GetParentLayout().ForceRefreshDetails();
				}));

			//only display the current page, hide others
			CustomRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this,PageIndex]()-> EVisibility
			{
				if (PageIndex != CurrentPageNumber - 1)
				{
					return EVisibility::Collapsed;
				}
				else 
				{
					return EVisibility::Visible;
				}
			})));

		}
	}
	//if the number of rows (note count) changes, refresh
	NotesHandle->SetOnNumElementsChanged(FSimpleDelegate::CreateLambda([this, &MidiStepSequenceCategory, NotesHandle]()
		{
			MidiStepSequenceCategory.GetParentLayout().ForceRefreshDetails();
		}));
}

void FMidiStepSequenceDetailCustomization::HandleContinuationChange(bool bEnabling, TSharedPtr<IPropertyHandle> EnabledHandle, TSharedPtr<IPropertyHandle> ContinuationHandle, TSharedPtr<IPropertyHandleArray> CellsHandle, int32 NumCells, int32 CellIndex)
{
	if (bEnabling)
	{
		bool bIsCellContinuable = IsCellContinuable(EnabledHandle, ContinuationHandle, CellsHandle, static_cast<int32>(NumCells), CellIndex);
		ContinuationHandle->SetValue(bIsCellContinuable);
	}
	else
	{
		//NOTE: we don't disable the current cell's continuation here, but check if following cells also have continuation enabled
		//if that's the case, disable them
		int32 NextCellIndex = CellIndex + 1 > NumCells - 1 ? 0 : CellIndex + 1;
		while (NextCellIndex != CellIndex)
		{
			TSharedPtr<IPropertyHandle> NextEnabledHandle = CellsHandle->GetElement(NextCellIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceCell, bEnabled));
			check(NextEnabledHandle.IsValid());
			bool bIsNextCellEnabled;
			NextEnabledHandle->GetValue(bIsNextCellEnabled);
			if (bIsNextCellEnabled) break;

			TSharedPtr<IPropertyHandle> NextContinuationHandle = CellsHandle->GetElement(NextCellIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceCell, bContinuation));
			check(NextContinuationHandle.IsValid());

			bool bNextContinuationEnabled;
			NextContinuationHandle->GetValue(bNextContinuationEnabled);
			if (bNextContinuationEnabled)
			{
				NextContinuationHandle->SetValue(false);
			}
			NextCellIndex++;

			//if we reach the last cell in the current row, loop back to the beginning
			if (NextCellIndex > NumCells - 1)
			{
				NextCellIndex = 0;
			}
		}
	}
}

bool FMidiStepSequenceDetailCustomization::IsCellContinuable(TSharedPtr<IPropertyHandle> EnabledHandle, TSharedPtr<IPropertyHandle> ContinuationHandle, TSharedPtr<IPropertyHandleArray> CellsHandle, int32 NumCells, int32 CellIndex)
{
	int32 PreviousCellIndex = CellIndex - 1 >= 0 ? CellIndex - 1 : NumCells - 1;
	while (PreviousCellIndex != CellIndex)
	{
		TSharedPtr<IPropertyHandle> PreviousEnabledHandle = CellsHandle->GetElement(PreviousCellIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceCell, bEnabled));
		check(PreviousEnabledHandle.IsValid());
		TSharedPtr<IPropertyHandle> PreviousContinuationHandle = CellsHandle->GetElement(PreviousCellIndex)->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStepSequenceCell, bContinuation));
		check(PreviousContinuationHandle.IsValid());
		bool bIsPreviousCellEnabled;
		PreviousEnabledHandle->GetValue(bIsPreviousCellEnabled);

		//check if the previous cell is already enabled
		if (bIsPreviousCellEnabled)
		{
			return true;
		}

		bool bPreviousContinuationEnabled;
		PreviousContinuationHandle->GetValue(bPreviousContinuationEnabled);
		//Or, check if the previous cell also enables continuation
		//if so, keeps moving backwards until finding a enabled cell, otherwise do not allow continuation to be enabled
		if (!bPreviousContinuationEnabled)
		{
			break;
		}

		PreviousCellIndex--;
		//if we reach the first cell in the row, loop back to the last cell
		if (PreviousCellIndex < 0)
		{
			PreviousCellIndex = NumCells - 1;
		}
	}
	return false;
}


