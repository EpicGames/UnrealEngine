// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Layout/Visibility.h"
#include "Widgets/Layout/SScrollBox.h"
#include "HarmonixMetasound/DataTypes/MidiStepSequence.h"

class FMidiStepSequenceDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FMidiStepSequenceDetailCustomization);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
private:
	TWeakObjectPtr<UObject> MidiStepSequence;
	TArray<TSharedPtr<SScrollBox>> CellWidgetContainerScrollBoxes;
	int32 CurrentPageNumber = 1;
	int32 NumPages = 1;

	//cell button colors for disabled, enabled, and continuation-enabled cells
	FSlateColor CellDisabledColor = FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f)); //gray
	FSlateColor CellEnabledColor = FSlateColor(FLinearColor(124 / 255.f, 252 / 255.f, 0.f)); //green
	FSlateColor CellContinuationColor = FSlateColor(FLinearColor(1.0, 1.0, 0.f)); //yellow

	void DrawCurrentPage(TSharedPtr<IPropertyHandle> StepTableHandle, IDetailCategoryBuilder& MidiStepSequenceCategory);
	
	/**
	* this handles the change in continuation of each cell upon a click on the cell button
	*/
	void HandleContinuationChange(bool bEnabling, TSharedPtr<IPropertyHandle> EnabledHandle, TSharedPtr<IPropertyHandle> ContinuationHandle, TSharedPtr<IPropertyHandleArray> CellsHandle, int32 NumCells, int32 CellIndex);
	
	bool IsCellContinuable(TSharedPtr<IPropertyHandle> EnabledHandle, TSharedPtr<IPropertyHandle> ContinuationHandle, TSharedPtr<IPropertyHandleArray> CellsHandle, int32 NumCells, int32 CellIndex);
};