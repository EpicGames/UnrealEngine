// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiStepSequenceFactory.h"
#include "HarmonixMetasound/DataTypes/MidiStepSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiStepSequenceFactory)

UMidiStepSequenceFactory::UMidiStepSequenceFactory()
{
	SupportedClass = UMidiStepSequence::StaticClass();
	bCreateNew = true;
}

FText UMidiStepSequenceFactory::GetDisplayName() const
{
	return NSLOCTEXT("MIDI", "MIDIStepSequenceFactoryName", "MIDI Step Sequence");
}

UObject* UMidiStepSequenceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UMidiStepSequence>(InParent, Class, Name, Flags, Context);
}
