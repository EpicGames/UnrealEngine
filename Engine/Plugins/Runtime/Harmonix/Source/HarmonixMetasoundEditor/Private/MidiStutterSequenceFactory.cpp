// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiStutterSequenceFactory.h"
#include "HarmonixMetasound/DataTypes/MidiStutterSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiStutterSequenceFactory)

UMidiStutterSequenceFactory::UMidiStutterSequenceFactory()
{
	SupportedClass = UMidiStutterSequence::StaticClass();
	bCreateNew = true;
}

FText UMidiStutterSequenceFactory::GetDisplayName() const
{
	return NSLOCTEXT("MIDI", "MIDIStutterSequenceFactoryName", "MIDI Stutter Sequence");
}

UObject* UMidiStutterSequenceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UMidiStutterSequence>(InParent, Class, Name, Flags, Context);
}
