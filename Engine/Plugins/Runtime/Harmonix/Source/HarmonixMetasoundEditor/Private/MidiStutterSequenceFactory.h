// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "MidiStutterSequenceFactory.generated.h"

UCLASS()
class UMidiStutterSequenceFactory : public UFactory
{
	GENERATED_BODY()
public:
	UMidiStutterSequenceFactory();
	//~ BEGIN UFactory interface
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ END UFactory interface
};

