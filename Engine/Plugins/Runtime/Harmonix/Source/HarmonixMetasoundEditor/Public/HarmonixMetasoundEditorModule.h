// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixMetasoundEditor, Log, All);

class UAssetDefinition_MidiStepSequence;

class FHarmonixMetasoundEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();
};
