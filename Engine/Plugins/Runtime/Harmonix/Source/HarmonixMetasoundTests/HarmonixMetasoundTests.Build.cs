// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixMetasoundTests : ModuleRules
{
	public HarmonixMetasoundTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"FunctionalTesting",
				"UnrealEd",
				"AudioExtensions",
				"MetasoundEngine",
				"MetasoundEngineTest",
				"MetasoundFrontend",
				"MetasoundGenerator",
				"MetasoundGraphCore",
				"MetasoundStandardNodes",
				"HarmonixDsp",
				"HarmonixMetasound",
				"HarmonixMidi",
				"Projects",
				"SignalProcessing"
			}
		);
	}
}
