// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"

#include "HarmonixMetasound/Analysis/MidiClockVertexAnalyzer.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiClockVertexAnalyzer
{
	template<typename DataType>
	TUniquePtr<Metasound::FMetasoundGenerator> BuildPassthroughGraph(
		FAutomationTestBase& Test,
		const FName& InputName,
		const FName& OutputName,
		const Metasound::FSampleRate SampleRate,
		const int32 NumSamplesPerBlock,
		FGuid& OutputGuid)
	{
		Metasound::Test::FNodeTestGraphBuilder Builder;
		const Metasound::Frontend::FNodeHandle InputNode = Builder.AddInput(InputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const Metasound::Frontend::FNodeHandle OutputNode = Builder.AddOutput(OutputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const Metasound::Frontend::FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(InputName);
		const Metasound::Frontend::FInputHandle InputToConnect = OutputNode->GetInputWithVertexName(OutputName);

		if (!Test.TestTrue("Connected input to output", InputToConnect->Connect(*OutputToConnect)))
		{
			return nullptr;
		}

		OutputGuid = OutputNode->GetID();

		// have to add an audio output for the generator to render
		Builder.AddOutput("Audio", Metasound::GetMetasoundDataTypeName<Metasound::FAudioBuffer>());
		
		return Builder.BuildGenerator(SampleRate, NumSamplesPerBlock);
	}

	void ResetAndStartClock(const HarmonixMetasound::FMidiClockWriteRef& ClockInput, float Tempo, float Speed, int32 TimeSigNumerator, int32 TimeSigDenominator)
	{
		const TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(Tempo, TimeSigNumerator, TimeSigDenominator);
		check(SongMaps);

		SongMaps->SetSongLengthTicks(std::numeric_limits<int32>::max());

		ClockInput->AttachToSongMapEvaluator(SongMaps);
		ClockInput->SeekTo(0,0,0);
		ClockInput->SetSpeed(0, Speed);
		ClockInput->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
	}
	
	void AdvanceClock(
		bool bNeedsPrepare,
		const HarmonixMetasound::FMidiClockWriteRef& ClockInput,
		const int32 NumSamples)
	{
		if (bNeedsPrepare)
		{
			ClockInput->PrepareBlock();
		}
		ClockInput->Advance(0, NumSamples);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiClockVertexAnalyzerTestBasic,
		"Harmonix.Metasound.Analysis.MidiClockVertexAnalyzer.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiClockVertexAnalyzerTestBasic::RunTest(const FString&)
	{
		const FName InputName = "MidiClockIn";
		const FName OutputName = "MidiClockOut";
		constexpr Metasound::FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 480;
		
		FGuid OutputGuid;
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
			BuildPassthroughGraph<HarmonixMetasound::FMidiClock>(
				*this,
				InputName,
				OutputName,
				SampleRate,
				NumSamplesPerBlock,
				OutputGuid);
		UTEST_TRUE("Generator is valid", Generator.IsValid());

		// Add an analyzer to get the timestamp
		{
			Metasound::Frontend::FAnalyzerAddress AnalyzerAddress;
			AnalyzerAddress.DataType = Metasound::GetMetasoundDataTypeName<HarmonixMetasound::FMidiClock>();
			AnalyzerAddress.InstanceID = 1234;
			AnalyzerAddress.OutputName = OutputName;
			AnalyzerAddress.AnalyzerName = HarmonixMetasound::Analysis::FMidiClockVertexAnalyzer::GetAnalyzerName();
			AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();
			AnalyzerAddress.AnalyzerMemberName = HarmonixMetasound::Analysis::FMidiClockVertexAnalyzer::FOutputs::Timestamp.Name;
			AnalyzerAddress.NodeID = OutputGuid;
			Generator->AddOutputVertexAnalyzer(AnalyzerAddress);
		}

		// Get the clock
		const TOptional<HarmonixMetasound::FMidiClockWriteRef> ClockRef = Generator->GetInputWriteReference<HarmonixMetasound::FMidiClock>(InputName);
		UTEST_TRUE("Got clock", ClockRef.IsSet());

		// Reset the clock
		constexpr float Tempo = 123;
		constexpr float Speed = 1.2f;
		const FTimeSignature TimeSignature{ 3, 4 };
		ResetAndStartClock(*ClockRef, Tempo, Speed, TimeSignature.Numerator, TimeSignature.Denominator);

		// Listen for changes
		bool CallbackSuccess = false;
		FMusicTimestamp ReceivedTimestamp;
		
		Generator->OnOutputChanged.AddLambda([ExpectedOutputName = OutputName, &CallbackSuccess, &ReceivedTimestamp](
			const FName AnalyzerName,
			const FName OutputName,
			const FName AnalyzerOutputName,
			TSharedPtr<Metasound::IOutputStorage> OutputData)
		{
			const bool IsExpectedCallback =
				OutputData->GetDataTypeName() == Metasound::GetMetasoundDataTypeName<FMusicTimestamp>()
				&& AnalyzerName == HarmonixMetasound::Analysis::FMidiClockVertexAnalyzer::GetAnalyzerName()
				&& OutputName == ExpectedOutputName
				&& AnalyzerOutputName == HarmonixMetasound::Analysis::FMidiClockVertexAnalyzer::FOutputs::Timestamp.Name;

			if (!IsExpectedCallback)
			{
				return;
			}

			CallbackSuccess = true;
			ReceivedTimestamp = static_cast<Metasound::TOutputStorage<FMusicTimestamp>*>(OutputData.Get())->Get();
		});

		// Render some blocks and make sure we're advancing at the expected rate
		constexpr int32 NumBlocks = 20;

		for(int32 i = 0; i < NumBlocks; ++i)
		{
			// Reset
			CallbackSuccess = false;
			ReceivedTimestamp.Reset();
			
			// Advance the clock
			AdvanceClock(i != 0, *ClockRef, NumSamplesPerBlock);
			const FMusicTimestamp ExpectedTimestamp = (*ClockRef)->GetMusicTimestampAtBlockEnd();

			// Render a block
			TArray<float> Buffer;
			Buffer.SetNumUninitialized(NumSamplesPerBlock);
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Check that we got correct data from the analyzer
			UTEST_TRUE("Callback succeeded", CallbackSuccess);
			UTEST_EQUAL("Timestamps match", ReceivedTimestamp, ExpectedTimestamp);
		}

		return true;
	}
}

#endif
