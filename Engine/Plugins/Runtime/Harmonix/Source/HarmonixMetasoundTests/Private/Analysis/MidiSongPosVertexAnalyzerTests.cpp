// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"

#include "HarmonixMetasound/Analysis/MidiSongPosVertexAnalyzer.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiSongPosVertexAnalyzer
{
	template<typename DataType>
	TUniquePtr<Metasound::FMetasoundGenerator> BuildPassthroughGraph(
		FAutomationTestBase& Test,
		const FName& InputName,
		const FName& OutputName,
		const Metasound::FSampleRate SampleRate,
		const int32 NumSamplesPerBlock,
		FGuid& OutputGuid)
	{
		Metasound::Test::FNodeTestGraphBuilder Builder;
		const Metasound::Frontend::FNodeHandle InputNode = Builder.AddInput(InputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const Metasound::Frontend::FNodeHandle OutputNode = Builder.AddOutput(OutputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const Metasound::Frontend::FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(InputName);
		const Metasound::Frontend::FInputHandle InputToConnect = OutputNode->GetInputWithVertexName(OutputName);

		if (!Test.TestTrue("Connected input to output", InputToConnect->Connect(*OutputToConnect)))
		{
			return nullptr;
		}

		OutputGuid = OutputNode->GetID();

		// have to add an audio output for the generator to render
		Builder.AddOutput("Audio", Metasound::GetMetasoundDataTypeName<Metasound::FAudioBuffer>());
		
		return Builder.BuildGenerator(SampleRate, NumSamplesPerBlock);
	}

	void ResetAndStartClock(const HarmonixMetasound::FMidiClockWriteRef& ClockInput)
	{
		const TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(240.3f, 4, 4);
		check(SongMaps);

		SongMaps->AddTimeSignatureAtBarIncludingCountIn(4, 7, 8);
		SongMaps->AddTempoChange(960, 200.0f);
		SongMaps->SetSongLengthTicks(std::numeric_limits<int32>::max());

		ClockInput->AttachToSongMapEvaluator(SongMaps);
		ClockInput->SeekTo(0,0,0);
		ClockInput->SetSpeed(0, 1.0f);
		ClockInput->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
		int32 Bar1Tick = SongMaps->BarBeatTickIncludingCountInToTick(1, 1, 0);
		int32 Bar7Tick = SongMaps->BarBeatTickIncludingCountInToTick(7, 1, 0);
		ClockInput->SetupPersistentLoop(Bar1Tick, Bar7Tick - Bar1Tick);
	}
	
	void AdvanceClock(
		bool bNeedsPrepare,
		const HarmonixMetasound::FMidiClockWriteRef& ClockInput,
		const int32 NumSamples)
	{
		if (bNeedsPrepare)
		{
			ClockInput->PrepareBlock();
		}
		ClockInput->Advance(0, NumSamples);
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiSongPosVertexAnalyzerTestBasic,
		"Harmonix.Metasound.Analysis.MidiSongPosVertexAnalyzer.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiSongPosVertexAnalyzerTestBasic::RunTest(const FString&)
	{
		const FName InputName = "MidiClockIn";
		const FName OutputName = "MidiClockOut";
		constexpr Metasound::FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 480;
		
		FGuid OutputGuid;
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
			BuildPassthroughGraph<HarmonixMetasound::FMidiClock>(
				*this,
				InputName,
				OutputName,
				SampleRate,
				NumSamplesPerBlock,
				OutputGuid);
		UTEST_TRUE("Generator is valid", Generator.IsValid());

		// Add an analyzer to get the timestamp
		Metasound::Frontend::FAnalyzerAddress AnalyzerAddress;
		AnalyzerAddress.DataType = Metasound::GetMetasoundDataTypeName<HarmonixMetasound::FMidiClock>();
		AnalyzerAddress.InstanceID = 1234;
		AnalyzerAddress.OutputName = OutputName;
		AnalyzerAddress.AnalyzerName = HarmonixMetasound::Analysis::FMidiSongPosVertexAnalyzer::GetAnalyzerName();
		AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();
		AnalyzerAddress.AnalyzerMemberName = HarmonixMetasound::Analysis::FMidiSongPosVertexAnalyzer::SongPosition.Name;
		AnalyzerAddress.NodeID = OutputGuid;
		Generator->AddOutputVertexAnalyzer(AnalyzerAddress);
		
		UMidiClockUpdateSubsystem::FClockHistoryPtr ClockHistory = UMidiClockUpdateSubsystem::GetOrCreateClockHistory(AnalyzerAddress);
		auto ClockHistoryCursor = ClockHistory->CreateReadCursor();

		// Get the clock
		const TOptional<HarmonixMetasound::FMidiClockWriteRef> ClockRef = Generator->GetInputWriteReference<HarmonixMetasound::FMidiClock>(InputName);
		UTEST_TRUE("Got clock", ClockRef.IsSet());

		// Reset the clock
		constexpr float Tempo = 240;
		constexpr float Speed = 1.0f;
		const FTimeSignature TimeSignature{ 3, 4 };
		ResetAndStartClock(*ClockRef);

		// Listen for changes
		bool CallbackSuccess = false;
		TArray<HarmonixMetasound::Analysis::FMidiClockSongPosition> ReceivedPositions;
		
		Generator->OnOutputChanged.AddLambda([ExpectedOutputName = OutputName, &CallbackSuccess, &ReceivedPositions](
			const FName AnalyzerName,
			const FName OutputName,
			const FName AnalyzerOutputName,
			TSharedPtr<Metasound::IOutputStorage> OutputData)
		{
			const bool IsExpectedCallback =
				OutputData->GetDataTypeName() == Metasound::GetMetasoundDataTypeName<HarmonixMetasound::Analysis::FMidiClockSongPosition>()
				&& AnalyzerName == HarmonixMetasound::Analysis::FMidiSongPosVertexAnalyzer::GetAnalyzerName()
				&& OutputName == ExpectedOutputName
				&& AnalyzerOutputName == HarmonixMetasound::Analysis::FMidiSongPosVertexAnalyzer::SongPosition.Name;

			if (!IsExpectedCallback)
			{
				return;
			}

			CallbackSuccess = true;
			ReceivedPositions.Add(static_cast<Metasound::TOutputStorage<HarmonixMetasound::Analysis::FMidiClockSongPosition>*>(OutputData.Get())->Get());
		});

		// Render some blocks and make sure we're advancing at the expected rate
		constexpr int32 NumBlocks = 60 * SampleRate / NumSamplesPerBlock;
		Metasound::FSampleCount SampleCount = 0;

		int32 LastTick = -1;
		TArray<float> Buffer;
		Buffer.SetNumUninitialized(NumSamplesPerBlock);

		FMidiSongPos NewSongPos;
		FMidiSongPos PreviousSongPos;
		HarmonixMetasound::Analysis::FMidiClockSongPosition::EMarkerType PreviousMarkerType = HarmonixMetasound::Analysis::FMidiClockSongPosition::EMarkerType::None;

		auto LogSongPos = [](const FMidiSongPos& Pos)
			{
			#if 0
				UE_LOGF(LogMIDI, Log, "\tSeconds From Bar One: %f", Pos.SecondsFromBarOne);
				UE_LOGF(LogMIDI, Log, "\tSeconds Including Count In: %f", Pos.SecondsIncludingCountIn);
				UE_LOGF(LogMIDI, Log, "\tTime Signatre: %d/%d", Pos.TimeSigNumerator, Pos.TimeSigDenominator);
				UE_LOGF(LogMIDI, Log, "\tTempo: %f", Pos.Tempo);
				UE_LOGF(LogMIDI, Log, "\tIncluding Count In: Bars = %f, Beats = %f", Pos.BarsIncludingCountIn, Pos.BeatsIncludingCountIn);
				UE_LOGF(LogMIDI, Log, "\tBeatType: %ls", *MusicalBeatTypeToString(Pos.BeatType));
				UE_LOGF(LogMIDI, Log, "\tTimestamp: %d : %f", Pos.Timestamp.Bar, Pos.Timestamp.Beat);
			#endif
			};

		for(int32 i = 0; i < NumBlocks; ++i)
		{
			// Reset
			CallbackSuccess = false;
			
			// Advance the clock
			AdvanceClock(i != 0, *ClockRef, NumSamplesPerBlock);

			// Render a block
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			while (auto PosRef = ClockHistoryCursor.ConsumeNext())
			{
				const HarmonixMetasound::Analysis::FMidiClockSongPosition& Pos = *PosRef;

				check(Pos.SampleCount >= SampleCount);
				UTEST_TRUE("Timestamps monotonically increasing", Pos.SampleCount >= SampleCount);
				SampleCount = Pos.SampleCount;

				if (Pos.UpToTick < LastTick)
				{
					UE_LOGF(LogMIDI, VeryVerbose, "----------------- LOOPED----------------- ");
				}
				LastTick = Pos.UpToTick;

				UE_LOG(LogMIDI, VeryVerbose, TEXT("POSITION: SampleCount = %" INT64_FMT ", Transport = %s"), Pos.SampleCount, *MusicPlayerTransportStateToString(Pos.CurrentTransportState));
				
				NewSongPos.SetByTick(LastTick, *(ClockHistory->GetLatestMapsForConsumer()->SongMaps));

				LogSongPos(NewSongPos);

				UE_LOGF(LogMIDI, VeryVerbose, "LERP POSITION:");
				if (PreviousMarkerType == HarmonixMetasound::Analysis::FMidiClockSongPosition::EMarkerType::LastPositionBeforeSeekLoop)
				{
					UE_LOGF(LogMIDI, VeryVerbose, "\t<none>");
				}
				else
				{

					LogSongPos(FMidiSongPos::Lerp(PreviousSongPos, NewSongPos, 0.5));
				}
				PreviousSongPos = NewSongPos;
				PreviousMarkerType = Pos.MarkerType;
			}
			ReceivedPositions.Reset();
		}

		return true;
	}
}

#endif
