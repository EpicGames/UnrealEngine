// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"

#include "HarmonixMetasound/Analysis/MidiStreamVertexAnalyzer.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiStreamVertexAnalyzer
{
	template<typename DataType>
	TUniquePtr<Metasound::FMetasoundGenerator> BuildPassthroughGraph(
		FAutomationTestBase& Test,
		const FName& InputName,
		const FName& OutputName,
		const Metasound::FSampleRate SampleRate,
		const int32 NumSamplesPerBlock,
		FGuid* OutputGuid)
	{
		Metasound::Test::FNodeTestGraphBuilder Builder;
		const Metasound::Frontend::FNodeHandle InputNode = Builder.AddInput(InputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const Metasound::Frontend::FNodeHandle OutputNode = Builder.AddOutput(OutputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const Metasound::Frontend::FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(InputName);
		const Metasound::Frontend::FInputHandle InputToConnect = OutputNode->GetInputWithVertexName(OutputName);

		if (!Test.TestTrue("Connected input to output", InputToConnect->Connect(*OutputToConnect)))
		{
			return nullptr;
		}

		if (nullptr != OutputGuid)
		{
			*OutputGuid = OutputNode->GetID();
		}

		// have to add an audio output for the generator to render
		Builder.AddOutput("Audio", Metasound::GetMetasoundDataTypeName<Metasound::FAudioBuffer>());
		
		return Builder.BuildGenerator(SampleRate, NumSamplesPerBlock);
	}

	void InitClock(
		HarmonixMetasound::FMidiClockWriteRef& Clock,
		const float Tempo,
		const int32 TimeSigNum,
		const int32 TimeSigDenom)
	{
		using namespace Metasound;

		// Make the tempo and time sig maps
		const TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(Tempo, TimeSigNum, TimeSigDenom);
		check(SongMaps);

		SongMaps->SetSongLengthTicks(std::numeric_limits<int32>::max());

		// Attach the maps
		Clock->AttachToSongMapEvaluator(SongMaps);
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiStreamVertexAnalyzerTestBasic,
	"Harmonix.Metasound.Analysis.MidiStreamVertexAnalyzer.Basic",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiStreamVertexAnalyzerTestBasic::RunTest(const FString&)
	{
		const FName MidiInputName = "MyInput";
		const FName MidiOutputName = "MyOutput";
		constexpr Metasound::FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;
		FGuid OutputNodeGuid;
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator = BuildPassthroughGraph<HarmonixMetasound::FMidiStream>(
			*this,
			MidiInputName,
			MidiOutputName,
			SampleRate,
			NumSamplesPerBlock,
			&OutputNodeGuid);
		UTEST_TRUE("Generator is valid", Generator.IsValid());

		// Add the analyzer
		Metasound::Frontend::FAnalyzerAddress AnalyzerAddress;
		AnalyzerAddress.DataType = Metasound::GetMetasoundDataTypeName<HarmonixMetasound::FMidiStream>();
		AnalyzerAddress.InstanceID = 1234;
		AnalyzerAddress.OutputName = MidiOutputName;
		AnalyzerAddress.AnalyzerName = HarmonixMetasound::Analysis::FMidiStreamVertexAnalyzer::GetAnalyzerName();
		AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();
		AnalyzerAddress.AnalyzerMemberName = HarmonixMetasound::Analysis::FMidiStreamVertexAnalyzer::FOutputs::GetValue().Name;
		AnalyzerAddress.NodeID = OutputNodeGuid;
		Generator->AddOutputVertexAnalyzer(AnalyzerAddress);

		// Get the MIDI in and attach a clock
		const TOptional<HarmonixMetasound::FMidiStreamWriteRef> MidiIn =
			Generator->GetInputWriteReference<HarmonixMetasound::FMidiStream>(MidiInputName);
		UTEST_TRUE("MIDI input is valid", MidiIn.IsSet());
		constexpr float Tempo = 87;
		HarmonixMetasound::FMidiClockWriteRef Clock = HarmonixMetasound::FMidiClockWriteRef::CreateNew(Generator->OperatorSettings);
		(*MidiIn)->SetClock(*Clock);

		bool CallbackSuccess = false;

		constexpr int32 EventIntervalTicks = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt;
		int32 NextExpectedNoteNumber = 60;
		int32 NextExpectedVelocity = 20;
		int32 NextExpectedEventTicks = 0;
		int32 NumEventsReceivedThisBlock = 0;
		
		// Subscribe for analyzer updates
		Generator->OnOutputChanged.AddLambda(
			[this, &AnalyzerAddress, &Clock, &CallbackSuccess, EventIntervalTicks, &NextExpectedNoteNumber, &NextExpectedVelocity, &NextExpectedEventTicks, &NumEventsReceivedThisBlock]
			(const FName AnalyzerName, const FName OutputName, const FName AnalyzerOutputName, TSharedPtr<Metasound::IOutputStorage> OutputData)
			{
				CallbackSuccess = TestEqual("Data types match", OutputData->GetDataTypeName(), Metasound::GetMetasoundDataTypeName<FMidiEventInfo>())
				&& TestEqual("Analyzer names match", AnalyzerName, AnalyzerAddress.AnalyzerName)
				&& TestEqual("Output names match", OutputName, AnalyzerAddress.OutputName)
				&& TestEqual("Analyzer output names match", AnalyzerOutputName, AnalyzerAddress.AnalyzerMemberName);

				if (!CallbackSuccess)
				{
					return;
				}

				const FMidiEventInfo& EventInfo = static_cast<Metasound::TOutputStorage<FMidiEventInfo>*>(OutputData.Get())->Get();

				// Check the note data
				CallbackSuccess = TestTrue("Is note on", EventInfo.IsNoteOn())
				&& TestEqual("Correct note number", EventInfo.GetNoteNumber(), NextExpectedNoteNumber)
				&& TestEqual("Correct velocity", EventInfo.GetVelocity(), NextExpectedVelocity);

				// Check the timestamp
				const int32 Tick = Clock->GetSongMapEvaluator().MusicTimestampToTick(EventInfo.Timestamp);
				CallbackSuccess = TestEqual("Timestamp matches", Tick, NextExpectedEventTicks);

				++NextExpectedNoteNumber;
				++NextExpectedVelocity;
				NextExpectedEventTicks += EventIntervalTicks;
				++NumEventsReceivedThisBlock;
			});

		constexpr int32 NumEventsToSend = 20;
		const int32 NumBlocksToTest = FMath::CeilToInt((static_cast<float>(NumEventsToSend) * SampleRate / (Tempo / 60)) / NumSamplesPerBlock);
		int32 SampleRemainder = 0;
		int32 SampleCount = 0;
		float TicksPerBlock = Harmonix::Midi::Constants::GTicksPerQuarterNote * NumSamplesPerBlock * (Tempo / 60) / SampleRate;
		float CurrentBlockTick = 0;
		int32 NextEventTick = 0;
		int32 NextNoteNumber = NextExpectedNoteNumber;
		int32 NextVelocity = NextExpectedVelocity;

		// Render and expect to get all the events
		for (int32 i = 0; i < NumBlocksToTest; ++i)
		{
			// Clear the MIDI input
			(*MidiIn)->PrepareBlock();
			
			// Add events if appropriate
			int32 NumEventsExpected = 0;
			CurrentBlockTick += TicksPerBlock;
			while (NextEventTick < CurrentBlockTick)
			{
				FMidiMsg Msg = FMidiMsg::CreateNoteOn(0, NextNoteNumber++, NextVelocity++);
				HarmonixMetasound::FMidiStreamEvent Event(0u, Msg);
				Event.CurrentMidiTick = NextEventTick;
				(*MidiIn)->AddMidiEvent(Event);
				NextEventTick += EventIntervalTicks;
				++NumEventsExpected;
			}
			
			// Advance the clock
			{
				if (i == 0)
				{
					Clock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
				}
				else
				{
					Clock->PrepareBlock();
				}
				int32 BlockOffset = SampleRemainder;
				SampleRemainder += NumSamplesPerBlock;
				constexpr int32 MidiGranularity = 128;
				while (SampleRemainder >= MidiGranularity)
				{
					SampleCount += MidiGranularity;
					SampleRemainder -= MidiGranularity;
					const float AdvanceToMs = static_cast<float>(SampleCount) * 1000.0f / SampleRate;
					Clock->AdvanceToMs(BlockOffset, AdvanceToMs);
					BlockOffset += MidiGranularity;
				}
			}
			
			// Render a block
			{
				CallbackSuccess = false;
				NumEventsReceivedThisBlock = 0;
				
				TArray<float> Buffer;
				Buffer.Reserve(NumSamplesPerBlock);
				UTEST_EQUAL("Generated the right number of samples.",
					Generator->OnGenerateAudio(Buffer.GetData(), NumSamplesPerBlock),
					NumSamplesPerBlock);
			}
			
			// If we were expecting an event, check to see that we got it
			if (NumEventsExpected > 0)
			{
				UTEST_TRUE(FString::Printf(TEXT("Callback success on iteration %i"), i), CallbackSuccess);
				UTEST_EQUAL(FString::Printf(TEXT("Right number of events in iteration %i"), i), NumEventsReceivedThisBlock, NumEventsExpected);
			}
		}
		
		return true;
	}
}

#endif