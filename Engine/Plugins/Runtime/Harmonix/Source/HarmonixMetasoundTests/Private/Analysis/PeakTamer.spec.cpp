// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/Analysis/PeakTamer.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::AudioReactivity::AudioAnalysis::PeakTamer::Tests
{
	BEGIN_DEFINE_SPEC(
		FHarmonixAudioReactivityPeakTamerTest,
		"Harmonix.AudioReactivity.AudioAnalysis.PeakTamer",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	
	END_DEFINE_SPEC(FHarmonixAudioReactivityPeakTamerTest)

	void FHarmonixAudioReactivityPeakTamerTest::Define()
	{
		Describe("Tamer update", [this]()
		{
			It("generates expected values with defaults", [this]()
			{
				FPeakTamer Tamer;

				if (!TestEqual("Peak starts at zero", Tamer.GetPeak(), 0.0f))
				{
					return;
				}

				if (!TestEqual("Value starts at zero", Tamer.GetValue(), 0.0f))
				{
					return;
				}

				FHarmonixPeakTamerSettings DefaultSettings;

				struct FValueDelta
				{
					float Value;
					float Delta;
				};

				const TArray<FValueDelta> ValueDeltaPairs
				{
					{ 1.0f, 0.5f },
					{ 0.5f, 1.0f },
					{ 4.0f, 0.2f }
				};

				for (const FValueDelta& ValueDelta : ValueDeltaPairs)
				{
					// calculate the expected peak
					float ExpectedPeak = 0.0f;
					{
						const float LastPeak = Tamer.GetPeak();
						const float SmoothTime = ValueDelta.Value > LastPeak
						? DefaultSettings.PeakAttackTimeSeconds
						: DefaultSettings.PeakReleaseTimeSeconds;
						ExpectedPeak = PeakTamerPrivate::SmoothValue(ValueDelta.Value, LastPeak, ValueDelta.Delta, SmoothTime);
					}

					// calculate the expected value
					float ExpectedValue = 0.0f;
					{
						const float LastValue = Tamer.GetValue();

						// compress the value
						const float Max = FMath::Max(ExpectedPeak, 1.0f);
						const float ScaledValue = ValueDelta.Value / Max;

						// If enabled, smooth the value
						if (DefaultSettings.bEnableValueSmoothing)
						{
							const float SmoothTime = ScaledValue > LastValue
							? DefaultSettings.ValueAttackTimeSeconds
							: DefaultSettings.ValueReleaseTimeSeconds;
							ExpectedValue = FMath::Min(PeakTamerPrivate::SmoothValue(ScaledValue, LastValue, ValueDelta.Delta, SmoothTime), 1.0f);
						}
						else
						{
							ExpectedValue = FMath::Min(ScaledValue, 1.0f);
						}
					}

					// update
					Tamer.Update(ValueDelta.Value, ValueDelta.Delta);

					// check the peak and value
					if (!TestEqual("Peak matched", Tamer.GetPeak(), ExpectedPeak))
					{
						return;
					}

					if (!TestEqual("Value matched", Tamer.GetValue(), ExpectedValue))
					{
						return;
					}
				}
			});
		});
	}
}

#endif