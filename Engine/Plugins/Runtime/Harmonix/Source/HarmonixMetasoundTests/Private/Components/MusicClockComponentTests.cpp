// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Delegates/Delegate.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMetasound/Components/MusicTimerHandle.h"
#include "HarmonixMetasound/DataTypes/MusicTimeInterval.h"
#include "Tests/AutomationEditorCommon.h"

namespace HarmonixMetasound::MidiStream::Tests
{
	class FWaitUntilFireLatentCommand : public IAutomationLatentCommand
	{
	public:
		FWaitUntilFireLatentCommand(TSharedRef<std::atomic<bool>, ESPMode::ThreadSafe> InFlag, float InTimeoutSeconds, TOptional<FSimpleDelegate> InDelegate)
			: Delegate(InDelegate)
			, Flag(MoveTemp(InFlag))
			, StartTimeSeconds(FPlatformTime::Seconds())
			, TimeoutSeconds(InTimeoutSeconds)
		{
		}

		virtual bool Update() override
		{
			bool bDone = false;

			// Fired
			if (Flag->load(std::memory_order_acquire))
			{
				bDone = true;
			}

			// Timeout
			if ((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds)
			{
				bDone = true;
			}

			if(bDone && Delegate.IsSet())
			{
				Delegate.GetValue().ExecuteIfBound();
			}

			return bDone;
		}

	private:
		TOptional<FSimpleDelegate> Delegate;
		TSharedRef<std::atomic<bool>, ESPMode::ThreadSafe> Flag;
		double StartTimeSeconds = 0.0;
		float TimeoutSeconds = 0.0f;
	};

	class FMusicClockTestHelper
	{
	public:
		FMusicClockTestHelper(FAutomationTestBase* InAutomationTestBase)
			: AutomationTestBase(InAutomationTestBase)
		{

		}

		void CreateMusicClock()
		{
			if(!AutomationTestBase)
			{
				return;
			}

			World = FAutomationEditorCommonUtils::CreateNewMap();
			AutomationTestBase->TestNotNull("World created", World.Get());

			Owner = World->SpawnActor<AActor>();
			AutomationTestBase->TestNotNull("Owner actor spawned", Owner.Get());

			bool bStart = true;
			bool bPersistAcrossLevelTransitions = false;
			UMidiFile* TempoMap = nullptr;
			MusicClockComponent = UMusicClockComponent::CreateWallClockDrivenMusicClock(World, TempoMap, bStart, bPersistAcrossLevelTransitions);
		}

		void StartTimer(const FMusicTimestamp& InStartTime, const FMusicTimeInterval& InInterval)
		{
			if (!MusicClockComponent || !AutomationTestBase)
			{
				return;
			}

			Interval = InInterval;
			auto OnTimerFinished = TDelegate<void()>::CreateLambda([this]()
			{
				TimerTriggered->store(true, std::memory_order_release);
			});

			StartTime = InStartTime;
			TimerHandle = MusicClockComponent->AddTimerNative(Interval, StartTime, Timebase, bLooping, OnTimerFinished);
			AutomationTestBase->TestTrue("Valid Timer Handle", TimerHandle.IsValid());
		}

		FMusicTimestamp GetExpectedEndTime() const
		{
			FMusicTimestamp EndTime = StartTime;
			Harmonix::IncrementTimestampByInterval(EndTime, Interval, TimeSignature);

			FMusicTimestamp OffsetTimeStamp = EndTime;
			Harmonix::IncrementTimestampByOffset(OffsetTimeStamp, Interval, TimeSignature);

			return OffsetTimeStamp;
		}

		FMusicTimestamp GetStartTime() const
		{
			return StartTime;
		}

		FMusicTimestamp GetCurrentTimestamp() const
		{
			if (!MusicClockComponent)
			{
				return FMusicTimestamp();
			}

			return MusicClockComponent->GetCurrentTimestamp(Timebase);
		}

		void WaitForTimer(TOptional<FSimpleDelegate> OnTimerFinished)
		{
			if (!AutomationTestBase)
			{
				return;
			}

			AutomationTestBase->AddCommand(new FWaitUntilFireLatentCommand(TimerTriggered, TimeoutInSeconds, OnTimerFinished));
		}

		bool WasFired() const
		{
			return TimerTriggered->load(std::memory_order_acquire);
		}

		FMusicTimerHandle GetTimerHandle() const
		{
			return TimerHandle;
		}

		FTimeSignature GetTimeSignature() const
		{
			return TimeSignature;
		}

		TObjectPtr<UMusicClockComponent> GetMusicClockComponent() const
		{
			return MusicClockComponent;
		}

		int32 TimeoutInSeconds = 60;
		bool bLooping = false;

	private:
		FMusicTimeInterval Interval;
		ECalibratedMusicTimebase Timebase = ECalibratedMusicTimebase::RawAudioRenderTime;
		FTimeSignature TimeSignature;
		FMusicTimestamp StartTime;

		FMusicTimerHandle TimerHandle;
		TObjectPtr<UWorld> World = nullptr;
		TObjectPtr<AActor> Owner = nullptr;
		TObjectPtr<UMusicClockComponent> MusicClockComponent = nullptr;
		FAutomationTestBase* AutomationTestBase = nullptr;

		TSharedRef<std::atomic<bool>, ESPMode::ThreadSafe> TimerTriggered = MakeShared<std::atomic<bool>, ESPMode::ThreadSafe>(false);
	};

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMusicClockTimerBeatTest, "Harmonix.Metasound.Components.MusicClockComponent.Timer.TimerBeat", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMusicClockTimerBeatTest::RunTest(const FString&)
	{
		FMusicTimeInterval Interval;
		Interval.Interval = EMidiClockSubdivisionQuantization::Beat;
		Interval.IntervalMultiplier = 25;

		TSharedRef<FMusicClockTestHelper> TestHelper = MakeShared<FMusicClockTestHelper>(this);
		TestHelper->CreateMusicClock();
		TestHelper->StartTimer(TestHelper->GetCurrentTimestamp(), Interval);

		auto OnTestFinished = TDelegate<void()>::CreateLambda([this, TestHelper]()
		{
			const FMusicTimestamp CurrentTime = TestHelper->GetCurrentTimestamp();
			const FMusicTimestamp ExpectedEndTime = TestHelper->GetExpectedEndTime();
			TestTrue("Timer callback was called", TestHelper->WasFired());
			TestTrue("Timer triggered at the correct time", CurrentTime.Bar == ExpectedEndTime.Bar && FMath::IsNearlyEqual(CurrentTime.Beat, ExpectedEndTime.Beat, 0.5f));
		});

		TestHelper->WaitForTimer(OnTestFinished);

		// Latent command, test is not over.
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMusicClockTimerBarTest, "Harmonix.Metasound.Components.MusicClockComponent.Timer.TimerBar", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMusicClockTimerBarTest::RunTest(const FString&)
	{
		FMusicTimeInterval Interval;
		Interval.Interval = EMidiClockSubdivisionQuantization::Bar;
		Interval.IntervalMultiplier = 10;

		TSharedRef<FMusicClockTestHelper> TestHelper = MakeShared<FMusicClockTestHelper>(this);
		TestHelper->CreateMusicClock();
		TestHelper->StartTimer(TestHelper->GetCurrentTimestamp(), Interval);

		auto OnTestFinished = TDelegate<void()>::CreateLambda([this, TestHelper]()
		{
			const FMusicTimestamp CurrentTime = TestHelper->GetCurrentTimestamp();
			const FMusicTimestamp ExpectedEndTime = TestHelper->GetExpectedEndTime();
			TestTrue("Timer callback was called", TestHelper->WasFired());
			TestTrue("Timer triggered at the correct time", CurrentTime.Bar == ExpectedEndTime.Bar && FMath::IsNearlyEqual(CurrentTime.Beat, ExpectedEndTime.Beat, 0.5f));
		});

		TestHelper->WaitForTimer(OnTestFinished);

		// Latent command, test is not over.
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMusicClockTimerNoteTest, "Harmonix.Metasound.Components.MusicClockComponent.Timer.TimerNote", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMusicClockTimerNoteTest::RunTest(const FString&)
	{
		FMusicTimeInterval Interval;
		Interval.Interval = EMidiClockSubdivisionQuantization::DottedEighthNote;
		Interval.IntervalMultiplier = 25;

		TSharedRef<FMusicClockTestHelper> TestHelper = MakeShared<FMusicClockTestHelper>(this);
		TestHelper->CreateMusicClock();
		TestHelper->StartTimer(TestHelper->GetCurrentTimestamp(), Interval);

		auto OnTestFinished = TDelegate<void()>::CreateLambda([this, TestHelper]()
		{
			const FMusicTimestamp CurrentTime = TestHelper->GetCurrentTimestamp();
			const FMusicTimestamp ExpectedEndTime = TestHelper->GetExpectedEndTime();
			TestTrue("Timer callback was called", TestHelper->WasFired());
			TestTrue("Timer triggered at the correct time", CurrentTime.Bar == ExpectedEndTime.Bar && FMath::IsNearlyEqual(CurrentTime.Beat, ExpectedEndTime.Beat, 0.5f));
		});

		TestHelper->WaitForTimer(OnTestFinished);

		// Latent command, test is not over.
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMusicClockTimerOffsetTest, "Harmonix.Metasound.Components.MusicClockComponent.Timer.TimerOffsets", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMusicClockTimerOffsetTest::RunTest(const FString&)
	{
		FMusicTimeInterval Interval;
		Interval.Interval = EMidiClockSubdivisionQuantization::Bar;
		Interval.IntervalMultiplier = 5;
		Interval.Offset = EMidiClockSubdivisionQuantization::Beat;
		Interval.OffsetMultiplier = 3;

		TSharedRef<FMusicClockTestHelper> TestHelper = MakeShared<FMusicClockTestHelper>(this);
		TestHelper->CreateMusicClock();
		TestHelper->StartTimer(TestHelper->GetCurrentTimestamp(), Interval);

		auto OnTestFinished = TDelegate<void()>::CreateLambda([this, TestHelper]()
		{
			const FMusicTimestamp CurrentTime = TestHelper->GetCurrentTimestamp();
			const FMusicTimestamp ExpectedEndTime = TestHelper->GetExpectedEndTime();
			TestTrue("Timer callback was called", TestHelper->WasFired());
			TestTrue("Timer triggered at the correct time", CurrentTime.Bar == ExpectedEndTime.Bar && FMath::IsNearlyEqual(CurrentTime.Beat, ExpectedEndTime.Beat, 0.5f));
		});

		TestHelper->WaitForTimer(OnTestFinished);

		// Latent command, test is not over.
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMusicClockTimerPauseTest, "Harmonix.Metasound.Components.MusicClockComponent.Timer.TimerPause", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMusicClockTimerPauseTest::RunTest(const FString&)
	{
		FMusicTimeInterval Interval;
		Interval.Interval = EMidiClockSubdivisionQuantization::Bar;
		Interval.IntervalMultiplier = 5;

		TSharedRef<FMusicClockTestHelper> TestHelper = MakeShared<FMusicClockTestHelper>(this);
		TestHelper->CreateMusicClock();
		const FMusicTimestamp StartTime = TestHelper->GetCurrentTimestamp();
		TestHelper->StartTimer(StartTime, Interval);
		TestHelper->GetMusicClockComponent()->PauseTimer(TestHelper->GetTimerHandle(), true);

		TSharedRef<FMusicTimestamp> PauseTime = MakeShared<FMusicTimestamp>();
		auto OnTimerTriggered = TDelegate<void()>::CreateLambda([this, TestHelper, PauseTime]()
		{
			TestTrue("Timer callback was called", TestHelper->WasFired());

			// Add the time we were paused to the expected end time
			const FMusicTimestamp CurrentTime = TestHelper->GetCurrentTimestamp();
			const FMusicTimestamp StartTime = TestHelper->GetStartTime();
			FMusicTimestamp ExpectedEndTime = TestHelper->GetExpectedEndTime();
			const FTimeSignature TimeSignature = TestHelper->GetTimeSignature();
			const int32 BeatsPerBar = TimeSignature.Numerator;
			const float StartTimeAbsoluteBeats = float(StartTime.Bar - 1) * BeatsPerBar + (StartTime.Beat - 1.0f);
			const float PausedbsoluteBeats = float(PauseTime->Bar - 1) * BeatsPerBar + (PauseTime->Beat - 1.0f);
			const float PausedDeltaBeats = PausedbsoluteBeats - StartTimeAbsoluteBeats;
			Harmonix::IncrementTimestampByBeats(ExpectedEndTime, PausedDeltaBeats, TimeSignature);

			TestTrue("Timer triggered at the correct time", CurrentTime.Bar == ExpectedEndTime.Bar && FMath::IsNearlyEqual(CurrentTime.Beat, ExpectedEndTime.Beat, 0.5f));
		});

		auto OnTimerTimeout = TDelegate<void()>::CreateLambda([this, TestHelper, OnTimerTriggered, PauseTime]()
		{
			// Timer was paused so should not have fired.
			TestFalse("Timer callback was called", TestHelper->WasFired());

			// Unpause
			TestHelper->GetMusicClockComponent()->PauseTimer(TestHelper->GetTimerHandle(), false);
			*PauseTime = TestHelper->GetCurrentTimestamp();
			TestHelper->WaitForTimer(OnTimerTriggered);
		});

		TestHelper->WaitForTimer(OnTimerTimeout);

		// Latent command, test is not over.
		return true;
	}
}

#endif
