// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MidiClock.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiClock
{
	using namespace HarmonixMetasound;
	BEGIN_DEFINE_SPEC(
		FHarmonixMetasoundMidiClockSpec,
		"Harmonix.Metasound.DataTypes.MidiClock",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	TUniquePtr<FMidiClock> TestClock;
	TSharedPtr<FMidiClock, ESPMode::NotThreadSafe> DrivingClock;
	TSharedPtr<FSongMaps> SongMaps;
	Metasound::FOperatorSettings OperatorSettings {48000, 100};

	void AddStateAtFrame(EMusicPlayerTransportState State, int32 Frame) const
	{
		check(TestClock.IsValid());
		TestClock->SetTransportState(Frame, State);
	}

	void ExecuteWriteAdvance(int32 StartFrameIndex, int32 EndFrameIndex, float Speed)
	{
		TestClock->PrepareBlock();
		TestClock->SeekTo(0, 0, 0);
		TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

		TestEqual("Clock.GetCurrentSongPosMs()", TestClock->GetCurrentSongPosMs(), 0.0f);
		TestEqual("Clock.GetLastProcessedMidiTick()", TestClock->GetLastProcessedMidiTick(), -1);
		TestEqual("Clock.GetNextMidiTickToProcess()", TestClock->GetNextMidiTickToProcess(), 0);

		// clear out the transport change message, etc. so that our
		// tests below can just look for speed and advance events...
		TestClock->PrepareBlock();

		float OldMs = TestClock->GetCurrentSongPosMs();
		int32 OldTick = TestClock->GetNextMidiTickToProcess();
		TestClock->SetSpeed(StartFrameIndex, Speed);
		TestClock->Advance(StartFrameIndex, EndFrameIndex);
		float NewMs = TestClock->GetCurrentSongPosMs();
		int32 NewTick = TestClock->GetNextMidiTickToProcess();

		TestTrue("Non looping clock advanced forward in time", NewMs > OldMs);
		TestTrue("Non looping Clock Advanced forward in ticks", NewTick > OldTick);

		int32 DeltaFrames = FMidiClock::kMidiGranularity * Speed;
		float DeltaMs = DeltaFrames * 1000.0f / OperatorSettings.GetSampleRate();
		float Ms = OldMs;

		int32 BlockFrameIndex = 0; 
		int32 FromTick = (int32)(TestClock->GetSongMapEvaluator().MsToTick(Ms) + 0.5f);
		for (int32 i = 0; i < TestClock->GetMidiClockEventsInBlock().Num(); ++i)
		{
			const FMidiClockEvent& Event = TestClock->GetMidiClockEventsInBlock()[i];

			if (i == 0 && Speed != 1.0f)
			{
				// the first message should be the speed message UNLESS the speed was 1.0...
				TestEqual(FString::Printf(TEXT("Frame-%d: Event.BlockFrameIndex"), Event.BlockFrameIndex), Event.BlockFrameIndex, BlockFrameIndex);
				TestTrue(FString::Printf(TEXT("Frame-%d: Event.Type"), Event.BlockFrameIndex), Event.Msg.IsType<MidiClockMessageTypes::FSpeedChange>());
				TestEqual(FString::Printf(TEXT("Frame-%d: Event.Speed"), Event.BlockFrameIndex), Event.Msg.Get<MidiClockMessageTypes::FSpeedChange>().Speed, Speed);
			}
			else
			{
				// all other messages should be advance messages...
				Ms += DeltaMs;
				int32 UpToTick = FMath::RoundToInt32(TestClock->GetSongMapEvaluator().MsToTick(Ms));

				TestEqual(FString::Printf(TEXT("Frame-%d: Event.BlockFrameIndex"), Event.BlockFrameIndex), Event.BlockFrameIndex, BlockFrameIndex);
				TestTrue(FString::Printf(TEXT("Frame-%d: Event.Type"), Event.BlockFrameIndex), Event.Msg.IsType<MidiClockMessageTypes::FAdvance>());
				TestEqual(FString::Printf(TEXT("Frame-%d: Event.FirstTickToProcess"), Event.BlockFrameIndex), Event.Msg.Get<MidiClockMessageTypes::FAdvance>().FirstTickToProcess, FromTick);
				int32 NumberOfTicksToProcess = Event.Msg.Get<MidiClockMessageTypes::FAdvance>().NumberOfTicksToProcess;
				TestEqual(FString::Printf(TEXT("Frame-%d: Event.NumberOfTicksToProcess"), Event.BlockFrameIndex), Event.Msg.Get<MidiClockMessageTypes::FAdvance>().NumberOfTicksToProcess, UpToTick - FromTick);
				TestEqual(FString::Printf(TEXT("Frame-%d: Event.LastTickToProcess"), Event.BlockFrameIndex), Event.Msg.Get<MidiClockMessageTypes::FAdvance>().LastTickToProcess(), UpToTick - 1);
			
				BlockFrameIndex += FMidiClock::kMidiGranularity;
				FromTick = UpToTick;
			}
		}

		int32 Tick = FMath::RoundToInt32(TestClock->GetSongMapEvaluator().MsToTick(Ms));
		TestEqual("Clock.GetNextMidiTickToProcess()", TestClock->GetLastProcessedMidiTick(), Tick - 1);
		TestEqual("Clock.GetCurrentSongPosMs()", TestClock->GetCurrentSongPosMs(), Ms, DeltaMs / 2.0f);
	};
	
	int32 GetLastTransportStateChangeBlockSample(FMidiClock* InClock)
	{
		const TArray<FMidiClockEvent>& ClockEvents = InClock->GetMidiClockEventsInBlock();
		for (auto it = ClockEvents.rbegin(); it != ClockEvents.rend(); ++it)
		{
			if ((*it).IsType<MidiClockMessageTypes::FTransportChange>())
			{
				return (*it).BlockFrameIndex;
			}
		}
		return -1;
	}

	END_DEFINE_SPEC(FHarmonixMetasoundMidiClockSpec)

	void FHarmonixMetasoundMidiClockSpec::Define()
	{
		BeforeEach([this]()
		{
			TestClock = MakeUnique<FMidiClock>(OperatorSettings);
		});

		AfterEach([this]()
		{
			TestClock.Reset();
		});
		
		Describe("AddTransportStateChangeToBlock(NewState)", [this]()
		{
			It("should always add NewState if there are no changes in the block", [this]()
			{
				TestClock->PrepareBlock();

				TestFalse("No transport changes in block", TestClock->HasTransportStateChangesInBlock());

				constexpr EMusicPlayerTransportState NewState = EMusicPlayerTransportState::Playing;
				AddStateAtFrame(NewState, 0);

				TestTrue("There is a transport state change in block", TestClock->HasTransportStateChangesInBlock());
				TestEqual("State at end of block matches the one we added", TestClock->GetTransportStateAtEndOfBlock(), NewState);
			});

			It("should add NewState if its frame is greater than the last one in the block", [this]()
			{
				AddStateAtFrame(EMusicPlayerTransportState::Playing, 0);
				TestTrue("There is already a transport state change in the block", TestClock->HasTransportStateChangesInBlock());

				const int32 NumInitialStates = TestClock->GetNumTransportStateChangesInBlock();
				const int32 LastStateFrame = GetLastTransportStateChangeBlockSample(TestClock.Get());

				constexpr EMusicPlayerTransportState NewState = EMusicPlayerTransportState::Pausing;
				AddStateAtFrame(NewState, LastStateFrame + 1);

				TestEqual("There is another transport state change in block", TestClock->GetNumTransportStateChangesInBlock(), NumInitialStates + 1);
				TestEqual("State at end of block matches the one we added", TestClock->GetTransportStateAtEndOfBlock(), NewState);
			});

			It("should add NewState if it has the same frame as the last one in the block", [this]()
			{
				AddStateAtFrame(EMusicPlayerTransportState::Playing, 0);
				TestTrue("There is already a transport state change in the block", TestClock->HasTransportStateChangesInBlock());

				const int32 NumInitialStates = TestClock->GetNumTransportStateChangesInBlock();
				const int32 LastStateFrame = GetLastTransportStateChangeBlockSample(TestClock.Get());

				constexpr EMusicPlayerTransportState NewState = EMusicPlayerTransportState::Paused;
				AddStateAtFrame(NewState, LastStateFrame );

				TestEqual("There is another transport state change in block", TestClock->GetNumTransportStateChangesInBlock(), NumInitialStates + 1);
				TestEqual("State at end of block matches the one we added", TestClock->GetTransportStateAtEndOfBlock(), NewState);
			});
		});

		Describe("PrepareBlock()", [this]()
		{
			It("should reset transport, speed, tempo, and clock states", [this]()
			{
				const float LastTempo = TestClock->GetTempoAtEndOfBlock();
				const float LastSpeed = TestClock->GetSpeedAtEndOfBlock();
				TestClock->PrepareBlock();
				TestEqual("Clock.TransportChangesInBlock.Num()", TestClock->GetNumTransportStateChangesInBlock(), 0);
				TestEqual("Clock.SpeedChangesInBlock.Num()", TestClock->GetNumSpeedChangesInBlock(), 0);
				TestEqual("Clock.SpeedAtBlockSampleFrame(0)", TestClock->GetSpeedAtBlockSampleFrame(0), LastSpeed);
				TestEqual("Clock.SpeedAtEndOfBlock()", TestClock->GetSpeedAtEndOfBlock(), LastSpeed);
				TestFalse("Clock.HasSpeedChangesInBlock()", TestClock->HasSpeedChangesInBlock());
				TestEqual("Clock.TempoChangesInBlock.Num()", TestClock->GetNumTempoChangesInBlock(), 0);
				TestEqual("Clock.TempoAtBlockSampleFrame(0)", TestClock->GetTempoAtBlockSampleFrame(0), LastTempo);
				TestEqual("Clock.GetTempoAtEndOfBlock()", TestClock->GetTempoAtEndOfBlock(), LastTempo);
				TestFalse("Clock.HasTempoChangesInBlock()", TestClock->HasTempoChangesInBlock());
				TestEqual("Clock.GetMidiClockEventsInBlock().Num()", TestClock->GetMidiClockEventsInBlock().Num(), 0);
			});
		});

		Describe("ResetAndStart()", [this]()
		{
			It("should reset speed and tempo changes, and be \"playing\"", [this]()
			{
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
				TestEqual("Clock.GetTransportStateAtEndOfBlock()", TestClock->GetTransportStateAtEndOfBlock(), EMusicPlayerTransportState::Playing);
				TestEqual("Clock.SpeedAtBlockSampleFrame(0)", TestClock->GetSpeedAtBlockSampleFrame(0), 1.0f);
				TestEqual("Clock.SpeedAtEndOfBlock()", TestClock->GetSpeedAtEndOfBlock(), 1.0f);
				TestTrue("Clock.HasSpeedChangesInBlock()", TestClock->HasSpeedChangesInBlock());
				TestEqual("Clock.TempoChangesInBlock.Num()", TestClock->GetNumTempoChangesInBlock(), 1);
				TestEqual("Clock.TempoAtBlockSampleFrame(0)", TestClock->GetTempoAtBlockSampleFrame(0), 120.0f);
				TestEqual("Clock.GetTempoAtEndOfBlock()", TestClock->GetTempoAtEndOfBlock(), 120.0f);
				TestTrue("Clock.HasTempoChangesInBlock()", TestClock->HasTempoChangesInBlock());
			});

/*
			It("should seek to 0 when reset requested", [this]()
			{
				int32 BlockFrame = 100;
				FMusicSeekTarget SeekTarget;
				SeekTarget.Type = ESeekPointType::BarBeat;
				SeekTarget.BarBeat = FMusicTimestamp(2, 1.0f);
				TestClock->SeekTo(0, SeekTarget);
				TestTrue("Clock seeked, Clock.GetNextMidiTickToProcess() != 0", TestClock->GetNextMidiTickToProcess() != 0);
				
				TestClock->Seek ResetAndStart(BlockFrame, true);
				TestEqual("Clock.GetLastProcessedMidiTick()", TestClock->GetLastProcessedMidiTick(), -1);
				TestEqual("Clock.GetNextMidiTickToProcess()", TestClock->GetNextMidiTickToProcess(), 0);
			});
*/

/*
			It("should NOT seek to 0 when reset requested", [this]()
			{
				int32 BlockFrame = 100;
				FMusicSeekTarget SeekTarget;
				SeekTarget.Type = ESeekPointType::BarBeat;
				SeekTarget.BarBeat = FMusicTimestamp(2, 1.0f);
				TestClock->SeekTo(0, SeekTarget);
				int32 NewMidiTick = TestClock->GetNextMidiTickToProcess();
				TestTrue("Clock seeked, Clock.GetNextMidiTickToProcess() != 0", TestClock->GetNextMidiTickToProcess() != 0);
							
				TestClock->ResetAndStart(BlockFrame, false);
				TestEqual("After reset, Clock.GetNextMidiTickToProcess()", TestClock->GetNextMidiTickToProcess(), NewMidiTick);
			});
*/

		});

		Describe("SetLoop()", [this]()
		{
			It("should correctly set the tempo", [this]()
			{
				// set transport to playing so the block gets the initial tempo, time signature, etc...
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

				const float TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNote;
				const float TempoBPM = 120.0f;
				int32 LoopStartTick = 0;
				int32 LoopEndTick = 1000;

				// initial clock tempo
				TestEqual("Clock.Tempo", TestClock->GetTempoAtEndOfBlock(), TempoBPM);
				TestEqual("Clock.Tempo", TestClock->GetSongMapEvaluator().GetTempoAtTick(0), TempoBPM);

				// microseconds per quarternote
				int32 MidiTempo = Harmonix::Midi::Constants::BPMToMidiTempo(TempoBPM);
							
				float MsPerTick = MidiTempo / TicksPerQuarterNote / 1000.0f;
				float LoopStartMs = MsPerTick * LoopStartTick;
				float LoopEndMs = MsPerTick * LoopEndTick;

				TestFalse("Initial -> Clock.HasPersistentLoop()", TestClock->HasPersistentLoop());
				TestClock->SetupPersistentLoop(LoopStartTick, LoopEndTick - LoopStartTick);
				TestTrue("SetLoop -> Clock.HasPersistentLoop()", TestClock->HasPersistentLoop());
				TestEqual("Clock.GetFirstTickInLoop()", TestClock->GetFirstTickInLoop(), LoopStartTick);
				TestEqual("Clock.GetLoopLengthTicks()", TestClock->GetLoopLengthTicks(), LoopEndTick - LoopStartTick);
				TestEqual("Clock.LoopStartMs()", TestClock->GetLoopStartMs(), LoopStartMs);
				TestEqual("Clock.LoopEndMs()", TestClock->GetLoopEndMs(), LoopEndMs);

				TestClock->ClearPersistentLoop();
				TestFalse("Cleared -> Clock.HasPersistentLoop()", TestClock->HasPersistentLoop());
			});

		});

		Describe("WriteAdvance", [this]()
		{
			It("should advance one block correctly with speed 1", [this]
			{
				int32 StartFrame = 0;
				int32 EndFrame = StartFrame + OperatorSettings.GetNumFramesPerBlock();
				float Speed = 1.0f;
				ExecuteWriteAdvance(0, EndFrame, Speed);
			});

			It("should advance one block correctly with speed 2", [this]
			{
				int32 StartFrame = 0;
				int32 EndFrame = StartFrame + OperatorSettings.GetNumFramesPerBlock();
				float Speed = 2.0f;
				ExecuteWriteAdvance(0, EndFrame, Speed);
			});
			
			It("should advance one block correctly with speed 1/2", [this]
			{
				int32 StartFrame = 0;
				int32 EndFrame = StartFrame + OperatorSettings.GetNumFramesPerBlock();
				float Speed = 0.5f;
				ExecuteWriteAdvance(0, EndFrame, Speed);
			});
			
		});

		Describe("ProcessClockEvent(EventType)", [this]()
		{
			BeforeEach([&, this]
			{
				DrivingClock = MakeShared<FMidiClock, ESPMode::NotThreadSafe>(OperatorSettings);
				TestClock->SetDrivingClock(DrivingClock);
			});

			AfterEach([&, this]
			{
				TestClock->ClearPersistentLoop();
				TestClock->SetDrivingClock(nullptr);
				DrivingClock.Reset();
			});

			It("EventType::AdvanceThru.NonLooping", [&, this]()
			{
				DrivingClock->SeekTo(0,0,0);
				DrivingClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

				float DrivingClockSpeed = 1.0f;
				int32 StartFrame = 0;
				int32 DeltaFrames = FMidiClock::kMidiGranularity * DrivingClockSpeed;
				float DeltaMs = DeltaFrames * 1000.0f / OperatorSettings.GetSampleRate();
				int32 Tick = (int32)(TestClock->GetSongMapEvaluator().MsToTick(DeltaMs) + 0.5f);

				DrivingClock->AdvanceToTick(0, Tick, Tick);

				TestClock->PrepareBlock();
				int32 OldEventsNum = TestClock->GetMidiClockEventsInBlock().Num();
				TestClock->Advance(*DrivingClock, 0, OperatorSettings.GetNumFramesPerBlock());

				if (!TestTrue("Clock has new clock events", TestClock->GetMidiClockEventsInBlock().Num() > OldEventsNum))
				{
					return;
				}
				TestTrue("Last Clock Event in block", TestClock->GetMidiClockEventsInBlock().Last().Msg.IsType<MidiClockMessageTypes::FAdvance>());
				const FMidiClockEvent& Event = TestClock->GetMidiClockEventsInBlock().Last();
				TestEqual("Clock Last Processed Tick", TestClock->GetLastProcessedMidiTick(), Event.Msg.Get<MidiClockMessageTypes::FAdvance>().LastTickToProcess());
				TestEqual("Clock Next Tick To Process", TestClock->GetNextMidiTickToProcess(), Event.Msg.Get<MidiClockMessageTypes::FAdvance>().LastTickToProcess() + 1);
			});

			It("EventType::AdvanceThru.Looping", [&, this]()
			{
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
				DrivingClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

				FMusicTimestamp LoopEndTimestamp;
				LoopEndTimestamp.Bar = 2;
				LoopEndTimestamp.Beat = 1.0f;
				int32 LoopStartTick = 0;
				int32 LoopEndTick = TestClock->GetSongMapEvaluator().MusicTimestampToTick(LoopEndTimestamp);
				
				TestClock->SetupPersistentLoop(LoopStartTick, LoopEndTick - LoopStartTick);

				TestClock->PrepareBlock();
				DrivingClock->PrepareBlock();

				int32 TestStartTick = LoopEndTick - 1;;

				DrivingClock->SeekTo(0, TestStartTick, TestStartTick);

				float DeltaMs = FMidiClock::kMidiGranularity * 1000.0f / OperatorSettings.GetSampleRate();
				int32 DeltaTicks = (int32)(DrivingClock->GetSongMapEvaluator().MsToTick(DeltaMs) + 0.5f);
				int32 Tick = DeltaTicks;


				int32 ExpectedTick = TestStartTick + DeltaTicks;
				if (ExpectedTick > LoopEndTick)
				{
					ExpectedTick = ExpectedTick - LoopEndTick + LoopStartTick;
				}

				DrivingClock->Advance(0, FMidiClock::kMidiGranularity);
				TestClock->Advance(*DrivingClock, 0, FMidiClock::kMidiGranularity);

				if (!TestTrue("Clock has new clock events", TestClock->GetMidiClockEventsInBlock().Num() > 0))
				{
					return;
				}
				TestTrue("Last Clock Event in block is advance", TestClock->GetMidiClockEventsInBlock().Last().Msg.IsType<MidiClockMessageTypes::FAdvance>());
				TestEqual("Clock Current Tick", TestClock->GetNextMidiTickToProcess(), ExpectedTick);
			});


			It("ProccessClockEvent(SeekTo)", [&, this]()
			{
				int32 StartFrame = 0;
				int32 Tick = 1000;

				TestClock->PrepareBlock();
				int32 OldEventsNum = TestClock->GetMidiClockEventsInBlock().Num();

				TestClock->SeekTo(StartFrame, 1000, 1000);

				if (!TestTrue("Clock has new clock events", TestClock->GetMidiClockEventsInBlock().Num() > OldEventsNum))
				{
					return;
				}
				TestTrue("Last Clock Event in block", TestClock->GetMidiClockEventsInBlock()[0].Msg.IsType<MidiClockMessageTypes::FSeek>());
				TestEqual("Clock Current Tick", TestClock->GetNextMidiTickToProcess(), Tick);
			});
		});

		Describe("Tempo Changes", [this]()
		{
			It("Without driving clock - One Tempo Change At Span End", [this]()
			{
				SongMaps = MakeShared<FSongMaps>(123, 5, 8);
				constexpr int32 TempoChangeTick = 234;
				constexpr float TempoChangeTempo = 89;
				SongMaps->AddTempoChange(TempoChangeTick, TempoChangeTempo);
				TestClock->AttachToSongMapEvaluator(SongMaps);
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

				while (TestClock->GetLastProcessedMidiTick() < TempoChangeTick)
				{
					if (TestClock->GetLastProcessedMidiTick() > 0)
					{
						TestClock->PrepareBlock();
					}
					TestClock->Advance(0, OperatorSettings.GetNumFramesPerBlock());
				}

				bool HasTempoEvent = false;

				for (const FMidiClockEvent& Event : TestClock->GetMidiClockEventsInBlock())
				{
					if (Event.Msg.IsType<MidiClockMessageTypes::FTempoChange>())
					{
						const MidiClockMessageTypes::FTempoChange& TempoChange = Event.Msg.Get<MidiClockMessageTypes::FTempoChange>();

						if (!TestEqual("Tempo is correct", TempoChange.Tempo, TempoChangeTempo, 0.001f))
						{
							return;
						}

						if (!TestEqual("Tick is correct", TempoChange.Tick, TempoChangeTick))
						{
							return;
						}
						
						HasTempoEvent = true;
						break;
					}
				}

				TestTrue("Got tempo event", HasTempoEvent);
			});

			It("Without driving clock - One Tempo Change At Span Start", [this]()
			{
				SongMaps = MakeShared<FSongMaps>(123, 5, 8);
				constexpr int32 TempoChangeTick = 230;
				constexpr float TempoChangeTempo = 89;
				SongMaps->AddTempoChange(TempoChangeTick, TempoChangeTempo);
				TestClock->AttachToSongMapEvaluator(SongMaps);
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

				while (TestClock->GetLastProcessedMidiTick() < TempoChangeTick)
				{
					if (TestClock->GetLastProcessedMidiTick() > 0)
					{
						TestClock->PrepareBlock();
					}
					TestClock->Advance(0, OperatorSettings.GetNumFramesPerBlock());
				}

				bool HasTempoEvent = false;

				for (const FMidiClockEvent& Event : TestClock->GetMidiClockEventsInBlock())
				{
					if (Event.Msg.IsType<MidiClockMessageTypes::FTempoChange>())
					{
						const MidiClockMessageTypes::FTempoChange& TempoChange = Event.Msg.Get<MidiClockMessageTypes::FTempoChange>();

						if (!TestEqual("Tempo is correct", TempoChange.Tempo, TempoChangeTempo, 0.001f))
						{
							return;
						}

						if (!TestEqual("Tick is correct", TempoChange.Tick, TempoChangeTick))
						{
							return;
						}

						HasTempoEvent = true;
						break;
					}
				}

				TestTrue("Got tempo event", HasTempoEvent);
			});

			It("Without driving clock - Many Tempo Changes In Span", [this]()
			{
				SongMaps = MakeShared<FSongMaps>(123, 5, 8);
				constexpr int32 NumChanges = 4;
				constexpr int32 TempoChangeTicks[NumChanges] = { 230, 231, 232, 233 };
				constexpr float TempoChangeTempos [NumChanges] = { 89.0f, 89.2f, 89.4f, 89.6f };
				for (int32 i = 0; i < NumChanges; ++i)
				{
					SongMaps->AddTempoChange(TempoChangeTicks[i], TempoChangeTempos[i]);
				}
				TestClock->AttachToSongMapEvaluator(SongMaps);
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

				while (TestClock->GetLastProcessedMidiTick() < TempoChangeTicks[NumChanges - 1])
				{
					if (TestClock->GetLastProcessedMidiTick() > 0)
					{
						TestClock->PrepareBlock();
					}
					TestClock->Advance(0, OperatorSettings.GetNumFramesPerBlock());
				}

				int NumTempoEvents = 0;

				for (const FMidiClockEvent& Event : TestClock->GetMidiClockEventsInBlock())
				{
					if (Event.Msg.IsType<MidiClockMessageTypes::FTempoChange>())
					{
						const MidiClockMessageTypes::FTempoChange& TempoChange = Event.Msg.Get<MidiClockMessageTypes::FTempoChange>();

						if (!TestEqual("Tempo is correct", TempoChange.Tempo, TempoChangeTempos[NumTempoEvents], 0.001f))
						{
							return;
						}

						if (!TestEqual("Tick is correct", TempoChange.Tick, TempoChangeTicks[NumTempoEvents]))
						{
							return;
						}

						NumTempoEvents++;
					}
				}

				TestEqual("Got correct number of tempo events", NumTempoEvents, NumChanges);
			});

		});

		Describe("Time signature Changes", [this]()
		{
			It("BarBeatTickIncludingCountInToTick", [this]()
			{
				SongMaps = MakeShared<FSongMaps>(123, 4, 4);

				{
					// calculate the test tick
					int32 ExpectedTick = 10;

					// make sure this method returns the correct value
					const int32 Tick = SongMaps->BarBeatTickIncludingCountInToTick(0, 1, 10);

								
					if (!TestEqual("Tick at Bar 1 Beat 1 Tick 10", Tick , ExpectedTick))
					{
						return;
					}
				}

				{
					// calculate the test tick
					int32 ExpectedTick = 10 + SongMaps->GetTicksPerQuarterNote();

					// make sure this method returns the correct value
					const int32 Tick = SongMaps->BarBeatTickIncludingCountInToTick(0, 2, 10);

					if (!TestEqual("Tick at Bar 1 Beat 2 Tick 10", Tick , ExpectedTick))
					{
						return;
					}
				}

				{
					// calculate the test tick
					int32 ExpectedTick = 10 + 2 * SongMaps->GetTicksPerQuarterNote();

					// make sure this method returns the correct value
					const int32 Tick = SongMaps->BarBeatTickIncludingCountInToTick(0, 3, 10);

				
					if (!TestEqual("Tick at Bar 1 Beat 3 Tick 10", Tick , ExpectedTick))
					{
						return;
					}
				}
			});
			
			It("Without driving clock - One Change", [this]()
			{
				SongMaps = MakeShared<FSongMaps>(123, 5, 8);
				const int32 TimeSigChangeTick = SongMaps->BarBeatTickIncludingCountInToTick(2, 1, 0);
				const FTimeSignature NewTimeSig{ 3, 4 };
				SongMaps->AddTimeSigChange(TimeSigChangeTick, NewTimeSig.Numerator, NewTimeSig.Denominator);
				TestClock->AttachToSongMapEvaluator(SongMaps);
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
						
				while (TestClock->GetLastProcessedMidiTick() < TimeSigChangeTick)
				{
					if (TestClock->GetLastProcessedMidiTick() > 0)
					{
						TestClock->PrepareBlock();
					}
					TestClock->Advance(0, OperatorSettings.GetNumFramesPerBlock());
				}

				bool HasTimeSignatureEvent = false;

				for (const FMidiClockEvent& Event : TestClock->GetMidiClockEventsInBlock())
				{
					if (Event.Msg.IsType<MidiClockMessageTypes::FTimeSignatureChange>())
					{
						const MidiClockMessageTypes::FTimeSignatureChange& TimeSigChange = Event.Msg.Get<MidiClockMessageTypes::FTimeSignatureChange>();

						if (!TestEqual("Time signature is correct", TimeSigChange.TimeSignature, NewTimeSig))
						{
							return;
						}

						if (!TestEqual("Tick is correct", TimeSigChange.Tick, TimeSigChangeTick))
						{
							return;
						}
								
						HasTimeSignatureEvent = true;
						break;
					}
				}

				TestTrue("Got time sig event", HasTimeSignatureEvent);
			});

			It("Without driving clock - One Change with tempos", [this]()
			{
				SongMaps = MakeShared<FSongMaps>(123, 5, 8);
				const int32 TimeSigChangeTick = SongMaps->BarBeatTickIncludingCountInToTick(2, 1, 0);
				constexpr int32 NumTempoChanges = 3;
				constexpr int32 TempoChangeTicks[NumTempoChanges] = { 4799, 4800, 4801 };
				constexpr float TempoChangeTempos[NumTempoChanges] = { 155.0f, 157.2f, 158.4f };
				SongMaps->AddTempoChange(TempoChangeTicks[0], TempoChangeTempos[0]);
				const FTimeSignature NewTimeSig{ 3, 4 };
				SongMaps->AddTimeSigChange(TimeSigChangeTick, NewTimeSig.Numerator, NewTimeSig.Denominator);
				SongMaps->AddTempoChange(TempoChangeTicks[1], TempoChangeTempos[1]);
				SongMaps->AddTempoChange(TempoChangeTicks[2], TempoChangeTempos[2]);
				TestClock->AttachToSongMapEvaluator(SongMaps);
				TestClock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

				while (TestClock->GetLastProcessedMidiTick() < TimeSigChangeTick)
				{
					if (TestClock->GetLastProcessedMidiTick() > 0)
					{
						TestClock->PrepareBlock();
					}
					TestClock->Advance(0, OperatorSettings.GetNumFramesPerBlock());
				}

				bool HasTimeSignatureEvent = false;
				int32 NumFoundTempoChanges = 0;

				for (const FMidiClockEvent& Event : TestClock->GetMidiClockEventsInBlock())
				{
					if (Event.Msg.IsType<MidiClockMessageTypes::FTimeSignatureChange>())
					{
						const MidiClockMessageTypes::FTimeSignatureChange& TimeSigChange = Event.Msg.Get<MidiClockMessageTypes::FTimeSignatureChange>();

						if (!TestEqual("Time signature is correct", TimeSigChange.TimeSignature, NewTimeSig))
						{
							return;
						}

						if (!TestEqual("Tick is correct", TimeSigChange.Tick, TimeSigChangeTick))
						{
							return;
						}

						TestFalse("Already found time signature", HasTimeSignatureEvent);

						HasTimeSignatureEvent = true;
					}
					else if (const MidiClockMessageTypes::FTempoChange* AsTempoChange = Event.TryGet<MidiClockMessageTypes::FTempoChange>())
					{
						if (!TestEqual("Tempo is correct", AsTempoChange->Tempo, TempoChangeTempos[NumFoundTempoChanges], 0.001f))
						{
							return;
						}

						if (!TestEqual("Tick is correct", AsTempoChange->Tick, TempoChangeTicks[NumFoundTempoChanges]))
						{
							return;
						}

						NumFoundTempoChanges++;
					}
				}

				TestTrue("Got time sig event", HasTimeSignatureEvent);
				TestEqual("Found correct number of tempo changes", NumFoundTempoChanges, NumTempoChanges);
			});
		});
	}


}

#endif