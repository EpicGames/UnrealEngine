// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasound::MidiStream::Tests
{
	FMidiStream MakeTestMidiStream(const Metasound::FOperatorSettings& OperatorSettings)
	{
		FMidiStream Stream;

		// Make a bunch of note events
		constexpr int32 NumNotes = 3;
		uint8 NoteNumber = 0;
		int32 EventAudioFrame = 0;
		
		for (int32 i = 0; i < NumNotes; ++i)
		{
			FMidiStreamEvent NoteOn{static_cast<uint32>(0), FMidiMsg::CreateNoteOn(5, NoteNumber, 123)};
			NoteOn.BlockSampleFrameIndex = EventAudioFrame;
			Stream.InsertMidiEvent(NoteOn);

			constexpr int32 AudioFrameDelta = 11;
			EventAudioFrame += AudioFrameDelta;
			
			FMidiStreamEvent NoteOff{static_cast<uint32>(0), FMidiMsg::CreateNoteOff(5, NoteNumber)};
			NoteOff.BlockSampleFrameIndex = EventAudioFrame;
			Stream.InsertMidiEvent(NoteOff);

			constexpr uint8 NoteNumberDelta = 17;
			NoteNumber += NoteNumberDelta;
		}

		// Make a bunch of CC events
		for (int32 i = 0; i < OperatorSettings.GetNumFramesPerBlock(); ++i)
		{
			FMidiStreamEvent CC{ static_cast<uint32>(0), FMidiMsg::CreateControlChange(7, 45, i % 128)};
			CC.BlockSampleFrameIndex = i;
			Stream.InsertMidiEvent(CC);
		}
		
		return Stream;
	}

	bool EventsAreEqual(const FMidiStreamEvent& A, const FMidiStreamEvent& B, bool MatchVoiceId = true)
	{
		return (!MatchVoiceId || A.GetVoiceId() == B.GetVoiceId())
		&& A.TrackIndex == B.TrackIndex
		&& A.BlockSampleFrameIndex == B.BlockSampleFrameIndex
		&& A.MidiMessage == B.MidiMessage;
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiStreamCopyTest,
		"Harmonix.Metasound.DataTypes.MidiStream.Copy",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiStreamCopyTest::RunTest(const FString&)
	{
		Metasound::FOperatorSettings OperatorSettings{ 48000, 100 };
		const FMidiStream FromStream = MakeTestMidiStream(OperatorSettings);
		FMidiStream ToStream;

		// Copy with no filter or transform
		{
			FMidiStream::Copy(FromStream, ToStream);

			const TArray<FMidiStreamEvent>& FromEvents = FromStream.GetEventsInBlock();
			const TArray<FMidiStreamEvent>& ToEvents = ToStream.GetEventsInBlock();
			UTEST_EQUAL("Same number of events", ToEvents.Num(), FromEvents.Num());
			
			for (int i = 0; i < FromEvents.Num(); ++i)
			{
				UTEST_TRUE("Events match", EventsAreEqual(FromEvents[i], ToEvents[i]));
			}
		}

		// Copy with filter
		{
			const auto Filter = [](const FMidiStreamEvent& Event)
			{
				return Event.MidiMessage.IsNoteOn();
			};
			
			FMidiStream::Copy(FromStream, ToStream, Filter);

			const TArray<FMidiStreamEvent>& FromEvents = FromStream.GetEventsInBlock();
			const TArray<FMidiStreamEvent>& ToEvents = ToStream.GetEventsInBlock();
			
			for (int i = 0; i < FromEvents.Num(); ++i)
			{
				if (Filter(FromEvents[i]))
				{
					const FMidiStreamEvent FromEvent = FromEvents[i];
					const bool Found = ToEvents.ContainsByPredicate([FromEvent](const FMidiStreamEvent& Event)
					{
						return EventsAreEqual(Event, FromEvent);
					});
					UTEST_TRUE("Event passed through", Found);
				}
			}
		}

		// Copy with transformation
		{
			const auto Transform = [](const FMidiStreamEvent& Event)
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					FMidiStreamEvent TransformedEvent = Event;
					TransformedEvent.MidiMessage.Data1 = TransformedEvent.MidiMessage.Data1 + 2;
					return TransformedEvent;
				}

				return Event;
			};
			FMidiStream::Copy(FromStream, ToStream, FMidiStream::NoOpFilter, Transform);

			const TArray<FMidiStreamEvent>& FromEvents = FromStream.GetEventsInBlock();
			const TArray<FMidiStreamEvent>& ToEvents = ToStream.GetEventsInBlock();
			UTEST_EQUAL("Same number of events", ToEvents.Num(), FromEvents.Num());
			
			for (int i = 0; i < FromEvents.Num(); ++i)
			{
				FMidiStreamEvent ExpectedEvent = FromEvents[i];
				
				if (FromEvents[i].MidiMessage.IsNoteOn())
				{
					ExpectedEvent.MidiMessage.Data1 = ExpectedEvent.MidiMessage.Data1 + 2;
				}

				UTEST_TRUE("Events match", EventsAreEqual(ExpectedEvent, ToEvents[i]));
			}
		}
		
		return true;
	}

	void MakeMergeStreams(
		const Metasound::FOperatorSettings& OperatorSettings,
		FMidiStream& FromA,
		FMidiStream& FromB,
		FMidiStream& Expected,
		const FMidiStream::FEventFilter& Filter = FMidiStream::NoOpFilter,
		const FMidiStream::FEventTransformer& Transformer = FMidiStream::NoOpTransformer)
	{
		FromA.PrepareBlock();
		FromA = MakeTestMidiStream(OperatorSettings);
		
		// Change the values of the std messages in the second stream
		FromB.PrepareBlock();
		for (const FMidiStreamEvent& Event : FromA.GetEventsInBlock())
		{
			if (Event.MidiMessage.IsStd())
			{
				FMidiStreamEvent Transformed = Event;
				const uint8 Data1 = Transformed.MidiMessage.Data1;
				Transformed.MidiMessage.Data1 = Transformed.MidiMessage.Data2;
				Transformed.MidiMessage.Data2 = Data1;
				FromB.InsertMidiEvent(Transformed);
			}
			else
			{
				FromB.InsertMidiEvent(Event);
			}
		}

		// Generate the expected outcome from merging
		Expected.PrepareBlock();
		for (const FMidiStreamEvent& Event : FromA.GetEventsInBlock())
		{
			if (Filter(Event))
			{
				Expected.InsertMidiEvent(Transformer(Event));
			}
		}
		for (const FMidiStreamEvent& Event : FromB.GetEventsInBlock())
		{
			if (Filter(Event))
			{
				Expected.InsertMidiEvent(Transformer(Event));
			}
		}
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiStreamMergeTest,
		"Harmonix.Metasound.DataTypes.MidiStream.Merge",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiStreamMergeTest::RunTest(const FString&)
	{
		Metasound::FOperatorSettings OperatorSettings{ 48000, 100 };
		FMidiStream FromA;
		FMidiStream FromB;
		FMidiStream Expected;
		FMidiStream To;

		// no filter or transform
		{
			MakeMergeStreams(OperatorSettings, FromA, FromB, Expected);
			To.PrepareBlock();
			FMidiStream::Merge(FromA, FromB, To);

			const TArray<FMidiStreamEvent>& ExpectedEvents = Expected.GetEventsInBlock();
			const TArray<FMidiStreamEvent>& ToEvents = To.GetEventsInBlock();
			UTEST_EQUAL("Same number of events", ToEvents.Num(), ExpectedEvents.Num());
			
			for (int i = 0; i < ExpectedEvents.Num(); ++i)
			{
				UTEST_TRUE("Events match", EventsAreEqual(ExpectedEvents[i], ToEvents[i], false));
			}
		}

		// with filter
		{
			const auto Filter = [](const FMidiStreamEvent& Event)
			{
				return Event.MidiMessage.IsNoteOn();
			};
			
			MakeMergeStreams(OperatorSettings, FromA, FromB, Expected, Filter);
			To.PrepareBlock();
			FMidiStream::Merge(FromA, FromB, To, Filter);

			const TArray<FMidiStreamEvent>& ExpectedEvents = Expected.GetEventsInBlock();
			const TArray<FMidiStreamEvent>& ToEvents = To.GetEventsInBlock();
			UTEST_EQUAL("Same number of events", ToEvents.Num(), ExpectedEvents.Num());
			
			for (int i = 0; i < ExpectedEvents.Num(); ++i)
			{
				UTEST_TRUE("Events match", EventsAreEqual(ExpectedEvents[i], ToEvents[i], false));
			}
		}

		// with transformation
		{
			const auto Transform = [](const FMidiStreamEvent& Event)
			{
				if (Event.MidiMessage.IsControlChange())
				{
					FMidiStreamEvent Transformed = Event;
					Transformed.MidiMessage.Data2 = 123;
					return Transformed;
				}

				return Event;
			};
			
			MakeMergeStreams(OperatorSettings, FromA, FromB, Expected, FMidiStream::NoOpFilter, Transform);
			To.PrepareBlock();
			FMidiStream::Merge(FromA, FromB, To, FMidiStream::NoOpFilter, Transform);

			const TArray<FMidiStreamEvent>& ExpectedEvents = Expected.GetEventsInBlock();
			const TArray<FMidiStreamEvent>& ToEvents = To.GetEventsInBlock();
			UTEST_EQUAL("Same number of events", ToEvents.Num(), ExpectedEvents.Num());
			
			for (int i = 0; i < ExpectedEvents.Num(); ++i)
			{
				UTEST_TRUE("Events match", EventsAreEqual(ExpectedEvents[i], ToEvents[i], false));
			}
		}
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiStreamMergeClockTest,
	"Harmonix.Metasound.DataTypes.MidiStream.Merge.ClockIsSet",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiStreamMergeClockTest::RunTest(const FString&)
	{
		// Make sure merging streams results in the correct clock being set on the output stream
		Metasound::FOperatorSettings OperatorSettings{ 48000, 100 };
		FMidiStream FromA;
		FMidiStream FromB;
		FMidiStream To;

		TSharedRef<const FMidiClock, ESPMode::NotThreadSafe> ClockA = MakeShared<FMidiClock, ESPMode::NotThreadSafe>(OperatorSettings);
		FromA.SetClock(*ClockA);

		FMidiStream::Merge(FromA, FromB, To);
		TSharedPtr<const FMidiClock, ESPMode::NotThreadSafe> ToClock = To.GetClock();
		UTEST_EQUAL("Output clock is clock A", ClockA.ToSharedPtr(), ToClock);
		
		return true;
	}
}

#endif