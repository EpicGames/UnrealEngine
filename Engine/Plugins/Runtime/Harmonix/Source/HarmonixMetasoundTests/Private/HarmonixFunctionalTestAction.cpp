// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixFunctionalTestAction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixFunctionalTestAction)

void UHarmonixFunctionalTestActionSequence::Prepare_Implementation(AFunctionalTest* Test)
{
	ActionStack.Reserve(ActionSequence.Num());
	for (int32 Idx = ActionSequence.Num() - 1; Idx >= 0; --Idx)
	{
		ActionStack.Add(ActionSequence[Idx]);
	}

	if (ActionStack.Num() > 0)
	{
		CurrentAction = ActionStack.Pop();
	}

	if (CurrentAction)
	{
		CurrentAction->Prepare(Test);
	}
}

void UHarmonixFunctionalTestActionSequence::OnStart_Implementation(AFunctionalTest* Test)
{
	if (IsFinished())
	{
		return;
	}

	if (CurrentAction)
	{
		CurrentAction->OnStart(Test);
		if (CurrentAction->IsFinished())
		{
			if (!CurrentAction->ShouldContinue())
			{
				Finish(false);
			}
			CurrentAction = nullptr;
		}
	}
}

void UHarmonixFunctionalTestActionSequence::Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds)
{
	if (IsFinished())
	{
		return;
	}
		
	if (CurrentAction == nullptr && ActionStack.Num() > 0)
	{
		CurrentAction = ActionStack.Pop();
		CurrentAction->Prepare(Test);
		CurrentAction->OnStart(Test);
	}
		
	if (CurrentAction)
	{
		CurrentAction->Tick(Test, DeltaSeconds);
		if (CurrentAction->IsFinished())
		{
			if (!CurrentAction->ShouldContinue())
			{
				Finish(false);
			}
			CurrentAction = nullptr;
		}
	}

	if (!IsFinished() && CurrentAction == nullptr && ActionStack.IsEmpty())
	{
		Finish(true);
	}
}

void UHarmonixFunctionalTestActionSequence::OnFinished_Implementation()
{
	CurrentAction = nullptr;
	ActionStack.Empty();
}

void UHarmonixFunctionalTestActionParallel::Prepare_Implementation(AFunctionalTest* Test)
{
	ActionStack.Reserve(ParallelActions.Num());
	for (int32 Idx = ParallelActions.Num() - 1; Idx >= 0; --Idx)
	{
		ActionStack.Add(ParallelActions[Idx]);
	}

	int32 ActionIdx = ActionStack.Num() - 1;
	while (ActionIdx >= 0)
	{
		ActionStack[ActionIdx]->Prepare(Test);
		--ActionIdx;
	}
}

void UHarmonixFunctionalTestActionParallel::OnStart_Implementation(AFunctionalTest* Test)
{
	int32 ActionIdx = ActionStack.Num() - 1;
	while (ActionIdx >= 0)
	{
		TObjectPtr<UHarmonixFunctionalTestAction> Action = ActionStack[ActionIdx];

		Action->OnStart(Test);
		if (Action->IsFinished())
		{
			if (!Action->ShouldContinue())
			{
				ActionIdx = -1;
				FinishAllActions(false);
				break;
			}
			
			ActionStack.RemoveAt(ActionIdx);
		}
		--ActionIdx;
	}
}

void UHarmonixFunctionalTestActionParallel::Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds)
{
	if (IsFinished())
	{
		return;
	}

	bool bContinue = true;
	int32 ActionIdx = ActionStack.Num() - 1;
	while (ActionIdx >= 0)
	{
		TObjectPtr<UHarmonixFunctionalTestAction> Action = ActionStack[ActionIdx];

		Action->Tick(Test, DeltaSeconds);
		if (Action->IsFinished())
		{
			if (!Action->ShouldContinue())
			{
				ActionIdx = -1;
				bContinue = false;
				FinishAllActions(bContinue);
				break;
			}
			
			ActionStack.RemoveAt(ActionIdx);
		}
		--ActionIdx;
	}

	if (!IsFinished() && ActionStack.IsEmpty())
	{
		Finish(bContinue);
	}
}

void UHarmonixFunctionalTestActionParallel::OnFinished_Implementation()
{
	FinishAllActions(ShouldContinue());
}

void UHarmonixFunctionalTestActionParallel::FinishAllActions(bool bContinue)
{
	while (ActionStack.Num() > 0)
	{
		ActionStack.Pop()->Finish(bContinue);
	}
}

void UHarmonixFunctionalTestActionDelay::OnStart_Implementation(AFunctionalTest* Test)
{
	TotalTime = 0.0f;
}

void UHarmonixFunctionalTestActionDelay::Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds)
{
	if (IsFinished())
	{
		return;
	}
	
	TotalTime += DeltaSeconds;

	if (TotalTime > DelaySeconds)
	{
		Finish(true);
	}
}

void UHarmonixFunctionalTestActionFinishTest::OnStart_Implementation(AFunctionalTest* Test)
{
	Test->FinishTest(Result, Message);
	Finish(false);
}

