// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "FunctionalTest.h"

#include "HarmonixFunctionalTestAction.generated.h"

class AFunctionalTest;

UCLASS(Blueprintable, EditInlineNew, CollapseCategories, HideCategories=(Object), Abstract)
class UHarmonixFunctionalTestAction : public UObject
{
	GENERATED_BODY()
	
public:

	virtual void Prepare_Implementation(AFunctionalTest* Test) {};

	virtual void OnStart_Implementation(AFunctionalTest* Test) {};
	
	virtual void OnFinished_Implementation() {};

	virtual void Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds) {};

	bool ShouldContinue() const
	{
		return bShouldContinue;
	}
	
	UFUNCTION(BlueprintNativeEvent, Category="Functional Testing")
	void Prepare(AFunctionalTest* Test);

	UFUNCTION(BlueprintNativeEvent, Category="Functional Testing")
	void OnStart(AFunctionalTest* Test);

	UFUNCTION(BlueprintNativeEvent, Category="Functional Testing")
	void OnFinished();

	UFUNCTION(BlueprintNativeEvent, Category = "Functional Testing")
	void Tick(AFunctionalTest* Test, float DeltaSeconds);

	// called by the user when finished with the step
	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	void Finish(bool bContinue = true)
	{
		if (IsFinished())
		{
			return;
		}
		
		bShouldContinue = bContinue;
		bIsFinished = true;
		OnFinished();
	}

	UFUNCTION(BlueprintPure, Category = "Functional Testing")
	bool IsFinished() const { return bIsFinished; }
	
private:
	
	bool bIsFinished = false;
	bool bShouldContinue = false;
};

UCLASS(Blueprintable, meta=(DisplayName = "Action Sequence"))
class UHarmonixFunctionalTestActionSequence : public UHarmonixFunctionalTestAction
{
	GENERATED_BODY()

public:
	
	virtual void Prepare_Implementation(AFunctionalTest* Test) override;

	virtual void OnStart_Implementation(AFunctionalTest* Test) override;

	virtual void Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds) override;

	virtual void OnFinished_Implementation() override;

	UPROPERTY(EditAnywhere, Instanced, Category = "Functional Testing")
	TArray<TObjectPtr<UHarmonixFunctionalTestAction>> ActionSequence;

private:
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<UHarmonixFunctionalTestAction>> ActionStack;

	UPROPERTY(Transient)
	TObjectPtr<UHarmonixFunctionalTestAction> CurrentAction;
};

UCLASS(Blueprintable, meta=(DisplayName = "Parallel Actions"))
class UHarmonixFunctionalTestActionParallel : public UHarmonixFunctionalTestAction
{
	GENERATED_BODY()

public:
	
	virtual void Prepare_Implementation(AFunctionalTest* Test) override;

	virtual void OnStart_Implementation(AFunctionalTest* Test) override;

	virtual void Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds) override;

	virtual void OnFinished_Implementation() override;

	UPROPERTY(EditAnywhere, Instanced, Category = "Functional Testing")
	TArray<TObjectPtr<UHarmonixFunctionalTestAction>> ParallelActions;

private:
	
	void FinishAllActions(bool bContinue);
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<UHarmonixFunctionalTestAction>> ActionStack;
};

UCLASS(NotBlueprintable, meta=(DisplayName="Delay"))
class UHarmonixFunctionalTestActionDelay final : public UHarmonixFunctionalTestAction
{
public:

	GENERATED_BODY()
	
	virtual void OnStart_Implementation(AFunctionalTest* Test) override;
	
	virtual void Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds) override;

	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	float DelaySeconds = 1.0f;

private:

	float TotalTime = 0.0f;
};

UCLASS(NotBlueprintable, meta=(DisplayName="Wait For Timeout"))
class UHarmonixFunctionalTestActionWaitForTimeout final : public UHarmonixFunctionalTestAction
{
	GENERATED_BODY()
};

UCLASS(NotBlueprintable, meta=(DisplayName="Finish Test"))
class UHarmonixFunctionalTestActionFinishTest : public UHarmonixFunctionalTestAction
{
	GENERATED_BODY()

public:
	
	virtual void OnStart_Implementation(AFunctionalTest* Test) override;

	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	EFunctionalTestResult Result = EFunctionalTestResult::Default;

	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	FString Message = TEXT("Finish Test");
	
};
