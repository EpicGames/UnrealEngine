// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasoundFunctionalTest.h"
#include "HarmonixFunctionalTestAction.h"

#include "MetasoundGenerator.h"
#include "Components/AudioComponent.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixDsp/AudioAnalysis/AnalysisUtilities.h"

#include "Audio.h"
#include "Audio/SimpleWaveWriter.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerAudioBuffer.h"
#include "HAL/FileManager.h"
#include "HarmonixDsp/AudioAnalysis/WaveFileComparison.h"
#include "Interfaces/IPluginManager.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/FileHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixMetasoundFunctionalTest)


DEFINE_LOG_CATEGORY(LogHarmonixMetasoundTests)

namespace HarmonixMetasoundTests
{
	static int32 WriteOutputToFileCVar = 0;
	FAutoConsoleVariableRef CVarWriteOutputToFile(
		TEXT("harmonix.tests.WriteOutputToFile"),
		WriteOutputToFileCVar,
		TEXT("Whether to write the output of the unit tests to \".wav\" files store in the AudioCapture directory \\[ProjectDirectory]\\Saved\\AudioCaptures\\/\n")
		TEXT("0: Disabled 1: Always write output 2: Only write output on error"),
		ECVF_Default);

	
	FString MetasoundOutputValueAsString(const FMetaSoundOutput& Output)
	{
		if (float Value; Output.Get(Value))
		{
			return FString::Printf(TEXT("%f"), Value);
		}
		if (int32 Value; Output.Get(Value))
		{
			return FString::Printf(TEXT("%d"), Value);
		}
		if (bool Value; Output.Get(Value))
		{
			return FString::Printf(TEXT("%s"), Value ? TEXT("true") : TEXT("false"));
		}
		if (FString Value; Output.Get(Value))
		{
			return Value;
		}
		if (Metasound::FTime Value; Output.Get(Value))
		{
			return FString::Printf(TEXT("%f"), (float)Value.GetSeconds());
		}
		return FString::Printf(TEXT("Unsupported logging type: %s"), *Output.GetDataTypeName().ToString());
	}
}

bool UHarmonixMetasoundFunctionalTestLibrary::AddOutputLogger(UMetasoundGeneratorHandle* GeneratorHandle, FName OutputName, EAudioParameterType Type)
{
	UE_LOGF(LogHarmonixMetasoundTests, Log, "Adding output logger for output: %ls", *OutputName.ToString());
	TWeakObjectPtr<const UObject> WeakContext = GeneratorHandle;
	bool ReturnValue = GeneratorHandle->WatchOutput(
		OutputName, FOnMetasoundOutputValueChangedNative::CreateLambda(
			[WeakContext](FName OutputName, const FMetaSoundOutput& Output) 
			{
				FString Message = FString::Printf(TEXT("%s: %s"), *OutputName.ToString(), *HarmonixMetasoundTests::MetasoundOutputValueAsString(Output));
				UKismetSystemLibrary::PrintString(nullptr, Message);
			}
		)
	);
	if (!ReturnValue)
	{
		UE_LOGF(LogHarmonixMetasoundTests, Error, "Failed to add logger for output: %ls", *OutputName.ToString());
	}
	return ReturnValue;
}

bool UHarmonixMetasoundFunctionalTestLibrary::AddMidiStreamLogger(UMetasoundGeneratorHandle* GeneratorHandle, FName OutputName)
{
	using namespace HarmonixMetasound;
	return GeneratorHandle->WatchOutput(
		OutputName, FOnMetasoundOutputValueChangedNative::CreateLambda(
				[](FName OutputName, const FMetaSoundOutput& Output)
				{
					if (FMidiStream MidiStream; Output.Get<FMidiStream>(MidiStream))
					{
						for (const FMidiStreamEvent& Event : MidiStream.GetEventsInBlock())
						{
							FString Message = FString::Printf(TEXT("%s: Event: BlockSampleFrameIndex=%d, MidiTick=%d, IsNoteMessage=%s, IsNoteOn=%s, Std1=%d"),
								*OutputName.ToString(),
								Event.BlockSampleFrameIndex,
								Event.CurrentMidiTick,
								Event.MidiMessage.IsNoteMessage() ? TEXT("true") : TEXT("false"),
								Event.MidiMessage.IsNoteOn() ? TEXT("true") : TEXT("false"),
								Event.MidiMessage.IsStd() ? Event.MidiMessage.GetStdData1() : 0);
							UKismetSystemLibrary::PrintString(nullptr, Message);
						}
					}
				}
			)
		);
}

FString UHarmonixMetasoundFunctionalTestLibrary::WriteAudioToFile(const FString& Filename, int32 SampleRate, int32 NumChannels, const Audio::FAlignedFloatBuffer& Audio)
{
	// write output to file in the "Audio Capture Directory"
	static const FString RootPath = FPaths::AudioCaptureDir();
	FString OutFilename = RootPath / Filename;
	TUniquePtr<FArchive> Stream{ IFileManager::Get().CreateFileWriter(*OutFilename, IO_WRITE) };
	// create the Wave Writer to write output to file
	const TUniquePtr<Audio::FSimpleWaveWriter> Writer = MakeUnique<Audio::FSimpleWaveWriter>(MoveTemp(Stream), SampleRate, NumChannels, true);
	Writer->Write(MakeArrayView(Audio.GetData(), Audio.Num()));
	return OutFilename;
}

bool UHarmonixMetasoundFunctionalTestLibrary::ReadAudioFromFile(const FString& Filepath, Audio::FAlignedFloatBuffer& OutAudio, int32& OutSampleRate, int32& OutNumFrames, int32& OutNumChannels, uint16& OutFormatTag)
{
	OutAudio.Reset();
	OutSampleRate = -1;
	OutNumChannels = -1;
	OutFormatTag = 0;
	
	if (!FPaths::FileExists(Filepath))
	{
		UE_LOGF(LogHarmonixMetasoundTests, Error, "Failed to read wave file %ls: File does not exist", *Filepath);
		return false;
	}

	TArray64<uint8> FileData;
	if (!ensure(FFileHelper::LoadFileToArray(FileData, *Filepath)))
	{
		UE_LOGF(LogHarmonixMetasoundTests, Error, "Failed to read wave file %ls: Unable to convert data to array", *Filepath)
		return false;
	}

	FWaveModInfo WaveInfo;
	FString ErrorMessage;

	if (!ensure(WaveInfo.ReadWaveInfo(FileData.GetData(), FileData.Num(), &ErrorMessage)))
	{
		UE_LOGF(LogHarmonixMetasoundTests, Error, "Failed to read wave file %ls: %ls", *Filepath, *ErrorMessage);
		return false;
	}
	
	// if (!ensure(*WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_PCM || *WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_IEEE_FLOAT))
	// {
	// 	UE_LOGF(LogHarmonixMetasoundTests, Error, "Failed to read wave file %ls: Unable to read format. Must be PCM short or Float");
	// 	return false;
	// }

	if (*WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_PCM)
	{
		if (!ensure(*WaveInfo.pBitsPerSample == 16))
		{
			UE_LOGF(LogHarmonixMetasoundTests, Error, "Failed to read wave file %ls: Unable to read format. Must be 16 bit PCM", *Filepath);
			return false;
		}
		
		OutFormatTag = *WaveInfo.pFormatTag;
		OutSampleRate = *WaveInfo.pSamplesPerSec;
		OutNumChannels= *WaveInfo.pChannels;
		OutNumFrames = WaveInfo.SampleDataSize / OutNumChannels / (sizeof(int16));
		OutAudio.AddUninitialized(OutNumFrames * OutNumChannels);
		const int16* RawDataPtr = (const int16*)WaveInfo.SampleDataStart;
		constexpr float Max16BitAsFloat = static_cast<float>(TNumericLimits<int16>::Max());
		for (int32 Frame = 0; Frame < OutNumFrames; ++Frame)
		{
			for (int32 Channel = 0; Channel < OutNumChannels; ++Channel)
			{
				int32 SampleIdx = Frame * OutNumChannels + Channel;
				OutAudio[SampleIdx] = RawDataPtr[SampleIdx] / Max16BitAsFloat;
			}
		}

		return true;
	}

	if (*WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_IEEE_FLOAT)
	{
		OutFormatTag = *WaveInfo.pFormatTag;
		OutSampleRate = *WaveInfo.pSamplesPerSec;
		OutNumChannels= *WaveInfo.pChannels;
		OutNumFrames = WaveInfo.SampleDataSize / OutNumChannels / (sizeof(float));
		OutAudio.AddUninitialized(OutNumFrames * OutNumChannels);
		const float* RawDataPtr = (const float*)WaveInfo.SampleDataStart;
		for (int32 Frame = 0; Frame < OutNumFrames; ++Frame)
		{
			for (int32 Channel = 0; Channel < OutNumChannels; ++Channel)
			{
				int32 SampleIdx = Frame * OutNumChannels + Channel;
				OutAudio[SampleIdx] = RawDataPtr[SampleIdx];
			}
		}
		return true;
	}

	UE_LOGF(LogHarmonixMetasoundTests, Error, "Failed to read wave file %ls: Unable to read format. Must be 16 bit PCM or IEEE float", *Filepath);
	return false;
}


AHarmonixMetasoundFunctionalTest::AHarmonixMetasoundFunctionalTest(const FObjectInitializer& ObjectInitializer)
	:AFunctionalTest(ObjectInitializer)
{

	AudioComponent = CreateDefaultSubobject<UAudioComponent>(TEXT("AudioComponent"));
	AudioComponent->SetupAttachment(RootComponent);
	check(AudioComponent);
}

bool AHarmonixMetasoundFunctionalTest::IsReady_Implementation()
{
	bool OutIsReady = AFunctionalTest::IsReady_Implementation() && GeneratorHandle;
	UE_LOGF(LogHarmonixMetasoundTests, Log, "%ls -- Is Ready: %d", *TestLabel, OutIsReady);
	return OutIsReady;
}

void AHarmonixMetasoundFunctionalTest::CompareResults()
{
	if (WavFilename_Expected.IsEmpty())
	{
		return;
	}
	
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("Harmonix"));
	check(Plugin);
	FString TestAudioDir = Plugin->GetContentDir() / TEXT("Editor/Tests/Audio");
	FString Filepath = TestAudioDir / WavFilename_Expected;
	Audio::FAlignedFloatBuffer AudioData;
	int32 SampleRate;
	int32 NumChannels;
	int32 NumFrames;
	uint16 FormatTag;
	bool Success = UHarmonixMetasoundFunctionalTestLibrary::ReadAudioFromFile(Filepath, AudioData, SampleRate, NumFrames, NumChannels, FormatTag);

	if (!AssertTrue(Success, FString::Printf(TEXT("ReadAudioFromFile: %s"), *WavFilename_Expected), this))
	{
		return;
	}

	int32 NumChannels_Actual = 1;
	if (!AssertEqual_Int(NumChannels_Actual, NumChannels, TEXT("NumChannels"), this))
	{
		return;
	}

	if (!AssertTrue(AudioCaptureSampleRate > 0.0f, TEXT("AudioCaptureSampleRate > 0"), this))
	{
		return;
	}

	if (!AssertEqual_Float(AudioCaptureSampleRate, SampleRate, TEXT("SampleRate")))
	{
		return;
	}

	float AudioCaptureDuration = AudioCaptureOutput.Num() / AudioCaptureSampleRate;
	float AudioDuration = AudioData.Num() / (float)SampleRate;
	if (!AssertEqual_Float(AudioCaptureDuration, AudioDuration, TEXT("AudioCaptureDuration"), 0.1f, this))
	{
		return;
	}

	int32 NumFramesToCompare = FMath::Min(AudioCaptureOutput.Num(), AudioData.Num());
	float PSNR = Harmonix::Dsp::AudioAnalysis::CalculatePSNR(AudioCaptureOutput.GetData(), AudioData.GetData(), NumChannels, NumFramesToCompare);

	static constexpr float PSNRThreshold = 60.0f;
	AssertTrue(PSNR >= PSNRThreshold, FString::Printf(TEXT("PSNR = %.2f where the acceptable range is (PSNR >= %.2f) Compared %d frames."), PSNR, PSNRThreshold, NumFramesToCompare), this);
}

void AHarmonixMetasoundFunctionalTest::StartTest()
{
	AFunctionalTest::StartTest();

	UE_LOGF(LogHarmonixMetasoundTests, Log, "%ls -- StartTest", *TestLabel);
	
	if (AudioComponent && AudioAutoStart)
	{
		AudioComponent->Play();
	}

	ActionSequence->OnStart(this);
}

void AHarmonixMetasoundFunctionalTest::FinishTest(EFunctionalTestResult TestResult, const FString& Message)
{
	if (ActionSequence)
	{
		ActionSequence->Finish(false);
	}
	
	if (AudioComponent)
	{
		AudioComponent->Stop();
	}

	CompareResults();
	
	AFunctionalTest::FinishTest(TestResult, Message);
}

void AHarmonixMetasoundFunctionalTest::Tick(float DeltaSeconds)
{
	if (IsRunning())
	{
		if (ActionSequence && !ActionSequence->IsFinished())
		{
			ActionSequence->Tick(this, DeltaSeconds);
			if (ActionSequence->IsFinished())
			{
				FinishTest(EFunctionalTestResult::Default, TEXT("Test completed"));
			}
		}
	}
	AFunctionalTest::Tick(DeltaSeconds);
}


void AHarmonixMetasoundFunctionalTest::PrepareTest()
{
	AFunctionalTest::PrepareTest();

	UE_LOGF(LogHarmonixMetasoundTests, Log, "%ls -- PrepareTest", *TestLabel);
	
	if (!AudioComponent)
	{
		return;
	}

	if (!TestSound)
	{
		return;
	}

	ActionSequence = NewObject<UHarmonixFunctionalTestActionSequence>(this);
	ActionSequence->ActionSequence = FunctionalTestActions;
	ActionSequence->Prepare(this);

	AudioComponent->Sound = TestSound;
	
	GeneratorHandle = UMetasoundGeneratorHandle::CreateMetaSoundGeneratorHandle(AudioComponent);
	
	GeneratorHandle->OnGeneratorHandleAttached.AddLambda([this]()
	{
		if (!GeneratorHandle)
		{
			return;
		}

		TSharedPtr<Metasound::FMetasoundGenerator> Generator = GeneratorHandle->GetGenerator();
		if (!Generator)
		{
			return;
		}

		AudioCaptureSampleRate = Generator->OperatorSettings.GetSampleRate();
		
		AudioCaptureOutput.Reset();
		AudioOutAnalyzerAddress.DataType = Metasound::GetMetasoundDataTypeName<Metasound::FAudioBuffer>();
		AudioOutAnalyzerAddress.InstanceID = 1234;
		AudioOutAnalyzerAddress.OutputName = AudioOutName;
		AudioOutAnalyzerAddress.AnalyzerName = Metasound::Frontend::FVertexAnalyzerAudioBuffer::GetAnalyzerName();
		AudioOutAnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();
		AudioOutAnalyzerAddress.AnalyzerMemberName = Metasound::Frontend::FVertexAnalyzerAudioBuffer::FOutputs::GetValue().Name;
			
		Generator->AddOutputVertexAnalyzer(AudioOutAnalyzerAddress);

		Generator->OnOutputChanged.AddLambda(
			[this](const FName AnalyzerName, const FName OutputName, const FName AnalyzerOutputName, TSharedPtr<Metasound::IOutputStorage> OutputData)
		{
			if (AnalyzerName == AudioOutAnalyzerAddress.AnalyzerName
				&& OutputName == AudioOutAnalyzerAddress.OutputName
				&& AnalyzerOutputName == AudioOutAnalyzerAddress.AnalyzerMemberName)
			{
				const Metasound::FAudioBuffer& AudioBuffer = static_cast<Metasound::TOutputStorage<Metasound::FAudioBuffer>*>(OutputData.Get())->Get();
				AudioCaptureOutput.Append(AudioBuffer.GetData(), AudioBuffer.Num());
			}
		});
	
	});
	
	OnTestFinished.AddDynamic(this, &AHarmonixMetasoundFunctionalTest::OnTestFinishedEvent);
}

void AHarmonixMetasoundFunctionalTest::OnTestFinishedEvent()
{
	UE_LOGF(LogHarmonixMetasoundTests, Log, "%ls -- OnTestFinished", *TestLabel);
	
	OnTestFinished.RemoveDynamic(this, &AHarmonixMetasoundFunctionalTest::OnTestFinishedEvent);
	
	if (GeneratorHandle)
	{
		if (HarmonixMetasoundTests::WriteOutputToFileCVar && !WavFilename_Output.IsEmpty() )
		{
			UHarmonixMetasoundFunctionalTestLibrary::WriteAudioToFile(WavFilename_Output, AudioCaptureSampleRate, 1, AudioCaptureOutput);
		}

		if (TSharedPtr<Metasound::FMetasoundGenerator> Generator = GeneratorHandle->GetGenerator())
		{
			Generator->RemoveOutputVertexAnalyzer(AudioOutAnalyzerAddress);
		}
	}

	if (!WavFilename_Expected.IsEmpty() && !WavFilename_Output.IsEmpty())
	{
			
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("Harmonix"));
		check(Plugin);
		FString TestAudioDir = Plugin->GetContentDir() / TEXT("Editor/Tests/Audio");
		FString Filepath_Expected = TestAudioDir / WavFilename_Expected;

		static const FString RootPath = FPaths::AudioCaptureDir();
		FString Filepath_Output = RootPath / WavFilename_Output;
		
		Harmonix::Dsp::AudioAnalysis::FWaveFileComparison FileComparison;
		FileComparison.LoadForCompare(Filepath_Expected, Filepath_Output);
		float PSNR =  FileComparison.GetPSNR();

		UE_LOGF(LogHarmonixMetasoundTests, Log, "PSNR of files is: %.2f", PSNR);
	}
}




