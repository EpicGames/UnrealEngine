// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FunctionalTest.h"

#include "AudioParameter.h"
#include "MetasoundGeneratorHandle.h"

#include "HarmonixMetasoundFunctionalTest.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHarmonixMetasoundTests, Log, All);

namespace Audio
{
	class FSimpleWaveWriter;
}

class UHarmonixFunctionalTestAction;
class UHarmonixFunctionalTestActionSequence;

UCLASS()
class UHarmonixMetasoundFunctionalTestLibrary final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category="Harmonix Metasound", Meta=(ExpandBoolAsExecs="ReturnValue"))
	static bool AddOutputLogger(UMetasoundGeneratorHandle* GeneratorHandle, FName OutputName, EAudioParameterType Type);

	UFUNCTION(BlueprintCallable, Category="Harmonix Metasound", Meta=(ExpandBoolAsExecs="ReturnValue"))
	static bool AddMidiStreamLogger(UMetasoundGeneratorHandle* GeneratorHandle, FName OutputName);
	
	static FString WriteAudioToFile(const FString& Filename, int32 SampleRate, int32 NumChannels, const Audio::FAlignedFloatBuffer& Audio);

	// absolute filepath
	// 
	// reads audio data from audio file, converting audio to float
	// audio is interleaved in the OutAudio buffer for multi-channel audio
	// OutFormatTag is the format the audio file was saved in
	static bool ReadAudioFromFile(const FString& Filepath, Audio::FAlignedFloatBuffer& OutAudio, int32& OutSampleRate, int32& OutNumFrames, int32& OutNumChannels, uint16& OutFormatTag);
};

UCLASS(Blueprintable)
class AHarmonixMetasoundFunctionalTest : public AFunctionalTest
{
	GENERATED_BODY()
public:

	AHarmonixMetasoundFunctionalTest(const FObjectInitializer& ObjectInitializer);

	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	TObjectPtr<USoundBase> TestSound;

	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	FName AudioOutName = "AudioOut";

	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	bool AudioAutoStart = true;

	// name of wave file with the expected audio output.
	// (searches the Content/Tests/AudioSource directory)
	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	FString WavFilename_Expected;

	// optionally write the output to a file
	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	FString WavFilename_Output;

	UPROPERTY(EditAnywhere, Instanced, Category = "Functional Testing")
	TArray<TObjectPtr<UHarmonixFunctionalTestAction>> FunctionalTestActions;

	virtual void FinishTest(EFunctionalTestResult TestResult, const FString& Message) override;

	virtual void Tick(float DeltaSeconds) override;

protected:
	
	virtual void PrepareTest() override;

	virtual void StartTest() override;

	virtual bool IsReady_Implementation() override;

	void CompareResults();

	UFUNCTION()
	void OnTestFinishedEvent();

	UPROPERTY()
	TObjectPtr<UAudioComponent> AudioComponent;

	UPROPERTY(Transient)
	TObjectPtr<UMetasoundGeneratorHandle> GeneratorHandle;

	UPROPERTY(Transient)
	TObjectPtr<UHarmonixFunctionalTestActionSequence> ActionSequence;

	// the audio captured from the metasound
	Audio::FAlignedFloatBuffer AudioCaptureOutput;
	float AudioCaptureSampleRate = 0.0f;
	
	Metasound::Frontend::FAnalyzerAddress AudioOutAnalyzerAddress;
	
	FTimerHandle TimerHandle;
};