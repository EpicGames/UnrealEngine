// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasoundFunctionalTestAction.h"
#include "Components/AudioComponent.h"
#include "FunctionalTest.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HAL/FileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarmonixMetasoundFunctionalTestAction)

void UHarmonixMetasoundFunctionalTestActionSetAudioParameter::OnStart_Implementation(AFunctionalTest* Test)
{
	if (Test)
	{
		if (TObjectPtr<UAudioComponent> AudioComponent = Test->FindComponentByClass<UAudioComponent>())
		{
			AudioComponent->SetParameter(FAudioParameter(AudioParameter));
			Finish(true);
		}
		else
		{
			Finish(false);
		}
	}
}

void UHarmonixMetasoundFunctionalTestActionSetWaitForAudioFinished::OnStart_Implementation(AFunctionalTest* Test)
{
	if (Test)
	{
		if (TObjectPtr<UAudioComponent> AudioComponent = Test->FindComponentByClass<UAudioComponent>())
		{
			AudioComponent->OnAudioFinished.AddDynamic(this, &UHarmonixMetasoundFunctionalTestActionSetWaitForAudioFinished::OnAudioFinished);
		}
	}
}

void UHarmonixMetasoundFunctionalTestActionSetWaitForAudioFinished::OnAudioFinished()
{
	Finish(true);
}

void UHarmonixMetasoundFunctonalTestActionRecordClockOutput::Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds)
{
	if (IsFinished())
	{
		return;
	}
	
	if (RecordTime <= 0.0f)
	{
		Finish(true);
		return;
	}
	
	if (UMusicClockComponent* MusicClock = Test->GetComponentByClass<UMusicClockComponent>())
	{
		if (MusicClock->GetState() == EMusicClockState::Running)
		{
			if (!bClockStarted)
			{
				bClockStarted = true;
				StartTime = TotalTime;
			}
			FMidiSongPos SongPos = MusicClock->GetSongPos();
			// TODO: uncomment these
			float BarDuration = -1.0f; // MusicClock->GetCurrentSecondsPerBar();
			float BeatDuration = -1.0f; //MusicClock->GetCurrentSecondsPerBeat();
			ClockFrameData.Add({ TotalTime - StartTime, SongPos , BarDuration, BeatDuration, MusicClock->GetState()});
		}
	}
	TotalTime += DeltaSeconds;
	if (TotalTime >= RecordTime)
	{
		Finish(true);
	}
	
}

void UHarmonixMetasoundFunctonalTestActionRecordClockOutput::OnFinished_Implementation()
{
	IConsoleVariable* WriteOutputToFileCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("harmonix.tests.WriteOutputToFile"));
	int32 CVarValue = WriteOutputToFileCVar ? WriteOutputToFileCVar->GetInt() : 0;
	bool bWriteOutputToFile = CVarValue == 1;

	if (!bWriteOutputToFile)
	{
		return;
	}
	
	const FString FilePath = FString::Printf(TEXT("%s%s.csv"), *FPaths::ProjectSavedDir(), *OutputFilename);
	if (TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(*FilePath)); Archive.IsValid())
	{
		const FString Header(TEXT("Time,State,SecondsFromBarOne,SecondsIncludingCountIn,TimeSig,Tempo,BarsIncludingCountIn,BeatsIncludingCountIn,BeatType,SecondsPerBar,SecondsPerBeat,Timestamp,SongSection\n"));
		Archive->Serialize(TCHAR_TO_ANSI(*Header), Header.Len());
		for (const FClockFrameData& Data : ClockFrameData)
		{
			const FString Entry = FString::Printf(
				TEXT("%.4f,%s,%.4f,%.4f,%d/%d,%.4f,%.4f,%.4f,%s,%s,%s,%d:%.4f,%s\n"),
				Data.Time, *StaticEnum<EMusicClockState>()->GetNameByValue((int64)Data.ClockState).ToString(),
				Data.SongPos.SecondsFromBarOne, Data.SongPos.SecondsIncludingCountIn,
				Data.SongPos.TimeSigNumerator, Data.SongPos.TimeSigDenominator, Data.SongPos.Tempo,
				Data.SongPos.BarsIncludingCountIn, Data.SongPos.BeatsIncludingCountIn,
				*StaticEnum<EMusicalBeatType>()->GetNameByValue((int64)Data.SongPos.BeatType).ToString(),
				TEXT("TODO: FIX ME"), TEXT("TODO: FIX ME"),
				Data.SongPos.Timestamp.Bar, Data.SongPos.Timestamp.Beat,
				Data.SongPos.CurrentSongSection.Name.IsEmpty() ? TEXT("None") : *Data.SongPos.CurrentSongSection.Name);
			Archive->Serialize(TCHAR_TO_ANSI(*Entry), Entry.Len());
		}
		Archive->Flush();
		Archive->Close();
	}
}

