// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixFunctionalTestAction.h"
#include "AudioParameter.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMidi/MidiSongPos.h"

#include "HarmonixMetasoundFunctionalTestAction.generated.h"

UCLASS(NotBlueprintable, Meta=(DisplayName="Set Audio Parameter"))
class UHarmonixMetasoundFunctionalTestActionSetAudioParameter : public UHarmonixFunctionalTestAction
{
	GENERATED_BODY()
public:
	virtual void OnStart_Implementation(AFunctionalTest* Test) override;

	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	FAudioParameter AudioParameter;
};

UCLASS(NotBlueprintable, Meta=(DisplayName="Wait For Audio Finished"))
class UHarmonixMetasoundFunctionalTestActionSetWaitForAudioFinished : public UHarmonixFunctionalTestAction
{
	GENERATED_BODY()
public:
	virtual void OnStart_Implementation(AFunctionalTest* Test) override;
protected:
	UFUNCTION()
	void OnAudioFinished();
};

UCLASS(NotBlueprintable, meta=(DisplayName="Record Clock Output"))
class UHarmonixMetasoundFunctonalTestActionRecordClockOutput : public UHarmonixFunctionalTestAction
{
	GENERATED_BODY()
public:
	
	virtual void Tick_Implementation(AFunctionalTest* Test, float DeltaSeconds) override;

	virtual void OnFinished_Implementation() override;

	UPROPERTY(EditAnywhere, Category = "Functional Testing", meta=(ClampMin="0.0", UIMin="0.0"))
	float RecordTime = 1.0f;

	// optionally write the output to a file
	UPROPERTY(EditAnywhere, Category = "Functional Testing")
	FString OutputFilename = FString("Clock.csv");

private:

	float TotalTime = 0.0f;
	float StartTime = 0.0f;
	bool bClockStarted = false;

	struct FClockFrameData
	{
		float Time = 0.0f;
		FMidiSongPos SongPos;
		float BarDuration = 0.0f;
		float BeatDuration = 0.0f;
		EMusicClockState ClockState;
	};

	TArray<FClockFrameData> ClockFrameData;
};