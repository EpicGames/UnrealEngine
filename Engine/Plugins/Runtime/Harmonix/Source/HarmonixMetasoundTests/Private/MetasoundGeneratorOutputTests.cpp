// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MetasoundGeneratorOutput
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace HarmonixMetasound;
	using namespace Metasound;
	using namespace Metasound::Frontend;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOutputMidiClockTest,
	"Harmonix.Metasound.Generator.Outputs.MidiClock",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOutputMidiClockTest::RunTest(const FString&)
	{
		// Build a graph with a metronome node
		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;
		GraphBuilder Builder;
		const FNodeHandle MetronomeNode = Builder.AddNode(
			{ HarmonixMetasound::HarmonixNodeNamespace, "Metronome", "" },
			0);

		const FName InputName = CommonPinNames::Inputs::TransportName;
		const FName OutputName = CommonPinNames::Outputs::MidiClockName;

		// Connect the metronome transport input to the graph input
		{
			const FNodeHandle InputNode = Builder.AddInput(InputName, Metasound::GetMetasoundDataTypeName<FMusicTransportEventStream>());
			const FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(InputName);
			const FInputHandle InputToConnect = MetronomeNode->GetInputWithVertexName(InputName);
			UTEST_TRUE("Connected transport input", InputToConnect->Connect(*OutputToConnect));
		}

		// Connect the metronome clock output to the graph output
		{
			const FNodeHandle OutputNode = Builder.AddOutput(OutputName, Metasound::GetMetasoundDataTypeName<FMidiClock>());
			const FOutputHandle OutputToConnect = MetronomeNode->GetOutputWithVertexName(OutputName);
			const FInputHandle InputToConnect = OutputNode->GetInputWithVertexName(OutputName);
			UTEST_TRUE("Connected clock output", InputToConnect->Connect(*OutputToConnect));
		}

		// Build the graph
		const TUniquePtr<FMetasoundGenerator> Generator = Builder.BuildGenerator(SampleRate, NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// Grab the output read ref
		const TOptional<TDataReadReference<FMidiClock>> OutputReadRef = Generator->GetOutputReadReference<FMidiClock>(OutputName);
		UTEST_TRUE("Got the read ref", OutputReadRef.IsSet());

		return true;
	}
}

#endif