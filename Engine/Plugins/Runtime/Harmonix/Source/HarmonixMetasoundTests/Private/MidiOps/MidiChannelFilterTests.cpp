// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiChannelFilter.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Midi::Ops::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiChannelFilterSetGetTest,
		"Harmonix.Midi.Ops.MidiChannelFilter.SetGet",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiChannelFilterSetGetTest::RunTest(const FString&)
	{
		FMidiChannelFilter Filter;

		// Default: all channels off
		UTEST_FALSE("All channels disabled", Filter.GetChannelEnabled(0));

		// Set all channels on and expect them to be enabled
		Filter.SetChannelEnabled(0, true);
		UTEST_TRUE("All channels enabled", Filter.GetChannelEnabled(0));

		// Set all channels off and expect them to be disabled
		Filter.SetChannelEnabled(0, false);
		UTEST_FALSE("All channels disabled", Filter.GetChannelEnabled(0));

		// Set each channel on and off and check them
		for (uint8 Channel = 1; Channel <= 16; ++Channel)
		{
			Filter.SetChannelEnabled(Channel, true);
			UTEST_TRUE(FString::Printf(TEXT("Channel %i enabled"), Channel), Filter.GetChannelEnabled(Channel));
			Filter.SetChannelEnabled(Channel, false);
			UTEST_FALSE(FString::Printf(TEXT("Channel %i disabled"), Channel), Filter.GetChannelEnabled(Channel));
		}
		
		return true;
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiChannelFilterProcessTest,
		"Harmonix.Midi.Ops.MidiChannelFilter.Process",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiChannelFilterProcessTest::RunTest(const FString&)
	{
		FMidiChannelFilter Filter;
		
		HarmonixMetasound::FMidiStream InputStream;
		HarmonixMetasound::FMidiStream OutputStream;

		// Add some events to the input stream
		constexpr uint8 NoteChannelIdx = 1;
		constexpr uint8 NoteNumber = 76;
		constexpr uint8 NoteVelocity = 67;
		InputStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(NoteChannelIdx, NoteNumber, NoteVelocity)});
		{
			HarmonixMetasound::FMidiStreamEvent NoteOffEvent{ static_cast<uint32>(0), FMidiMsg::CreateNoteOff(NoteChannelIdx, NoteNumber) };
			NoteOffEvent.BlockSampleFrameIndex = 1;
			InputStream.AddMidiEvent(NoteOffEvent);
		}

		// Default: only transport events pass
		Filter.Process(InputStream, OutputStream);
		UTEST_EQUAL("No MIDI events", OutputStream.GetEventsInBlock().Num(), 0);

		// Channel mismatch: no MIDI events pass, transport state hasn't changed, so we won't get another one of those
		constexpr uint8 NoteChannel = NoteChannelIdx + 1;
		Filter.SetChannelEnabled(NoteChannel + 1, true);
		OutputStream.PrepareBlock();
		Filter.Process(InputStream, OutputStream);
		UTEST_EQUAL("No MIDI events", OutputStream.GetEventsInBlock().Num(), 0);

		// Channel match: MIDI events pass, transport state hasn't changed, so we won't get another one of those
		Filter.SetChannelEnabled(NoteChannel, true);
		OutputStream.PrepareBlock();
		Filter.Process(InputStream, OutputStream);
		UTEST_EQUAL("No MIDI events", OutputStream.GetEventsInBlock().Num(), 2);
		for (const HarmonixMetasound::FMidiStreamEvent& Event : OutputStream.GetEventsInBlock())
		{
			UTEST_EQUAL("Channel matches", Event.MidiMessage.GetStdChannel(), NoteChannelIdx);
			UTEST_EQUAL("Note number matches", Event.MidiMessage.GetStdData1(), NoteNumber);
			if (Event.MidiMessage.IsNoteOn())
			{
				UTEST_EQUAL("Velocity matches", Event.MidiMessage.GetStdData2(), NoteVelocity);
			}
		}
		
		return true;
	}
}

#endif