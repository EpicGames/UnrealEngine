// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiNoteFilter.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Midi::Ops::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiNoteFilterProcessTest,
		"Harmonix.Midi.Ops.MidiNoteFilter.Process",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiNoteFilterProcessTest::RunTest(const FString&)
	{
		FMidiNoteFilter Filter;
		
		HarmonixMetasound::FMidiStream InputStream;
		HarmonixMetasound::FMidiStream OutputStream;

		FMidiMsg NoteOnMsg = FMidiMsg::CreateNoteOn(2, 3, 4);
		HarmonixMetasound::FMidiStreamEvent NoteOnEvent{ static_cast<uint32>(0), NoteOnMsg };
		
		// Default: All events pass
		{
			FMidiMsg CCMsg = FMidiMsg::CreateControlChange(3, 4, 5);
			InputStream.AddMidiEvent(NoteOnEvent);
			InputStream.AddMidiEvent({ static_cast<uint32>(0), CCMsg });

			Filter.Process(InputStream, OutputStream);

			bool GotNoteOn = false;
			bool GotCC = false;
			
			for (const HarmonixMetasound::FMidiStreamEvent& Event : OutputStream.GetEventsInBlock())
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					UTEST_EQUAL("Channel matches", Event.MidiMessage.GetStdChannel(), NoteOnMsg.GetStdChannel());
					UTEST_EQUAL("Note number matches", Event.MidiMessage.GetStdData1(), NoteOnMsg.GetStdData1());
					UTEST_EQUAL("Velocity matches", Event.MidiMessage.GetStdData2(), NoteOnMsg.GetStdData2());
					GotNoteOn = true;
				}
				else if (Event.MidiMessage.IsControlChange())
				{
					UTEST_EQUAL("Channel matches", Event.MidiMessage.GetStdChannel(), CCMsg.GetStdChannel());
					UTEST_EQUAL("Control number matches", Event.MidiMessage.GetStdData1(), CCMsg.GetStdData1());
					UTEST_EQUAL("Value matches", Event.MidiMessage.GetStdData2(), CCMsg.GetStdData2());
					GotCC = true;
				}
				else
				{
					UTEST_TRUE("Got an unexpected message", false);
				}
			}

			UTEST_TRUE("Got note on", GotNoteOn);
			UTEST_TRUE("Got CC", GotCC);
		}

		// Change the filter such that the first note is filtered out. We should get a note off.
		{
			// Clear out the last block's events
			InputStream.PrepareBlock();
			UTEST_EQUAL("No events in input", InputStream.GetEventsInBlock().Num(), 0);
			OutputStream.PrepareBlock();
			UTEST_EQUAL("No events in output", InputStream.GetEventsInBlock().Num(), 0);
			
			Filter.MinNoteNumber = NoteOnMsg.GetStdData1() + 1;
			
			Filter.Process(InputStream, OutputStream);

			const TArray<HarmonixMetasound::FMidiStreamEvent>& EventsOut = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Got one event out", EventsOut.Num(), 1);
			UTEST_TRUE("Event is note off", EventsOut[0].MidiMessage.IsNoteOff());
			UTEST_EQUAL("Note off is on correct channel", EventsOut[0].MidiMessage.GetStdChannel(), NoteOnMsg.GetStdChannel());
			UTEST_EQUAL("Note off is on correct note number", EventsOut[0].MidiMessage.GetStdData1(), NoteOnMsg.GetStdData1());
			UTEST_EQUAL("Note off has correct voice id", EventsOut[0].GetVoiceId(), NoteOnEvent.GetVoiceId());
		}

		// Fire the note on again, this time expect it to get filtered out
		{
			// Clear out the last block's events
			InputStream.PrepareBlock();
			UTEST_EQUAL("No events in input", InputStream.GetEventsInBlock().Num(), 0);
			OutputStream.PrepareBlock();
			UTEST_EQUAL("No events in output", InputStream.GetEventsInBlock().Num(), 0);
			
			InputStream.AddMidiEvent(NoteOnEvent);
			
			Filter.Process(InputStream, OutputStream);
			
			const TArray<HarmonixMetasound::FMidiStreamEvent>& EventsOut = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Got no events", EventsOut.Num(), 0);
		}

		// Toggle "IncludeOtherEvents" off, send the CC again, this time expect it to get filtered out
		{
			// Clear out the last block's events
			InputStream.PrepareBlock();
			UTEST_EQUAL("No events in input", InputStream.GetEventsInBlock().Num(), 0);
			OutputStream.PrepareBlock();
			UTEST_EQUAL("No events in output", InputStream.GetEventsInBlock().Num(), 0);
			
			InputStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateControlChange(4, 5, 6) });

			Filter.IncludeOtherEvents.Set(false);
			
			Filter.Process(InputStream, OutputStream);

			const TArray<HarmonixMetasound::FMidiStreamEvent>& EventsOut = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Got no events", EventsOut.Num(), 0);
		}

		return true;
	}
}

#endif