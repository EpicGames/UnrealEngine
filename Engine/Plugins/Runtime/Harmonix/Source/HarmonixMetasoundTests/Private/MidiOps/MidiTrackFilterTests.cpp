// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/MidiTrackFilter.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Midi::Ops::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiTrackFilterProcessTest,
		"Harmonix.Midi.Ops.MidiTrackFilter.Process",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiTrackFilterProcessTest::RunTest(const FString&)
	{
		FMidiTrackFilter Filter;
		
		HarmonixMetasound::FMidiStream InputStream;
		HarmonixMetasound::FMidiStream OutputStream;

		constexpr uint16 MinTrack = 1;
		constexpr uint16 MaxTrack = 37;
		constexpr uint8 Channel = 7;
		constexpr uint8 Data1 = 78;
		constexpr uint8 Data2 = 87;
		constexpr float Tempo = 135.7f;
		const int32 MidiTempo = Harmonix::Midi::Constants::BPMToMidiTempo(Tempo);
		constexpr uint8 TimeSigNumerator = 7;
		constexpr uint8 TimeSigDenominator = 8;

		const auto AddMidiMessageToTrack = [](HarmonixMetasound::FMidiStream& Stream, FMidiMsg&& Msg, uint16 TrackIndex)
		{
			HarmonixMetasound::FMidiStreamEvent Event{ static_cast<uint32>(0), Msg };
			Event.TrackIndex = TrackIndex;
			Stream.AddMidiEvent(Event);
		};
		
		// Default: no events pass
		{
			// Add a tempo and time sig change
			AddMidiMessageToTrack(InputStream, FMidiMsg{ MidiTempo }, 0);
			AddMidiMessageToTrack(InputStream, FMidiMsg{ TimeSigNumerator, TimeSigDenominator }, 0);
			// Add events to each track
			for (uint16 TrackIdx = MinTrack; TrackIdx <= MaxTrack; ++TrackIdx)
			{
				AddMidiMessageToTrack(InputStream, FMidiMsg::CreateNoteOn(Channel, Data1, Data2), TrackIdx);
				AddMidiMessageToTrack(InputStream, FMidiMsg::CreateControlChange(Channel, Data1, Data2), TrackIdx);
			}

			Filter.Process(InputStream, OutputStream);
			UTEST_EQUAL("No events in output", OutputStream.GetEventsInBlock().Num(), 0);
		}

		// Include conductor track
		{
			Filter.SetTrackRange(0, 0, true);
			Filter.Process(InputStream, OutputStream);
			const TArray<HarmonixMetasound::FMidiStreamEvent>& Events = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Only tempo and time sig events in output", Events.Num(), 2);
			for (const HarmonixMetasound::FMidiStreamEvent& Event : Events)
			{
				if (Event.MidiMessage.Type == FMidiMsg::EType::Tempo)
				{
					UTEST_EQUAL("Tempo is correct", Event.MidiMessage.GetMicrosecPerQuarterNote(), MidiTempo);
				}
				else if (Event.MidiMessage.Type == FMidiMsg::EType::TimeSig)
				{
					UTEST_EQUAL("Time sig numerator is correct", Event.MidiMessage.GetTimeSigNumerator(), TimeSigNumerator);
					UTEST_EQUAL("Time sig denominator is correct", Event.MidiMessage.GetTimeSigDenominator(), TimeSigDenominator);
				}
			}
		}

		// Include some tracks within the range
		constexpr uint16 MinTrackToInclude = MinTrack + 3;
		constexpr uint16 MaxTrackToInclude = MaxTrack - 4;
		check(MaxTrackToInclude >= MinTrackToInclude);
		constexpr uint16 NumTracksIncluded = (MaxTrackToInclude - MinTrackToInclude) + 1;
		{
			OutputStream.PrepareBlock();
			Filter.SetTrackRange(MinTrackToInclude, MaxTrackToInclude, false);
			Filter.Process(InputStream, OutputStream);
			constexpr int32 NumEventsExpected = NumTracksIncluded * 2; // we include a note on and a CC above
			UTEST_EQUAL("Correct number of events", OutputStream.GetEventsInBlock().Num(), NumEventsExpected);
		}

		// Change the filter and expect notes to become unstuck
		{
			InputStream.PrepareBlock();
			OutputStream.PrepareBlock();
			Filter.SetTrackRange(0, 0, false);
			Filter.Process(InputStream, OutputStream);
			UTEST_EQUAL("Correct number of note offs", OutputStream.GetEventsInBlock().Num(), NumTracksIncluded);
		}
		
		return true;
	}
}

#endif