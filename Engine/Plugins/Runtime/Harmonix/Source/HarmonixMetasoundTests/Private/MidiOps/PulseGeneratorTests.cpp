// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/PulseGenerator.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Midi::Ops::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiPulseGeneratorBasicTest,
		"Harmonix.Midi.Ops.PulseGenerator.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiPulseGeneratorBasicTest::RunTest(const FString&)
	{
		FMidiPulseGenerator PulseGenerator;
		
		Metasound::FOperatorSettings OperatorSettings{ 48000, 100 };

		constexpr float Tempo = 123;
		constexpr uint8 TimeSigNumerator = 4;
		constexpr uint8 TimeSigDenominator = 4;
		const auto Clock = MakeShared<HarmonixMetasound::FMidiClock, ESPMode::NotThreadSafe>(OperatorSettings);
		Clock->AttachToSongMapEvaluator(MakeShared<FSongMaps>(Tempo, TimeSigNumerator, TimeSigDenominator));
		
		HarmonixMetasound::FMidiStream OutputStream;

		// Default: a pulse every beat
		{
			constexpr int32 NotesUntilWeAreSatisfiedThisWorks = 23;
			int32 NumNotesReceived = 0;

			Clock->SeekTo(0,0,0);
			Clock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

			FTimeSignature TimeSignature(4, 4);
			const FTimeSignature* TimeSigPtr = Clock->GetSongMapEvaluator().GetTimeSignatureAtTick(0);
			if (TimeSigPtr)
			{
				TimeSignature = *TimeSigPtr;
			}
			FMusicTimeInterval Interval = PulseGenerator.GetInterval();
			FMusicTimestamp NextPulse{ 1, 1 };
			IncrementTimestampByOffset(NextPulse, Interval, TimeSignature);

			int32 EndTick = 0;
			{
				FMusicTimestamp EndTimestamp{ NextPulse };
				for (int i = 0; i < NotesUntilWeAreSatisfiedThisWorks; ++i)
				{
					IncrementTimestampByInterval(EndTimestamp, Interval, TimeSignature);
				}
				EndTick = Clock->GetSongMapEvaluator().MusicTimestampToTick(EndTimestamp);
			}
			
			while (Clock->GetLastProcessedMidiTick() < EndTick)
			{
				// Advance the clock, which will advance the play cursor in the pulse generator
				Clock->Advance(0, OperatorSettings.GetNumFramesPerBlock());

				// Process, which will pop the next notes
				OutputStream.PrepareBlock();
				PulseGenerator.Process(*Clock, OutputStream);
				
				// If this is a block where we should get a pulse, check that we got it
				if (Clock->GetLastProcessedMidiTick() >= Clock->GetSongMapEvaluator().MusicTimestampToTick(NextPulse))
				{
					const bool ShouldGetNoteOff = NumNotesReceived > 0;

					bool GotNoteOn = false;
					bool GotNoteOff = false;

					const TArray<HarmonixMetasound::FMidiStreamEvent> Events = OutputStream.GetEventsInBlock();

					if (Events.Num() != (ShouldGetNoteOff ? 2 : 1))
					{
						return false;
					}

					UTEST_EQUAL("Got the right number of events", Events.Num(), ShouldGetNoteOff ? 2 : 1);

					for (const HarmonixMetasound::FMidiStreamEvent& Event : Events)
					{
						if (Event.MidiMessage.IsNoteOn())
						{
							GotNoteOn = true;

							UTEST_EQUAL("Right track", Event.TrackIndex, PulseGenerator.Track);
							UTEST_EQUAL("Right channel", Event.MidiMessage.GetStdChannel(), PulseGenerator.Channel - 1);
							UTEST_EQUAL("Right note number", Event.MidiMessage.GetStdData1(), PulseGenerator.NoteNumber);
							UTEST_EQUAL("Right velocity", Event.MidiMessage.GetStdData2(), PulseGenerator.Velocity);

							++NumNotesReceived;
						}
						else if (Event.MidiMessage.IsNoteOff())
						{
							GotNoteOff = true;

							UTEST_EQUAL("Right track", Event.TrackIndex, PulseGenerator.Track);
							UTEST_EQUAL("Right channel", Event.MidiMessage.GetStdChannel(), PulseGenerator.Channel - 1);
							UTEST_EQUAL("Right note number", Event.MidiMessage.GetStdData1(), PulseGenerator.NoteNumber);
						}
						else
						{
							UTEST_TRUE("Unexpected event", false);
						}
					}

					UTEST_TRUE("Got note on", GotNoteOn);

					if (ShouldGetNoteOff)
					{
						UTEST_TRUE("Got note off", GotNoteOff);
					}
					else
					{
						UTEST_FALSE("Did not get note off", GotNoteOff);
					}

					IncrementTimestampByInterval(NextPulse, Interval, TimeSignature);
				}

				// we can prepare the clock for the next block...
				Clock->PrepareBlock();
			}
			
			UTEST_TRUE("Got all the notes at the right time", NumNotesReceived >= NotesUntilWeAreSatisfiedThisWorks);
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiPulseGeneratorSeekTest,
	"Harmonix.Midi.Ops.PulseGenerator.Seek",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiPulseGeneratorSeekTest::RunTest(const FString&)
	{
		FPulseGenerator PulseGenerator;
		
		Metasound::FOperatorSettings OperatorSettings{ 48000, 100 };

		constexpr float Tempo = 123;
		const FTimeSignature TimeSignature(4, 4);
		const auto Clock = MakeShared<HarmonixMetasound::FMidiClock, ESPMode::NotThreadSafe>(OperatorSettings);
		Clock->AttachToSongMapEvaluator(MakeShared<FSongMaps>(Tempo, TimeSignature.Numerator, TimeSignature.Denominator));

		// Advance forward a few pulses, then seek back and ensure we keep getting notes
		{
			constexpr int32 PulsesToDo = 10;
			int32 NumPulsesReceived = 0;
			int32 NumPulseOffReceived = 0;

			Clock->SeekTo(0,0,0);
			Clock->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
			const FMusicTimeInterval Interval = PulseGenerator.GetInterval();
			FMusicTimestamp NextPulse{ 1, 1 };
			IncrementTimestampByOffset(NextPulse, Interval, TimeSignature);

			FMusicTimestamp EndTimestamp{ NextPulse };
			for (int i = 0; i < PulsesToDo; ++i)
			{
				IncrementTimestampByInterval(EndTimestamp, Interval, TimeSignature);
			}
			
			while (Clock->GetMusicTimestampAtBlockEnd() < EndTimestamp)
			{
				// Advance the clock, which will advance the play cursor in the pulse generator
				Clock->Advance(0, OperatorSettings.GetNumFramesPerBlock());

				// Process
				TArray<FPulseGenerator::FPulseInfo> PulsesThisBlock;
				TArray<FPulseGenerator::FPulseInfo> PulseOffThisBlock;
				PulseGenerator.Process(*Clock,
					[&PulsesThisBlock](const FPulseGenerator::FPulseInfo& Pulse)
					{
						PulsesThisBlock.Add(Pulse);
					},
					[&PulseOffThisBlock](const FPulseGenerator::FPulseInfo& Pulse)
					{
						PulseOffThisBlock.Add(Pulse);
					});
				NumPulsesReceived += PulsesThisBlock.Num();
				NumPulseOffReceived += PulseOffThisBlock.Num();
				
				// If this is a block where we should get a pulse, check that we got it
				if (Clock->GetLastProcessedMidiTick() >= Clock->GetSongMapEvaluator().MusicTimestampToTick(NextPulse))
				{
					UTEST_EQUAL("Got the right number of pulses", PulsesThisBlock.Num(), 1);

					IncrementTimestampByInterval(NextPulse, Interval, TimeSignature);
				}

				// we can prepare the clock for the next block...
				Clock->PrepareBlock();
			}
			
			UTEST_TRUE("Before seek: Got all the pulses at the right time", NumPulsesReceived >= PulsesToDo);
			UTEST_TRUE("Before seek: Got all the \"pulse-off\" at the right time", NumPulseOffReceived >= PulsesToDo);

			// Seek and reset the next pulse
			Clock->SeekTo(0, 0, 0);
			NumPulsesReceived = 0;
			NextPulse = { 1, 1 };
			IncrementTimestampByOffset(NextPulse, Interval, TimeSignature);

			while (Clock->GetMusicTimestampAtBlockEnd() < EndTimestamp)
			{
				// Advance the clock, which will advance the play cursor in the pulse generator
				Clock->Advance(0, OperatorSettings.GetNumFramesPerBlock());

				// Process
				TArray<FPulseGenerator::FPulseInfo> PulsesThisBlock;
				TArray<FPulseGenerator::FPulseInfo> PulseOffThisBlock;
				PulseGenerator.Process(*Clock,
					[&PulsesThisBlock](const FPulseGenerator::FPulseInfo& Pulse)
					{
						PulsesThisBlock.Add(Pulse);
					},
					[&PulseOffThisBlock](const FPulseGenerator::FPulseInfo& Pulse)
					{
						PulseOffThisBlock.Add(Pulse);
					});
				
				NumPulsesReceived += PulsesThisBlock.Num();
				NumPulseOffReceived += PulseOffThisBlock.Num();
				
				// If this is a block where we should get a pulse, check that we got it
				if (Clock->GetLastProcessedMidiTick() >= Clock->GetSongMapEvaluator().MusicTimestampToTick(NextPulse))
				{
					UTEST_EQUAL("Got the right number of pulses", PulsesThisBlock.Num(), 1);

					IncrementTimestampByInterval(NextPulse, Interval, TimeSignature);
				}

				// we can prepare the clock for the next block...
				Clock->PrepareBlock();
			}
			
			UTEST_TRUE("After seek: Got all the notes at the right time", NumPulsesReceived >= PulsesToDo);
			UTEST_TRUE("After seek: Gott all the \"pulse off\" at the right time", NumPulseOffReceived >= PulsesToDo);
		}

		return true;
	}
}

#endif