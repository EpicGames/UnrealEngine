// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMetasound/MidiOps/StuckNoteGuard.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Harmonix::Midi::Ops::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FStuckNoteGuardBasicTest,
		"Harmonix.Midi.Ops.StuckNoteGuard.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FStuckNoteGuardBasicTest::RunTest(const FString&)
	{
		FStuckNoteGuard Guard;

		HarmonixMetasound::FMidiStream InputStream;
		HarmonixMetasound::FMidiStream OutputStream;

		// Use a filter that only includes messages on channel 3
		const auto Channel3Filter = [](const HarmonixMetasound::FMidiStreamEvent& Event)
		{
			return Event.MidiMessage.IsStd() && Event.MidiMessage.GetStdChannel() == 3;
		};

		constexpr uint8 NoteNumber = 43;
		constexpr uint8 Velocity = 54;

		// Add some note ons on all the channels
		for (uint8 ChannelIdx = 0; ChannelIdx < Constants::GNumChannels; ++ChannelIdx)
		{
			InputStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(ChannelIdx, NoteNumber, Velocity)});
		}

		// Process and expect no messages (because we don't have any stuck notes yet)
		Guard.Process(InputStream, OutputStream, Channel3Filter);
		UTEST_EQUAL("Got no events", OutputStream.GetEventsInBlock().Num(), 0);

		// Now change the filter to include only messages on channel 4, process, and expect to get a note off for the newly-stuck note on channel 3
		const auto Channel4Filter = [](const HarmonixMetasound::FMidiStreamEvent& Event)
		{
			return Event.MidiMessage.IsStd() && Event.MidiMessage.GetStdChannel() == 4;
		};
		InputStream.PrepareBlock();
		OutputStream.PrepareBlock();
		Guard.Process(InputStream, OutputStream, Channel4Filter);
		const auto& Events = OutputStream.GetEventsInBlock();
		UTEST_EQUAL("Got one event", Events.Num(), 1);
		UTEST_TRUE("Got a note off", Events[0].MidiMessage.IsNoteOff());
		UTEST_EQUAL("Got the right note number", Events[0].MidiMessage.GetStdData1(), NoteNumber);
		UTEST_EQUAL("Got the note off on the right channel", Events[0].MidiMessage.GetStdChannel(), 3);

		// Add some note ons on all the channels
		InputStream.PrepareBlock();
		OutputStream.PrepareBlock();
		for (uint8 ChannelIdx = 0; ChannelIdx < Constants::GNumChannels; ++ChannelIdx)
		{
			InputStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(ChannelIdx, NoteNumber, Velocity)});
		}

		// Process and expect no messages (because we don't have any stuck notes yet)
		Guard.Process(InputStream, OutputStream, Channel4Filter);
		UTEST_EQUAL("Got no events", OutputStream.GetEventsInBlock().Num(), 0);

		// Now send note offs on all the channels, process, and expect no additional note offs
		InputStream.PrepareBlock();
		OutputStream.PrepareBlock();
		for (uint8 ChannelIdx = 0; ChannelIdx < Constants::GNumChannels; ++ChannelIdx)
		{
			InputStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOff(ChannelIdx, NoteNumber)});
		}
		Guard.Process(InputStream, OutputStream, Channel4Filter);
		UTEST_EQUAL("Got no events", OutputStream.GetEventsInBlock().Num(), 0);

		// Change the filter back to channel 3, and expect no new note offs
		InputStream.PrepareBlock();
		OutputStream.PrepareBlock();
		Guard.Process(InputStream, OutputStream, Channel3Filter);
		UTEST_EQUAL("Got no events", OutputStream.GetEventsInBlock().Num(), 0);
		
		return true;
	}
}

#endif
