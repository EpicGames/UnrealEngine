// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "HarmonixMetasound/MusicClock/MusicClock.h"
#include "HarmonixMetasound/MusicClock/TimeSource.h"
#include "HarmonixMetasound/MusicSource/MusicSource.h"
#include "HarmonixMetasound/MusicSource/RuntimeMusicSource.h"
#include "HarmonixMetasound/MusicSource/OffsetMusicSource.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"
#include "HarmonixMetasound/Components/MusicClockComponent.h"
#include "HarmonixMidi/MidiFile.h"
#include "Tests/AutomationEditorCommon.h"

namespace HarmonixMetasound::MusicClock::Tests
{
	/**
	 * A manually-driven time source for testing.
	 * Allows precise control of time without depending on UWorld.
	 */
	class FManualTimeSource : public Harmonix::ITimeSource
	{
	public:
		virtual FString GetDisplayName() const override { return TEXT("ManualTimeSource"); }
		virtual TOptional<FVector> TryGetAudioSourceLocation() const override { return {}; }

		virtual void Update() override
		{
			if (State == Harmonix::ESourceState::Running)
			{
				CurrentTime += DeltaTimeToApply;
				LatestEvent = Harmonix::ESourceEvent::Advance;
			}
			else
			{
				LatestEvent = Harmonix::ESourceEvent::None;
			}
			DeltaTimeToApply = 0.0;
		}

		virtual double GetCurrentTime() const override { return CurrentTime; }
		virtual float GetSpeed() const override { return Speed; }
		virtual Harmonix::ESourceState GetCurrentSourceState() const override { return State; }
		virtual Harmonix::ESourceEvent GetLatestSourceEvent() const override { return LatestEvent; }

		void Start()
		{
			CurrentTime = 0.0;
			State = Harmonix::ESourceState::Running;
			LatestEvent = Harmonix::ESourceEvent::Start;
		}

		void Stop()
		{
			State = Harmonix::ESourceState::Stopped;
			LatestEvent = Harmonix::ESourceEvent::Stop;
			CurrentTime = 0.0;
		}

		void Pause()
		{
			if (State == Harmonix::ESourceState::Running)
			{
				State = Harmonix::ESourceState::Paused;
				LatestEvent = Harmonix::ESourceEvent::Pause;
			}
		}

		void Continue()
		{
			if (State == Harmonix::ESourceState::Paused)
			{
				State = Harmonix::ESourceState::Running;
				LatestEvent = Harmonix::ESourceEvent::Continue;
			}
		}

		// ITimeSource optional transport
		virtual void RequestStart(float InStartTime = 0.f) override { Start(); }
		virtual void RequestStop() override { Stop(); }
		virtual void RequestPause() override { Pause(); }
		virtual void RequestContinue() override { Continue(); }

		/** Queue a time delta to be applied on the next Update() call. */
		void AdvanceBy(double Seconds) { DeltaTimeToApply = Seconds; }

		float Speed = 1.0f;

	private:
		double CurrentTime = 0.0;
		double DeltaTimeToApply = 0.0;
		Harmonix::ESourceState State = Harmonix::ESourceState::Stopped;
		Harmonix::ESourceEvent LatestEvent = Harmonix::ESourceEvent::None;
	};

	/** Helper to create a source + clock pair with default 120bpm 4/4 maps. */
	struct FTestClockSetup
	{
		TSharedPtr<FManualTimeSource> TimeSource;
		URuntimeMusicSource* Source = nullptr;
		UMusicClock* Clock = nullptr;

		void Create(UObject* Outer)
		{
			TimeSource = MakeShared<FManualTimeSource>();
			Source = NewObject<URuntimeMusicSource>(Outer);
			Source->Initialize(TimeSource, nullptr);
			Clock = NewObject<UMusicClock>(Outer);
			Clock->SetSource(TScriptInterface<IMusicSource>(Source));
		}

		/** Advance time, update source, then update clock. Simulates one frame. */
		void Tick(double DeltaSeconds)
		{
			TimeSource->AdvanceBy(DeltaSeconds);
			Source->Update();
			++GFrameCounter;
			Clock->UpdateForGameFrame();
		}

		/** Update source and clock without advancing time. */
		void TickNoAdvance()
		{
			Source->Update();
			++GFrameCounter;
			Clock->UpdateForGameFrame();
		}

		void Start()
		{
			Source->Start();
			TickNoAdvance();
		}

		void Pause()
		{
			Source->Pause();
			TickNoAdvance();
		}

		void Continue()
		{
			Source->Continue();
		}

		void Stop()
		{
			Source->Stop();
			TickNoAdvance();
		}
	};

	// =========================================================================
	// Test 1: URuntimeMusicSource.BasicUpdate
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FRuntimeMusicSourceBasicTest,
		"Harmonix.Metasound.MusicClock.RuntimeMusicSource.BasicUpdate",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FRuntimeMusicSourceBasicTest::RunTest(const FString&)
	{
		TSharedRef<FManualTimeSource> TimeSource = MakeShared<FManualTimeSource>();
		URuntimeMusicSource* Source = NewObject<URuntimeMusicSource>();
		Source->Initialize(TimeSource, nullptr);

		// Not started yet — state should be Stopped
		UTEST_EQUAL("Initial state is Stopped", Source->GetSourceState(), Harmonix::ESourceState::Stopped);

		TimeSource->Start();
		TimeSource->AdvanceBy(0.5);
		Source->Update();

		UTEST_EQUAL("State is Running after start", Source->GetSourceState(), Harmonix::ESourceState::Running);
		UTEST_GREATER("Position advanced past 0", Source->GetCurrentSongPos().SecondsIncludingCountIn, 0.f);

		float FirstPos = Source->GetCurrentSongPos().SecondsIncludingCountIn;

		TimeSource->AdvanceBy(0.5);
		Source->Update();

		UTEST_NEARLY_EQUAL("Previous pos holds prior frame's value",
			Source->GetPreviousSongPos().SecondsIncludingCountIn, FirstPos, 0.01f);
		UTEST_GREATER("Current pos advanced further",
			Source->GetCurrentSongPos().SecondsIncludingCountIn, FirstPos);

		return true;
	}

	// =========================================================================
	// Test 2: URuntimeMusicSource.WithMidiFile
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FRuntimeMusicSourceMidiFileTest,
		"Harmonix.Metasound.MusicClock.RuntimeMusicSource.WithMidiFile",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FRuntimeMusicSourceMidiFileTest::RunTest(const FString&)
	{
		UMidiFile* MidiFile = NewObject<UMidiFile>();
		MidiFile->GetSongMaps()->EmptyAllMaps();
		MidiFile->GetSongMaps()->Init(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt);
		MidiFile->GetSongMaps()->GetTempoMap().AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(90.0f), 0);
		MidiFile->GetSongMaps()->GetBarMap().AddTimeSignatureAtBarIncludingCountIn(0, 3, 4);

		TSharedRef<FManualTimeSource> TimeSource = MakeShared<FManualTimeSource>();
		URuntimeMusicSource* Source = NewObject<URuntimeMusicSource>();
		Source->InitializeWithMidi(TimeSource, MidiFile);

		TimeSource->Start();
		// At 90bpm in 3/4 time: 1 bar = 3 beats = 2 seconds
		TimeSource->AdvanceBy(2.0);
		Source->Update();

		const FMidiSongPos& Pos = Source->GetCurrentSongPos();
		UTEST_GREATER("Position advanced", Pos.SecondsIncludingCountIn, 0.f);
		UTEST_EQUAL("Time sig numerator is 3", Pos.TimeSigNumerator, 3);
		UTEST_EQUAL("Time sig denominator is 4", Pos.TimeSigDenominator, 4);
		UTEST_NEARLY_EQUAL("Approximately 1 bar elapsed", Pos.BarsIncludingCountIn, 1.0f, 0.15f);

		return true;
	}

	// =========================================================================
	// Test 3: URuntimeMusicSource.TransportStates
	// Behavioral contracts from the clock's point of view:
	//   - Stopped => time is 0
	//   - Paused  => time is frozen at the paused-at value
	//   - Continue => resumes from paused time
	//   - Stop    => resets time to 0
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FRuntimeMusicSourceTransportTest,
		"Harmonix.Metasound.MusicClock.RuntimeMusicSource.TransportStates",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FRuntimeMusicSourceTransportTest::RunTest(const FString&)
	{
		FTestClockSetup Setup;
		Setup.Create(GetTransientPackage());

		// --- Before start: clock reads 0, state is Stopped ---
		UTEST_EQUAL("Initial clock state is Stopped", Setup.Clock->GetState(), EMusicClockState::Stopped);
		UTEST_NEARLY_EQUAL("Initial clock time is 0", Setup.Clock->GetSecondsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);

		// --- Start and advance ---
		Setup.Start();
		Setup.Tick(1.0); // Advance 1 second

		UTEST_EQUAL("Clock is Running", Setup.Clock->GetState(), EMusicClockState::Running);
		float TimeAfterOneSecond = Setup.Clock->GetSecondsIncludingCountIn();
		UTEST_GREATER("Clock time > 0 after advancing", TimeAfterOneSecond, 0.f);

		// --- Pause: time freezes ---
		Setup.Pause();
		float TimeAtPause = Setup.Clock->GetSecondsIncludingCountIn();
		UTEST_EQUAL("Clock is Paused", Setup.Clock->GetState(), EMusicClockState::Paused);
		UTEST_GREATER("Time at pause > 0", TimeAtPause, 0.f);

		// Tick again while paused — time should not advance
		Setup.TickNoAdvance();
		UTEST_NEARLY_EQUAL("Time unchanged while paused", Setup.Clock->GetSecondsIncludingCountIn(), TimeAtPause, 0.001f);
		UTEST_NEARLY_EQUAL("Delta beat is 0 while paused", Setup.Clock->GetDeltaBeat(), 0.f, KINDA_SMALL_NUMBER);

		// --- Continue: resumes from paused time ---
		Setup.Continue();
		Setup.Tick(0.5);
		UTEST_EQUAL("Clock is Running after continue", Setup.Clock->GetState(), EMusicClockState::Running);
		UTEST_GREATER("Time advanced past pause point", Setup.Clock->GetSecondsIncludingCountIn(), TimeAtPause);

		// --- Stop: resets to 0 ---
		Setup.Stop();
		UTEST_EQUAL("Clock is Stopped", Setup.Clock->GetState(), EMusicClockState::Stopped);
		UTEST_NEARLY_EQUAL("Time is 0 after stop", Setup.Clock->GetSecondsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);

		// --- Restart: begins from 0 again ---
		Setup.Start();
		Setup.Tick(0.5);
		UTEST_EQUAL("Clock is Running after restart", Setup.Clock->GetState(), EMusicClockState::Running);
		UTEST_GREATER("Time advancing from 0 after restart", Setup.Clock->GetSecondsIncludingCountIn(), 0.f);
		UTEST_LESS("Time is less than before stop (restarted fresh)", Setup.Clock->GetSecondsIncludingCountIn(), TimeAfterOneSecond);

		return true;
	}

	// =========================================================================
	// Test 4: UMusicClock.ReadOnlyView
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMusicClockReadOnlyViewTest,
		"Harmonix.Metasound.MusicClock.UMusicClock.ReadOnlyView",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMusicClockReadOnlyViewTest::RunTest(const FString&)
	{
		FTestClockSetup Setup;
		Setup.Create(GetTransientPackage());
		Setup.Start();

		// Advance 2 seconds at 120bpm 4/4 = 4 beats = 1 bar
		Setup.Tick(2.0);

		UMusicClock* Clock = Setup.Clock;

		UTEST_EQUAL("Clock is Running", Clock->GetState(), EMusicClockState::Running);
		UTEST_GREATER("Seconds > 0", Clock->GetSecondsIncludingCountIn(), 0.f);
		UTEST_GREATER("Bars > 0", Clock->GetBarsIncludingCountIn(), 0.f);
		UTEST_GREATER("Beats > 0", Clock->GetBeatsIncludingCountIn(), 0.f);
		UTEST_NEARLY_EQUAL("Tempo is ~120", Clock->GetCurrentTempo(), 120.f, 1.f);
		UTEST_NEARLY_EQUAL("BPM is ~120", Clock->GetCurrentBeatsPerMinute(), 120.f, 1.f);
		UTEST_NEARLY_EQUAL("Beats/sec is ~2", Clock->GetCurrentBeatsPerSecond(), 2.f, 0.1f);
		UTEST_GREATER("DeltaBeat > 0", Clock->GetDeltaBeat(), 0.f);
		UTEST_GREATER("DeltaBar > 0", Clock->GetDeltaBar(), 0.f);

		int Num = 0, Denom = 0;
		Clock->GetCurrentTimeSignature(Num, Denom);
		UTEST_EQUAL("Time sig num is 4", Num, 4);
		UTEST_EQUAL("Time sig denom is 4", Denom, 4);

		FMusicTimestamp Timestamp = Clock->GetCurrentTimestamp();
		UTEST_GREATER_EQUAL("Timestamp bar >= 1", Timestamp.Bar, 1);

		const ISongMapEvaluator* Maps = Clock->GetSongMaps();
		UTEST_NOT_NULL("Song maps available", Maps);

		return true;
	}

	// =========================================================================
	// Test 5: UMusicClock.StoppedReturnsZero
	// Explicit test that a stopped clock always returns 0
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMusicClockStoppedZeroTest,
		"Harmonix.Metasound.MusicClock.UMusicClock.StoppedReturnsZero",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMusicClockStoppedZeroTest::RunTest(const FString&)
	{
		FTestClockSetup Setup;
		Setup.Create(GetTransientPackage());

		// Before ever starting
		UTEST_NEARLY_EQUAL("Seconds is 0 before start", Setup.Clock->GetSecondsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);
		UTEST_NEARLY_EQUAL("Bars is 0 before start", Setup.Clock->GetBarsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);
		UTEST_NEARLY_EQUAL("Beats is 0 before start", Setup.Clock->GetBeatsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);

		// Run for a bit then stop
		Setup.Start();
		Setup.Tick(2.0);
		UTEST_GREATER("Time advanced while running", Setup.Clock->GetSecondsIncludingCountIn(), 0.f);

		Setup.Stop();
		UTEST_NEARLY_EQUAL("Seconds is 0 after stop", Setup.Clock->GetSecondsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);
		UTEST_NEARLY_EQUAL("Bars is 0 after stop", Setup.Clock->GetBarsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);
		UTEST_NEARLY_EQUAL("Beats is 0 after stop", Setup.Clock->GetBeatsIncludingCountIn(), 0.f, KINDA_SMALL_NUMBER);
		UTEST_NEARLY_EQUAL("Tempo is 0 after stop", Setup.Clock->GetCurrentTempo(), 0.f, KINDA_SMALL_NUMBER);
		UTEST_NEARLY_EQUAL("DeltaBeat is 0 after stop", Setup.Clock->GetDeltaBeat(), 0.f, KINDA_SMALL_NUMBER);

		return true;
	}

	// =========================================================================
	// Test 6: OffsetMusicSource.PositiveOffset
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FOffsetClockPositiveTest,
		"Harmonix.Metasound.MusicClock.OffsetMusicSource.PositiveOffset",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FOffsetClockPositiveTest::RunTest(const FString&)
	{
		FTestClockSetup Setup;
		Setup.Create(GetTransientPackage());

		// Offset source reads directly from the source, not from a clock
		UOffsetMusicSource* OffsetSource = NewObject<UOffsetMusicSource>(GetTransientPackage());
		OffsetSource->SetParentSource(TScriptInterface<IMusicSource>(Setup.Source));
		OffsetSource->SetOffsetMs(500.f);
		UMusicClock* OffsetClock = NewObject<UMusicClock>(GetTransientPackage());
		OffsetClock->SetSource(TScriptInterface<IMusicSource>(OffsetSource));

		Setup.Start();

		for (int32 i = 0; i < 10; ++i)
		{
			Setup.Tick(0.2);
			OffsetSource->Update();
			++GFrameCounter;
			OffsetClock->UpdateForGameFrame();
		}

		float BaseSeconds = Setup.Clock->GetSecondsIncludingCountIn();
		float OffsetSeconds = OffsetClock->GetSecondsIncludingCountIn();

		UTEST_GREATER("Offset clock is ahead of base", OffsetSeconds, BaseSeconds);
		float DiffMs = (OffsetSeconds - BaseSeconds) * 1000.f;
		UTEST_NEARLY_EQUAL("Offset is approximately 500ms", DiffMs, 500.f, 50.f);

		return true;
	}

	// =========================================================================
	// Test 7: OffsetMusicSource.NegativeOffset
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FOffsetClockNegativeTest,
		"Harmonix.Metasound.MusicClock.OffsetMusicSource.NegativeOffset",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FOffsetClockNegativeTest::RunTest(const FString&)
	{
		FTestClockSetup Setup;
		Setup.Create(GetTransientPackage());

		UOffsetMusicSource* OffsetSource = NewObject<UOffsetMusicSource>(GetTransientPackage());
		OffsetSource->SetParentSource(TScriptInterface<IMusicSource>(Setup.Source));
		OffsetSource->SetOffsetMs(-200.f);
		UMusicClock* OffsetClock = NewObject<UMusicClock>(GetTransientPackage());
		OffsetClock->SetSource(TScriptInterface<IMusicSource>(OffsetSource));

		Setup.Start();

		for (int32 i = 0; i < 10; ++i)
		{
			Setup.Tick(0.2);
			OffsetSource->Update();
			++GFrameCounter;
			OffsetClock->UpdateForGameFrame();
		}

		float BaseSeconds = Setup.Clock->GetSecondsIncludingCountIn();
		float OffsetSeconds = OffsetClock->GetSecondsIncludingCountIn();

		UTEST_LESS("Offset clock is behind base", OffsetSeconds, BaseSeconds);
		float DiffMs = (BaseSeconds - OffsetSeconds) * 1000.f;
		UTEST_NEARLY_EQUAL("Offset is approximately 200ms", DiffMs, 200.f, 50.f);

		return true;
	}

	// =========================================================================
	// Test 8: OffsetMusicSource.Chain
	// Offsets chain directly: Source -> OffsetA (+100ms) -> OffsetB (+50ms)
	// No intermediate clocks needed in the chain — offset sources read from
	// IMusicSource, and each offset source is itself an IMusicSource.
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FOffsetClockChainTest,
		"Harmonix.Metasound.MusicClock.OffsetMusicSource.Chain",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FOffsetClockChainTest::RunTest(const FString&)
	{
		FTestClockSetup Setup;
		Setup.Create(GetTransientPackage());

		// Chain: Source -> OffsetA (+100ms) -> OffsetB (+50ms) -> ClockB
		UOffsetMusicSource* OffsetA = NewObject<UOffsetMusicSource>(GetTransientPackage());
		OffsetA->SetParentSource(TScriptInterface<IMusicSource>(Setup.Source));
		OffsetA->SetOffsetMs(100.f);

		UOffsetMusicSource* OffsetB = NewObject<UOffsetMusicSource>(GetTransientPackage());
		OffsetB->SetParentSource(TScriptInterface<IMusicSource>(OffsetA));
		OffsetB->SetOffsetMs(50.f);

		UMusicClock* ChainClock = NewObject<UMusicClock>(GetTransientPackage());
		ChainClock->SetSource(TScriptInterface<IMusicSource>(OffsetB));

		Setup.Start();

		for (int32 i = 0; i < 10; ++i)
		{
			Setup.Tick(0.2);
			OffsetA->Update();
			OffsetB->Update();
			++GFrameCounter;
			ChainClock->UpdateForGameFrame();
		}

		float BaseSeconds = Setup.Clock->GetSecondsIncludingCountIn();
		float ChainSeconds = ChainClock->GetSecondsIncludingCountIn();

		float TotalDiffMs = (ChainSeconds - BaseSeconds) * 1000.f;
		UTEST_NEARLY_EQUAL("Chained offset is approximately 150ms", TotalDiffMs, 150.f, 30.f);

		return true;
	}

	// =========================================================================
	// Test 9: OffsetMusicSource.LoopBoundaryWrap
	// Validates that a negative offset does NOT wrap on the first pass through
	// a loop, but DOES wrap correctly after the parent source loops.
	//
	// Setup: 2-bar loop at 120bpm 4/4 = 4000ms (0 to 4000ms), offset -100ms
	//   First pass, parent at 50ms  -> Offset = -50ms  -> stays negative (no wrap)
	//   First pass, parent at 200ms -> Offset = 100ms  -> normal (in bounds)
	//   After loop, parent at 50ms  -> Offset = -50ms  -> wraps to 3950ms
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FOffsetClockLoopBoundaryTest,
		"Harmonix.Metasound.MusicClock.OffsetMusicSource.LoopBoundaryWrap",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FOffsetClockLoopBoundaryTest::RunTest(const FString&)
	{
		// At 120bpm 4/4: 1 bar = 2 seconds. 2 bars = 4 seconds = 4000ms.
		TSharedRef<FManualTimeSource> TimeSource = MakeShared<FManualTimeSource>();
		URuntimeMusicSource* Source = NewObject<URuntimeMusicSource>(GetTransientPackage());
		Source->Initialize(TimeSource, nullptr); // Default 120bpm 4/4
		Source->SetLoopRegionByBars(1, 2); // Loop from bar 1 for 2 bars

		UTEST_EQUAL("Source is looping", Source->IsLooping(), true);
		float LoopLenMs = Source->GetLoopLengthMs();
		UTEST_NEARLY_EQUAL("Loop length is ~4000ms", LoopLenMs, 4000.f, 100.f);

		// Base clock for reading the source's position in assertions
		UMusicClock* BaseClock = NewObject<UMusicClock>(GetTransientPackage());
		BaseClock->SetSource(TScriptInterface<IMusicSource>(Source));

		// Offset source reads directly from the source
		UOffsetMusicSource* OffsetSource = NewObject<UOffsetMusicSource>(GetTransientPackage());
		OffsetSource->SetParentSource(TScriptInterface<IMusicSource>(Source));
		OffsetSource->SetOffsetMs(-100.f); // 100ms behind
		UMusicClock* OffsetClock = NewObject<UMusicClock>(GetTransientPackage());
		OffsetClock->SetSource(TScriptInterface<IMusicSource>(OffsetSource));

		TFunction<void(double)> TickAll = [&](double DeltaSec)
		{
			TimeSource->AdvanceBy(DeltaSec);
			Source->Update();
			++GFrameCounter;
			BaseClock->UpdateForGameFrame();
			OffsetSource->Update();
			++GFrameCounter;
			OffsetClock->UpdateForGameFrame();
		};

		// Start
		TimeSource->Start();
		TickAll(0.0); // Process start

		// --- First pass: parent at ~50ms, offset = -50ms. Should NOT wrap. ---
		TickAll(0.05);
		float OffsetMs1 = OffsetClock->GetSecondsIncludingCountIn() * 1000.f;
		UTEST_LESS("First pass: negative offset stays before loop start", OffsetMs1, 100.f);

		// --- Advance past the offset threshold so offset time goes positive ---
		TickAll(0.15); // Parent at ~200ms, offset at ~100ms
		float OffsetMs1b = OffsetClock->GetSecondsIncludingCountIn() * 1000.f;
		UTEST_GREATER("First pass: offset advances into loop region", OffsetMs1b, 0.f);

		// --- Advance to mid-loop ---
		TickAll(1.8); // Parent at ~2000ms
		float BaseMs2 = BaseClock->GetSecondsIncludingCountIn() * 1000.f;
		float OffsetMs2 = OffsetClock->GetSecondsIncludingCountIn() * 1000.f;
		float Diff2 = BaseMs2 - OffsetMs2;
		UTEST_NEARLY_EQUAL("Mid-loop: offset is ~100ms behind", Diff2, 100.f, 30.f);

		// --- Cross the loop boundary ---
		TickAll(2.1); // Parent past 4000ms, loops to ~100ms
		float OffsetMs3 = OffsetClock->GetSecondsIncludingCountIn() * 1000.f;
		// After the parent loops, the offset clock should wrap (parent near start
		// with negative offset → wraps to near loop end)
		UTEST_GREATER("After loop: offset clock wrapped to near end of loop", OffsetMs3, 3800.f);
		// Loop should have been detected
		UTEST_EQUAL("Offset clock detected loop", OffsetClock->LoopedThisFrame(), true);

		return true;
	}

	// =========================================================================
	// Test 10: UMidiClockUpdateSubsystem.TrackNewTypes
	// =========================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FSubsystemTrackNewTypesTest,
		"Harmonix.Metasound.MusicClock.Subsystem.TrackNewTypes",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FSubsystemTrackNewTypesTest::RunTest(const FString&)
	{
		UTEST_NOT_NULL("GEngine exists", GEngine);
		UMidiClockUpdateSubsystem* Subsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>();
		UTEST_NOT_NULL("Subsystem exists", Subsystem);

		TSharedRef<FManualTimeSource> TimeSource = MakeShared<FManualTimeSource>();
		URuntimeMusicSource* Source = NewObject<URuntimeMusicSource>();
		Source->Initialize(TimeSource, nullptr);

		UMusicClock* Clock = NewObject<UMusicClock>();
		Clock->SetSource(TScriptInterface<IMusicSource>(Source));

		UMidiClockUpdateSubsystem::TrackMusicSource(Source);
		UMidiClockUpdateSubsystem::TrackMusicClock(Clock);

		TimeSource->Start();
		TimeSource->AdvanceBy(1.0);

		// Tick subsystem — should update source and clock
		Subsystem->TickForTesting();

		UTEST_GREATER("Source position advanced", Source->GetCurrentSongPos().SecondsIncludingCountIn, 0.f);
		UTEST_GREATER("Clock position advanced", Clock->GetSecondsIncludingCountIn(), 0.f);

		// Unregister
		UMidiClockUpdateSubsystem::StopTrackingMusicClock(Clock);
		UMidiClockUpdateSubsystem::StopTrackingMusicSource(Source);

		float PosBeforeSecondTick = Clock->GetSecondsIncludingCountIn();
		TimeSource->AdvanceBy(1.0);
		Subsystem->TickForTesting();

		UTEST_NEARLY_EQUAL("Clock not updated after unregister", Clock->GetSecondsIncludingCountIn(), PosBeforeSecondTick, 0.01f);

		return true;
	}
}

#endif
