// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixDsp/Effects/Delay.h"
#include "HarmonixMetasound/Nodes/DelayNode.h"
#include "Misc/AutomationTest.h"
#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/DataTypes/DelayFilterType.h"
#include "HarmonixMetasound/DataTypes/DelayStereoType.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/TimeSyncOption.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::DelayNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace HarmonixMetasound;
	using namespace Metasound;
	
	class FTestFixture
	{
	public:
		FTestFixture(float InSampleRate, int32 NumSamplesPerBlock, FAutomationTestBase& InTest, bool WithClock)
			: SampleRate(InSampleRate)
			, Test(InTest)
		{
			constexpr int32 NumChannels = HarmonixMetasound::DelayNode::Constants::NumChannels;
			Audio::SetMultichannelBufferSize(NumChannels, NumSamplesPerBlock, ComparisonBuffer);
			GeneratorBufferInterleaved.SetNumZeroed(NumChannels * NumSamplesPerBlock);
			
			DelayForComparison.Prepare(
				InSampleRate,
				HarmonixMetasound::DelayNode::Constants::NumChannels,
				HarmonixMetasound::DelayNode::Constants::MaxDelayTime);

			using namespace Metasound;
			
			GraphBuilder Builder;
			const Frontend::FNodeHandle NodeHandle = Builder.AddNode(
				{HarmonixNodeNamespace, "Delay", "" }, 0);

			check(NodeHandle->IsValid());

			// add the inputs and connect them
			for (const Frontend::FInputHandle& Input : NodeHandle->GetInputs())
			{
				// ...but skip the midi clock if we're not using one
				if (Input->GetDataType() == "MidiClock" && !WithClock)
				{
					continue;
				}
					
				const Frontend::FNodeHandle InputNode = Builder.AddInput(Input->GetName(), Input->GetDataType());
				check(InputNode->IsValid());

				const Frontend::FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(Input->GetName());
				const Frontend::FInputHandle InputToConnect = NodeHandle->GetInputWithVertexName(Input->GetName());
					
				if (!InputToConnect->Connect(*OutputToConnect))
				{
					check(false);
					return;
				}
			}

			// add the outputs and connect them
			for (const Frontend::FOutputHandle& Output : NodeHandle->GetOutputs())
			{
				const Frontend::FNodeHandle OutputNode = Builder.AddOutput(Output->GetName(), Output->GetDataType());
	
				check(OutputNode->IsValid());

				Frontend::FOutputHandle OutputToConnect = NodeHandle->GetOutputWithVertexName(Output->GetName());
				const Frontend::FInputHandle InputToConnect = OutputNode->GetInputWithVertexName(Output->GetName());
	
				if (!InputToConnect->Connect(*OutputToConnect))
				{
					check(false);
					return;
				}
			}

			// build the graph
			Generator = Builder.BuildGenerator(SampleRate, NumSamplesPerBlock);
		}

		bool RenderAndCompare(bool AddImpulse)
		{
			using namespace Metasound;

			// zero the input buffers
			Algo::ForEach(ComparisonBuffer, [](Audio::FAlignedFloatBuffer& Channel)
			{
				check(Channel.Num() > 0);
				FMemory::Memzero(Channel.GetData(), Channel.Num() * sizeof(float));
			});
			TOptional<TDataWriteReference<FAudioBuffer>> InputAudioLeft =
				Generator->GetInputWriteReference<FAudioBuffer>(HarmonixMetasound::DelayNode::Inputs::AudioLeftName);
			TOptional<TDataWriteReference<FAudioBuffer>> InputAudioRight =
				Generator->GetInputWriteReference<FAudioBuffer>(HarmonixMetasound::DelayNode::Inputs::AudioRightName);
			if (!Test.TestTrue("Got input buffers", InputAudioLeft.IsSet() && InputAudioRight.IsSet()))
			{
				return false;
			}
			(*InputAudioLeft)->Zero();
			(*InputAudioRight)->Zero();
			
			// if requested, add an impulse to the input buffers
			if (AddImpulse)
			{
				ComparisonBuffer[0][0] = 1;
				ComparisonBuffer[1][0] = 1;

				check((*InputAudioLeft)->Num() == (*InputAudioRight)->Num());
				(*InputAudioLeft)->GetData()[0] = 1;
				(*InputAudioRight)->GetData()[0] = 1;
			}

			// render
			check(GeneratorBufferInterleaved.Num() > 0);
			FMemory::Memzero(GeneratorBufferInterleaved.GetData(), GeneratorBufferInterleaved.Num() * sizeof(float));
			Generator->OnGenerateAudio(GeneratorBufferInterleaved.GetData(), GeneratorBufferInterleaved.Num());
			Audio::FMultichannelBufferView ComparisonBufferView = Audio::MakeMultichannelBufferView(ComparisonBuffer);
			DelayForComparison.Process(ComparisonBufferView);
			
			// check that the output buffers are equal
			{
				constexpr int32 NumChannels = HarmonixMetasound::DelayNode::Constants::NumChannels;
				const int32 NumFrames = Generator->OperatorSettings.GetNumFramesPerBlock();
				
				for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
				{
					for (int32 FrameIdx = 0; FrameIdx < NumFrames; ++FrameIdx)
					{
						if (!Test.TestEqual(
							FString::Printf(TEXT("Channel %i samples match at idx %i"), ChannelIdx, FrameIdx),
							GeneratorBufferInterleaved[FrameIdx * NumChannels + ChannelIdx],
							ComparisonBuffer[ChannelIdx][FrameIdx]))
						{
							return false;
						}
					}
				}
			}

			return true;
		}

		struct FParams
		{
			ETimeSyncOption DelayTimeType;
			float DelayTime;
			float Feedback;
			float DryLevel;
			float WetLevel;
			bool WetFilterEnabled;
			bool FeedbackFilterEnabled;
			EDelayFilterType FilterType;
			float FilterCutoff;
			float FilterQ;
			bool LFOEnabled;
			ETimeSyncOption LFOTimeType;
			float LFOFrequency;
			float LFODepth;
			EDelayStereoType StereoType;
			float StereoSpreadLeft;
			float StereoSpreadRight;

			FParams()
			{
				const Harmonix::Dsp::Effects::FDelay Def;
				DelayTimeType = Def.GetTimeSyncOption();
				DelayTime = Def.GetDelaySeconds();
				Feedback = Def.GetFeedbackGain();
				DryLevel = Def.GetDryGain();
				WetLevel = Def.GetWetGain();
				WetFilterEnabled = Def.GetWetFilterEnabled();
				FeedbackFilterEnabled = Def.GetFeedbackFilterEnabled();
				FilterType = static_cast<EDelayFilterType>(Def.GetFilterType());
				FilterCutoff = Def.GetFilterFreq();
				FilterQ = Def.GetFilterQ();
				LFOEnabled = Def.GetLfoEnabled();
				LFOTimeType = Def.GetLfoTimeSyncOption();
				LFOFrequency = Def.GetLfoFreq();
				LFODepth = Def.GetLfoDepth();
				StereoType = Def.GetStereoType();
				StereoSpreadLeft = Def.GetStereoSpreadLeft();
				StereoSpreadRight = Def.GetStereoSpreadRight();
			}
		};

		void SetParams(const FParams& Params)
		{
			using namespace Metasound;

			// set the comparison delay's params
			DelayForComparison.SetTimeSyncOption(Params.DelayTimeType);
			DelayForComparison.SetDelaySeconds(Params.DelayTime);
			DelayForComparison.SetFeedbackGain(Params.Feedback);
			DelayForComparison.SetDryGain(Params.DryLevel);
			DelayForComparison.SetWetGain(Params.WetLevel);
			DelayForComparison.SetWetFilterEnabled(Params.WetFilterEnabled);
			DelayForComparison.SetFeedbackFilterEnabled(Params.FeedbackFilterEnabled);
			DelayForComparison.SetFilterType(Params.FilterType);
			DelayForComparison.SetFilterFreq(Params.FilterCutoff);
			DelayForComparison.SetFilterQ(Params.FilterQ);
			DelayForComparison.SetLfoEnabled(Params.LFOEnabled);
			DelayForComparison.SetLfoTimeSyncOption(Params.LFOTimeType);
			DelayForComparison.SetLfoFreq(Params.LFOFrequency);
			DelayForComparison.SetLfoDepth(Params.LFODepth);
			DelayForComparison.SetStereoType(Params.StereoType);
			DelayForComparison.SetStereoSpreadLeft(Params.StereoSpreadLeft);
			DelayForComparison.SetStereoSpreadRight(Params.StereoSpreadRight);
			
			// set the operator's params
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::DelayTimeTypeName, FEnumTimeSyncOption{ Params.DelayTimeType });
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::DelayTimeName, Params.DelayTime);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::FeedbackName, Params.Feedback);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::DryLevelName, Params.DryLevel);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::WetLevelName, Params.WetLevel);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::WetFilterEnabledName, Params.WetFilterEnabled);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::FeedbackFilterEnabledName, Params.FeedbackFilterEnabled);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::FilterTypeName, FEnumDelayFilterType{ Params.FilterType });
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::FilterCutoffName, Params.FilterCutoff);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::FilterQName, Params.FilterQ);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::LFOEnabledName, Params.LFOEnabled);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::LFOTimeTypeName, FEnumTimeSyncOption{ Params.LFOTimeType });
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::LFOFrequencyName, Params.LFOFrequency);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::LFODepthName, Params.LFODepth);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::StereoTypeName, FEnumDelayStereoType{ Params.StereoType });
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::StereoSpreadLeftName, Params.StereoSpreadLeft);
			Generator->SetInputValue(HarmonixMetasound::DelayNode::Inputs::StereoSpreadRightName, Params.StereoSpreadRight);
		}
		

		bool ResetAndStartClock(float Tempo, float Speed, int32 TimeSigNum, int32 TimeSigDenom)
		{
			TOptional<FMidiClockWriteRef> ClockInput = Generator->GetInputWriteReference<FMidiClock>(CommonPinNames::Inputs::MidiClockName);
			if (!Test.TestTrue("Got clock", ClockInput.IsSet()))
			{
				return false;
			}
			const TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(Tempo, TimeSigNum, TimeSigDenom);
			check(SongMaps);

			SongMaps->SetSongLengthTicks(std::numeric_limits<int32>::max());

			(*ClockInput)->AttachToSongMapEvaluator(SongMaps);
			(*ClockInput)->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

			SampleRemainder = 0;
			SampleCount = 0;

			DelayForComparison.SetTempo(Tempo);
			DelayForComparison.SetSpeed(Speed);

			return true;
		}

		bool AdvanceClock()
		{
			using namespace Metasound;
			TOptional<FMidiClockWriteRef> ClockInput = Generator->GetInputWriteReference<FMidiClock>(CommonPinNames::Inputs::MidiClockName);
			if (!Test.TestTrue("Got clock", ClockInput.IsSet()))
			{
				return false;
			}
			(*ClockInput)->PrepareBlock();
			const int32 NumSamples = Audio::GetMultichannelBufferNumFrames(ComparisonBuffer);
			int32 BlockFrameIndex = SampleRemainder;
			SampleRemainder += NumSamples;
			constexpr int32 MidiGranularity = 128;
			while (SampleRemainder >= MidiGranularity)
			{
				SampleCount += MidiGranularity;
				SampleRemainder -= MidiGranularity;
				const float AdvanceToMs = static_cast<float>(SampleCount) * 1000.0f / SampleRate;
				(*ClockInput)->AdvanceToMs(BlockFrameIndex, AdvanceToMs);
				BlockFrameIndex += MidiGranularity;
			}

			return true;
		}

		float SampleRate;
		FAutomationTestBase& Test;
		Harmonix::Dsp::Effects::FDelay DelayForComparison;
		Audio::FMultichannelBuffer ComparisonBuffer;
		TUniquePtr<FMetasoundGenerator> Generator;
		Audio::FAlignedFloatBuffer GeneratorBufferInterleaved;
		Metasound::FSampleCount SampleCount = 0;
		Metasound::FSampleCount SampleRemainder = 0;
	};
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDelayNodeTestRenderNoClockDefaults, 
		"Harmonix.Metasound.Nodes.Delay.Render.NoClock.Defaults", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDelayNodeTestRenderNoClockDefaults::RunTest(const FString&)
	{
		constexpr float SampleRate = 48000;
		constexpr int32 NumSamples = 256;
		FTestFixture TestFixture(SampleRate, NumSamples, *this, false);

		constexpr int32 NumBlocksToRender = 200;
		FTestFixture::FParams Params;

		// test with defaults
		{
			TestFixture.SetParams(Params);
			for (int32 i = 0; i < NumBlocksToRender; ++i)
			{
				UTEST_TRUE(FString::Printf(TEXT("Render test iteration %i"), i), TestFixture.RenderAndCompare(i == 0));
			}
		}
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FDelayNodeTestRenderWithClockDefaults, 
	"Harmonix.Metasound.Nodes.Delay.Render.WithClock.Defaults", 
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDelayNodeTestRenderWithClockDefaults::RunTest(const FString&)
	{
		constexpr float SampleRate = 48000;
		constexpr int32 NumSamples = 256;
		FTestFixture TestFixture(SampleRate, NumSamples, *this, true);

		constexpr int32 NumBlocksToRender = 200;
		FTestFixture::FParams Params;
		Params.DelayTimeType = ETimeSyncOption::TempoSync;

		// test with defaults
		{
			TestFixture.SetParams(Params);
			UTEST_TRUE("Started clock", TestFixture.ResetAndStartClock(120, 1, 4, 4));
			for (int32 i = 0; i < NumBlocksToRender; ++i)
			{
				UTEST_TRUE(FString::Printf(TEXT("Advance clock iteration %i"), i), TestFixture.AdvanceClock());
				UTEST_TRUE(FString::Printf(TEXT("Render test iteration %i"), i), TestFixture.RenderAndCompare(i == 0));
			}
		}
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDelayNodeTestRenderNoClockMinDelayLfoEnabled, 
		"Harmonix.Metasound.Nodes.Delay.Render.NoClock.MinDelayLfoEnabled", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDelayNodeTestRenderNoClockMinDelayLfoEnabled::RunTest(const FString&)
	{
		constexpr float SampleRate = 48000;
		constexpr int32 NumSamples = 256;
		FTestFixture TestFixture(SampleRate, NumSamples, *this, false);

		constexpr int32 NumBlocksToRender = 200;
		FTestFixture::FParams Params;
		Params.DelayTime = 0;
		Params.LFOEnabled = true;

		// test with defaults
		{
			TestFixture.SetParams(Params);
			for (int32 i = 0; i < NumBlocksToRender; ++i)
			{
				UTEST_TRUE(FString::Printf(TEXT("Render test iteration %i"), i), TestFixture.RenderAndCompare(i == 0));
			}
		}
		
		return true;
	}
}

#endif
