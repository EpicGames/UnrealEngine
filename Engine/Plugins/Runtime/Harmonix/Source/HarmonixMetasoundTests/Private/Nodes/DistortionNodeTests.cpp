// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixDsp/Generate.h"
#include "HarmonixMetasound/Common.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::DistortionNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	// Validate node creation
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDistortionCreateNodeTest,
		"Harmonix.Metasound.Nodes.Distortion.CreateNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FDistortionCreateNodeTest::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "Distortion", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// execute a block
		{
			TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
			Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
		}

		// No need to validate output pin, finish test

		return true;
	}

	//  Validate audio output is generated when node is getting audio input
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FDistortionGenerateAudioTest,
		"Harmonix.Metasound.Nodes.Distortion.GenerateAudioTest",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FDistortionGenerateAudioTest::RunTest(const FString&)
	{
		constexpr float SampleRate = 48000;
		constexpr int32 NumSamples = 256;

		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "Distortion", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// execute a block
		{
			// fill the input with white noise
			TOptional<FAudioBufferWriteRef> InputAudio = Generator->GetInputWriteReference<FAudioBuffer>("In");
			UTEST_TRUE("Got input audio", InputAudio.IsSet());
			HarmonixDsp::GenerateWhiteNoise((*InputAudio)->GetData(), (*InputAudio)->Num());
			Generator->SetInputValue("Wet Gain", 1.0f);
			// render
			TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete };
			Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
			// check that sound came out the other side
			UTEST_TRUE("Some audio came out the other side", Buffer.GetPeak() > 0.0f);
		}

		return true;
	}
}

#endif
