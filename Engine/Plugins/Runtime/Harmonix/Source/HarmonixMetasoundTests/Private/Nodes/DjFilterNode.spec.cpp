// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGenerator.h"
#include "NodeTestGraphBuilder.h"

#include "HarmonixDsp/Effects/DjFilter.h"

#include "HarmonixMetasound/Nodes/DjFilterNode.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MorphingLFONode
{
	BEGIN_DEFINE_SPEC(
		FHarmonixMetasoundDjFilterNodeSpec,
		"Harmonix.Metasound.Nodes.DjFilterNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	Harmonix::Dsp::Effects::FDjFilter FilterForComparison{ 0 };
	TUniquePtr<Metasound::FMetasoundGenerator> Generator;

	void BuildGenerator()
	{
		using FBuilder = Metasound::Test::FNodeTestGraphBuilder;
		FBuilder Builder;
		Generator = Builder.MakeSingleNodeGraph(HarmonixMetasound::Nodes::DjFilter::GetClassName(), 0);
	}

	void GenerateSaw(Metasound::FAudioBuffer* Buffer, float Frequency, float SampleRate, float& Phase)
	{
		const float PhaseInc = Frequency / SampleRate;
		
		for (int32 SampleIdx = 0; SampleIdx < Buffer->Num(); ++SampleIdx)
		{
			Buffer->GetData()[SampleIdx] = Phase * 2 - 1;

			Phase += PhaseInc;

			while (Phase > 1.0f)
			{
				Phase -= 1.0f;
			}
		}
	}

	END_DEFINE_SPEC(FHarmonixMetasoundDjFilterNodeSpec)

	void FHarmonixMetasoundDjFilterNodeSpec::Define()
	{
		Describe("With defaults", [this]()
		{
			BeforeEach([this]()
			{
				BuildGenerator();
				FilterForComparison.Reset(Generator->OperatorSettings.GetSampleRate());
			});

			// Test a few different amounts
			const TArray<float> AmountsToTest = { -1.0f, -0.4f, 0.0f, 0.6f, 1.0f };

			for (const float Amount : AmountsToTest)
			{
				It(FString::Printf(TEXT("matches raw DSP module with amount %d%%"), static_cast<int32>(Amount * 100)), [this, Amount]()
				{
					TOptional<Metasound::FAudioBufferWriteRef> InputBuffer =
						Generator->GetInputWriteReference<Metasound::FAudioBuffer>(HarmonixMetasound::Nodes::DjFilter::Inputs::AudioMonoName);

					if (!TestTrue("Got input buffer", InputBuffer.IsSet()))
					{
						return;
					}

					// Set the amount on the generator and the comparison filter
					Generator->SetInputValue(HarmonixMetasound::Nodes::DjFilter::Inputs::AmountName, Amount);
					FilterForComparison.Amount = Amount;

					constexpr int32 NumBlocks = 40;
					constexpr float Frequency = 110.0f;
					float Phase = 0;

					// Run some sound through the node and raw DSP and expect them to output the same sound
					for (int32 BlockIdx = 0; BlockIdx < NumBlocks; ++BlockIdx)
					{
						GenerateSaw((*InputBuffer).Get(), Frequency, Generator->OperatorSettings.GetSampleRate(), Phase);

						Audio::FAlignedFloatBuffer ExpectedOutput;
						ExpectedOutput.SetNumZeroed((*InputBuffer)->Num());
						FilterForComparison.Process(*(*InputBuffer), ExpectedOutput);

						Audio::FAlignedFloatBuffer ActualOutput;
						ActualOutput.SetNumZeroed(ExpectedOutput.Num());
						Generator->OnGenerateAudio(ActualOutput.GetData(), ActualOutput.Num());

						for (int32 SampleIdx = 0; SampleIdx < ActualOutput.Num(); ++SampleIdx)
						{
							if (!TestEqual(
								FString::Printf(TEXT("samples match at block %i, sample %i"), BlockIdx, SampleIdx),
								ActualOutput[SampleIdx],
								ExpectedOutput[SampleIdx]))
							{
								return;
							}
						}
					}
				});
			}
		});
	}

}

#endif