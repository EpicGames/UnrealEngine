// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGenerator.h"
#include "NodeTestGraphBuilder.h"

#include "HarmonixDsp/AudioAnalysis/FFTAnalyzer.h"
#include "HarmonixDsp/Generate.h"

#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/FFTAnalyzerResult.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::FFTAnalyzerNode
{
	BEGIN_DEFINE_SPEC(
		FHarmonixMetasoundFFTAnalyzerNodeSpec,
		"Harmonix.Metasound.Nodes.FFTAnalyzer",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	
	END_DEFINE_SPEC(FHarmonixMetasoundFFTAnalyzerNodeSpec)

	void FHarmonixMetasoundFFTAnalyzerNodeSpec::Define()
	{
		Describe("When passing in some white noise", [this]()
		{
			It("reports the same result as the raw DSP", [this]()
			{
				const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
					Metasound::Test::FNodeTestGraphBuilder::MakeSingleNodeGraph(
						{HarmonixMetasound::HarmonixNodeNamespace, TEXT("FFT Analyzer"), TEXT("")},
						0);

				Harmonix::Dsp::AudioAnalysis::FFFTAnalyzer FFTAnalyzerForComparison{ Generator->OperatorSettings.GetSampleRate() };

				// Make some noise and copy it to the node's input
				Audio::FAlignedFloatBuffer InputBuffer;
				InputBuffer.SetNumUninitialized(Generator->OperatorSettings.GetNumFramesPerBlock());
				HarmonixDsp::GenerateWhiteNoise(InputBuffer.GetData(), InputBuffer.Num());

				auto GeneratorInputBuffer = Generator->GetInputWriteReference<Metasound::FAudioBuffer>("In");

				if (!TestTrue("Got audio input", GeneratorInputBuffer.IsSet()))
				{
					return;
				}

				FHarmonixFFTAnalyzerResults RawResults;

				constexpr int32 NumBlocksToRender = 4;
				Metasound::FAudioBuffer OutputBuffer{ InputBuffer.Num() };
				
				for (int32 i = 0; i < NumBlocksToRender; ++i)
				{
					FMemory::Memcpy(
						(*GeneratorInputBuffer)->GetData(),
						InputBuffer.GetData(),
						InputBuffer.Num() * sizeof(float));
				
					// Process the node
					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());

					// Process the raw DSP
					FFTAnalyzerForComparison.Process(InputBuffer, RawResults);
				}

				auto GeneratorResults = Generator->GetOutputReadReference<FHarmonixFFTAnalyzerResults>("Results");

				if (!TestTrue("Got results output", GeneratorResults.IsSet()))
				{
					return;
				}

				if (!TestEqual("Same number of result bins", (*GeneratorResults)->Spectrum.Num(), RawResults.Spectrum.Num()))
				{
					return;
				}

				if (!TestTrue("More than zero result bins", RawResults.Spectrum.Num() > 0))
				{
					return;
				}

				for (int32 BinIdx = 0; BinIdx < RawResults.Spectrum.Num(); ++BinIdx)
				{
					if (!TestTrue("Bin matches", FMath::IsNearlyEqual(RawResults.Spectrum[BinIdx], (*GeneratorResults)->Spectrum[BinIdx])))
					{
						return;
					}
				}
			});

			It("reports nothing when disabled", [this]()
			{
				TUniquePtr<Metasound::FMetasoundGenerator> Generator =
					Metasound::Test::FNodeTestGraphBuilder::MakeSingleNodeGraph(
						{HarmonixMetasound::HarmonixNodeNamespace, TEXT("FFT Analyzer"), TEXT("")},
						0);

				auto GeneratorInputBuffer = Generator->GetInputWriteReference<Metasound::FAudioBuffer>("In");

				if (!TestTrue("Got audio input", GeneratorInputBuffer.IsSet()))
				{
					return;
				}

				auto GeneratorResults = Generator->GetOutputReadReference<FHarmonixFFTAnalyzerResults>("Results");

				if (!TestTrue("Got results output", GeneratorResults.IsSet()))
				{
					return;
				}

				// Render a few times and expect some results
				constexpr int32 NumBlocksToRender = 4;

				for (int32 i = 0; i < NumBlocksToRender; ++i)
				{
					// Make some noise
					HarmonixDsp::GenerateWhiteNoise((*GeneratorInputBuffer)->GetData(), (*GeneratorInputBuffer)->Num());

					// Process the node
					Metasound::FAudioBuffer OutputBuffer{ (*GeneratorInputBuffer)->Num() };
					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());
				}
				
				if (!TestTrue("More than zero result bins", (*GeneratorResults)->Spectrum.Num() > 0))
				{
					return;
				}

				bool GotSomeValidResults = false;
				
				for (int32 BinIdx = 0; BinIdx < (*GeneratorResults)->Spectrum.Num(); ++BinIdx)
				{
					if ((*GeneratorResults)->Spectrum[BinIdx] > 0.0f)
					{
						GotSomeValidResults = true;
						return;
					}
				}

				if (!TestTrue("Got some valid results", GotSomeValidResults))
				{
					return;
				}

				// Disable and expect no results
				Generator->SetInputValue("Enable", false);

				for (int32 i = 0; i < NumBlocksToRender; ++i)
				{
					// Make some noise
					HarmonixDsp::GenerateWhiteNoise((*GeneratorInputBuffer)->GetData(), (*GeneratorInputBuffer)->Num());

					// Process the node
					Metasound::FAudioBuffer OutputBuffer{ (*GeneratorInputBuffer)->Num() };
					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());

					if (!TestTrue("Zero result bins", (*GeneratorResults)->Spectrum.Num() == 0))
					{
						return;
					}
				}

				// Re-enable and expect some results
				for (int32 i = 0; i < NumBlocksToRender; ++i)
				{
					// Make some noise
					HarmonixDsp::GenerateWhiteNoise((*GeneratorInputBuffer)->GetData(), (*GeneratorInputBuffer)->Num());

					// Process the node
					Metasound::FAudioBuffer OutputBuffer{ (*GeneratorInputBuffer)->Num() };
					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());
				}
				
				if (!TestTrue("More than zero result bins", (*GeneratorResults)->Spectrum.Num() > 0))
				{
					return;
				}

				GotSomeValidResults = false;
				
				for (int32 BinIdx = 0; BinIdx < (*GeneratorResults)->Spectrum.Num(); ++BinIdx)
				{
					if ((*GeneratorResults)->Spectrum[BinIdx] > 0.0f)
					{
						GotSomeValidResults = true;
						return;
					}
				}

				if (!TestTrue("Got some valid results", GotSomeValidResults))
				{
					return;
				}
			});
		});
	}
}

#endif