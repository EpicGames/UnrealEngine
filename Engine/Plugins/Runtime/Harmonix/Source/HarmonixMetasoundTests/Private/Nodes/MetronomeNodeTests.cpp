// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "Misc/AutomationTest.h"

DEFINE_LOG_CATEGORY_STATIC(LogMetronomeNodeTests, Log, All);

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MetronomeNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	static const FString TempoChangeTestString = TEXT("Test Tempo Change while Playing");

	class FBasicMetronomeTest
	{
	public:
		struct FParameters
		{
			// rendering
			int32 NumSamplesPerBlock = 256;
			const float SampleRate = 48000.0f;
			int32 NumBlocks = 100;

			// clock parameters
			float Tempo = 120.0f;
			float Speed = 1.0f;

			int32 TimeSigNumerator = 4;
			int32 TimeSigDenominator = 4;

			bool Loop = false;
			int32 LoopLengthBars = 1;
			int32 PreRollBars = 8;
		};

		static bool RunTest(FAutomationTestBase& InTest, const FParameters& Params, const FString& TestCaseString)
		{
			GraphBuilder Builder;
			const FNodeHandle MetronomeNodeHandle = Builder.AddNode(
				{ HarmonixMetasound::HarmonixNodeNamespace, "Metronome", "" }, 0
			);

			auto Testf = [TestCaseString](const FString& TestString) -> FString
			{
				return FString::Printf(TEXT("%s: %s"), *TestCaseString, *TestString);
			};

			if (!InTest.TestTrue(Testf("Metronome node should be Valid"), MetronomeNodeHandle->IsValid()))
			{
				return false;
			}

			using namespace CommonPinNames;
			
			Builder.AddAndConnectConstructorInput(MetronomeNodeHandle, Inputs::LoopName, Params.Loop);
			Builder.AddAndConnectConstructorInput(MetronomeNodeHandle, Inputs::LoopLengthBarsName, Params.LoopLengthBars);
			Builder.AddAndConnectConstructorInput(MetronomeNodeHandle, Inputs::PrerollBarsName, Params.PreRollBars);
			
			bool AddedDataReferences = Builder.AddAndConnectDataReferenceInputs(MetronomeNodeHandle);

			if (!InTest.TestTrue(Testf("Added All Data References"), AddedDataReferences))
			{
				return false;
			}

			Builder.AddAndConnectDataReferenceOutput(MetronomeNodeHandle, Outputs::MidiClockName, GetMetasoundDataTypeName<FMidiClock>());

			// have to make an audio output for the generator to do anything
			Builder.AddOutput("AudioOut", GetMetasoundDataTypeName<FAudioBuffer>());

			const TUniquePtr<FMetasoundGenerator> Generator = Builder.BuildGenerator(Params.SampleRate, Params.NumSamplesPerBlock);

			if (!InTest.TestTrue(Testf("Graph successfully built"), Generator.IsValid()))
			{
				return false;
			}

			if (!InTest.TestTrue(Testf("Graph has audio output"), Generator->GetNumChannels() > 0))
			{
				return false;
			}

			Generator->ApplyToInputValue<FMusicTransportEventStream>(Inputs::TransportName, [](FMusicTransportEventStream& Transport)
				{
					Transport.AddTransportRequest(EMusicPlayerTransportRequest::Prepare, 0);
					Transport.AddTransportRequest(EMusicPlayerTransportRequest::Play, 1);
				}
			);

			Generator->SetInputValue<int32>(Inputs::TimeSigNumeratorName, Params.TimeSigNumerator);
			Generator->SetInputValue<int32>(Inputs::TimeSigDenominatorName, Params.TimeSigDenominator);
			Generator->SetInputValue<float>(Inputs::TempoName, Params.Tempo);
			Generator->SetInputValue<float>(Inputs::SpeedName, Params.Speed);

			TOptional<FMidiClockReadRef> OutputMidiClock = Generator->GetOutputReadReference<FMidiClock>(Outputs::MidiClockName);
			// validate midi output
			if (!InTest.TestTrue(Testf("MIDI clock output exists"), OutputMidiClock.IsSet()))
			{
				return false;
			}

			if (!InTest.TestEqual(Testf("Midi Clock Looping"), (*OutputMidiClock)->HasPersistentLoop(), Params.Loop))
			{
				return false;
			}

			float DefaultTicksPerSec = 120.0f * Harmonix::Midi::Constants::GTicksPerQuarterNote / 60.0f;
			float DefaultTicksPerMs = DefaultTicksPerSec / 1000.0f;

			// do some math to figure out how fast the clock should be advancing...
			float TicksPerSec = Params.Tempo * Harmonix::Midi::Constants::GTicksPerQuarterNote / 60.0f;
			float TicksPerMs = TicksPerSec / 1000.0f;
			float SecsPerBlock = Params.NumSamplesPerBlock / Params.SampleRate;
			float TicksPerBlock = TicksPerSec * SecsPerBlock;

			TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(Params.Tempo, Params.TimeSigNumerator, Params.TimeSigDenominator);
			int32 LoopLengthTicks = SongMaps->BarIncludingCountInToTick(Params.LoopLengthBars);
			
			//test for tempo consistency by stopping and restarting the transport (clock output and tempo map)
			if (TestCaseString.Equals(TempoChangeTestString))
			{
				float DifferentTempo = Params.Tempo + 10.f;
				constexpr int32 TempoPointsChangeIndexAtStart = 0;
				constexpr int32 TempoPointsChangeIndexAfterStop = 1;
				constexpr int32 ExpectedNumTempoChangeAtStart = 1;
				constexpr int32 ExpectedNumTempoChangeBeforeStop = 2;
				constexpr int32 ExpectedNumTempoChangeStopAndRestart = 1;

				//generate a block 
				TAudioBuffer<float> TempoTestBuffer{ Generator->GetNumChannels(), Params.NumSamplesPerBlock, EAudioBufferCleanupMode::Delete };
				Generator->OnGenerateAudio(TempoTestBuffer.GetRawChannelData(0), TempoTestBuffer.GetNumTotalValidSamples());
				
				//check tempo (original)
				if (!InTest.TestEqual("Expect original tempo at the end of the block", (*OutputMidiClock)->GetTempoAtEndOfBlock(), Params.Tempo))
				{
					return false;
				}

				//expect 1 tempo change point at the beginning (original tempo)
				int32 NumTempoChange = (*OutputMidiClock)->GetSongMapEvaluator().GetNumTempoChanges();
				if (!InTest.TestEqual("Expect 1 tempo change", NumTempoChange, ExpectedNumTempoChangeAtStart))
				{
					return false;
				}

				//check tempo in tempo map
				int32 TempoChangeTick = (*OutputMidiClock)->GetSongMapEvaluator().GetTempoChangePointTick(TempoPointsChangeIndexAtStart);
				float CurrentTempoBPM = (*OutputMidiClock)->GetSongMapEvaluator().GetTempoAtTick(TempoChangeTick);
				if (!InTest.TestEqual("Expect original tempo at tick 0", CurrentTempoBPM, Params.Tempo, 0.001f))
				{
					return false;
				}

				//change tempo
				Generator->SetInputValue<float>(Inputs::TempoName, DifferentTempo);
	
				//generate a few blocks
				for (int32 BlockIndex = 0; BlockIndex < Params.NumBlocks; ++BlockIndex)
				{
					Generator->OnGenerateAudio(TempoTestBuffer.GetRawChannelData(0), TempoTestBuffer.GetNumTotalValidSamples());
					//check tempo at the end of each block (different tempo) 
					if (!InTest.TestEqual("Expect tempo different from original at the end of the block", (*OutputMidiClock)->GetTempoAtEndOfBlock(), DifferentTempo, 0.001f))
					{
						return false;
					}
				}

				//expect 2 tempo change points in tempo map: 1 at the beginning (original), 1 at the current tick (different)
				NumTempoChange = (*OutputMidiClock)->GetSongMapEvaluator().GetNumTempoChanges();
				if (!InTest.TestEqual("Expect 2 tempo change", NumTempoChange, ExpectedNumTempoChangeBeforeStop))
				{
					return false;
				}

				//check tempo in tempo map
				TempoChangeTick = (*OutputMidiClock)->GetSongMapEvaluator().GetTempoChangePointTick(TempoPointsChangeIndexAfterStop);
				CurrentTempoBPM = (*OutputMidiClock)->GetSongMapEvaluator().GetTempoAtTick(TempoChangeTick);
				if (!InTest.TestEqual("Expect a different tempo from the original at the current tick", CurrentTempoBPM, DifferentTempo,0.001f))
				{
					return false;
				}

				//stop transport 
				Generator->ApplyToInputValue<FMusicTransportEventStream>(Inputs::TransportName, [](FMusicTransportEventStream& Transport)
					{
						Transport.AddTransportRequest(EMusicPlayerTransportRequest::Stop, 1);
					}
				);

				//reset tempo to original 
				Generator->SetInputValue<float>(Inputs::TempoName, Params.Tempo);
				Generator->OnGenerateAudio(TempoTestBuffer.GetRawChannelData(0), TempoTestBuffer.GetNumTotalValidSamples());

				//restart transport 
				Generator->ApplyToInputValue<FMusicTransportEventStream>(Inputs::TransportName, [](FMusicTransportEventStream& Transport)
					{
						Transport.AddTransportRequest(EMusicPlayerTransportRequest::Prepare, 1);
						Transport.AddTransportRequest(EMusicPlayerTransportRequest::Play, 2);
					}
				);

				//generate a few blocks
				for (int32 BlockIndex = 0; BlockIndex < Params.NumBlocks; ++BlockIndex)
				{
					Generator->OnGenerateAudio(TempoTestBuffer.GetRawChannelData(0), TempoTestBuffer.GetNumTotalValidSamples());
					//check tempo (original)
					if (!InTest.TestEqual("Expect original tempo at the end of the block", (*OutputMidiClock)->GetTempoAtEndOfBlock(), Params.Tempo))
					{
						return false;
					}
				}
				
				//expect 1 tempo change at the start 
				NumTempoChange = (*OutputMidiClock)->GetSongMapEvaluator().GetNumTempoChanges();
				if (!InTest.TestEqual("Expect 1 tempo change", NumTempoChange, ExpectedNumTempoChangeStopAndRestart))
				{
					return false;
				}

				//check tempo in tempo map (original tempo) 
				TempoChangeTick = (*OutputMidiClock)->GetSongMapEvaluator().GetTempoChangePointTick(TempoPointsChangeIndexAtStart);
				CurrentTempoBPM = (*OutputMidiClock)->GetSongMapEvaluator().GetTempoAtTick(TempoChangeTick);
				if (!InTest.TestEqual("Expect original tempo at tick 0", CurrentTempoBPM, Params.Tempo,0.001f))
				{
					return false;
				}

				return true;
			}
			
			// execute 
			bool AllTicksEqual = true;
			for (int32 BlockIndex = 0; BlockIndex < Params.NumBlocks; ++BlockIndex)
			{
				TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), Params.NumSamplesPerBlock, EAudioBufferCleanupMode::Delete };
				Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());

				int32 ExpectedTick = FMath::RoundToInt32(TicksPerBlock * (BlockIndex + 1));

				if (!InTest.TestEqual(Testf("Midi Clock Looping"), (*OutputMidiClock)->HasPersistentLoop(), Params.Loop))
				{
					return false;
				}

				float ClockTempo = (*OutputMidiClock)->GetTempoAtEndOfBlock();

				if (!InTest.TestEqual(Testf(FString::Printf(TEXT("Midi Clock Tempo at block: %d"), BlockIndex)), ClockTempo, Params.Tempo, 0.001f))
				{
					return false;
				}

				if (Params.Loop)
				{
					ExpectedTick %= LoopLengthTicks;
				}

				int32 ActualTick = (*OutputMidiClock)->GetNextMidiTickToProcess();
				// allow for single tick tolerance?
				ExpectedTick = FMath::Abs(ExpectedTick - ActualTick) <= 1 ? ActualTick : ExpectedTick;
				if (AllTicksEqual && (ActualTick != ExpectedTick))
				{
					FString What = Testf(FString::Printf(TEXT("All ticks not equal. First failure at block: %d"), BlockIndex));
					InTest.AddError(FString::Printf(TEXT("%s. Expected tick to be %d, but it was %d."), *What, ExpectedTick, ActualTick));
					AllTicksEqual = false;
				}

				// test looping here since it the values may not be updated until the first execution
				if ((*OutputMidiClock)->HasPersistentLoop())
				{
					int32 LoopStartTick = 0;
					int32 LoopEndTick = LoopLengthTicks;

					float LoopStartMs = LoopStartTick / TicksPerMs;
					float LoopEndMs = LoopEndTick / TicksPerMs;

					if (!InTest.TestEqual(Testf("Midi Clock Loop Start Tick"), (*OutputMidiClock)->GetFirstTickInLoop(), LoopStartTick))
					{
						return false;
					}

					if (!InTest.TestEqual(Testf("Midi Clock Loop Length"), (*OutputMidiClock)->GetLoopLengthTicks(), LoopEndTick - LoopStartTick))
					{
						return false;
					}

					// slightly less aggressive tolerance since it doesn't have to be _that_ precise
					if (!InTest.TestEqual(Testf("Midi Clock Loop Start Ms"), (*OutputMidiClock)->GetLoopStartMs(), LoopStartMs, 0.1f))
					{
						return false;
					}

					if (!InTest.TestEqual(Testf("Midi Clock Loop End Ms"), (*OutputMidiClock)->GetLoopEndMs(), LoopEndMs, 0.1f))
					{
						return false;
					}
				}

				Generator->ApplyToInputValue<FMusicTransportEventStream>(Inputs::TransportName, [](FMusicTransportEventStream& Transport)
					{
						Transport.Reset();
					}
				);
			}

			if (!AllTicksEqual)
			{
				return false;
			}

			return true;
		}

	private:

		FBasicMetronomeTest() {};
	};


	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetronomeCreateNodeTestDefaults,
		"Harmonix.Metasound.Nodes.Metronome.CreateAndPlay_120BPM_4/4",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMetronomeCreateNodeTestDefaults::RunTest(const FString&)
	{
		FBasicMetronomeTest::FParameters Params;
		return FBasicMetronomeTest::RunTest(*this, Params, "Test Defaults");
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetronomeCreateNodeTestLooping,
		"Harmonix.Metasound.Nodes.Metronome.Looping_240BPM_4/4",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMetronomeCreateNodeTestLooping::RunTest(const FString&)
	{
		FBasicMetronomeTest::FParameters Params;
		Params.Loop = true;
		Params.Tempo = 240.0f;
		// render enough blocks to experience a loop
		Params.NumBlocks = 500;
		return FBasicMetronomeTest::RunTest(*this, Params, "Test Looping");
	}

	// test tempos 4-240
	// tempos below 4 aren't well supported
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetronomeCreateNodeTestLoopingTempoRange,
		"Harmonix.Metasound.Nodes.Metronome.Looping_4-240BPM_4/4",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMetronomeCreateNodeTestLoopingTempoRange::RunTest(const FString&)
	{
		int32 Min = 4;
		int32 Max = 240;
		for (int32 Tempo = Min; Tempo <= Max; ++Tempo)
		{
			FBasicMetronomeTest::FParameters Params;
			Params.Loop = true;
			Params.NumBlocks = 500;
			Params.Tempo = (float)Tempo;
			Params.TimeSigNumerator = 4;
			Params.TimeSigDenominator = 4;
			FString TestString = FString::Printf(TEXT("Test %d BPM"), Tempo);
			if (!FBasicMetronomeTest::RunTest(*this, Params, TestString))
			{
				return false;
			}
		}

		return !HasAnyErrors();
	}

	// Test range a good range of time signatures: 1/1 - 12/12.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetronomeCreateNodeTestLoopingAllTimeSigs,
		"Harmonix.Metasound.Nodes.Metronome.Looping_120BPM_1/1-12/12",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMetronomeCreateNodeTestLoopingAllTimeSigs::RunTest(const FString&)
		{
			int32 Min = 1;
			int32 Max = 12;
			for (int32 Numerator = Min; Numerator <= Max; ++Numerator)
			{
				for (int32 Denominator = Min; Denominator <= Max; ++Denominator)
				{
					FBasicMetronomeTest::FParameters Params;
					Params.Loop = true;
					// only testing loop lengths, so don't need to advance clock
					Params.NumBlocks = 500;
					Params.TimeSigNumerator = Numerator;
					Params.TimeSigDenominator = Denominator;
					FString TestString = FString::Printf(TEXT("Test %d/%d Time"), Numerator, Denominator);
					if (!FBasicMetronomeTest::RunTest(*this, Params, TestString))
					{
						return false;
					}
				}
			}

			return !HasAnyErrors();
		}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetronomeCreateNodeTestTempoChangeInBlock,
		"Harmonix.Metasound.Nodes.Metronome.TempoChangeWhilePlaying",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMetronomeCreateNodeTestTempoChangeInBlock::RunTest(const FString&)
		{
			FBasicMetronomeTest::FParameters Params;
			Params.NumBlocks = 500;
			return FBasicMetronomeTest::RunTest(*this, Params, TempoChangeTestString);
		}
}

#endif
