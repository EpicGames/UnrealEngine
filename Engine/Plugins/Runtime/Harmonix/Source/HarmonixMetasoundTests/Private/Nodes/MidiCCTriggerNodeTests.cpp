// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "Misc/AutomationTest.h"
#include "HarmonixMetasound/DataTypes/MidiControllerID.h"
#include "HarmonixMetasound/Nodes/MidiCCTriggerNode.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiCCTriggerNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;
	using namespace Metasound::Test;
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiCCTriggerNodeTestBasic,
		"Harmonix.Metasound.Nodes.MidiCCTriggerNode.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiCCTriggerNodeTestBasic::RunTest(const FString&)
	{
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			Nodes::MidiCCTriggerNode::GetClassName(),
			Nodes::MidiCCTriggerNode::GetCurrentMajorVersion());
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		/*
		 * Get refs to inputs we want to manipulate
		 */
		TOptional<FBoolWriteRef> InputEnable = Generator->GetInputWriteReference<bool>(Nodes::MidiCCTriggerNode::Inputs::EnableName);
		UTEST_TRUE("Got enable input", InputEnable.IsSet());

		TOptional<FMidiStreamWriteRef> InputMidiStream =
			Generator->GetInputWriteReference<FMidiStream>(Nodes::MidiCCTriggerNode::Inputs::MidiStreamName);
		UTEST_TRUE("Got MIDI stream input", InputMidiStream.IsSet());

		TOptional<FEnumStdMidiControllerIDWriteRef> InputControlNumber =
			Generator->GetInputWriteReference<FEnumStdMidiControllerID>(Nodes::MidiCCTriggerNode::Inputs::InputMidiControllerIDName);
		UTEST_TRUE("Got control number input", InputControlNumber.IsSet());

		/*
		 * Get refs to outputs we want to check.
		 */
		TOptional<FInt32ReadRef> OutputControlChangeValueInt32 =
			Generator->GetOutputReadReference<int32>(Nodes::MidiCCTriggerNode::Outputs::OutputControlChangeValueInt32Name);
		UTEST_TRUE("Output exists", OutputControlChangeValueInt32.IsSet());
		UTEST_EQUAL("Control Change value (int32) at start", **OutputControlChangeValueInt32, 0);

		constexpr uint8 Channel = 3;
		constexpr uint8 ControlNumber = 24;
		constexpr uint8 Value = 100;

		**InputControlNumber = FEnumStdMidiControllerID{ ControlNumber };
		
		/*
		 * Add a CC to the input stream and expect to get the value
		 */
		{
			// Add a CC to the input stream
			const FMidiStreamEvent CCEvent{ static_cast<uint32>(0), FMidiMsg::CreateControlChange(Channel, ControlNumber, Value) };
			(*InputMidiStream)->AddMidiEvent(CCEvent);
			
			// Process
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Check that we got the CC info out the other side
			UTEST_EQUAL("Got correct value", **OutputControlChangeValueInt32, Value);

			// Advance the MIDI stream
			(*InputMidiStream)->PrepareBlock();
		}

		/*
		 * Process again and expect the same output values
		 */
		{
			// Process
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Check that we got nothing new out the other side
			UTEST_EQUAL("Got correct value", **OutputControlChangeValueInt32, Value);

			// Advance the MIDI stream
			(*InputMidiStream)->PrepareBlock();
		}

		/*
		 * Add a CC with a different number and value to the input stream, and expect to get the same output values
		 */
		{
			// Add a CC to the input stream
			const FMidiStreamEvent CCEvent{ static_cast<uint32>(0), FMidiMsg::CreateControlChange(Channel, ControlNumber + 1, Value + 1) };
			(*InputMidiStream)->AddMidiEvent(CCEvent);
			
			// Process
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Check that we got the CC info out the other side
			UTEST_EQUAL("Got correct value", **OutputControlChangeValueInt32, Value);

			// Advance the MIDI stream
			(*InputMidiStream)->PrepareBlock();
		}


		/*
		 * Add a CC to the input stream, turn off the node, and expect the same output values
		 */
		{
			// Add a CC to the input stream
			const FMidiStreamEvent CCEvent{ static_cast<uint32>(0), FMidiMsg::CreateControlChange(Channel, ControlNumber, Value) };
			(*InputMidiStream)->AddMidiEvent(CCEvent);

			// Turn off the node
			**InputEnable = false;
			
			// Process
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Check that we got the CC info out the other side
			UTEST_EQUAL("Got correct value", **OutputControlChangeValueInt32, Value);

			// Advance the MIDI stream
			(*InputMidiStream)->PrepareBlock();
		}

		return true;
	}
}
#endif