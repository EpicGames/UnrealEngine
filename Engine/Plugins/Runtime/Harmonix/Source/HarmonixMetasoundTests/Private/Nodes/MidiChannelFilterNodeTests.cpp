// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/MidiOps/MidiChannelFilter.h"
#include "HarmonixMetasound/Nodes/MidiChannelFilterNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasound::Nodes::MidiChannelFilter::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiChannelFilterNodeParityTest,
	"Harmonix.Metasound.Nodes.MidiChannelFilter.Parity",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiChannelFilterNodeParityTest::RunTest(const FString&)
	{
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
			Metasound::Test::FNodeTestGraphBuilder::MakeSingleNodeGraph(GetClassName(), GetCurrentMajorVersion());
		TOptional<Metasound::TDataReadReference<FMidiStream>> NodeMidiOutput = Generator->GetOutputReadReference<FMidiStream>(Outputs::MidiStreamName);
		UTEST_TRUE("Got node MIDI output", NodeMidiOutput.IsSet());
		
		Harmonix::Midi::Ops::FMidiChannelFilter FilterForComparison;
		FMidiStream InputStream;
		FMidiStream OutputStream;

		FMidiStreamEvent NoteOnEvent{ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(2, 43, 45) };
		const uint8 ChannelOneIndexed = NoteOnEvent.MidiMessage.GetStdChannel() + 1;
		
		// Channel mismatch, no events output
		{
			// Set the wrong channel
			Generator->SetInputValue<int32>(Inputs::ChannelName, ChannelOneIndexed + 1);
			FilterForComparison.SetChannelEnabled(ChannelOneIndexed + 1, true);
			
			// Add an event to the node input
			Generator->ApplyToInputValue<FMidiStream>(Inputs::MidiStreamName, [&NoteOnEvent](FMidiStream& Stream)
			{
				Stream.AddMidiEvent(NoteOnEvent);
			});

			// Process the node
			Metasound::FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Process the filter
			InputStream.AddMidiEvent(NoteOnEvent);
			FilterForComparison.Process(InputStream, OutputStream);

			// Make sure we got no events
			UTEST_EQUAL("Got no events (test)", (*NodeMidiOutput)->GetEventsInBlock().Num(), 0);
			UTEST_EQUAL("Got no events (control)", OutputStream.GetEventsInBlock().Num(), 0);

			// Leave the inputs where they are
		}

		// Turn on the channel for the note on and expect the message to come out the other end
		{
			Generator->SetInputValue<int32>(Inputs::ChannelName, ChannelOneIndexed);
			FilterForComparison.SetChannelEnabled(0, false); // Reset
			FilterForComparison.SetChannelEnabled(ChannelOneIndexed, true);

			// Process the node
			Metasound::FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Process the filter
			FilterForComparison.Process(InputStream, OutputStream);

			const TArray<FMidiStreamEvent>& NodeEvents = (*NodeMidiOutput)->GetEventsInBlock();
			UTEST_EQUAL("Got one event (test)", NodeEvents.Num(), 1);
			UTEST_TRUE("Event is note on (test)", NodeEvents[0].MidiMessage.IsNoteOn());
			const TArray<FMidiStreamEvent>& ControlEvents = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Got one event (control)", ControlEvents.Num(), NodeEvents.Num());
			UTEST_TRUE("Event is note on (control)", ControlEvents[0].MidiMessage.IsNoteOn());

			UTEST_EQUAL("Same channel", NodeEvents[0].MidiMessage.GetStdChannel(), ControlEvents[0].MidiMessage.GetStdChannel());
			UTEST_EQUAL("Same note number", NodeEvents[0].MidiMessage.GetStdData1(), ControlEvents[0].MidiMessage.GetStdData1());
			UTEST_EQUAL("Same velocity", NodeEvents[0].MidiMessage.GetStdData2(), ControlEvents[0].MidiMessage.GetStdData2());
		}
		
		return true;
	}
}

#endif