// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/MidiOps/PulseGenerator.h"
#include "HarmonixMetasound/Nodes/MidiClockSubdivisionTriggerNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasound::Nodes::MidiClockSubdivisionTriggerNode::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiClockSubdivisionTriggerNodeParityTest,
	"Harmonix.Metasound.Nodes.MidiClockSubdivisionTrigger.PulseGeneratorParity",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiClockSubdivisionTriggerNodeParityTest::RunTest(const FString&)
	{
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
			Metasound::Test::FNodeTestGraphBuilder::MakeSingleNodeGraph(GetClassName(), GetCurrentMajorVersion());
		TOptional<Metasound::TDataReadReference<Metasound::FTrigger>> NodeTriggerOutput =
			Generator->GetOutputReadReference<Metasound::FTrigger>(Outputs::TriggerOutputName);
		UTEST_TRUE("Got node MIDI output", NodeTriggerOutput.IsSet());

		TOptional<FMidiClockWriteRef> Clock = Generator->GetInputWriteReference<FMidiClock>(Inputs::MidiClockName);
		UTEST_TRUE("Got clock", Clock.IsSet());
		(*Clock)->SetTransportState(0, EMusicPlayerTransportState::Playing);

		Harmonix::Midi::Ops::FPulseGenerator PulseGenerator;

		// Render for a bit and expect the same output from both the node and the raw processor
		constexpr int32 NumBlocks = 1000;
		Metasound::FAudioBuffer Buffer{ Generator->OperatorSettings };
		
		for (int32 BlockIdx = 0; BlockIdx < NumBlocks; ++BlockIdx)
		{
			// Advance the clock, which will advance the play cursor in the pulse generators
			(*Clock)->PrepareBlock();
			(*Clock)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());

			// Process
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			TArray<int32> PulseGeneratorTriggers;
			PulseGenerator.Process(*Clock.GetValue(),
				[&PulseGeneratorTriggers](const Harmonix::Midi::Ops::FPulseGenerator::FPulseInfo& Pulse)
				{
					PulseGeneratorTriggers.Add(Pulse.BlockFrameIndex);
				},
				[&PulseGeneratorTriggers](const Harmonix::Midi::Ops::FPulseGenerator::FPulseInfo& Pulse)
				{
					
				});

			const TArray<int32>& NodeTriggers = (*NodeTriggerOutput)->GetTriggeredFrames();
			
			UTEST_EQUAL("Same number of triggers", (*NodeTriggerOutput)->NumTriggeredInBlock(), PulseGeneratorTriggers.Num());

			for (const int32 Frame : NodeTriggers)
			{
				UTEST_TRUE("Trigger frame was in both arrays", PulseGeneratorTriggers.Contains(Frame));
			}
		}
		
		return true;
	}
}

#endif