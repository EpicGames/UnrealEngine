// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/MidiOps/MidiNoteFilter.h"
#include "HarmonixMetasound/Nodes/MidiNoteFilterNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasound::Nodes::MidiNoteFilter::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiNoteFilterNodeParityTest,
	"Harmonix.Metasound.Nodes.MidiNoteFilter.Parity",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiNoteFilterNodeParityTest::RunTest(const FString&)
	{
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
			Metasound::Test::FNodeTestGraphBuilder::MakeSingleNodeGraph(GetClassName(), GetCurrentMajorVersion());
		TOptional<Metasound::TDataReadReference<FMidiStream>> NodeMidiOutput = Generator->GetOutputReadReference<FMidiStream>(Outputs::MidiStreamName);
		UTEST_TRUE("Got node MIDI output", NodeMidiOutput.IsSet());
		
		Harmonix::Midi::Ops::FMidiNoteFilter FilterForComparison;
		FMidiStream InputStream;
		FMidiStream OutputStream;

		FMidiStreamEvent NoteOnEvent{ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(2, 43, 45) };
		FMidiStreamEvent CCEvent{ static_cast<uint32>(0), FMidiMsg::CreateControlChange(3, 4, 5) };
		Metasound::FAudioBuffer Buffer{ Generator->OperatorSettings };

		// Default: all events pass
		{
			Generator->ApplyToInputValue<FMidiStream>(Inputs::MidiStreamName, [&NoteOnEvent, &CCEvent](FMidiStream& Stream)
			{
				Stream.AddMidiEvent(NoteOnEvent);
				Stream.AddMidiEvent(CCEvent);
			});

			InputStream.AddMidiEvent(NoteOnEvent);
			InputStream.AddMidiEvent(CCEvent);
			
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			FilterForComparison.Process(InputStream, OutputStream);

			const TArray<FMidiStreamEvent>& NodeEvents = (*NodeMidiOutput)->GetEventsInBlock();
			const TArray<FMidiStreamEvent>& FilterEvents = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Right number of events", NodeEvents.Num(), 2);
			UTEST_EQUAL("Same number of events", FilterEvents.Num(), NodeEvents.Num());
		}

		// Disabled: no events pass
		{
			// keep the same inputs from the previous block
			Generator->SetInputValue<bool>(Inputs::EnableName, false);
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
			UTEST_EQUAL("No events", (*NodeMidiOutput)->GetEventsInBlock().Num(), 0);
		}

		// Filter out the note: the CC comes through, as well as a note off because the note was previously sounding
		{
			OutputStream.PrepareBlock();
			
			Generator->SetInputValue(Inputs::EnableName, true);
			Generator->SetInputValue(Inputs::MaxVelocityName, NoteOnEvent.MidiMessage.GetStdData2() - 1);
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			FilterForComparison.MaxVelocity = NoteOnEvent.MidiMessage.GetStdData2() - 1;
			FilterForComparison.Process(InputStream, OutputStream);

			const TArray<FMidiStreamEvent>& NodeEvents = (*NodeMidiOutput)->GetEventsInBlock();
			const TArray<FMidiStreamEvent>& FilterEvents = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Right number of events", NodeEvents.Num(), 2);
			UTEST_EQUAL("Same number of events", FilterEvents.Num(), NodeEvents.Num());

			bool bGotCC = false;
			bool bGotNoteOff = false;

			for (const FMidiStreamEvent& Event : NodeEvents)
			{
				if (Event.MidiMessage.IsControlChange())
				{
					UTEST_EQUAL("CC: Right channel", Event.MidiMessage.GetStdChannel(), CCEvent.MidiMessage.GetStdChannel());
					UTEST_EQUAL("CC: Right control number", Event.MidiMessage.GetStdData1(), CCEvent.MidiMessage.GetStdData1());
					UTEST_EQUAL("CC: Right value", Event.MidiMessage.GetStdData2(), CCEvent.MidiMessage.GetStdData2());
					bGotCC = true;
				}
				else if (Event.MidiMessage.IsNoteOff())
				{
					UTEST_EQUAL("Note off: Right channel", Event.MidiMessage.GetStdChannel(), NoteOnEvent.MidiMessage.GetStdChannel());
					UTEST_EQUAL("Note off: Right note number", Event.MidiMessage.GetStdData1(), NoteOnEvent.MidiMessage.GetStdData1());
					UTEST_EQUAL("Note off: Right voice id", Event.GetVoiceId(), NoteOnEvent.GetVoiceId());
					bGotNoteOff = true;
				}
				else
				{
					UTEST_TRUE("Unexpected event", false);
				}
			}

			UTEST_TRUE("Got CC", bGotCC);
			UTEST_TRUE("Got Note off", bGotNoteOff);
		}
		
		return true;
	}
}

#endif