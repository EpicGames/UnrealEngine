// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/Nodes/MidiNoteTriggerNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiNoteTriggerNode
{
	using FGraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiNoteTriggerCreateNodeTest,
		"Harmonix.Metasound.Nodes.MidiNoteTriggerNode.Defaults",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiNoteTriggerCreateNodeTest::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = FGraphBuilder::MakeSingleNodeGraph(
			Nodes::MidiNoteTriggerNode::GetClassName(),
			Nodes::MidiNoteTriggerNode::GetCurrentMajorVersion(),
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// execute a block
		{
			TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
			Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
		}

		// Validate defaults.
		TOptional<FTriggerReadRef> OutputNoteOn = Generator->GetOutputReadReference<FTrigger>(Nodes::MidiNoteTriggerNode::Outputs::NoteOnTriggerName);
		UTEST_TRUE("Output exists", OutputNoteOn.IsSet());
		UTEST_EQUAL("Number of Note On triggers", (*OutputNoteOn)->NumTriggeredInBlock(), 0);
		UTEST_EQUAL("Note On Trigger sample", (*OutputNoteOn)->First(), -1);

		TOptional<FTriggerReadRef> OutputNoteOff = Generator->GetOutputReadReference<FTrigger>(Nodes::MidiNoteTriggerNode::Outputs::NoteOffTriggerName);
		UTEST_TRUE("Output exists", OutputNoteOff.IsSet());
		UTEST_EQUAL("Number of Note Off triggers", (*OutputNoteOff)->NumTriggeredInBlock(), 0);
		UTEST_EQUAL("Note Off Trigger sample", (*OutputNoteOff)->First(), -1);

		TOptional<FInt32ReadRef> OutputMidiNoteNum = Generator->GetOutputReadReference<int32>(Nodes::MidiNoteTriggerNode::Outputs::MidiNoteNumberName);
		UTEST_TRUE("Output exists", OutputMidiNoteNum.IsSet());
		UTEST_EQUAL("Midi Note Num check", **OutputMidiNoteNum, 0);
		
		TOptional<FInt32ReadRef> OutputVelocity = Generator->GetOutputReadReference<int32>(Nodes::MidiNoteTriggerNode::Outputs::MidiVelocityName);
		UTEST_TRUE("Output exists", OutputVelocity.IsSet());
		UTEST_EQUAL("Velocity check", **OutputVelocity, 0);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiNoteTriggerNodeTestBasic,
		"Harmonix.Metasound.Nodes.MidiNoteTriggerNode.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiNoteTriggerNodeTestBasic::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = FGraphBuilder::MakeSingleNodeGraph(
			Nodes::MidiNoteTriggerNode::GetClassName(),
			Nodes::MidiNoteTriggerNode::GetCurrentMajorVersion(),
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// Get the outputs
		TOptional<FTriggerReadRef> OutputNoteOn = Generator->GetOutputReadReference<FTrigger>(Nodes::MidiNoteTriggerNode::Outputs::NoteOnTriggerName);
		UTEST_TRUE("Output exists", OutputNoteOn.IsSet());

		TOptional<FTriggerReadRef> OutputNoteOff = Generator->GetOutputReadReference<FTrigger>(Nodes::MidiNoteTriggerNode::Outputs::NoteOffTriggerName);
		UTEST_TRUE("Output exists", OutputNoteOff.IsSet());

		TOptional<FInt32ReadRef> OutputMidiNoteNum = Generator->GetOutputReadReference<int32>(Nodes::MidiNoteTriggerNode::Outputs::MidiNoteNumberName);
		UTEST_TRUE("Output exists", OutputMidiNoteNum.IsSet());
		
		TOptional<FInt32ReadRef> OutputVelocity = Generator->GetOutputReadReference<int32>(Nodes::MidiNoteTriggerNode::Outputs::MidiVelocityName);
		UTEST_TRUE("Output exists", OutputVelocity.IsSet());

		// Send a note on and expect the right values
		constexpr uint8 Note1Channel = 3;
		constexpr uint8 Note1NoteNumber = 45;
		constexpr uint8 Note1Velocity = 79;
		{
			// Add a note on
			Generator->ApplyToInputValue<FMidiStream>(
				Nodes::MidiNoteTriggerNode::Inputs::MidiStreamName,
				[Note1Channel, Note1NoteNumber, Note1Velocity](FMidiStream& MidiStream)
				{
					MidiStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(Note1Channel, Note1NoteNumber, Note1Velocity)});
				});

			// Render a block
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Make sure we got the right stuff out the other side
			UTEST_EQUAL("Number of Note On triggers", (*OutputNoteOn)->NumTriggeredInBlock(), 1);
			UTEST_EQUAL("Note On Trigger sample", (*OutputNoteOn)->First(), 0);

			UTEST_EQUAL("Number of Note Off triggers", (*OutputNoteOff)->NumTriggeredInBlock(), 0);
			UTEST_EQUAL("Note Off Trigger sample", (*OutputNoteOff)->First(), -1);

			UTEST_EQUAL("Midi Note Num check", **OutputMidiNoteNum, Note1NoteNumber);

			UTEST_EQUAL("Velocity check", **OutputVelocity, Note1Velocity);

			// Advance the MIDI stream
			Generator->ApplyToInputValue<FMidiStream>(
				Nodes::MidiNoteTriggerNode::Inputs::MidiStreamName,
				[](FMidiStream& MidiStream)
				{
					MidiStream.PrepareBlock();
				});
		}

		// Send a note off and expect the right values
		{
			// Add a note on
			Generator->ApplyToInputValue<FMidiStream>(
				Nodes::MidiNoteTriggerNode::Inputs::MidiStreamName,
				[Note1Channel, Note1NoteNumber](FMidiStream& MidiStream)
				{
					MidiStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOff(Note1Channel, Note1NoteNumber)});
				});

			// Render a block
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Make sure we got the right stuff out the other side
			UTEST_EQUAL("Number of Note On triggers", (*OutputNoteOn)->NumTriggeredInBlock(), 0);
			UTEST_EQUAL("Note On Trigger sample", (*OutputNoteOn)->First(), -1);

			UTEST_EQUAL("Number of Note Off triggers", (*OutputNoteOff)->NumTriggeredInBlock(), 1);
			UTEST_EQUAL("Note Off Trigger sample", (*OutputNoteOff)->First(), 0);

			UTEST_EQUAL("Midi Note Num check", **OutputMidiNoteNum, Note1NoteNumber);

			UTEST_EQUAL("Velocity check", **OutputVelocity, 0);

			// Advance the MIDI stream
			Generator->ApplyToInputValue<FMidiStream>(
				Nodes::MidiNoteTriggerNode::Inputs::MidiStreamName,
				[](FMidiStream& MidiStream)
				{
					MidiStream.PrepareBlock();
				});
		}

		// Send a note on, but turn off the node, expect the same values as before
		constexpr uint8 Note2NoteNumber = 98;
		constexpr uint8 Note2Velocity = 123;
		{
			// Turn off the node
			Generator->SetInputValue<bool>(Nodes::MidiNoteTriggerNode::Inputs::EnableName, false);
			
			// Add a note on
			Generator->ApplyToInputValue<FMidiStream>(
				Nodes::MidiNoteTriggerNode::Inputs::MidiStreamName,
				[Note1Channel, Note2NoteNumber, Note2Velocity](FMidiStream& MidiStream)
				{
					MidiStream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(Note1Channel, Note2NoteNumber, Note2Velocity)});
				});

			// Render a block
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Make sure we got the right stuff out the other side
			UTEST_EQUAL("Number of Note On triggers", (*OutputNoteOn)->NumTriggeredInBlock(), 0);
			UTEST_EQUAL("Note On Trigger sample", (*OutputNoteOn)->First(), -1);

			UTEST_EQUAL("Number of Note Off triggers", (*OutputNoteOff)->NumTriggeredInBlock(), 0);
			UTEST_EQUAL("Note Off Trigger sample", (*OutputNoteOff)->First(), -1);

			UTEST_EQUAL("Midi Note Num check", **OutputMidiNoteNum, Note1NoteNumber);

			UTEST_EQUAL("Velocity check", **OutputVelocity, 0);

			// Advance the MIDI stream
			Generator->ApplyToInputValue<FMidiStream>(
				Nodes::MidiNoteTriggerNode::Inputs::MidiStreamName,
				[](FMidiStream& MidiStream)
				{
					MidiStream.PrepareBlock();
				});
		}

		return true;
	}
}

#endif
