// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMidi/MidiFile.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiAsset.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiPlayerNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	namespace NodeNames
	{
		namespace MidiPlayer
		{
			const FName Loop = "MidiPlayer Loop";
			const FName Speed = "MidiPlayer Speed";
			const FName PreRollBars = "MidiPlayer ReRoll Bars";
			const FName ClockOut = "MidiPlayer Clock (Out)";
		};

		namespace Metronome
		{
			const FName Loop = "Metronome Loop";
			const FName Speed = "Metronome Speed";
			const FName PreRollBars = "Metronome PreRoll Bars";
			const FName ClockOut = "Metronome Clock (Out)";
		};
	};

	class FBasicMidiPlayerNodeTest
	{
	public:

		struct FMidiPlayerInputs
		{
			TSharedPtr<FMidiFileData> MidiFile = nullptr;
			bool Loop = false;
			float Speed = 1.0f;
			int32 PreRollBars = 8;
		};

		struct FMetronomeInputs
		{
			float Tempo = 120.0f;
			float Speed = 1.0f;
			FTimeSignature TimeSig = { 4, 4 };
			bool Loop = false;
			int32 LoopLengthBars = 1;
			int32 PreRollBars = 8;
		};

		struct FParameters
		{
			FMidiPlayerInputs MidiPlayer;
			FMetronomeInputs Metronome;

			bool UseMetronome = false;

			int32 NumSamplesPerBlock = 256;
			float SampleRate = 48000.0f;
			int32 NumBlocks = 100;
		};

		static bool RunTest(FAutomationTestBase& InTest, FParameters Params)
		{
			GraphBuilder Builder;
			const FNodeHandle MidiPlayerNode = Builder.AddNode(
				{ HarmonixMetasound::HarmonixNodeNamespace, "MidiPlayer", "" }, 0
			);

			if (!InTest.TestTrue("MidiPlayerNode should be valid", MidiPlayerNode->IsValid()))
			{
				return false;
			}

			using namespace HarmonixMetasound;
			using namespace CommonPinNames;

			FNodeHandle InTransportNode = Builder.AddAndConnectDataReferenceInput(MidiPlayerNode, Inputs::TransportName, GetMetasoundDataTypeName<FMusicTransportEventStream>());

			Builder.AddAndConnectDataReferenceInput(MidiPlayerNode, Inputs::MidiFileAssetName, GetMetasoundDataTypeName<FMidiAsset>());

			if (Params.UseMetronome)
			{
				const FNodeHandle MetronomeNode = Builder.AddNode(
					{ HarmonixMetasound::HarmonixNodeNamespace, "Metronome", "" }, 0
						);

						if (!InTest.TestTrue("Metronome connected to MidiPlayer", Builder.ConnectNodes(MetronomeNode, Outputs::MidiClockName, MidiPlayerNode, Inputs::MidiClockName)))
						{
							return false;
						}

						if (!InTest.TestTrue("Transport connected to Metronome", Builder.ConnectNodes(InTransportNode, Outputs::TransportName, MetronomeNode, Inputs::TransportName)))
						{
							return false;
						}

						Builder.AddAndConnectDataReferenceInput(MetronomeNode, Inputs::TempoName, GetMetasoundDataTypeName<float>());
						Builder.AddAndConnectDataReferenceInput(MetronomeNode, Inputs::SpeedName, GetMetasoundDataTypeName<float>(), NodeNames::Metronome::Speed);
						Builder.AddAndConnectDataReferenceInput(MetronomeNode, Inputs::TimeSigNumeratorName, GetMetasoundDataTypeName<int32>());
						Builder.AddAndConnectDataReferenceInput(MetronomeNode, Inputs::TimeSigDenominatorName, GetMetasoundDataTypeName<int32>());

						Builder.AddAndConnectConstructorInput(MetronomeNode, Inputs::LoopName, Params.Metronome.Loop, NodeNames::Metronome::Loop);
						Builder.AddAndConnectConstructorInput(MetronomeNode, Inputs::LoopLengthBarsName, Params.Metronome.LoopLengthBars);
						Builder.AddAndConnectConstructorInput(MetronomeNode, Inputs::PrerollBarsName, Params.Metronome.PreRollBars, NodeNames::Metronome::PreRollBars);

						Builder.AddAndConnectDataReferenceOutput(MetronomeNode, Outputs::MidiClockName, GetMetasoundDataTypeName<FMidiClock>(), NodeNames::Metronome::ClockOut);
			}

			Builder.AddAndConnectConstructorInput(MidiPlayerNode, Inputs::LoopName, Params.MidiPlayer.Loop, NodeNames::MidiPlayer::Loop);
			Builder.AddAndConnectConstructorInput(MidiPlayerNode, Inputs::PrerollBarsName, Params.MidiPlayer.PreRollBars, NodeNames::MidiPlayer::PreRollBars);

			Builder.AddAndConnectDataReferenceOutput(MidiPlayerNode, Outputs::MidiClockName, GetMetasoundDataTypeName<FMidiClock>(), NodeNames::MidiPlayer::ClockOut);
			Builder.AddAndConnectDataReferenceOutput(MidiPlayerNode, Outputs::MidiStreamName, GetMetasoundDataTypeName<FMidiStream>());

			// have to make an audio output for the generator to do anything
			Builder.AddOutput("AudioOut", GetMetasoundDataTypeName<FAudioBuffer>());

			const TUniquePtr<FMetasoundGenerator> Generator = Builder.BuildGenerator(Params.SampleRate, Params.NumSamplesPerBlock);

			if (!InTest.TestTrue("Graph successfully built", Generator.IsValid()))
			{
				return false;
			}

			if (!InTest.TestTrue("Graph has audio output", Generator->GetNumChannels() > 0))
			{
				return false;
			}

			Generator->ApplyToInputValue<FMusicTransportEventStream>(Inputs::TransportName, [](FMusicTransportEventStream& Transport)
				{
					Transport.AddTransportRequest(EMusicPlayerTransportRequest::Prepare, 0);
					Transport.AddTransportRequest(EMusicPlayerTransportRequest::Play, 1);
				}
			);

			TOptional<FMidiClockReadRef> MetronomeClockOut;

			int32 MetronomeLoopLengthTicks = 0;
			if (Params.UseMetronome)
			{
				Generator->SetInputValue<int32>(Inputs::TimeSigNumeratorName, Params.Metronome.TimeSig.Numerator);
				Generator->SetInputValue<int32>(Inputs::TimeSigDenominatorName, Params.Metronome.TimeSig.Denominator);
				Generator->SetInputValue<float>(Inputs::TempoName, Params.Metronome.Tempo);
				Generator->SetInputValue<float>(NodeNames::Metronome::Speed, Params.Metronome.Speed);

				MetronomeClockOut = Generator->GetOutputReadReference<FMidiClock>(NodeNames::Metronome::ClockOut);

				TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(Params.Metronome.Tempo, Params.Metronome.TimeSig.Numerator, Params.Metronome.TimeSig.Denominator);
				MetronomeLoopLengthTicks = SongMaps->BarIncludingCountInToTick(Params.Metronome.LoopLengthBars);
			}

			if (Params.MidiPlayer.MidiFile.IsValid())
			{
				TSharedPtr<FMidiFileProxy> MidiFileProxy = MakeShared<FMidiFileProxy>(Params.MidiPlayer.MidiFile);
				TSharedPtr<FMidiAsset> MidiAsset = MakeShared<FMidiAsset>(MidiFileProxy);

				Generator->SetInputValue<FMidiAsset>(Inputs::MidiFileAssetName, *MidiAsset);
			}

			TOptional<FMidiClockReadRef> MidiPlayerClockOut = Generator->GetOutputReadReference<FMidiClock>(NodeNames::MidiPlayer::ClockOut);


			float DefaultTicksPerSec = 120.0f * Harmonix::Midi::Constants::GTicksPerQuarterNote / 60.0f;
			float DefaultTicksPerMs = DefaultTicksPerSec / 1000.0f;

			// do some math to figure out how fast the clock should be advancing...
			float Tempo = Params.MidiPlayer.MidiFile ? Params.MidiPlayer.MidiFile->SongMaps.GetTempoAtTick(0) : 120.0f;
			float TicksPerSec = Tempo * Harmonix::Midi::Constants::GTicksPerQuarterNote / 60.0f;
			float TicksPerMs = TicksPerSec / 1000.0f;
			float SecsPerBlock = Params.NumSamplesPerBlock / Params.SampleRate;
			float TicksPerBlock = TicksPerSec * SecsPerBlock;

			const FSongLengthData& MidiFileLength = Params.MidiPlayer.MidiFile->SongMaps.GetSongLengthData();

			int32 MidiPlayerLoopLengthTicks = MidiFileLength.LengthTicks;

			bool MetronomeAllTicksEqual = true;
			bool MidiPlayerAllTicksEqual = true;
			for (int32 BlockIndex = 0; BlockIndex < Params.NumBlocks; ++BlockIndex)
			{
				TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), Params.NumSamplesPerBlock, EAudioBufferCleanupMode::Delete };
				Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());

				int32 ExpectedTick = FMath::RoundToInt32(TicksPerBlock * (BlockIndex + 1));
				int32 MidiPlayerExpectedTick = ExpectedTick;
				int32 MetronomeExpectedTick = ExpectedTick;

				if (Params.UseMetronome && Params.Metronome.Loop)
				{
					MetronomeExpectedTick %= MetronomeLoopLengthTicks;
					MidiPlayerExpectedTick = MetronomeExpectedTick;
				}

				if (Params.MidiPlayer.Loop)
				{
					MidiPlayerExpectedTick %= MidiPlayerLoopLengthTicks;
				}

				if (!InTest.TestEqual("Midi Player Looping", (*MidiPlayerClockOut)->HasPersistentLoop(), Params.MidiPlayer.Loop))
				{
					return false;
				}

				if (MetronomeClockOut)
				{
					if (!InTest.TestEqual("Metronome Looping", (*MetronomeClockOut)->HasPersistentLoop(), Params.Metronome.Loop))
					{
						return false;
					}

					int32 MetronomeActualTick = (*MetronomeClockOut)->GetNextMidiTickToProcess();
					MetronomeExpectedTick = FMath::Abs(MidiPlayerExpectedTick - MetronomeActualTick) <= 1 ? MetronomeActualTick : MetronomeExpectedTick;
					if (MetronomeAllTicksEqual && (MetronomeActualTick != MetronomeExpectedTick))
					{
						MetronomeAllTicksEqual = false;
						FString What = FString::Printf(TEXT("Metronome Looping: Metronome Out Clock, Tick at block: %d"), BlockIndex);
						InTest.AddError(FString::Printf(TEXT("%s: Expected Tick to be: %d, but was %d"), *What, MetronomeExpectedTick, MetronomeActualTick));
					}
				}

				int32 MidiPlayerActualTick = (*MidiPlayerClockOut)->GetNextMidiTickToProcess();
				MidiPlayerExpectedTick = FMath::Abs(MidiPlayerExpectedTick - MidiPlayerActualTick) <= 1 ? MidiPlayerActualTick : MidiPlayerExpectedTick;

				if (MidiPlayerAllTicksEqual && (MidiPlayerActualTick != MidiPlayerExpectedTick))
				{
					MidiPlayerAllTicksEqual = false;
					FString What = FString::Printf(TEXT("Metronome Looping: Midi Player Out Clock, Tick at block: %d"), BlockIndex);
					InTest.AddError(FString::Printf(TEXT("%s: Expected Tick to be: %d, but was %d"), *What, MidiPlayerExpectedTick, MidiPlayerActualTick));
				}

				Generator->ApplyToInputValue<FMusicTransportEventStream>(Inputs::TransportName, [](FMusicTransportEventStream& Transport)
					{
						Transport.Reset();
					}
				);
			}

			return true;
		}

		static TSharedPtr<FMidiFileData> MakeTestMidiData(float InTempoBpm, int32 InTimeSigNum, int32 InTimeSigDen, int32 InLengthBars)
		{
			TSharedPtr<FMidiFileData> OutMidiData = MakeShared<FMidiFileData>();

			// clear it all out for good measure...
			OutMidiData->SongMaps.EmptyAllMaps();
			OutMidiData->Tracks.Empty();

			// create conductor track
			FMidiTrack& Track = OutMidiData->Tracks.Add_GetRef(FMidiTrack(TEXT("conductor")));

			// add time sig info
			int32 TimeSigNum = FMath::Clamp(InTimeSigNum, 1, 64);
			int32 TimeSigDen = FMath::Clamp(InTimeSigDen, 1, 64);
			Track.AddEvent(FMidiEvent(0, FMidiMsg((uint8)TimeSigNum, (uint8)TimeSigDen)));
			OutMidiData->SongMaps.AddTimeSignatureAtBarIncludingCountIn(0, TimeSigNum, TimeSigDen);

			// add tempo info
			float TempoBpm = FMath::Max(1.0f, InTempoBpm);
			int32 MidiTempo = Harmonix::Midi::Constants::BPMToMidiTempo(TempoBpm);
			Track.AddEvent(FMidiEvent(0, FMidiMsg(MidiTempo)));
			OutMidiData->SongMaps.AddTempoInfoPoint(MidiTempo, 0);

			Track.Sort();
			OutMidiData->SongMaps.SetLengthTotalBars(InLengthBars);

			return OutMidiData;
		}
	};

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiPlayerCreateNodeTest,
		"Harmonix.Metasound.Nodes.MidiPlayerNode.CreateNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiPlayerCreateNodeTest::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "MidiPlayer", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// execute a block
		{
			TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
			Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
		}
	
		
		Generator->SetInputValue<bool>(CommonPinNames::Inputs::LoopName, false);

		// Validate output.
		TOptional<FMidiClockReadRef> OutputMidiClock = Generator->GetOutputReadReference<FMidiClock>(CommonPinNames::Outputs::MidiClockName);
		UTEST_TRUE("Output exists", OutputMidiClock.IsSet());
		UTEST_EQUAL("Current Midi Tick Test", (*OutputMidiClock)->GetLastProcessedMidiTick(), -1);
		UTEST_EQUAL("Next Midi Tick Test", (*OutputMidiClock)->GetNextMidiTickToProcess(), 0);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiPlayerBasicTest,
		"Harmonix.Metasound.Nodes.MidiPlayerNode.BasicTest",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiPlayerBasicTest::RunTest(const FString&)
	{
		FBasicMidiPlayerNodeTest::FParameters Params;
		Params.MidiPlayer.MidiFile = FBasicMidiPlayerNodeTest::MakeTestMidiData(120.0f, 4, 4, 1);
		return FBasicMidiPlayerNodeTest::RunTest(*this, Params);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiPlayerExternallyClocked,
		"Harmonix.Metasound.Nodes.MidiPlayerNode.ExternallyClocked",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiPlayerExternallyClocked::RunTest(const FString&)
	{
		FBasicMidiPlayerNodeTest::FParameters Params;
		Params.UseMetronome = true;
		Params.MidiPlayer.MidiFile = FBasicMidiPlayerNodeTest::MakeTestMidiData(120.0f, 4, 4, 1);
		return FBasicMidiPlayerNodeTest::RunTest(*this, Params);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiPlayerExternallyClockedLoopingMetronome,
		"Harmonix.Metasound.Nodes.MidiPlayerNode.ExternallyClocked.LoopingMetronome",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiPlayerExternallyClockedLoopingMetronome::RunTest(const FString&)
	{
		FBasicMidiPlayerNodeTest::FParameters Params;
		Params.NumBlocks = 500;
		Params.UseMetronome = true;
		Params.Metronome.Loop = true;
		Params.MidiPlayer.MidiFile = FBasicMidiPlayerNodeTest::MakeTestMidiData(120.0f, 4, 4, 1);
		return FBasicMidiPlayerNodeTest::RunTest(*this, Params);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiPlayerExternallyClockedLoopingMidiPlayer,
		"Harmonix.Metasound.Nodes.MidiPlayerNode.ExternallyClocked.LoopingMidiPlayer",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiPlayerExternallyClockedLoopingMidiPlayer::RunTest(const FString&)
	{
		FBasicMidiPlayerNodeTest::FParameters Params;
		Params.NumBlocks = 500;
		Params.UseMetronome = true;
		Params.MidiPlayer.Loop = true;
		Params.Metronome.LoopLengthBars = 2;
		Params.MidiPlayer.MidiFile = FBasicMidiPlayerNodeTest::MakeTestMidiData(120.0f, 4, 4, 1);
		return FBasicMidiPlayerNodeTest::RunTest(*this, Params);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiPlayerExternallyClockedBothLooping,
		"Harmonix.Metasound.Nodes.MidiPlayerNode.ExternallyClocked.BothLooping",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiPlayerExternallyClockedBothLooping::RunTest(const FString&)
	{
		FBasicMidiPlayerNodeTest::FParameters Params;
		Params.NumBlocks = 1000;
		Params.UseMetronome = true;
		Params.Metronome.Loop = true;
		Params.MidiPlayer.Loop = true;
		Params.MidiPlayer.MidiFile = FBasicMidiPlayerNodeTest::MakeTestMidiData(120.0f, 4, 4, 1);
		return FBasicMidiPlayerNodeTest::RunTest(*this, Params);
	}
}

#endif
