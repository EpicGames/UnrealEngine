// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/MidiOps/PulseGenerator.h"
#include "HarmonixMetasound/Nodes/MidiPulseGeneratorNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasound::Nodes::MidiPulseGeneratorNode::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiPulseGeneratorNodeParityTest,
	"Harmonix.Metasound.Nodes.MidiPulseGenerator.Parity",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiPulseGeneratorNodeParityTest::RunTest(const FString&)
	{
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
			Metasound::Test::FNodeTestGraphBuilder::MakeSingleNodeGraph(GetClassName(), GetCurrentMajorVersion());
		TOptional<Metasound::TDataReadReference<FMidiStream>> NodeMidiOutput = Generator->GetOutputReadReference<FMidiStream>(Outputs::MidiStreamName);
		UTEST_TRUE("Got node MIDI output", NodeMidiOutput.IsSet());

		TOptional<FMidiClockWriteRef> Clock = Generator->GetInputWriteReference<FMidiClock>(Inputs::MidiClockName);
		UTEST_TRUE("Got clock", Clock.IsSet());
		(*Clock)->SetTransportState(0, EMusicPlayerTransportState::Playing);

		Harmonix::Midi::Ops::FMidiPulseGenerator PulseGenerator;
		FMidiStream PulseGeneratorMidiOutput;

		// Render for a bit and expect the same output from both the node and the raw processor
		constexpr int32 NumBlocks = 1000;
		Metasound::FAudioBuffer Buffer{ Generator->OperatorSettings };
		
		for (int32 BlockIdx = 0; BlockIdx < NumBlocks; ++BlockIdx)
		{
			// Advance the clock, which will advance the play cursor in the pulse generators
			(*Clock)->PrepareBlock();
			(*Clock)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());

			// Process
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
			PulseGeneratorMidiOutput.PrepareBlock();
			PulseGenerator.Process(*Clock.GetValue(), PulseGeneratorMidiOutput);

			// If there are notes in the pulse generator output, expect them in the node output
			const TArray<FMidiStreamEvent> NodeEvents = (*NodeMidiOutput)->GetEventsInBlock();
			const TArray<FMidiStreamEvent> PulseGeneratorEvents = PulseGeneratorMidiOutput.GetEventsInBlock();

			for (const FMidiStreamEvent& PulseGeneratorEvent : PulseGeneratorEvents)
			{
				UTEST_TRUE("Event is in node output", NodeEvents.ContainsByPredicate([&PulseGeneratorEvent](const FMidiStreamEvent& NodeEvent)
				{
					return NodeEvent.BlockSampleFrameIndex == PulseGeneratorEvent.BlockSampleFrameIndex
					&& NodeEvent.AuthoredMidiTick == PulseGeneratorEvent.AuthoredMidiTick
					&& NodeEvent.CurrentMidiTick == PulseGeneratorEvent.CurrentMidiTick
					&& NodeEvent.TrackIndex == PulseGeneratorEvent.TrackIndex
					&& NodeEvent.MidiMessage == PulseGeneratorEvent.MidiMessage;
				}));
			}
		}
		
		return true;
	}
}

#endif