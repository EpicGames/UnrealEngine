// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiStreamLoggerNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiStreamLoggerCreateNodeTest,
		"Harmonix.Metasound.Nodes.MidiStreamLoggerNode.CreateNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiStreamLoggerCreateNodeTest::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "MidiStreamLogger", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// execute a block
		{
			TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
			Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
		}

		// No output to validate, finish test

		return true;
	}
}

#endif
