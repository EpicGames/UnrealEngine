// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiStreamMergeNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiStreamMergeCreateNodeTest,
		"Harmonix.Metasound.Nodes.MidiStreamMergeNode.Merge",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiStreamMergeCreateNodeTest::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "MidiStreamMerge", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// add some messages to the inputs
		Generator->ApplyToInputValue<FMidiStream>("Midi Stream A", [](FMidiStream& Stream)
		{
			const FMidiStreamEvent Event
			{
				uint32{ 0 },
				FMidiMsg::CreateNoteOn(2, 34, 56)
			};
			Stream.AddMidiEvent(Event);
		});
		Generator->ApplyToInputValue<FMidiStream>("Midi Stream B", [](FMidiStream& Stream)
		{
			const FMidiStreamEvent Event
			{
				uint32{ 0 },
				FMidiMsg::CreateNoteOff(1, 23)
			};
			Stream.AddMidiEvent(Event);
		});
		
		// execute a block
		{
			FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
		}

		// expect to see those messages at the output
		TOptional<TDataReadReference<FMidiStream>> OutputStream = Generator->GetOutputReadReference<FMidiStream>("Midi Stream");
		UTEST_TRUE("Got output ref", OutputStream.IsSet());
		const TArray<FMidiStreamEvent> Events = (*OutputStream)->GetEventsInBlock();
		UTEST_EQUAL("Got the right number of events", Events.Num(), 2);

		for (const FMidiStreamEvent& Event : Events)
		{
			if (Event.MidiMessage.IsNoteOn())
			{
				UTEST_EQUAL("Right channel", Event.MidiMessage.GetStdChannel(), 2);
				UTEST_EQUAL("Right note number", Event.MidiMessage.GetStdData1(), 34);
				UTEST_EQUAL("Right velocity", Event.MidiMessage.GetStdData2(), 56);
			}
			else if (Event.MidiMessage.IsNoteOff())
			{
				UTEST_EQUAL("Right channel", Event.MidiMessage.GetStdChannel(), 1);
				UTEST_EQUAL("Right note number", Event.MidiMessage.GetStdData1(), 23);
			}
			else
			{
				UTEST_TRUE("Wrong message", false);
			}
		}

		return true;
	}
}

#endif
