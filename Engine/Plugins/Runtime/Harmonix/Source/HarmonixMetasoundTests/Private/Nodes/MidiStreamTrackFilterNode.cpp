// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/MidiOps/MidiTrackFilter.h"
#include "HarmonixMetasound/Nodes/MidiStreamTrackFilterNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasound::Nodes::MidiTrackFilter::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiTrackFilterNodeParityTest,
	"Harmonix.Metasound.Nodes.MidiTrackFilter.Parity",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiTrackFilterNodeParityTest::RunTest(const FString&)
	{
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator =
			Metasound::Test::FNodeTestGraphBuilder::MakeSingleNodeGraph(GetClassName(), GetCurrentMajorVersion());
		UTEST_TRUE("Built graph", Generator.IsValid());
		TOptional<Metasound::TDataReadReference<FMidiStream>> NodeMidiOutput = Generator->GetOutputReadReference<FMidiStream>(Outputs::MidiStreamName);
		UTEST_TRUE("Got node MIDI output", NodeMidiOutput.IsSet());
		
		Harmonix::Midi::Ops::FMidiTrackFilter FilterForComparison;
		FMidiStream InputStream;
		FMidiStream OutputStream;

		FMidiStreamEvent NoteOnEvent{ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(2, 43, 45) };
		NoteOnEvent.TrackIndex = 3;
		
		// Default: no tracks included, no events output
		{
			// Add an event to the node input
			Generator->ApplyToInputValue<FMidiStream>(Inputs::MidiStreamName, [&NoteOnEvent](FMidiStream& Stream)
			{
				Stream.AddMidiEvent(NoteOnEvent);
			});

			// Process the node
			Metasound::FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Process the filter
			InputStream.AddMidiEvent(NoteOnEvent);
			FilterForComparison.Process(InputStream, OutputStream);

			// Make sure we got no events
			UTEST_EQUAL("Got no events (test)", (*NodeMidiOutput)->GetEventsInBlock().Num(), 0);
			UTEST_EQUAL("Got no events (control)", OutputStream.GetEventsInBlock().Num(), 0);

			// Leave the inputs where they are
		}

		// Include the track for the note on and expect the message to come out the other end
		{
			Generator->SetInputValue<int32>(Inputs::MinTrackIndexName, NoteOnEvent.TrackIndex);
			Generator->SetInputValue<int32>(Inputs::MaxTrackIndexName, NoteOnEvent.TrackIndex);
			FilterForComparison.SetTrackRange(NoteOnEvent.TrackIndex, NoteOnEvent.TrackIndex, false);

			// Process the node
			Metasound::FAudioBuffer Buffer{ Generator->OperatorSettings };
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// Process the filter
			FilterForComparison.Process(InputStream, OutputStream);

			const TArray<FMidiStreamEvent>& NodeEvents = (*NodeMidiOutput)->GetEventsInBlock();
			UTEST_EQUAL("Got one event (test)", NodeEvents.Num(), 1);
			UTEST_TRUE("Event is note on (test)", NodeEvents[0].MidiMessage.IsNoteOn());
			const TArray<FMidiStreamEvent>& ControlEvents = OutputStream.GetEventsInBlock();
			UTEST_EQUAL("Got one event (control)", ControlEvents.Num(), NodeEvents.Num());
			UTEST_TRUE("Event is note on (control)", ControlEvents[0].MidiMessage.IsNoteOn());

			UTEST_EQUAL("Same channel", NodeEvents[0].MidiMessage.GetStdChannel(), ControlEvents[0].MidiMessage.GetStdChannel());
			UTEST_EQUAL("Same note number", NodeEvents[0].MidiMessage.GetStdData1(), ControlEvents[0].MidiMessage.GetStdData1());
			UTEST_EQUAL("Same velocity", NodeEvents[0].MidiMessage.GetStdData2(), ControlEvents[0].MidiMessage.GetStdData2());
		}
		
		return true;
	}
}

#endif