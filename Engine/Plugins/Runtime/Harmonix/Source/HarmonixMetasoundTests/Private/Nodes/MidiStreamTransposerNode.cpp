// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/Nodes/MidiStreamTransposerNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiNoteTransposeNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiStreamTransposerCreateNodeTest,
		"Harmonix.Metasound.Nodes.MidiNoteTransposeNode.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiStreamTransposerCreateNodeTest::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			Nodes::MidiNoteTranspose::GetClassName(),
			Nodes::MidiNoteTranspose::GetCurrentMajorVersion(),
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		TOptional<TDataReadReference<FMidiStream>> NodeMidiOutput =
			Generator->GetOutputReadReference<FMidiStream>(Nodes::MidiNoteTranspose::Outputs::MidiStreamName);
		UTEST_TRUE("Got node MIDI output", NodeMidiOutput.IsSet());
		
		FAudioBuffer Buffer{ Generator->OperatorSettings };
		
		const TArray<uint8> NoteNumbers{ 3, 23, 57, 87, 99, 120 };

		// Default: no transposition
		{
			// Add the notes
			Generator->ApplyToInputValue<FMidiStream>(
				Nodes::MidiNoteTranspose::Inputs::MidiStreamName,
				[&NoteNumbers](FMidiStream& Stream)
				{
					for (const uint8 NoteNumber : NoteNumbers)
					{
						Stream.AddMidiEvent({ static_cast<uint32>(0), FMidiMsg::CreateNoteOn(7, NoteNumber, 127) });
					}
				});

			// Process
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// The notes should be unchanged in the output
			const TArray<FMidiStreamEvent>& Events = (*NodeMidiOutput)->GetEventsInBlock();
			UTEST_EQUAL("Got the right number of events", Events.Num(), NoteNumbers.Num());
			const TArray<uint8> ExpectedNoteNumbers = NoteNumbers;
			
			for (const FMidiStreamEvent& Event : Events)
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					UTEST_TRUE("Note number unchanged", ExpectedNoteNumbers.Contains(Event.MidiMessage.GetStdData1()));
				}
				else
				{
					UTEST_TRUE("Unexpected event", false);
				}
			}
		}

		// Try some different transposition values
		const TArray<int32> TranspositionAmounts{ -23, -3, 0, 4, 12, 40, 1000 };
		for (const int32 TranspositionAmount : TranspositionAmounts)
		{
			// Set transposition
			Generator->SetInputValue<int32>(Nodes::MidiNoteTranspose::Inputs::TranspositionName, TranspositionAmount);
			// Process
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			// The notes should be transposed, but clamped to MIDI note range (0-127)
			const TArray<FMidiStreamEvent>& Events = (*NodeMidiOutput)->GetEventsInBlock();
			UTEST_EQUAL("Got the right number of events", Events.Num(), NoteNumbers.Num());
			TArray<uint8> ExpectedNoteNumbers;
			for (const uint8 OriginalNoteNumber : NoteNumbers)
			{
				ExpectedNoteNumbers.Add(FMath::Clamp(OriginalNoteNumber + TranspositionAmount, 0, 127));
			}
			
			for (const FMidiStreamEvent& Event : Events)
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					UTEST_TRUE("Note number unchanged", ExpectedNoteNumbers.Contains(Event.MidiMessage.GetStdData1()));
				}
				else
				{
					UTEST_TRUE("Unexpected event", false);
				}
			}
		}

		return true;
	}
}

#endif
