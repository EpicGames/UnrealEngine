// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiAsset.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiTextTriggerNode
{
	using namespace Metasound::Test;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	UMidiFile* BuildMidiFile()
	{
		// Make a midi file...
		UMidiFile* TheMidi = NewObject<UMidiFile>();
		// Set the initial time signature
		TheMidi->GetSongMaps()->AddTimeSignatureAtBarIncludingCountIn(0, 4, 4);
		// Set the initial tempo...
		TheMidi->GetSongMaps()->AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(120),0);
		// Make a track to hold some text events. This will be track 1 (track 0 is always the conductor track)...
		FMidiTrack* TextTrack = TheMidi->AddTrack("TextEventTest");
		// Make a text event...
		uint16 TextIndex = TextTrack->AddText("ThisIsText");
		FMidiEvent ATextEvent(0, FMidiMsg::CreateText(TextIndex, Harmonix::Midi::Constants::GMeta_Text));
		TextTrack->AddEvent(ATextEvent);
		// Make another text event...
		TextIndex = TextTrack->AddText("ThisIsAnotherText");
		FMidiEvent ASecondTextEvent(TheMidi->GetSongMaps()->MsToTick(15.0f), FMidiMsg::CreateText(TextIndex, Harmonix::Midi::Constants::GMeta_Text));
		TextTrack->AddEvent(ASecondTextEvent);
		// Tell the midi file its tracks have been changed so it can recalculate song length data...
		TheMidi->ScanTracksForSongLengthChange();
		// Now round appropriately...
		return TheMidi;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiTextTriggerCreateNodeTest,
		"Harmonix.Metasound.Nodes.MidiTextTriggerNode.TestFunctionality",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiTextTriggerCreateNodeTest::RunTest(const FString&)
	{
		//************************************************************************************************************************
		// Build the graph.
		//************************************************************************************************************************
		FNodeTestGraphBuilder Builder;
		Builder.AddOutput("AudioOut", GetMetasoundDataTypeName<FAudioBuffer>());

		// transport...
		Frontend::FNodeHandle TransportNode = Builder.AddNode({ HarmonixMetasound::HarmonixNodeNamespace, "TriggerToTransport", "" }, 0);
		UTEST_TRUE("Transport Node Created", TransportNode.Get().IsValid());
		// hoist up play trigger input...
		Builder.AddAndConnectDataReferenceInput(TransportNode, CommonPinNames::Inputs::TransportPlayName, GetMetasoundDataTypeName<FTrigger>());

		// midi player...
		Frontend::FNodeHandle MidiPlayerNode = Builder.AddNode({ HarmonixMetasound::HarmonixNodeNamespace, "MidiPlayer", "" }, 0);
		UTEST_TRUE("MidiPlayer Node Created", MidiPlayerNode.Get().IsValid());
		// hoist up midi asset input...
		Builder.AddAndConnectDataReferenceInput(MidiPlayerNode, CommonPinNames::Inputs::MidiFileAssetName, GetMetasoundDataTypeName<FMidiAsset>());

		// wire transport to midi player...
		bool ConnectionSuccess = Builder.ConnectNodes(TransportNode, CommonPinNames::Outputs::TransportName, MidiPlayerNode, CommonPinNames::Inputs::TransportName);
		UTEST_TRUE("Transport Out connected to MidiPlayer node", ConnectionSuccess);

		// Midi text trigger node...
		FNodeHandle TextTriggerNode = Builder.AddNode({ HarmonixMetasound::HarmonixNodeNamespace, "MidiTextTrigger", "" }, 1);
		UTEST_TRUE("MidiTextTrigger Node Created", TextTriggerNode.Get().IsValid());
		// hoist up text to match input...
		Builder.AddAndConnectDataReferenceInput(TextTriggerNode, "Text", GetMetasoundDataTypeName<FString>());
		// hoist up output trigger...
		Builder.AddAndConnectDataReferenceOutput(TextTriggerNode, "Trigger Out", GetMetasoundDataTypeName<FTrigger>(), "TextTiggerOut");

		// wire midi player to text trigger...
		ConnectionSuccess = Builder.ConnectNodes(MidiPlayerNode, CommonPinNames::Outputs::MidiStreamName, TextTriggerNode, CommonPinNames::Inputs::MidiStreamName);
		UTEST_TRUE("MidiPlayer Out connected to MidiTextTrigger node", ConnectionSuccess);

		//************************************************************************************************************************
		// Make the generator.
		//************************************************************************************************************************
		constexpr int32 NumSamplesPerBlock = 480; // 10ms per block
		const TUniquePtr<FMetasoundGenerator> Generator = Builder.BuildGenerator(48000, NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		//************************************************************************************************************************
		// Get reference to outputs we want to check.
		//************************************************************************************************************************
		TOptional<FTriggerReadRef> OutputTextTrigger = Generator->GetOutputReadReference<FTrigger>("TextTiggerOut");
		UTEST_TRUE("Output exists", OutputTextTrigger.IsSet());
		UTEST_EQUAL("Number of Text triggers at start", (*OutputTextTrigger)->NumTriggeredInBlock(), 0);
		UTEST_EQUAL("Text Trigger sample", (*OutputTextTrigger)->First(), -1);

		//************************************************************************************************************************
		// Make initial inputs.
		//************************************************************************************************************************
		// Midi file to play...
		UMidiFile* MidiFile = BuildMidiFile();
		TSharedPtr<Audio::IProxyData> MidiFileProxy = MidiFile->CreateProxyData({ "MidiTextTriggerTest" });
		Generator->SetInputValue<FMidiAsset>(CommonPinNames::Inputs::MidiFileAssetName, MidiFileProxy);

		// Text to look for...
		Generator->SetInputValue<FString>("Text", "ThisIsText");

		// Trigger transport...
		//Generator->ApplyToInputValue<FTrigger>("OnPlay", [](FTrigger& TriggerIn){ TriggerIn.TriggerFrame(0); });
		Generator->ApplyToInputValue<FTrigger>(CommonPinNames::Inputs::TransportPlayName, [](FTrigger& TriggerIn){ TriggerIn.TriggerFrame(0); });
		
		//************************************************************************************************************************
		// Generate 10ms...
		//************************************************************************************************************************
		TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
		Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
		UTEST_EQUAL("Number of Text triggers after first block", (*OutputTextTrigger)->NumTriggeredInBlock(), 1);
		UTEST_EQUAL("Text Trigger sample", (*OutputTextTrigger)->First(), 0);

		//************************************************************************************************************************
		// Generate 10ms...
		//************************************************************************************************************************
		// Text to look for...
		Generator->SetInputValue<FString>("Text", "ThisIsAnotherText");
		Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
		UTEST_EQUAL("Number of Text triggers after second block", (*OutputTextTrigger)->NumTriggeredInBlock(), 1);

		return true;
	}
}

#endif
