// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGenerator.h"
#include "NodeTestGraphBuilder.h"

#include "HarmonixDsp/Modulators/MorphingLfo.h"

#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/DataTypes/TimeSyncOption.h"
#include "HarmonixMetasound/Nodes/MorphingLfoNode.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MorphingLFONode
{
	BEGIN_DEFINE_SPEC(
		FHarmonixMetasoundMorphingLfoNodeSpec,
		"Harmonix.Metasound.Nodes.MorphingLFONode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	Harmonix::Dsp::Modulators::FMorphingLFO LFOForComparison{ 0 };
	TUniquePtr<Metasound::FMetasoundGenerator> Generator;

	template<typename OutputDataType>
	void BuildGenerator()
	{
		using FBuilder = Metasound::Test::FNodeTestGraphBuilder;
		FBuilder Builder;
		Generator = Builder.MakeSingleNodeGraph(HarmonixMetasound::Nodes::MorphingLFO::GetClassName<OutputDataType>(), 0);
	}

	void SetParams(
		ETimeSyncOption SyncType,
		float Frequency,
		float Shape,
		bool Invert)
	{
		LFOForComparison.SyncType = SyncType;
		LFOForComparison.Frequency = Frequency;
		LFOForComparison.Shape = Shape;
		LFOForComparison.Invert = Invert;

		using namespace HarmonixMetasound::Nodes::MorphingLFO::Inputs;
		Generator->SetInputValue(LFOSyncTypeName, Metasound::FEnumTimeSyncOption{ SyncType });
		Generator->SetInputValue(LFOFrequencyName, Frequency);
		Generator->SetInputValue(LFOShapeName, Shape);
		Generator->SetInputValue(LFOInvertName, Invert);
	}

	Metasound::FSampleCount SampleCount = 0;
	Metasound::FSampleCount SampleRemainder = 0;
	
	bool ResetAndStartClock(float Tempo, float Speed, int32 TimeSigNumerator, int32 TimeSigDenominator)
	{
		TOptional<HarmonixMetasound::FMidiClockWriteRef> ClockInput =
			Generator->GetInputWriteReference<HarmonixMetasound::FMidiClock>(HarmonixMetasound::Nodes::MorphingLFO::Inputs::MidiClockName);

		if (!TestTrue("Got clock", ClockInput.IsSet()))
		{
			return false;
		}
		
		const TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(Tempo, TimeSigNumerator, TimeSigDenominator);
		check(SongMaps);

		SongMaps->SetSongLengthTicks(std::numeric_limits<int32>::max());

		(*ClockInput)->AttachToSongMapEvaluator(SongMaps);
		(*ClockInput)->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
		(*ClockInput)->SetSpeed(0, Speed);

		SampleCount = 0;
		SampleRemainder = 0;

		return true;
	}
	
	bool AdvanceClock()
	{
		using namespace Metasound;
		TOptional<HarmonixMetasound::FMidiClockWriteRef> ClockInput =
			Generator->GetInputWriteReference<HarmonixMetasound::FMidiClock>(HarmonixMetasound::Nodes::MorphingLFO::Inputs::MidiClockName);

		if (!TestTrue("Got clock", ClockInput.IsSet()))
		{
			return false;
		}
		
		(*ClockInput)->PrepareBlock();

		const int32 NumSamples = Generator->OperatorSettings.GetNumFramesPerBlock();
		int32 BlockFrameIndex = SampleRemainder;
		SampleRemainder += NumSamples;
		constexpr int32 MidiGranularity = 128;
		while (SampleRemainder >= MidiGranularity)
		{
			SampleCount += MidiGranularity;
			SampleRemainder -= MidiGranularity;
			const float AdvanceToMs = static_cast<float>(SampleCount) * 1000.0f / Generator->OperatorSettings.GetSampleRate();
			(*ClockInput)->AdvanceToMs(BlockFrameIndex, AdvanceToMs);
			BlockFrameIndex += MidiGranularity;
		}

		return true;
	}

	FMusicTimestamp GetClockTimestampAtBlockOffset(int32 Offset)
	{
		TOptional<HarmonixMetasound::FMidiClockWriteRef> ClockInput =
			Generator->GetInputWriteReference<HarmonixMetasound::FMidiClock>(HarmonixMetasound::Nodes::MorphingLFO::Inputs::MidiClockName);

		if (!TestTrue("Got clock", ClockInput.IsSet()))
		{
			return {};
		}

		return (*ClockInput)->GetMusicTimestampAtBlockOffset(Offset);
	}

	bool RenderAndCompareFloat(
		ETimeSyncOption SyncType,
		float Frequency,
		float Shape,
		bool Invert,
		Harmonix::Dsp::Modulators::FMorphingLFO::FMusicTimingInfo* MusicTimingInfo = nullptr)
	{
		SetParams(SyncType, Frequency, Shape, Invert);

		// reset the clock if appropriate
		if (nullptr != MusicTimingInfo)
		{
			const bool Success = ResetAndStartClock(
				MusicTimingInfo->Tempo,
				MusicTimingInfo->Speed,
				MusicTimingInfo->TimeSignature.Numerator,
				MusicTimingInfo->TimeSignature.Denominator);

			if (!TestTrue("Reset clock", Success))
			{
				return false;
			}
		}

		constexpr int32 NumBlocks = 10;

		for (int32 i = 0; i < NumBlocks; ++i)
		{
			// Advance the clock
			if (nullptr != MusicTimingInfo && !TestTrue("Advance clock", AdvanceClock()))
			{
				return false;
			}

			// Render a block
			{
				TArray<float> Buffer;
				Buffer.SetNumUninitialized(Generator->OperatorSettings.GetNumFramesPerBlock());
				Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
			}
			
			// Advance the comparison LFO
			float ExpectedOutput = -1;

			if (nullptr != MusicTimingInfo)
			{
				MusicTimingInfo->Timestamp = GetClockTimestampAtBlockOffset(0);
			}
			
			LFOForComparison.Advance(Generator->OperatorSettings.GetNumFramesPerBlock(), ExpectedOutput, MusicTimingInfo);

			// Check the output
			{
				TOptional<Metasound::TDataReadReference<float>> Output =
					Generator->GetOutputReadReference<float>(HarmonixMetasound::Nodes::MorphingLFO::Outputs::LFOName);

				if (!TestTrue("Got output", Output.IsSet()))
				{
					return false;
				}

				const float ActualValue = *(*Output);
				
				if (!TestEqual(
					FString::Printf(TEXT("Output matches (block %d)"), i),
					ActualValue,
					ExpectedOutput))
				{
					return false;
				}
			}
		}

		return true;
	}

	bool RenderAndCompareBuffer(
		ETimeSyncOption SyncType,
		float Frequency,
		float Shape,
		bool Invert,
		Harmonix::Dsp::Modulators::FMorphingLFO::FMusicTimingInfo* MusicTimingInfo = nullptr)
	{
		SetParams(SyncType, Frequency, Shape, Invert);

		// reset the clock if appropriate
		if (nullptr != MusicTimingInfo)
		{
			const bool Success = ResetAndStartClock(
				MusicTimingInfo->Tempo,
				MusicTimingInfo->Speed,
				MusicTimingInfo->TimeSignature.Numerator,
				MusicTimingInfo->TimeSignature.Denominator);

			if (!TestTrue("Reset clock", Success))
			{
				return false;
			}
		}

		constexpr int32 NumBlocks = 10;

		for (int32 BlockIdx = 0; BlockIdx < NumBlocks; ++BlockIdx)
		{
			// Advance the clock
			if (nullptr != MusicTimingInfo && !TestTrue("Advance clock", AdvanceClock()))
			{
				return false;
			}

			// Render a block
			TArray<float> Buffer;
			Buffer.SetNumUninitialized(Generator->OperatorSettings.GetNumFramesPerBlock());
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
			
			// Advance the comparison LFO
			TArray<float> ExpectedBuffer;
			ExpectedBuffer.SetNumUninitialized(Generator->OperatorSettings.GetNumFramesPerBlock());

			if (nullptr != MusicTimingInfo)
			{
				MusicTimingInfo->Timestamp = GetClockTimestampAtBlockOffset(0);
			}
			
			LFOForComparison.Advance(ExpectedBuffer.GetData(), ExpectedBuffer.Num(), MusicTimingInfo);

			// Check the output
			if (!TestEqual("Buffer sizes match", Buffer.Num(), ExpectedBuffer.Num()))
			{
				return false;
			}

			for (int32 SampleIdx = 0; SampleIdx < ExpectedBuffer.Num(); ++SampleIdx)
			{
				if (!TestEqual(
					FString::Printf(TEXT("Output matches (block %d, sample %d"), BlockIdx, SampleIdx),
					Buffer[SampleIdx],
					ExpectedBuffer[SampleIdx]))
				{
					return false;
				}
			}

		}

		return true;
	}

	END_DEFINE_SPEC(FHarmonixMetasoundMorphingLfoNodeSpec)

	void FHarmonixMetasoundMorphingLfoNodeSpec::Define()
	{
		using FMusicTimingInfo = Harmonix::Dsp::Modulators::FMorphingLFO::FMusicTimingInfo;
		
		Describe("float", [this]()
		{
			BeforeEach([this]()
			{
				BuildGenerator<float>();
				LFOForComparison.Reset(Generator->OperatorSettings.GetSampleRate());
			});

			It("SyncType = None", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 0.9f;
				constexpr float Shape = 1.3f;
				constexpr bool Invert = false;

				RenderAndCompareFloat(ETimeSyncOption::None, Frequency, Shape, Invert);
			});

			It("SyncType = TempoSync", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 1.5f;
				constexpr float Shape = 0.2f;
				constexpr bool Invert = true;

				FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Timestamp.Bar = 1;
				MusicTimingInfo.Timestamp.Beat = 1;
				MusicTimingInfo.TimeSignature.Numerator = 4;
				MusicTimingInfo.TimeSignature.Denominator = 4;
				MusicTimingInfo.Tempo = 134.0f;

				RenderAndCompareFloat(ETimeSyncOption::TempoSync, Frequency, Shape, Invert, &MusicTimingInfo);
			});

			It("SyncType = SpeedScale", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 30.0f;
				constexpr float Shape = 2.0f;
				constexpr bool Invert = false;

				FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Speed = 0.3f;

				RenderAndCompareFloat(ETimeSyncOption::SpeedScale, Frequency, Shape, Invert, &MusicTimingInfo);
			});
		});

		Describe("FAudioBuffer", [this]()
		{
			BeforeEach([this]()
			{
				BuildGenerator<Metasound::FAudioBuffer>();
				LFOForComparison.Reset(Generator->OperatorSettings.GetSampleRate());
			});

			It("SyncType = None", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 0.9f;
				constexpr float Shape = 1.3f;
				constexpr bool Invert = false;

				RenderAndCompareBuffer(ETimeSyncOption::None, Frequency, Shape, Invert);
			});

			It("SyncType = TempoSync", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 1.5f;
				constexpr float Shape = 0.2f;
				constexpr bool Invert = true;

				FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Timestamp.Bar = 1;
				MusicTimingInfo.Timestamp.Beat = 1;
				MusicTimingInfo.TimeSignature.Numerator = 4;
				MusicTimingInfo.TimeSignature.Denominator = 4;
				MusicTimingInfo.Tempo = 134.0f;

				RenderAndCompareBuffer(ETimeSyncOption::TempoSync, Frequency, Shape, Invert, &MusicTimingInfo);
			});

			It("SyncType = SpeedScale", [this]()
			{
				// TODO: parameterize these tests
				constexpr float Frequency = 30.0f;
				constexpr float Shape = 2.0f;
				constexpr bool Invert = false;

				FMusicTimingInfo MusicTimingInfo{};
				MusicTimingInfo.Speed = 0.3f;

				RenderAndCompareBuffer(ETimeSyncOption::SpeedScale, Frequency, Shape, Invert, &MusicTimingInfo);
			});
		});
	}

}

#endif