// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEnvelopeFollowerTypes.h"
#include "MetasoundGenerator.h"
#include "NodeTestGraphBuilder.h"

#include "HarmonixDsp/Generate.h"

#include "HarmonixMetasound/Nodes/MultibandAnalyzerNode.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MultibandAnalyzerNode
{
	BEGIN_DEFINE_SPEC(
		FHarmonixMetasoundMultibandAnalyzerNodeSpec,
		"Harmonix.Metasound.Nodes.MultibandAnalyzerNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	
	static TUniquePtr<Metasound::FMetasoundGenerator> BuildGenerator(const HarmonixMetasound::Nodes::MultibandAnalyzer::FSettings& NodeSettings)
	{
		using FBuilder = Metasound::Test::FNodeTestGraphBuilder;
		FBuilder Builder;

		const Metasound::Frontend::FNodeHandle AnalyzerNode =
			Builder.AddNode(HarmonixMetasound::Nodes::MultibandAnalyzer::GetClassName(), 0);

		Builder.AddAndConnectDataReferenceInput(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::EnableName,
			Metasound::GetMetasoundDataTypeName<bool>());
		Builder.AddAndConnectDataReferenceInput(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::AudioMonoName,
			Metasound::GetMetasoundDataTypeName<Metasound::FAudioBuffer>());
		Builder.AddAndConnectConstructorInput(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::CrossoverFrequenciesName,
			NodeSettings.CrossoverFrequencies);
		Builder.AddAndConnectConstructorInput(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::ApplySmoothingName,
			NodeSettings.ApplySmoothing);
		Builder.AddAndConnectConstructorInput<Metasound::FTime, float>(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::AttackTimeName,
			static_cast<float>(NodeSettings.AttackTime.GetSeconds()));
		Builder.AddAndConnectConstructorInput<Metasound::FTime, float>(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::ReleaseTimeName,
			static_cast<float>(NodeSettings.ReleaseTime.GetSeconds()));
		Builder.AddAndConnectConstructorInput<Metasound::FEnumEnvelopePeakMode, int32>(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::PeakModeName,
			static_cast<int32>(NodeSettings.PeakMode));

		Builder.AddAndConnectDataReferenceOutput(
			AnalyzerNode,
			HarmonixMetasound::Nodes::MultibandAnalyzer::Outputs::BandLevelsName,
			Metasound::GetMetasoundDataTypeName<TArray<float>>());

		// have to add an audio output for the generator to render
		Builder.AddOutput("Audio", Metasound::GetMetasoundDataTypeName<Metasound::FAudioBuffer>());

		return Builder.BuildGenerator();
	}
	
	END_DEFINE_SPEC(FHarmonixMetasoundMultibandAnalyzerNodeSpec)

	void FHarmonixMetasoundMultibandAnalyzerNodeSpec::Define()
	{
		Describe("When passing in a sine within each band", [this]()
		{
			It("reports energy within that band when enabled and none when disabled.", [this]()
			{
				HarmonixMetasound::Nodes::MultibandAnalyzer::FSettings Settings;
				Settings.CrossoverFrequencies = { 200, 800, 3200 };
				const TUniquePtr<Metasound::FMetasoundGenerator> Generator = BuildGenerator(Settings);

				TOptional<Metasound::TDataReadReference<TArray<float>>> BandLevelsOutput =
					Generator->GetOutputReadReference<TArray<float>>(HarmonixMetasound::Nodes::MultibandAnalyzer::Outputs::BandLevelsName);

				if (!TestTrue("Got output", BandLevelsOutput.IsSet()))
				{
					return;
				}

				const Metasound::TDataReadReference<TArray<float>> BandLevelsRef = *BandLevelsOutput;
									
				TOptional<Metasound::FAudioBufferWriteRef> InputBuffer =
					Generator->GetInputWriteReference<Metasound::FAudioBuffer>(HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::AudioMonoName);

				if (!TestTrue("Got input buffer", InputBuffer.IsSet()))
				{
					return;
				}

				TOptional<Metasound::FBoolWriteRef> InputEnable =
					Generator->GetInputWriteReference<bool>(HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::EnableName);

				if (!TestTrue("Got enable input ref", InputEnable.IsSet()))
				{
					return;
				}

				const int32 NumBands = Settings.CrossoverFrequencies.Num() + 1;
				
				for (int32 BandIdx = 0; BandIdx < NumBands; ++BandIdx)
				{
					const float BandLowFreq = BandIdx > 0 ? Settings.CrossoverFrequencies[BandIdx - 1] : 20;
					const float BandHighFreq = BandIdx < NumBands - 1 ? Settings.CrossoverFrequencies[BandIdx] : 20000;
					const float SineFreq = (BandLowFreq + BandHighFreq) / 2;
					float SinePhase = 0;
					
					const float AttackTimeSamples = Settings.AttackTime.GetSeconds() * Generator->OperatorSettings.GetSampleRate();
					const int32 NumBlocksToRender = FMath::CeilToInt(AttackTimeSamples / Generator->OperatorSettings.GetNumFramesPerBlock());

					TArray<float> OutputBuffer;
					OutputBuffer.SetNumUninitialized(Generator->OperatorSettings.GetNumFramesPerBlock());

					*(*InputEnable) = true;

					for (int32 i = 0; i < NumBlocksToRender; ++i)
					{
						HarmonixDsp::GenerateSine(
							(*InputBuffer)->GetData(),
							(*InputBuffer)->Num(),
							SineFreq,
							Generator->OperatorSettings.GetSampleRate(),
							SinePhase);
						
						Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());
					}

					TArray<float> BandLevels = *BandLevelsRef;

					if (!TestTrue("Band is present in output", BandLevels.IsValidIndex(BandIdx)))
					{
						return;
					}
					
					float BandLevel = BandLevels[BandIdx];
						
					if (!TestTrue("Energy reported in band", BandLevel > 0.0f))
					{
						return;
					}

					*(*InputEnable) = false;

					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());

					BandLevels = *BandLevelsRef;
					BandLevel = BandLevels[BandIdx];

					if (!TestTrue("Bypassed: No energy reported in band", BandLevel == 0.0f))
					{
						return;
					}
				}
			});

			It("Reports the raw peak when smoothing is disabled", [this]()
			{
				HarmonixMetasound::Nodes::MultibandAnalyzer::FSettings Settings;
				Settings.CrossoverFrequencies = { 100, 1000 };
				Settings.ApplySmoothing = false;
				const TUniquePtr<Metasound::FMetasoundGenerator> Generator = BuildGenerator(Settings);

				Generator->SetInputValue(HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::EnableName, true);
				
				TOptional<Metasound::TDataReadReference<TArray<float>>> BandLevelsOutput =
					Generator->GetOutputReadReference<TArray<float>>(HarmonixMetasound::Nodes::MultibandAnalyzer::Outputs::BandLevelsName);

				if (!TestTrue("Got output", BandLevelsOutput.IsSet()))
				{
					return;
				}

				const Metasound::TDataReadReference<TArray<float>> BandLevelsRef = *BandLevelsOutput;
													
				TOptional<Metasound::FAudioBufferWriteRef> InputBuffer =
					Generator->GetInputWriteReference<Metasound::FAudioBuffer>(HarmonixMetasound::Nodes::MultibandAnalyzer::Inputs::AudioMonoName);

				if (!TestTrue("Got input buffer", InputBuffer.IsSet()))
				{
					return;
				}

				const int32 NumBands = Settings.CrossoverFrequencies.Num() + 1;
				
				for (int32 BandIdx = 0; BandIdx < NumBands; ++BandIdx)
				{
					const float BandLowFreq = BandIdx > 0 ? Settings.CrossoverFrequencies[BandIdx - 1] : 20;
					const float BandHighFreq = BandIdx < NumBands - 1 ? Settings.CrossoverFrequencies[BandIdx] : 20000;
					const float SineFreq = (BandLowFreq + BandHighFreq) / 2;
					float SinePhase = 0;

					TArray<float> OutputBuffer;
					OutputBuffer.SetNumUninitialized(Generator->OperatorSettings.GetNumFramesPerBlock());

					// Generate a block with no sound and expect no energy
					(*InputBuffer)->Zero();
					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());
					
					TArray<float> BandLevels = *BandLevelsRef;

					if (!TestTrue("Band is present in output", BandLevels.IsValidIndex(BandIdx)))
					{
						return;
					}
										
					float BandLevel = BandLevels[BandIdx];
											
					if (!TestEqual("No energy reported in band", BandLevel, 0.0f))
					{
						return;
					}
					
					// Generate a block with some sound and expect some energy immediately
					HarmonixDsp::GenerateSine(
						(*InputBuffer)->GetData(),
						(*InputBuffer)->Num(),
						SineFreq,
						Generator->OperatorSettings.GetSampleRate(),
						SinePhase);
					
					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());

					BandLevels = *BandLevelsRef;
					BandLevel = BandLevels[BandIdx];

					if (!TestNotEqual("Energy reported in band", BandLevel, 0.0f))
					{
						return;
					}

					// Zero the input and expect no energy
					// Render a few times to clear the filter history
					(*InputBuffer)->Zero();
					for (int32 i = 0; i < 6; ++i)
					{
						Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());
					}

					BandLevels = *BandLevelsRef;
					BandLevel = BandLevels[BandIdx];

					if (!TestEqual("No energy reported in band", BandLevel, 0.0f))
					{
						return;
					}
				}
			});
		});
	}

}

#endif