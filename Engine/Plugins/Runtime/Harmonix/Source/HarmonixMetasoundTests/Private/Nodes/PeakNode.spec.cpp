// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGenerator.h"
#include "NodeTestGraphBuilder.h"

#include "HarmonixMetasound/Nodes/PeakNode.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::PeakNode
{
	BEGIN_DEFINE_SPEC(
		FHarmonixMetasoundPeakNodeSpec,
		"Harmonix.Metasound.Nodes.PeakNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	TUniquePtr<Metasound::FMetasoundGenerator> Generator;

	void BuildGenerator()
	{
		using FBuilder = Metasound::Test::FNodeTestGraphBuilder;
		FBuilder Builder;
		Generator = Builder.MakeSingleNodeGraph(HarmonixMetasound::Nodes::Peak::GetClassName(), 0);
	}
	
	END_DEFINE_SPEC(FHarmonixMetasoundPeakNodeSpec)

	void FHarmonixMetasoundPeakNodeSpec::Define()
	{
		Describe("When processing a block", [this]()
		{
			BeforeEach([this]()
			{
				BuildGenerator();
				TestTrue("Built generator", Generator.IsValid());
			});


			It("the expected peak is output for a variety of input waveforms", [this]()
			{
				const TArray<float> PeakValues { 0.1f, 0.2f, 0.4f, 0.567f, 1.0f, 3.4f };

				for (const float PeakValue : PeakValues)
				{
					// Fill the input buffer
					TOptional<Metasound::FAudioBufferWriteRef> InputAudio =
						Generator->GetInputWriteReference<Metasound::FAudioBuffer>(
							HarmonixMetasound::Nodes::Peak::Inputs::AudioMonoName);
					if (!TestTrue("Got input", InputAudio.IsSet()))
					{
						return;
					}
									
					float Value = -PeakValue;
					constexpr float Inc = 0.123f;
									
					for (int32 i = 0; i < (*InputAudio)->Num(); ++i)
					{
						(*InputAudio)->GetData()[i] = Value;

						// Wrap around in range [-PeakValue, PeakValue]
						Value += Inc;
						if (Value > PeakValue)
						{
							Value -= PeakValue * 2;
						}
					}

					// Process
					Metasound::FAudioBuffer OutputBuffer{ Generator->OperatorSettings };
					Generator->OnGenerateAudio(OutputBuffer.GetData(), OutputBuffer.Num());

					// Make sure the peak matches
					TOptional<Metasound::FFloatReadRef> OutputPeak =
						Generator->GetOutputReadReference<float>(HarmonixMetasound::Nodes::Peak::Outputs::PeakName);
					if (!TestTrue("Got output", OutputPeak.IsSet()))
					{
						return;
					}
									
					TestEqual("Peak matches expected value", **OutputPeak, PeakValue);
				}
			});
		});
	}
}

#endif