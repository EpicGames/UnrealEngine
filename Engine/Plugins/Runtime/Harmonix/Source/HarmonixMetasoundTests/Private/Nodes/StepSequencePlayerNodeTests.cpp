// Copyright Epic Games, Inc. All Rights Reserved.
#include "NodeTestGraphBuilder.h"
#include "HarmonixMetasound/DataTypes/MidiStepSequence.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/Nodes/StepSequencePlayerNode.h"
#include "Misc/AutomationTest.h"
#if WITH_DEV_AUTOMATION_TESTS
namespace HarmonixMetasoundTests::StepSequencePlayerNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;



	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FStepSequencePlayerNoStuckNotesOnTransposeTest,
		"Harmonix.Metasound.Nodes.StepSequencePlayerNode.NoStuckNotesOnTranspose",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FStepSequencePlayerNoStuckNotesOnTransposeTest::RunTest(const FString&)
	{
		using namespace HarmonixMetasound::Nodes::StepSequencePlayer;
		
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			GetClassName(),
			GetCurrentMajorVersion(),
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// Start the clock
		auto ClockInput = Generator->GetInputWriteReference<HarmonixMetasound::FMidiClock>(Inputs::MidiClockName);
		UTEST_TRUE("Got clock", ClockInput.IsSet());
		(*ClockInput)->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

		// Start the transport
		auto TransportInput = Generator->GetInputWriteReference<HarmonixMetasound::FMusicTransportEventStream>(Inputs::TransportName);
		UTEST_TRUE("Got transport", TransportInput.IsSet());
		(*TransportInput)->AddTransportRequest(HarmonixMetasound::EMusicPlayerTransportRequest::Play, 0);

		// Create a sequence asset with a cell turned on
		auto SequenceAssetInput = Generator->GetInputWriteReference<HarmonixMetasound::FMidiStepSequenceAsset>(Inputs::SequenceAssetName);
		UTEST_TRUE("Got sequence asset", SequenceAssetInput.IsSet());

		UMidiStepSequence* SequenceAsset = NewObject<UMidiStepSequence>();
		SequenceAsset->SetCell(0, 0, true);
		**SequenceAssetInput = SequenceAsset->CreateProxyData({});

		// Render once, should get a note on in the output
		(*ClockInput)->PrepareBlock();
		(*ClockInput)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());
		Metasound::FAudioBuffer Buffer{ NumSamplesPerBlock };
		Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

		auto MidiOutput = Generator->GetOutputReadReference<HarmonixMetasound::FMidiStream>(Outputs::MidiStreamName);
		UTEST_TRUE("Got MIDI output", MidiOutput.IsSet());

		uint8 Channel = Harmonix::Midi::Constants::GNumChannels;
		uint8 NoteNumber = Harmonix::Midi::Constants::GMaxNote + 1;
		FMidiVoiceId VoiceId = FMidiVoiceId::None();
		
		for (const auto& Event : (*MidiOutput)->GetEventsInBlock())
		{
			if (Event.MidiMessage.IsNoteOn())
			{
				Channel = Event.MidiMessage.GetStdChannel();
				NoteNumber = Event.MidiMessage.GetStdData1();
				VoiceId = Event.GetVoiceId();
				break;
			}
		}

		UTEST_LESS("Note on: Channel was valid", Channel, Harmonix::Midi::Constants::GNumChannels);
		UTEST_LESS("Note on: Note number was valid", NoteNumber, Harmonix::Midi::Constants::GMaxNote + 1);
		UTEST_NOT_EQUAL("Note on: voice id was valid", VoiceId, FMidiVoiceId::None());

		// Transpose and render until we get the note off, it should have the same voice id as the note on
		auto AdditionalOctavesInput = Generator->GetInputWriteReference<float>(Inputs::AdditionalOctavesName);
		UTEST_TRUE("Got additional octaves", AdditionalOctavesInput.IsSet());

		constexpr float AdditionalOctaves = 2;
		**AdditionalOctavesInput = AdditionalOctaves;

		constexpr int32 MaxTries = 1000;
		bool GotNoteOff = false;

		for (int i = 0; i < MaxTries; ++i)
		{
			(*ClockInput)->PrepareBlock();
			(*ClockInput)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

			for (const auto& Event : (*MidiOutput)->GetEventsInBlock())
			{
				if (Event.MidiMessage.IsNoteOff())
				{
					UTEST_EQUAL("Correct channel", Event.MidiMessage.GetStdChannel(), Channel);
					// NB: the actual message doesn't matter for note offs
					UTEST_EQUAL("Correct voice id", Event.GetVoiceId(), VoiceId);
					GotNoteOff = true;
					break;
				}
			}

			if (GotNoteOff)
			{
				break;
			}
		}

		UTEST_TRUE("No stuck notes", GotNoteOff);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FStepSequencePlayerEnabledAndLooping,
		"Harmonix.Metasound.Nodes.StepSequencePlayerNode.StepSequencePlayerEnabledAndLooping",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FStepSequencePlayerEnabledAndLooping::RunTest(const FString&)
	{
		using namespace HarmonixMetasound::Nodes::StepSequencePlayer;

		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<Metasound::FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			GetClassName(),
			GetCurrentMajorVersion(),
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// Start the clock
		auto ClockInput = Generator->GetInputWriteReference<HarmonixMetasound::FMidiClock>(Inputs::MidiClockName);
		UTEST_TRUE("Got clock", ClockInput.IsSet());
		(*ClockInput)->SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);
		// Start the transport
		auto TransportInput = Generator->GetInputWriteReference<HarmonixMetasound::FMusicTransportEventStream>(Inputs::TransportName);
		UTEST_TRUE("Got transport", TransportInput.IsSet());
		(*TransportInput)->AddTransportRequest(HarmonixMetasound::EMusicPlayerTransportRequest::Play, 0);
		// Create a sequence asset with a cell turned on
		auto SequenceAssetInput = Generator->GetInputWriteReference<HarmonixMetasound::FMidiStepSequenceAsset>(Inputs::SequenceAssetName);
		UTEST_TRUE("Got sequence asset", SequenceAssetInput.IsSet());
		UMidiStepSequence* SequenceAsset = NewObject<UMidiStepSequence>();
		SequenceAsset->SetCell(0, 0, true);
		**SequenceAssetInput = SequenceAsset->CreateProxyData({});

		Metasound::FAudioBuffer Buffer{ NumSamplesPerBlock };
		auto MidiOutput = Generator->GetOutputReadReference<HarmonixMetasound::FMidiStream>(Outputs::MidiStreamName);
		UTEST_TRUE("Got MIDI output", MidiOutput.IsSet());

		// Turn looping on
		auto LoopInput = Generator->GetInputWriteReference<bool>(Inputs::LoopName);
		UTEST_TRUE("Got loop input", LoopInput.IsSet());
		**LoopInput = true;

		constexpr int32 MaxTries = 10000;
		int32 TotalNotesOn = 0;
		for (int i = 0; i < MaxTries; ++i)
		{
			(*ClockInput)->PrepareBlock();
			(*ClockInput)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
			for (const auto& Event : (*MidiOutput)->GetEventsInBlock())
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					TotalNotesOn++;
					break;
				}
			}
			if (TotalNotesOn > 1)
			{
				break;
			}
		}

		UTEST_GREATER_EQUAL("Looping on: Sequencer Played At Least Two Notes", TotalNotesOn, 2);

		// Disable and turn off looping
		auto EnabledInput = Generator->GetInputWriteReference<bool>(Inputs::EnabledName);
		UTEST_TRUE("Got loop input", EnabledInput.IsSet());
		**EnabledInput = false;
		**LoopInput = false;

		(*ClockInput)->PrepareBlock();
		(*ClockInput)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());

		bool GotAnyNotes = false;
		for (int i = 0; i < MaxTries; ++i)
		{
			(*ClockInput)->PrepareBlock();
			(*ClockInput)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
			for (const auto& Event : (*MidiOutput)->GetEventsInBlock())
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					GotAnyNotes = true;
					break;
				}
			}
			if (GotAnyNotes)
			{
				break;
			}
		}

		UTEST_FALSE("Enabled off: No notes received", GotAnyNotes);

		// Re-enable
		**EnabledInput = true;

		TotalNotesOn = 0;
		for (int i = 0; i < MaxTries; ++i)
		{
			(*ClockInput)->PrepareBlock();
			(*ClockInput)->Advance(0, Generator->OperatorSettings.GetNumFramesPerBlock());
			Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());
			for (const auto& Event : (*MidiOutput)->GetEventsInBlock())
			{
				if (Event.MidiMessage.IsNoteOn())
				{
					TotalNotesOn++;
					break;
				}
			}
			if (TotalNotesOn > 1)
			{
				break;
			}
		}

		UTEST_EQUAL("Looping Off: One note received", TotalNotesOn, 1);

		return true;
	}

}
#endif