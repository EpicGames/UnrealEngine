// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiStream.h"
#include "HarmonixMetasound/DataTypes/MusicTimestamp.h"
#include "HarmonixMetasound/Nodes/SecsUntilMusicTimestampNode.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::TimestampRelated
{
	using FGraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;
	using namespace HarmonixMetasound::Nodes;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FSecsUntilTimestampTestDefaults,
		"Harmonix.Metasound.Nodes.SecsUntilMusicTimestamp.Defaults",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FSecsUntilTimestampTestDefaults::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = FGraphBuilder::MakeSingleNodeGraph(
			SecsUntilMusicTimestampNode::GetClassName(),
			SecsUntilMusicTimestampNode::GetCurrentMajorVersion(),
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// execute a block
		{
			TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
			Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
		}

		// Validate defaults.
		TOptional<FFloatReadRef> OutputSeconds = Generator->GetOutputReadReference<float>(SecsUntilMusicTimestampNode::Outputs::SecsUntilTimestampName);
		UTEST_TRUE("Output exists", OutputSeconds.IsSet());
		UTEST_EQUAL("Secs. while enabled but no clock connected.", *(*OutputSeconds) , std::numeric_limits<float>::max());

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FSecsUntilTimestampTestBasic,
		"Harmonix.Metasound.Nodes.SecsUntilMusicTimestamp.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FSecsUntilTimestampTestBasic::RunTest(const FString&)
	{
		FGraphBuilder Builder;

		constexpr int32 TimeSigNumerator = 6;
		constexpr int32 TimeSigDenominator = 8;
		constexpr float Tempo = 60.0;
		constexpr int32 Bar = 3;
		constexpr float Beat = 3.0f;

		const FNodeHandle TransportNode = Builder.AddNode({ HarmonixNodeNamespace, TEXT("TriggerToTransport"), TEXT("") },
			0);

		const FNodeHandle MetronomeNode = Builder.AddNode({HarmonixMetasound::HarmonixNodeNamespace, "Metronome", "" },
			0	);

		const FNodeHandle SecsUntilNode = Builder.AddNode(Nodes::SecsUntilMusicTimestampNode::GetClassName(),
			Nodes::SecsUntilMusicTimestampNode::GetCurrentMajorVersion());

		FNodeHandle StartInTrigger = Builder.AddAndConnectDataReferenceInput(TransportNode, CommonPinNames::Inputs::TransportPlayName, Metasound::GetMetasoundDataTypeName<FTrigger>(), "StartTest");
		FNodeHandle TempoIn = Builder.AddAndConnectDataReferenceInput(MetronomeNode, CommonPinNames::Inputs::TempoName, Metasound::GetMetasoundDataTypeName<float>(), "TestTempo");
		FNodeHandle TimeSigNumeratorIn = Builder.AddAndConnectDataReferenceInput(MetronomeNode, CommonPinNames::Inputs::TimeSigNumeratorName, Metasound::GetMetasoundDataTypeName<int32>(), "TestTSNum");
		FNodeHandle TimeSigDenominatorIn = Builder.AddAndConnectDataReferenceInput(MetronomeNode, CommonPinNames::Inputs::TimeSigDenominatorName, Metasound::GetMetasoundDataTypeName<int32>(), "TestTSDenom");
		FNodeHandle TimestampIn = Builder.AddAndConnectDataReferenceInput(SecsUntilNode, SecsUntilMusicTimestampNode::Inputs::TimestampName, Metasound::GetMetasoundDataTypeName<FMusicTimestamp>(), "TestTimestamp");
		FNodeHandle SecsOut = Builder.AddAndConnectDataReferenceOutput(SecsUntilNode, SecsUntilMusicTimestampNode::Outputs::SecsUntilTimestampName, Metasound::GetMetasoundDataTypeName<float>(), "TestResult");

		UTEST_TRUE("Connected Transport To Metronome",
			FGraphBuilder::ConnectNodes(TransportNode, CommonPinNames::Outputs::TransportName,
				MetronomeNode, CommonPinNames::Inputs::TransportName));

		UTEST_TRUE("Connected Metronome To Secs Until Node",
			FGraphBuilder::ConnectNodes(MetronomeNode, CommonPinNames::Outputs::MidiClockName,
				SecsUntilNode, CommonPinNames::Inputs::MidiClockName));

		Builder.AddOutput("AudioOut", GetMetasoundDataTypeName<FAudioBuffer>());

		constexpr int32 NumSamplesPerBlock = 256;
		constexpr FSampleRate SampleRate = 48000;

		TUniquePtr<FMetasoundGenerator> Generator = Builder.BuildGenerator(48000, NumSamplesPerBlock);
		UTEST_TRUE("Made Generator", Generator.IsValid());
		FAudioBuffer Buffer{ Generator->OperatorSettings };

		TOptional<FTriggerWriteRef> StartTestTrigger = Generator->GetInputWriteReference<FTrigger>("StartTest");
		UTEST_TRUE("Got Start Test Trigger", StartTestTrigger.IsSet());

		TOptional<FFloatWriteRef> TestTempoInput = Generator->GetInputWriteReference<float>("TestTempo");
		UTEST_TRUE("Got Test Tempo Input", TestTempoInput.IsSet());

		TOptional<FInt32WriteRef> TestTSNumInput = Generator->GetInputWriteReference<int32>("TestTSNum");
		UTEST_TRUE("Got Test Time Sig Numerator Input", TestTSNumInput.IsSet());

		TOptional<FInt32WriteRef> TestTSDenomInput = Generator->GetInputWriteReference<int32>("TestTSDenom");
		UTEST_TRUE("Got Test Time Sig Denominator Input", TestTSDenomInput.IsSet());

		TOptional<FMusicTimestampWriteRef> TestTimestampInput = Generator->GetInputWriteReference<FMusicTimestamp>("TestTimestamp");
		UTEST_TRUE("Got Test Timestamp Input", TestTimestampInput.IsSet());

		TOptional<FFloatReadRef> ResultOutput = Generator->GetOutputReadReference<float>("TestResult");
		UTEST_TRUE("Got Test Result Output", ResultOutput.IsSet());

		*(*TestTempoInput) = Tempo;
		(*TestTimestampInput)->Bar = Bar;
		(*TestTimestampInput)->Beat = Beat;
		*(*TestTSNumInput) = TimeSigNumerator;
		*(*TestTSDenomInput) = TimeSigDenominator;
		(*StartTestTrigger)->TriggerFrame(0);

		Generator->OnGenerateAudio(Buffer.GetData(), Buffer.Num());

		float TotalBeats = ((Bar - 1) * TimeSigNumerator) + (Beat - 1);
		float SecsPerQuarter = 60.0f / Tempo;
		float SecsPerBeat = SecsPerQuarter / (TimeSigDenominator / 4);
		float ExpectedSeconds = (SecsPerBeat * TotalBeats) - ((float)NumSamplesPerBlock / (float)SampleRate);
		float SecsUntil = *(*ResultOutput);

		UTEST_TRUE("Time Until As Expected", FMath::IsNearlyEqual(SecsUntil, ExpectedSeconds, 0.0005));
		
		return true;
	}
}

#endif
