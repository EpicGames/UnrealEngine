// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::TransportToTriggerNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	TOptional<FTriggerReadRef> GetTriggerOutput(const TUniquePtr<FMetasoundGenerator>& Generator, EMusicPlayerTransportRequest Request)
	{
		const TCHAR* PinName = nullptr;

		switch (Request)
		{
		case EMusicPlayerTransportRequest::Prepare:
			PinName = CommonPinNames::Outputs::TransportPrepareName;
			break;
		case EMusicPlayerTransportRequest::Play:
			PinName = CommonPinNames::Outputs::TransportPlayName;
			break;
		case EMusicPlayerTransportRequest::Pause:
			PinName = CommonPinNames::Outputs::TransportPauseName;
			break;
		case EMusicPlayerTransportRequest::Continue:
			PinName = CommonPinNames::Outputs::TransportContinueName;
			break;
		case EMusicPlayerTransportRequest::Stop:
			PinName = CommonPinNames::Outputs::TransportStopName;
			break;
		case EMusicPlayerTransportRequest::Kill:
			PinName = CommonPinNames::Outputs::TransportKillName;
			break;
		default:
			checkNoEntry();
			break;
		}

		return Generator->GetOutputReadReference<FTrigger>(PinName);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestBasic, 
		"Harmonix.Metasound.Nodes.TransportToTrigger.Basic", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTestBasic::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "TransportToTrigger", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// send transport requests of each type and expect to get a trigger out the other side
		TOptional<FMusicTransportEventStreamWriteRef> InputTransport =
			Generator->GetInputWriteReference<FMusicTransportEventStream>(CommonPinNames::Inputs::TransportName);
		UTEST_TRUE("Input exists", InputTransport.IsSet());
		
		constexpr uint8 FirstRequestType = static_cast<uint8>(EMusicPlayerTransportRequest::None) + 1;
		constexpr uint8 NumRequestTypes = static_cast<uint8>(EMusicPlayerTransportRequest::Count);

		for (uint8 i = FirstRequestType; i < NumRequestTypes; ++i)
		{
			const EMusicPlayerTransportRequest Req = static_cast<EMusicPlayerTransportRequest>(i);

			// skip seek, not supported for now
			if (Req == EMusicPlayerTransportRequest::Seek)
			{
				continue;;
			}
			
			// send a transport request and execute a block
			(*InputTransport)->AddTransportRequest(Req, 0);
			{
				TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
				Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
			}

			// check that we got a trigger out the other side
			TOptional<FTriggerReadRef> OutputPin = GetTriggerOutput(Generator, Req);
			UTEST_TRUE("Output exists", OutputPin.IsSet());
			UTEST_EQUAL("Num triggers", (*OutputPin)->NumTriggeredInBlock(), 1);
			UTEST_EQUAL("Trigger sample", (*OutputPin)->First(), 0);

			// check that we didn't get any other triggers
			for (uint8 j = FirstRequestType; j < NumRequestTypes; ++j)
			{
				// skip the one we're expecting
				if (j == i)
				{
					continue;
				}

				const EMusicPlayerTransportRequest OtherReq = static_cast<EMusicPlayerTransportRequest>(j);

				// skip seek, not supported for now
				if (OtherReq == EMusicPlayerTransportRequest::Seek)
				{
					continue;;
				}
				
				TOptional<FTriggerReadRef> OtherOutputPin = GetTriggerOutput(Generator, OtherReq);
				UTEST_TRUE("Output exists", OtherOutputPin.IsSet());
				UTEST_EQUAL("Num triggers", (*OtherOutputPin)->NumTriggeredInBlock(), 0);
			}

			// reset the input
			// NB: Typically, the output connected to the transport input would reset it.
			// But since this test's logic is providing the input, that's up to us.
			(*InputTransport)->Reset();
		}

		return true;
	}
}

#endif
