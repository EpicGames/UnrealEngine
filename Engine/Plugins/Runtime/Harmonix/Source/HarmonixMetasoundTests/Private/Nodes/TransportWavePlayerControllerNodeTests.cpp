// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::TransportWavePlayerControllerNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace HarmonixMetasound;
	using namespace Metasound;
	using namespace Metasound::Frontend;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTransportControllerCreateNodeTest,
		"Harmonix.Metasound.Nodes.TransportWavePlayerController.CreateNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTransportControllerCreateNodeTest::RunTest(const FString&)
	{

		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "TransportWavePlayerController", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTransportControllerValidateOutputTriggerTest,
		"Harmonix.Metasound.Nodes.TransportWavePlayerController.ValidateOutputTrigger",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTransportControllerValidateOutputTriggerTest::RunTest(const FString&)
	{
		// Build a graph with a transport controller node
		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;
		GraphBuilder Builder;
		const FNodeHandle TransportControllerNode = Builder.AddNode(
			{ HarmonixMetasound::HarmonixNodeNamespace, "TransportWavePlayerController", "" },
			0);

		const FName InputName = CommonPinNames::Inputs::TransportName;
		const FName PlayOutputName = CommonPinNames::Outputs::TransportPlayName;
		const FName StopOutputName = CommonPinNames::Outputs::TransportStopName;

		// Connect the transport controller transport input to the graph input
		{
			const FNodeHandle InputNode = Builder.AddInput(InputName, Metasound::GetMetasoundDataTypeName<FMusicTransportEventStream>());
			const FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(InputName);
			const FInputHandle InputToConnect = TransportControllerNode->GetInputWithVertexName(InputName);
			UTEST_TRUE("Connected transport input", InputToConnect->Connect(*OutputToConnect));
		}

		// Connect the play trigger output to the graph output
		{
			const FNodeHandle PlayOutputNode = Builder.AddOutput(PlayOutputName, Metasound::GetMetasoundDataTypeName<FTrigger>());
			const FOutputHandle OutputToConnect = TransportControllerNode->GetOutputWithVertexName(PlayOutputName);
			const FInputHandle InputToConnect = PlayOutputNode->GetInputWithVertexName(PlayOutputName);
			UTEST_TRUE("Connected play trigger output", InputToConnect->Connect(*OutputToConnect));
		}

		// Connect the stop trigger output to the graph output
		{
			const FNodeHandle StopOutputNode = Builder.AddOutput(StopOutputName, Metasound::GetMetasoundDataTypeName<FTrigger>());
			const FOutputHandle OutputToConnect = TransportControllerNode->GetOutputWithVertexName(StopOutputName);
			const FInputHandle InputToConnect = StopOutputNode->GetInputWithVertexName(StopOutputName);
			UTEST_TRUE("Connected stop trigger output", InputToConnect->Connect(*OutputToConnect));
		}

		// Build the graph
		const TUniquePtr<FMetasoundGenerator> Generator = Builder.BuildGenerator(SampleRate, NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// Validate output triggers

		const TOptional<TDataReadReference<FTrigger>> PlayOutputReadRef = Generator->GetOutputReadReference<FTrigger>(PlayOutputName);
		UTEST_TRUE("Play output exists", PlayOutputReadRef.IsSet());
		UTEST_EQUAL("Num of Play triggers", (*PlayOutputReadRef)->NumTriggeredInBlock(), 0);

		const TOptional<TDataReadReference<FTrigger>> StopOutputReadRef = Generator->GetOutputReadReference<FTrigger>(StopOutputName);
		UTEST_TRUE("Stop output exists", StopOutputReadRef.IsSet());
		UTEST_EQUAL("Num of Stop triggers", (*StopOutputReadRef)->NumTriggeredInBlock(), 0);

		return true;
	}
}

#endif
