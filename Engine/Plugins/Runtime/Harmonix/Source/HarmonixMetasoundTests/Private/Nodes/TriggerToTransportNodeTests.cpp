// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "HarmonixDsp/AudioBuffer.h"
#include "HarmonixMetasound/Common.h"
#include "HarmonixMetasound/DataTypes/MusicTransport.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::TriggerToTransportNode
{
	using GraphBuilder = Metasound::Test::FNodeTestGraphBuilder;
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace HarmonixMetasound;

	TOptional<FTriggerWriteRef> GetTriggerInput(const TUniquePtr<FMetasoundGenerator>& Generator, EMusicPlayerTransportRequest Request)
	{
		const TCHAR* PinName = nullptr;

		switch (Request)
		{
		case EMusicPlayerTransportRequest::Prepare:
			PinName = CommonPinNames::Inputs::TransportPrepareName;
			break;
		case EMusicPlayerTransportRequest::Play:
			PinName = CommonPinNames::Inputs::TransportPlayName;
			break;
		case EMusicPlayerTransportRequest::Pause:
			PinName = CommonPinNames::Inputs::TransportPauseName;
			break;
		case EMusicPlayerTransportRequest::Continue:
			PinName = CommonPinNames::Inputs::TransportContinueName;
			break;
		case EMusicPlayerTransportRequest::Stop:
			PinName = CommonPinNames::Inputs::TransportStopName;
			break;
		case EMusicPlayerTransportRequest::Kill:
			PinName = CommonPinNames::Inputs::TransportKillName;
			break;
		case EMusicPlayerTransportRequest::Seek:
			PinName = CommonPinNames::Inputs::TriggerSeekName;
			break;
		default:
			checkNoEntry();
			break;
		}

		return Generator->GetInputWriteReference<FTrigger>(PinName);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTriggerToTransportCreateNodeTest,
		"Harmonix.Metasound.Nodes.TriggerToTransport.CreateNode",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTriggerToTransportCreateNodeTest::RunTest(const FString&)
	{
		// Build the graph.
		constexpr int32 NumSamplesPerBlock = 256;
		const TUniquePtr<FMetasoundGenerator> Generator = GraphBuilder::MakeSingleNodeGraph(
			{ HarmonixMetasound::HarmonixNodeNamespace, "TriggerToTransport", "" },
			0,
			48000,
			NumSamplesPerBlock);
		UTEST_TRUE("Graph successfully built", Generator.IsValid());

		// Send trigger requests of each type and expect to get a transport out the other side
		constexpr uint8 FirstRequestType = static_cast<uint8>(EMusicPlayerTransportRequest::None) + 1;
		constexpr uint8 NumRequestTypes = static_cast<uint8>(EMusicPlayerTransportRequest::Count);
		constexpr int32 TriggerFrameValue = 1;

		for (uint8 i = FirstRequestType; i < NumRequestTypes; ++i)
		{
			const EMusicPlayerTransportRequest InputReq = static_cast<EMusicPlayerTransportRequest>(i);

			// Create and send a trigger request. then execute a block
			TOptional<FTriggerWriteRef> InputTriggerPin = GetTriggerInput(Generator, InputReq);
			UTEST_TRUE("Input exists", InputTriggerPin.IsSet());

			// Setting trigger on frame 1 for output validation test.
			(*InputTriggerPin)->TriggerFrame(TriggerFrameValue);
			{
				TAudioBuffer<float> Buffer{ Generator->GetNumChannels(), NumSamplesPerBlock, EAudioBufferCleanupMode::Delete};
				Generator->OnGenerateAudio(Buffer.GetRawChannelData(0), Buffer.GetNumTotalValidSamples());
			}

			TOptional<FMusicTransportEventStreamReadRef> OutputPin =
				Generator->GetOutputReadReference<FMusicTransportEventStream>(CommonPinNames::Outputs::TransportName);
			UTEST_TRUE("Output exists", OutputPin.IsSet());

			const FMusicTransportEventStream::FEventList& TransportEventsThisBlock = (*OutputPin)->GetTransportEventsInBlock();

			// Verify that output request matches the input trigger request.
			// Then verify that output trigger frame matches the one assigned to the input.
			for (const FMusicTransportEventStream::FRequestEvent& OutputEvent : TransportEventsThisBlock)
			{
				UTEST_EQUAL("Validating Request Event", OutputEvent.Request, InputReq);
				UTEST_EQUAL("Validating Trigger Sample", OutputEvent.SampleIndex, TriggerFrameValue);
			}

			(*InputTriggerPin)->Reset();
		}

		return true;
	}
}

#endif
