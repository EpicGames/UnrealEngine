// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Engine.h"
#include "HarmonixMetasound/DataTypes/MidiClock.h"
#include "HarmonixMetasound/Subsystems/MidiClockUpdateSubsystem.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::MidiClockUpdateSubsystem
{
	namespace Helpers
	{
		HarmonixMetasound::FMidiClock MakeAndStartClock(
			const Metasound::FOperatorSettings& OperatorSettings,
			float Tempo,
			int32 TimeSigNum,
			int32 TimeSigDenom)
		{
			const TSharedPtr<FSongMaps> SongMaps = MakeShared<FSongMaps>(Tempo, TimeSigNum, TimeSigDenom);
			check(SongMaps);
			
			SongMaps->SetSongLengthTicks(std::numeric_limits<int32>::max());

			HarmonixMetasound::FMidiClock Clock{ OperatorSettings };
			Clock.AttachToSongMapEvaluator(SongMaps);
			Clock.SetTransportState(0, HarmonixMetasound::EMusicPlayerTransportState::Playing);

			return Clock;
		}	
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMidiClockUpdateSubsystemBasicTest,
	"Harmonix.Metasound.Subsystems.MidiClockUpdateSubsystem.Basic",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMidiClockUpdateSubsystemBasicTest::RunTest(const FString&)
	{
		using namespace HarmonixMetasound;

		UTEST_NOT_NULL("GEngine exists", GEngine);
		UMidiClockUpdateSubsystem* Subsystem = GEngine->GetEngineSubsystem<UMidiClockUpdateSubsystem>();
		UTEST_NOT_NULL("Subsystem exists", Subsystem);

		constexpr float Tempo = 95;
		constexpr int32 TimeSigNum = 3;
		constexpr int32 TimeSigDenom = 4;
		Metasound::FOperatorSettings OperatorSettings { 48000, 100 };
		
		FMidiClock Clock = Helpers::MakeAndStartClock(OperatorSettings, Tempo, TimeSigNum, TimeSigDenom);

		// Update the clock and tick the subsystem a few times to make sure we're updating the low-r clock
		constexpr int32 NumIterations = 100;
		const int32 NumSamples = OperatorSettings.GetNumFramesPerBlock();
		int32 SampleRemainder = 0;
		int32 SampleCount = 0;
		
		for (int32 i = 0; i < NumIterations; ++i)
		{
			// Advance the high-resolution clock
			SampleRemainder += NumSamples;
			constexpr int32 MidiGranularity = 128;
			Clock.PrepareBlock();
			int32 BlockOffset = 0;
			while (SampleRemainder >= MidiGranularity)
			{
				SampleCount += MidiGranularity;
				SampleRemainder -= MidiGranularity;
				const float AdvanceToMs = static_cast<float>(SampleCount) * 1000.0f / OperatorSettings.GetSampleRate();
				Clock.Advance(BlockOffset, MidiGranularity);
				BlockOffset += MidiGranularity;
			}

			// Tick the subsystem (low-resolution clocks)
			Subsystem->TickForTesting();

			// Check that the high- and low-resolution clocks are at the same place
			// TO DO!

		}

		return true;
	}
}

#endif