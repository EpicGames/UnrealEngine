// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixMidi : ModuleRules
{
	public HarmonixMidi(ReadOnlyTargetRules Target) : base(Target)
	{
		//OptimizeCode = CodeOptimization.Never;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TimeManagement",
				"MusicEnvironment",
			});
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioExtensions",
				"Engine",
				"Harmonix",
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("AssetRegistry");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
