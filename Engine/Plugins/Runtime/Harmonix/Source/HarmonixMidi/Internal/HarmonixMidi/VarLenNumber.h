// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include <type_traits>

class FArchive;
class IFileHandle;

namespace Midi
{
	/////////////////////////////////////////////////////////////////////////////// 
	// A "variable length number" as used in an SMF file.  The number is 
	// serialized using 7 bits per byte; the MSB of each byte is set for all but 
	// the last byte (so the MSB acts as a flag indicating that there are more 
	// bytes to come).  The maximum number of bytes that can be used is 4, so the
	// largest number that can be represented is 0x0FFFFFFF.
	//    For example:
	// 0x01 is serialized as 0x01      (binary 0000 0001)
	// 0x7F is serialized as 0x7F      (binary 0111 1111)
	// 0x80 is serialized as 0x81 0x00 (binary 1000 0001 0000 0000)
	namespace VarLenNumber
	{
		template<typename STORAGE_T>
		int32 Write(STORAGE_T& Storage, int32 Value)
		{
			if constexpr(std::is_convertible_v<STORAGE_T&, FArchive&>)
			{
				check(!Storage.IsLoading());
			}

			check(Value < 0x0fffffff);

			int32 Length = 1;
			int32 WorkingValue = Value;
			int32 Buffer = WorkingValue & 0x7f; // high bit = 0 for least-significant-7bits

			// first, peel away 7 bits at a time and put in "buffer"
			while ((WorkingValue >>= 7) != 0 && Length < 4)
			{
				Buffer <<= 8;
				Buffer |= (WorkingValue & 0x7f) | 0x80; // high bit = 1 for higher-order bits
				Length++;
			}

			// now, write out buffer in reverse-order (resulting in hi-to-low byte ordering)
			for (int32 i = 0; i < Length; ++i)
			{
				uint8 Byte = Buffer & 0xff;
				if constexpr(std::is_convertible_v<STORAGE_T&, FArchive&>)
				{
					Storage << Byte;
				}
				else
				{
					Storage->Write(&Byte, 1);
				}
				Buffer >>= 8;
			}

			return Length;

		}

		template<typename STORAGE_T>
		int32  Read(STORAGE_T& Storage, int32* BytesConsumed = nullptr)
		{
			if constexpr(std::is_convertible_v<STORAGE_T&, FArchive&>)
			{
				check(Storage.IsLoading());
			}

			int32 Value = 0;
			int32 ReadBytes = 0;
			while (true)
			{
				uint8 Byte = 0;
				if constexpr (std::is_convertible_v<STORAGE_T&, FArchive&>)
				{
					Storage << Byte;
				}
				else
				{
					Storage->Read(&Byte, 1);
				}
				ReadBytes++;
				Value = (Value << 7) + (Byte & 0x7f);
				check(Value < 0x0fffffff); // can't have more than 4 bytes
				if (!(Byte & 0x80))  //// this is the last byte. All done
				{
					break;
				}
			}
			if (BytesConsumed)
			{
				*BytesConsumed = ReadBytes;
			}
			return Value;
		}
	};
}
