// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/SongMaps.h"

#include "Misc/ScopeLock.h"
#include "Serialization/Archive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BarMap)

const FTimeSignaturePoint FBarMap::sDefaultTimeSignature(0, 0, FTimeSignature(4, 4), 0, std::numeric_limits<int32>::max());

bool FBarMap::operator==(const FBarMap& Other) const
{
	if (StartBar != Other.StartBar || TicksPerQuarterNote != Other.TicksPerQuarterNote || Points.Num() != Other.Points.Num())
	{
		return false;
	}
	for (int32 PointIndex = 0; PointIndex < Points.Num(); ++PointIndex)
	{
		if (Points[PointIndex] != Other.Points[PointIndex])
		{
			return false;
		}
	}
	return true;
}

void FBarMap::Empty()
{
	Points.Empty();
}

void FBarMap::Copy(const FBarMap& Other, int32 StartTick, int32 EndTick)
{
	TicksPerQuarterNote = Other.TicksPerQuarterNote;
	FMusicMapUtl::Copy(Other.Points, Points, StartTick, EndTick);
}

bool FBarMap::IsEmpty() const
{
	return Points.IsEmpty();
}

int32 FBarMap::CalculateMidiTick(const FMusicTimestamp& Timestamp, const EMidiClockSubdivisionQuantization Quantize) const
{
	int32 TriggerTick = 0;
	if (Quantize == EMidiClockSubdivisionQuantization::None)
	{
		TriggerTick = FMath::RoundToInt32(MusicTimestampToTick(Timestamp));
	}
	else
	{
		int32 RawTick = FMath::RoundToInt32(MusicTimestampToTick(Timestamp));
		int32 BarTick = MusicTimestampBarToTick(Timestamp.Bar);
		int32 TicksPerQuantizationUnit = SubdivisionToMidiTicks(Quantize, RawTick);
		if (ensure(TicksPerQuantizationUnit > 0))
		{
			float NumUnits = ((float)(RawTick - BarTick)) / (float)TicksPerQuantizationUnit;
			int32 NumWholeUnits = FMath::RoundToInt32(NumUnits);
			TriggerTick = BarTick + (NumWholeUnits * TicksPerQuantizationUnit);
		}
		else
		{
			TriggerTick = FMath::RoundToInt32(MusicTimestampToTick(Timestamp));
		}
	}
	return TriggerTick;
}

int32 FBarMap::SubdivisionToMidiTicks(const EMidiClockSubdivisionQuantization Division, const int32 AtTick) const
{
	FTimeSignature TimeSignature(4, 4);
	int32 BarMapPointIndex = GetPointIndexForTick(AtTick);
	if (BarMapPointIndex >= 0)
	{
		const FTimeSignaturePoint& TimeSignaturePoint = GetTimeSignaturePoint(BarMapPointIndex);
		TimeSignature = TimeSignaturePoint.TimeSignature;
	}
	return Harmonix::Midi::Constants::SubdivisionToMidiTicks(Division, TimeSignature, TicksPerQuarterNote);
}

float FBarMap::MusicTimestampToTick(const FMusicTimestamp& Timestamp) const
{
	float Beat = Timestamp.Beat;
	if (Beat < 1.0f)
	{
		if (Beat >= 0.0f)
		{
			UE_LOGF(LogMIDI, Warning, "Beat %f specified in Music Timestamp! Beats in Music Timestamps are '1' based. Beat 1.0 is the first beat in the specified bar. Adding 1.0 to the specified Beat!", Beat);
			Beat += 1.0f;
		}
		else
		{
			UE_LOGF(LogMIDI, Warning, "Negative beat (%f) specified in Music Timestamp. This is not allowed! Beats are '1' based in Music Timestamps and must be positive. Using beat 1!", Beat);
			Beat = 1.0f;
		}
	}

	int32 BarIndex = Timestamp.Bar - StartBar;
	Beat -= 1.0f;
	if (Points.IsEmpty())
	{
		return BarIndex * 4 * TicksPerQuarterNote +
			Beat * TicksPerQuarterNote;
	}
	
	int32 Index = Algo::UpperBound(Points, BarIndex, FTimeSignaturePoint::BarLessThan());

	if (Index > 0)
	{
		--Index;
	}

	if (!Points.IsValidIndex(Index))
	{
		Index = 0;
	}

	int32 BarDelta = BarIndex - Points[Index].BarIndex;
	return Points[Index].StartTick +
		BarDelta * GetTicksInBarAfterPoint(Index) +
		Beat * GetTicksInBeatAfterPoint(Index);
}

int32 FBarMap::MusicTimestampBarToTick(int32 InBar, int32* OutBeatsPerBar, int32* OutTicksPerBeat) const
{
	int32 BarIndex = InBar - StartBar;
	return BarIncludingCountInToTick(BarIndex, OutBeatsPerBar, OutTicksPerBeat);
}

int32 FBarMap::BarIncludingCountInToTick(int32 BarIndex, int32* OutBeatsPerBar, int32* OutTicksPerBeat) const
{
	if (Points.IsEmpty())
	{
		if (OutBeatsPerBar)
		{
			*OutBeatsPerBar = 4;
		}
		if (OutTicksPerBeat)
		{
			*OutTicksPerBeat = TicksPerQuarterNote;
		}
		return BarIndex * 4 * TicksPerQuarterNote;
	}

	int32 TimeSigIndex = Algo::UpperBound(Points, BarIndex, FTimeSignaturePoint::BarLessThan());

	if (TimeSigIndex > 0)
	{
		--TimeSigIndex;
	}

	if (!Points.IsValidIndex(TimeSigIndex))
	{
		TimeSigIndex = 0;
	}

	if (OutBeatsPerBar)
	{
		*OutBeatsPerBar = Points[TimeSigIndex].TimeSignature.Numerator;
	}
	if (OutTicksPerBeat)
	{
		*OutTicksPerBeat = GetTicksInBeatAfterPoint(TimeSigIndex);
	}
	int32 BarDelta = BarIndex - Points[TimeSigIndex].BarIndex;
	return Points[TimeSigIndex].StartTick + (BarDelta * GetTicksInBarAfterPoint(TimeSigIndex));
}

int32 FBarMap::MusicTimestampBarBeatTickToTick(int32 Bar, int32 BeatInBar, int32 TickInBeat) const
{
	int32 BarIndex = Bar - StartBar;
	return BarBeatTickIncludingCountInToTick(BarIndex, BeatInBar, TickInBeat);
}

int32 FBarMap::BarBeatTickIncludingCountInToTick(int32 BarIndex, int32 BeatInBar, int32 TickInBeat) const
{
	if (BeatInBar < 1)
	{
		UE_LOGF(LogMIDI, Warning, "Beat %d specified as a \"beat in bar\". Beat 1 is the first beat in a bar. Using Beat 1!", BeatInBar);
		BeatInBar = 0;
	}
	else
	{
		BeatInBar--;
	}

	if (Points.IsEmpty())
	{
		// Assume 4/4 time.
		return	(BarIndex * 4 * TicksPerQuarterNote) +
				(BeatInBar * TicksPerQuarterNote) +
			     TickInBeat;
	}

	int32 TimeSigIndex = Algo::UpperBound(Points, BarIndex, FTimeSignaturePoint::BarLessThan());

	if (TimeSigIndex > 0)
	{
		--TimeSigIndex;
	}

	if (!Points.IsValidIndex(TimeSigIndex))
	{
		TimeSigIndex = 0;
	}

	int32 BarDelta = BarIndex - Points[TimeSigIndex].BarIndex;
	int32 TicksPerBeat = GetTicksInBeatAfterPoint(TimeSigIndex);
	if (BeatInBar >= Points[TimeSigIndex].TimeSignature.Numerator)
	{
		UE_LOGF(LogMIDI, Warning, "BarBeatTickToAbsoluteTick: Supplied 'BeatInBar' is greater than the number of beats in the specified bar!");
	}
	if (TickInBeat >= TicksPerBeat)
	{
		UE_LOGF(LogMIDI, Warning, "BarBeatTickToAbsoluteTick: Supplied 'TickInBeat' is greater than the number of ticks in each beat at the specified bar!");
	}
	return Points[TimeSigIndex].StartTick +
			(BarDelta * GetTicksInBarAfterPoint(TimeSigIndex)) +
			(BeatInBar * TicksPerBeat) +
			TickInBeat;
}

void FBarMap::TickToBarBeatTickIncludingCountIn(int32 InTick, int32& OutBarIndex, int32& OutBeatInBar, int32& OutTickIndexInBeat, int32* OutBeatsPerBar, int32* OutTicksPerBeat) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		OutBarIndex = InTick / (TicksPerQuarterNote * 4);
		InTick -= OutBarIndex * (TicksPerQuarterNote * 4);
		OutBeatInBar = InTick / TicksPerQuarterNote;
		OutTickIndexInBeat = InTick - (OutBeatInBar * TicksPerQuarterNote);
		OutBeatInBar++; // Beat is one based to be consistent with BarBeatTickIncludingCountInToTick
		if (OutBeatsPerBar)
		{
			*OutBeatsPerBar = 4;
		}
		if (OutTicksPerBeat)
		{
			*OutTicksPerBeat = TicksPerQuarterNote;
		}
		return;
	}
	int32 TimeSigIndex = FMusicMapUtl::GetPointIndexForTick(Points, InTick);
	if (!Points.IsValidIndex(TimeSigIndex))
	{
		TimeSigIndex = 0;
	}
	int32 TicksPerBar = GetTicksInBarAfterPoint(TimeSigIndex);
	int32 TicksPerBeat = GetTicksInBeatAfterPoint(TimeSigIndex);
	int32 TicksPast = InTick - Points[TimeSigIndex].StartTick;
	int32 BarsPassed = TicksPast / TicksPerBar;
	OutBarIndex = Points[TimeSigIndex].BarIndex + BarsPassed;
	TicksPast -= BarsPassed * TicksPerBar;
	OutBeatInBar = TicksPast / TicksPerBeat;
	OutTickIndexInBeat = TicksPast - (OutBeatInBar * TicksPerBeat);
	OutBeatInBar++; // Beat is one based to be consistent with BarBeatTickIncludingCountInToTick
	if (OutBeatsPerBar)
	{
		*OutBeatsPerBar = Points[TimeSigIndex].TimeSignature.Numerator;
	}
	if (OutTicksPerBeat)
	{
		*OutTicksPerBeat = TicksPerBeat;
	}
}

FMusicTimestamp FBarMap::TickToMusicTimestamp(float Tick, int32* OutBeatsPerBar) const
{
	FMusicTimestamp Result;
	if (Points.IsEmpty())
	{
		// Assume 4/4 time.
		Result.Bar = int32(Tick) / (TicksPerQuarterNote * 4);
		Tick -= (Result.Bar * TicksPerQuarterNote * 4);
		if (Tick < 0)
		{
			Result.Bar--;
			Tick += TicksPerQuarterNote * 4;
		}
		Result.Beat = Tick / TicksPerQuarterNote;
		Result.Bar += StartBar;
		Result.Beat += 1.0f; // 1 based
		if (OutBeatsPerBar) *OutBeatsPerBar = 4;
		return Result;
	}

	int32 TimeSigIndex = FMusicMapUtl::GetPointIndexForTick(Points, int32(Tick));
	if (!Points.IsValidIndex(TimeSigIndex))
	{
		TimeSigIndex = 0;
	}

	if (Points[TimeSigIndex].TimeSignature.Numerator == 0 || Points[TimeSigIndex].TimeSignature.Denominator == 0)
	{
		// Assume 4/4 time.
		Result.Bar = int32(Tick) / (TicksPerQuarterNote * 4);
		Tick -= (Result.Bar * TicksPerQuarterNote * 4);
		if (Tick < 0)
		{
			Result.Bar--;
			Tick += TicksPerQuarterNote * 4;
		}
		Result.Beat = Tick / TicksPerQuarterNote;
		Result.Bar += StartBar;
		Result.Beat += 1.0f; // 1 based
		if (OutBeatsPerBar) *OutBeatsPerBar = 4;
		return Result;
	}

	int32 TicksPerBar = GetTicksInBarAfterPoint(TimeSigIndex);
	float TicksPassed = Tick - Points[TimeSigIndex].StartTick;
	if (TicksPassed < 0)
	{
		int32 BarsPassed = -int32(TicksPassed) / TicksPerBar + 1;
		TicksPassed += BarsPassed * TicksPerBar;
		float BeatsPassed = TicksPassed / GetTicksInBeatAfterPoint(TimeSigIndex);
		Result.Bar = -BarsPassed + StartBar; // already - 1 based
		Result.Beat = BeatsPassed + 1.0f; // 1 based
	}
	else
	{
		int32 BarsPassed = int32(TicksPassed) / TicksPerBar;
		TicksPassed -= BarsPassed * TicksPerBar;
		float BeatsPassed = TicksPassed / GetTicksInBeatAfterPoint(TimeSigIndex);
		Result.Bar = Points[TimeSigIndex].BarIndex + BarsPassed + StartBar; // 1 based
		Result.Beat = BeatsPassed + 1.0f; // 1 based
	}
	if (OutBeatsPerBar) *OutBeatsPerBar = Points[TimeSigIndex].TimeSignature.Numerator;
	return Result;
}

FMusicTimestamp FBarMap::TickFromBarOneToMusicTimestamp(float InTickFromBarOne, int32* OutBeatsPerBar) const
{
	return TickToMusicTimestamp(InTickFromBarOne + GetTickOfBarOne(), OutBeatsPerBar);
}

float FBarMap::TickToFractionalBeatIncludingCountIn(float Tick) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		return Tick / (TicksPerQuarterNote);
	}
	
	int32 PointIndex = FMusicMapUtl::GetPointIndexForTick(Points, int32(Tick));
	if (!Points.IsValidIndex(PointIndex))
	{
		return 0.0f;
	}

	float TicksPassed = Tick - Points[PointIndex].StartTick;
	int32 TicksPerBeat = GetTicksInBeatAfterPoint(PointIndex);
	float BeatsPassed = TicksPassed / TicksPerBeat;
	return  Points[PointIndex].BeatIndex + BeatsPassed;
}

float FBarMap::TickToFractionalBarIncludingCountIn(float Tick) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		return Tick / (TicksPerQuarterNote * 4);
	}

	int32 TimeSigIndex = FMusicMapUtl::GetPointIndexForTick(Points, int32(Tick));
	if (!Points.IsValidIndex(TimeSigIndex))
	{
		TimeSigIndex = 0;
	}
	float TicksPast = Tick - Points[TimeSigIndex].StartTick;
	int32 TicksPerBar = GetTicksInBarAfterPoint(TimeSigIndex);
	float BarsPassed = TicksPast / TicksPerBar;
	return Points[TimeSigIndex].BarIndex + BarsPassed;
}

int32 FBarMap::TickToBarIncludingCountIn(int32 Tick) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		return Tick / (TicksPerQuarterNote * 4);
	}
	int32 TimeSigIndex = FMusicMapUtl::GetPointIndexForTick(Points, Tick);
	if (!Points.IsValidIndex(TimeSigIndex))
	{
		TimeSigIndex = 0;
	}
	int32 TicksPast = Tick - Points[TimeSigIndex].StartTick;
	int32 TicksPerBar = GetTicksInBarAfterPoint(TimeSigIndex);
	int32 BarsPassed = TicksPast / TicksPerBar;
	return Points[TimeSigIndex].BarIndex + BarsPassed;
}

int32 FBarMap::TickToBeatIncludingCountIn(float Tick) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		return Tick / (TicksPerQuarterNote);
	}
	
	int32 PointIndex = FMusicMapUtl::GetPointIndexForTick(Points, int32(Tick));
	if (!Points.IsValidIndex(PointIndex))
	{
		return 0;
	}

	int32 TicksPassed = Tick - Points[PointIndex].StartTick;
	int32 TicksPerBeat = GetTicksInBeatAfterPoint(PointIndex);
	int32 BeatsPassed = TicksPassed / TicksPerBeat;
	return Points[PointIndex].BeatIndex + BeatsPassed;
}

float FBarMap::FractionalBarIncludingCountInToTick(float Bar) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		return Bar * 4.0f * TicksPerQuarterNote;
	}

	int32 BarIndex = FMath::FloorToInt32(Bar);
	int32 TimeSigIndex = Algo::UpperBound(Points, BarIndex, FTimeSignaturePoint::BarLessThan());

	if (TimeSigIndex > 0)
	{
		--TimeSigIndex;
	}

	if (!Points.IsValidIndex(TimeSigIndex))
	{
		TimeSigIndex = 0;
	}

	float BarsPassed = Bar - Points[TimeSigIndex].BarIndex;
	int32 TicksPerBar = GetTicksInBarAfterPoint(TimeSigIndex);
	return Points[TimeSigIndex].StartTick + BarsPassed * TicksPerBar;
}

float FBarMap::FractionalBeatIncludingCountInToTick(float Beat) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		return Beat * TicksPerQuarterNote;
	}

	int32 BeatIndex = FMath::FloorToInt32(Beat);
	int32 TimeSigIndex = Algo::UpperBound(Points, BeatIndex, FTimeSignaturePoint::BeatLessThan());
	
	if (TimeSigIndex > 0)
	{
		--TimeSigIndex;
	}

	if (!Points.IsValidIndex(TimeSigIndex))
    {
    	TimeSigIndex = 0;
    }
	
	float BeatsPassed = Beat - Points[TimeSigIndex].BeatIndex;
	int32 TicksPerBeat = GetTicksInBeatAfterPoint(TimeSigIndex);
	return Points[TimeSigIndex].StartTick + BeatsPassed * TicksPerBeat;
}

bool FBarMap::AddTimeSignatureAtMusicTimestampBar(int32 BarNumber, int32 Numerator, int32 Denominator, bool SortNow, bool FailOnError)
{
	int BarIndex = BarNumber - StartBar;
	return AddTimeSignatureAtBarIncludingCountIn(BarIndex, Numerator, Denominator, SortNow, FailOnError);
}

bool FBarMap::AddTimeSignatureAtBarIncludingCountIn(int32 BarIndex, int32 Numerator, int32 Denominator, bool SortNow, bool FailOnError)
{
	if (BarIndex < 0)
	{
		BarIndex = 0;
	}

	if (BarIndex == 0)
	{
		if (!Points.IsEmpty())
		{
			if (Points[0].BarIndex == 0 && Points[0].TimeSignature.Numerator == Numerator && Points[0].TimeSignature.Denominator == Denominator)
			{
				// Some DAWs are exporting midi with multiple duplicate time signatures in the file. (Looking at you Ableton!)
				UE_LOGF(LogMIDI, VeryVerbose, "Ignoring duplicate time signature at beginning of the MIDI data.");
				return true;
			}

			if (FailOnError)
			{
				checkf(false, TEXT("Multiple time signatures at start of song"));
			}
			return false;
		}
		FMusicMapUtl::AddPoint<FTimeSignaturePoint, int32, int32, const FTimeSignature&>(0, 0, FTimeSignature(Numerator, Denominator), Points, 0, SortNow);
	}
	else
	{
		FTimeSignaturePoint& LastSig = Points.Last();
		int32 BarsPassed = BarIndex - LastSig.BarIndex;
		if (BarsPassed <= 0)
		{
			if (FailOnError)
			{
				checkf(false, TEXT("Multiple time signatures at bar %d"), BarIndex + StartBar);
			}
			return false;
		}
		int32 TicksPassed  = BarsPassed * GetTicksInBarAfterPoint(Points.Num()-1);
		int32 ThisTick     = LastSig.StartTick + TicksPassed;
		int32 BeatsPassed = BarsPassed * LastSig.TimeSignature.Numerator;
		int32 ThisBeat    = LastSig.BeatIndex + BeatsPassed;
		FMusicMapUtl::AddPoint<FTimeSignaturePoint, int32, int32, const FTimeSignature&>(BarIndex, ThisBeat, FTimeSignature(Numerator, Denominator), Points, ThisTick, SortNow);
	}
	return true;
}

void FBarMap::WipeTimeSignaturePoints(int32 Tick)
{
	FMusicMapUtl::RemovePointsOnAndAfterTick(Points, Tick);
}

int32 FBarMap::GetNumTimeSignaturePoints() const
{
	return Points.Num();
}

int32 FBarMap::GetPointIndexForTick(int32 Tick) const
{
	return FMusicMapUtl::GetPointIndexForTick(Points, Tick);
}

const FTimeSignaturePoint& FBarMap::GetTimeSignaturePoint(int32 Index) const
{
	if (Points.IsEmpty())
	{
		return sDefaultTimeSignature;
	}

	if (!Points.IsValidIndex(Index))
	{
		Index = 0;
	}

	return Points[Index];
}

FTimeSignaturePoint& FBarMap::GetTimeSignaturePoint(int32 Index)
{
	check(!Points.IsEmpty());

	if (!Points.IsValidIndex(Index))
	{
		Index = 0;
	}

	return Points[Index];
}

const FTimeSignature& FBarMap::GetTimeSignatureAtTick(int32 Tick) const
{
	if (Points.IsEmpty())
	{
		return sDefaultTimeSignature.TimeSignature;
	}
	
	int32 Index = FMusicMapUtl::GetPointIndexForTick(Points, Tick);

	if (!Points.IsValidIndex(Index))
	{
		Index = 0;
	}

	return Points[Index].TimeSignature;
}

const FTimeSignaturePoint* FBarMap::GetTimeSignaturePointForTick(int32 Tick) const
{
	if (Points.IsEmpty())
	{
		return nullptr;
	}

	int32 Index = FMusicMapUtl::GetPointIndexForTick(Points, Tick);
	if (!Points.IsValidIndex(Index))
	{
		Index = 0;
	}
	return &Points[Index];
}

const FTimeSignature& FBarMap::GetTimeSignatureAtBar(int32 Bar) const
{
	if (Points.IsEmpty())
	{
		return sDefaultTimeSignature.TimeSignature;
	}

	Bar -= StartBar;

	int32 Index = Algo::UpperBound(Points, Bar, FTimeSignaturePoint::BarLessThan());

	if (Index > 0)
	{
		--Index;
	}

	if (!Points.IsValidIndex(Index))
	{
		Index = 0;
	}

	return Points[Index].TimeSignature;
}

void FBarMap::Finalize(int32 InLastTick)
{
	if (Points.IsEmpty())
	{
		SupplyDefault();
	}
	FMusicMapUtl::Finalize(Points, InLastTick);
}

int32 FBarMap::GetTimeSignatureChangePointTick(int32 PointIndex) const
{
	if (!Points.IsValidIndex(PointIndex))
	{
		return 0;
	}
	return Points[PointIndex].StartTick;
}
