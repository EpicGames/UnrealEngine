// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/BeatMap.h"
#include "HarmonixMidi/MidiConstants.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BeatMap)

bool FBeatMap::operator==(const FBeatMap& Other) const
{
	if (TicksPerQuarterNote != Other.TicksPerQuarterNote || Points.Num() != Other.Points.Num() || Bars.Num() != Other.Bars.Num())
	{
		return false;
	}
	for (int32 PointIndex = 0; PointIndex < Points.Num(); ++PointIndex)
	{
		if (Points[PointIndex] != Other.Points[PointIndex])
		{
			return false;
		}
	}

	for (int32 BarIndex = 0; BarIndex < Bars.Num(); ++BarIndex)
	{
		if (Bars[BarIndex] != Other.Bars[BarIndex])
		{
			return false;
		}
	}
	return true;
}

void FBeatMap::Empty()
{
	Points.Empty();
}

void FBeatMap::Copy(const FBeatMap& Other, int32 StartTick, int32 EndTick)
{
	TicksPerQuarterNote = Other.TicksPerQuarterNote;
	FMusicMapUtl::Copy(Other.Points, Points, StartTick, EndTick);
	FMusicMapUtl::Copy(Other.Bars, Bars, StartTick, EndTick);
}

bool FBeatMap::IsEmpty() const
{
	return Points.IsEmpty();
}

int32 FBeatMap::GetNumMapPoints() const
{
	return Points.Num();
}

void FBeatMap::AddBeat(EMusicalBeatType InType, int32 Tick, bool SortNow)
{
	FMusicMapUtl::AddPoint<FBeatMapPoint,EMusicalBeatType>(InType, Points, Tick, SortNow);
}

const FBeatMapPoint* FBeatMap::GetPointInfoForTick(int32 Tick) const
{
	return FMusicMapUtl::GetPointInfoForTick(Points, Tick);
}

int32 FBeatMap::GetPointIndexForTick(int32 Tick) const
{
	return FMusicMapUtl::GetPointIndexForTick(Points, Tick);
}

float FBeatMap::GetFractionalBeatAtTick(float Tick) const
{
	if (Points.IsEmpty())
	{
		// Assume x/4 time with all plain old beats...
		// Don't just floating-point divide by TicksPerQuarterNote because that can
		// get optimized to a multiply by the reciprocal, resulting in
		// non-integral beats even when tick is divisible by TicksPerQuarterNote.
		const int32 WholeBeats = int32(Tick) / TicksPerQuarterNote;
		const float RemainderTicks = Tick - WholeBeats * TicksPerQuarterNote;
		const float FractionalBeats = RemainderTicks / (float)TicksPerQuarterNote;
		return WholeBeats + FractionalBeats + 1.0f; // +1 because position is always 1 based.
	}

	return FMusicMapUtl::GetFractionalPointForTick(Points, Tick);
}

float FBeatMap::GetFractionalTickAtBeat(float Beat) const
{
	if (Points.IsEmpty())
	{
		return Beat * (float)TicksPerQuarterNote;
	}

	return FMusicMapUtl::GetFractionalTickForFractionalPoint(Points, Beat);
}

EMusicalBeatType FBeatMap::GetBeatTypeAtTick(int32 Tick) const
{
	if (Points.IsEmpty())
	{
		// assume 4/4
		int32 Beat = Tick / TicksPerQuarterNote;
		if (Beat % 4 == 0)
		{
			return EMusicalBeatType::Downbeat;
		}
		return EMusicalBeatType::Normal;
	}

	const FBeatMapPoint* Point = FMusicMapUtl::GetPointInfoForTick(Points, Tick);
	if (!Point)
	{
		return EMusicalBeatType::Normal;
	}
	return Point->Type;
}

float FBeatMap::GetBeatInPulseBarAtTick(float Tick) const
{
	int32 PointIndex = FMusicMapUtl::GetPointIndexForTick(Points, FMath::FloorToInt(Tick));
	if (PointIndex == -1)
	{
		return 1.0f; // 1 because beats relative to a bar are always 1 based.
	}
	float ProgressInBeat = Points[PointIndex].Progress(Tick);
	if (Bars.IsEmpty())
	{
		return PointIndex + ProgressInBeat + 1.0f; // +1 because beats relative to a bar are always 1 based.
	}

	int32 BarIndex = Points[PointIndex].PulseBar;
	const FPulseBar& PulseBar = Bars[BarIndex];
	int32 BeatInBar = PointIndex - PulseBar.FirstIncludedBeatIndex;
	return BeatInBar + ProgressInBeat + 1.0f; // +1 because beats relative to a bar are always 1 based.
}

int32 FBeatMap::GetNumBeatsInPulseBarAt(int32 Tick) const
{
	if (Bars.IsEmpty())
	{
		return Points.Num();
	}

	int32 BeatIndex = FMusicMapUtl::GetPointIndexForTick(Points, Tick);
	if (BeatIndex == -1)
	{
		return 0;
	}

	const FPulseBar& PulseBar = Bars[Points[BeatIndex].PulseBar];
	return PulseBar.LastIncludedBeatIndex - PulseBar.FirstIncludedBeatIndex + 1;
}

bool FBeatMap::IsDownbeat(float Beat) const
{
	int32 BeatIndex = FMath::FloorToInt32(Beat);
	if (Points.IsEmpty())
	{
		return (BeatIndex % 4 == 0);
	}

	if (Beat >= Points.Num())
	{
		return false;
	}

	return Points[BeatIndex].Type == EMusicalBeatType::Downbeat;
}

int32 FBeatMap::FindDownbeatIndexAfterBeat(float Beat) const
{
	return FindDownbeatIndexAtOrAfterBeat(FMath::FloorToFloat(Beat) + 1.0f);
}

int32 FBeatMap::FindDownbeatIndexAtOrAfterBeat(float Beat) const
{
	int32 BeatIndex = FMath::FloorToInt32(Beat);
	if (Points.IsEmpty())
	{
		return (((BeatIndex + 3) / 4) * 4);
	}

	for (int32 TestIndex = BeatIndex; TestIndex < Points.Num(); ++TestIndex)
	{
		if (Points[TestIndex].Type == EMusicalBeatType::Downbeat)
		{
			return TestIndex;
		}
	}
	return -1; // NO DOWNBEAT AT OR AFTER SPECIFIED BEAT!
}

const FBeatMapPoint& FBeatMap::GetBeatPointInfo(int32 Index) const
{
	return Points[Index];
}

void FBeatMap::Finalize(int32 LastTick)
{
	Bars.Empty();
	if (Points.IsEmpty())
	{
		return;
	}

	// first the beat points...
	FMusicMapUtl::Finalize(Points, LastTick);

	// Now bars...
	int32 PulseBarStartTick = Points[0].StartTick;
	int32 PulseBarLength = Points[0].LengthTicks;
	int32 FirstBeatInBar = 0;
	int32 LastBeatInBar = 0;
	Points[0].PulseBar = 0;
	for (int32 i = 1; i < Points.Num(); ++i)
	{
		if (Points[i].Type == EMusicalBeatType::Downbeat)
		{
			Bars.Emplace(PulseBarStartTick, PulseBarLength, FirstBeatInBar, LastBeatInBar);

			Points[i].PulseBar = Bars.Num();
			PulseBarStartTick = Points[i].StartTick;
			PulseBarLength = Points[i].LengthTicks;
			LastBeatInBar = FirstBeatInBar = i;
		}
		else
		{
			Points[i].PulseBar = Bars.Num();
			PulseBarLength = Points[i].StartTick - PulseBarStartTick;
			LastBeatInBar = i;
		}
	}
	Bars.Emplace(PulseBarStartTick, PulseBarLength, FirstBeatInBar, LastBeatInBar);
}

FString HARMONIXMIDI_API MusicalBeatTypeToString(EMusicalBeatType BeatType)
{
	switch (BeatType)
	{
	case EMusicalBeatType::Downbeat: return TEXT("Downbeat");
	case EMusicalBeatType::Strong:	 return TEXT("Strong");
	case EMusicalBeatType::Normal:	 return TEXT("Normal");
	}
	return TEXT("<Unrecognized>");
}
