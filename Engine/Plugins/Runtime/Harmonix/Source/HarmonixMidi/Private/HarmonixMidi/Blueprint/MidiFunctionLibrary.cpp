// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMidi/Blueprint/MidiFunctionLibrary.h"

#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/MidiSongPos.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiFunctionLibrary)

FMidiNote UMidiNoteFunctionLibrary::GetMinMidiNote()
{
	return FMidiNote(Harmonix::Midi::Constants::GMinNote);
}

FMidiNote UMidiNoteFunctionLibrary::GetMaxMidiNote()
{
	return FMidiNote(Harmonix::Midi::Constants::GMaxNote);
}

uint8 UMidiNoteFunctionLibrary::GetMinNoteNumber()
{
	return Harmonix::Midi::Constants::GMinNote;
}

uint8 UMidiNoteFunctionLibrary::GetMaxNoteNumber()
{
	return Harmonix::Midi::Constants::GMaxNote;
}

int UMidiNoteFunctionLibrary::GetMaxNumNotes()
{
	return Harmonix::Midi::Constants::GMaxNumNotes;
}

uint8 UMidiNoteFunctionLibrary::GetMinNoteVelocity()
{
	return Harmonix::Midi::Constants::GMinVelocity;
}

uint8 UMidiNoteFunctionLibrary::GetMaxNoteVelocity()
{
	return Harmonix::Midi::Constants::GMaxVelocity;
}

float UMusicalTickFunctionLibrary::GetTicksPerQuarterNote()
{
	return Harmonix::Midi::Constants::GTicksPerQuarterNote;
}

int32 UMusicalTickFunctionLibrary::GetTicksPerQuarterNoteInt()
{
	return Harmonix::Midi::Constants::GTicksPerQuarterNoteInt;
}

float UMusicalTickFunctionLibrary::GetQuarterNotesPerTick()
{
	return Harmonix::Midi::Constants::GQuarterNotesPerTick;
}

float UMusicalTickFunctionLibrary::TickToQuarterNote(float InTick)
{
	return InTick * Harmonix::Midi::Constants::GQuarterNotesPerTick;
}

float UMusicalTickFunctionLibrary::QuarterNoteToTick(float InQuarterNote)
{
	return InQuarterNote * Harmonix::Midi::Constants::GTicksPerQuarterNote;
}

bool UMidiSongPosFunctionLibrary::IsSongPosValid(const FMidiSongPos& SongPos)
{
	return SongPos.IsValid();
}

FMidiSongPos UMidiSongPosFunctionLibrary::LerpSongPos(const FMidiSongPos& A, const FMidiSongPos& B, float Alpha)
{
	return FMidiSongPos::Lerp(A, B, Alpha);
}

FMidiSongPos UMidiSongPosFunctionLibrary::MakeSongPosFromTime(float TimeMs, float BeatsPerMinute, int32 TimeSignatureNumerator /*= 4*/, int32 TimeSignatureDenominator /*= 4*/, int32 StartBar /*= 1*/)
{
	FMidiSongPos Result;
	Result.SetByTime(TimeMs, BeatsPerMinute, TimeSignatureNumerator, TimeSignatureDenominator, StartBar);
	return Result;
}
