// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMidi/Blueprint/MidiNote.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MidiFunctionLibrary.generated.h"

#define UE_API HARMONIXMIDI_API

struct FMidiSongPos;

/**
* Function library for FMidiNote and various midi note constants
*/
UCLASS(MinimalAPI)
class UMidiNoteFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Meta = (DisplayName = "FMidiNote To int64", CompactNodeTitle = "->", BlueprintAutocast), Category = "Midi")
	static int MidiNoteToInt(const FMidiNote& InMidiNote) { return InMidiNote.NoteNumber; }

	UFUNCTION(BlueprintPure, Meta = (DisplayName = "int64 to FMidiNote", CompactNodeTitle = "->", BlueprintAutocast), Category = "Midi")
	static FMidiNote IntToMidiNote(int InInt) { return FMidiNote(InInt); }

	UFUNCTION(BlueprintPure, Meta = (DisplayName = "FMidiNote To byte", CompactNodeTitle = "->", BlueprintAutocast), Category = "Midi")
	static uint8 MidiNoteToByte(const FMidiNote& InMidiNote) { return InMidiNote.NoteNumber; }

	UFUNCTION(BlueprintPure, Meta = (DisplayName = "byte to FMidiNote", CompactNodeTitle = "->", BlueprintAutocast), Category = "Midi")
	static FMidiNote ByteToMidiNote(uint8 InByte) { return FMidiNote(InByte); }

	UFUNCTION(BlueprintPure, Category = "Midi")
	static FMidiNote MakeLiteralMidiNote(FMidiNote Value) { return Value; }

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API FMidiNote GetMinMidiNote();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API FMidiNote GetMaxMidiNote();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API uint8 GetMinNoteNumber();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API uint8 GetMaxNoteNumber();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API int GetMaxNumNotes();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API uint8 GetMinNoteVelocity();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API uint8 GetMaxNoteVelocity();
};

/**
* Function library for converting Ticks to Beats and Beats to Ticks and other midi constants
*/
UCLASS(MinimalAPI)
class UMusicalTickFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API float GetTicksPerQuarterNote();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API int GetTicksPerQuarterNoteInt();
	 
	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API float GetQuarterNotesPerTick();

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API float TickToQuarterNote(float InTick);

	UFUNCTION(BlueprintPure, Category = "Midi")
	static UE_API float QuarterNoteToTick(float InQuarterNote);
};

/**
* Function library for FMidiSongPos
*/
UCLASS(MinimalAPI)
class UMidiSongPosFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Midi", meta = (DisplayName = "Is MIDI Song Position Valid"))
	static UE_API bool IsSongPosValid(const FMidiSongPos& SongPos);

	UFUNCTION(BlueprintPure, Category = "Midi", meta = (DisplayName = "Lerp (MIDI Song Position)"))
	static UE_API FMidiSongPos LerpSongPos(const FMidiSongPos& A, const FMidiSongPos& B, float Alpha);

	UFUNCTION(BlueprintPure, Category="Midi", meta=(NativeMakeFunc, DisplayName = "Make MIDI Song Position from Time"))
	static UE_API FMidiSongPos MakeSongPosFromTime(float TimeMs, float BeatsPerMinute, int32 TimeSignatureNumerator = 4, int32 TimeSignatureDenominator = 4, int32 StartBar = 1);
};

#undef UE_API
