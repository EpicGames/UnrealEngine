// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMidi/Blueprint/MidiNote.h"
#include "HarmonixMidi/MidiConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiNote)

FText FMidiNote::GetDisplayText() const
{
	return FText::FromString(ToString());
}

FName FMidiNote::GetDisplayName() const
{
	return FName(*ToString());
}

FString FMidiNote::ToString() const
{
	int32 Octave = Harmonix::Midi::Constants::GetNoteOctaveFromNoteNumber(NoteNumber);

	Harmonix::Midi::Constants::ENoteNameEnharmonicStyle Style = Harmonix::Midi::Constants::ENoteNameEnharmonicStyle::Sharp;

	return FString::Printf(TEXT("%s%d"), *FString(Harmonix::Midi::Constants::GetNoteNameFromNoteNumber(NoteNumber, Style)), Octave);
}

FMidiNote FMidiNote::FromString(const FString& NoteName)
{
	uint8 NoteNumber = NoteNumberFromString(NoteName);
	return FMidiNote(NoteNumber);
}

uint8 FMidiNote::NoteNumberFromString(const FString& NoteName)
{
	return Harmonix::Midi::Constants::GetNoteNumberFromNoteName(TCHAR_TO_ANSI(*NoteName));
}

#ifdef WITH_EDITOR

FString FMidiNote::ToEditorString() const
{
	// pad by 5 so that the numbers are approximately the same width 
	FString NumberString = FString::FromInt(NoteNumber).RightPad(5);
	return FString::Printf(TEXT("%s%s"), *NumberString, *ToString());
}

FMidiNote FMidiNote::FromEditorString(const FString& EditorName)
{
	uint8 NoteNumber = NoteNumberFromEditorString(EditorName);
	return FMidiNote(NoteNumber);
}

uint8 FMidiNote::NoteNumberFromEditorString(const FString& EditorName)
{
	int32 Index;
	// remove the numbers from the beginning of the string
	// there should be a space before the start of the note name
	if (EditorName.FindLastChar(' ', Index))
	{
		FString NoteName = EditorName.RightChop(Index + 1);
		return Harmonix::Midi::Constants::GetNoteNumberFromNoteName(TCHAR_TO_ANSI(*NoteName));
	}
	return Harmonix::Midi::Constants::GetNoteNumberFromNoteName(TCHAR_TO_ANSI(*EditorName));
}

#endif
