// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/ChordMap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChordMap)

bool FChordProgressionMap::operator==(const FChordProgressionMap& Other) const
{
	if (TicksPerQuarterNote != Other.TicksPerQuarterNote || Points.Num() != Other.Points.Num() || ChordTrackIndex != Other.ChordTrackIndex)
	{
		return false;
	}

	for (int32 PointIndex = 0; PointIndex < Points.Num(); ++PointIndex)
	{
		if (Points[PointIndex] != Other.Points[PointIndex])
		{
			return false;
		}
	}
	return true;
}

void FChordProgressionMap::Finalize(int32 LastTick)
{
	FMusicMapUtl::Finalize(Points, LastTick);
}

void FChordProgressionMap::Copy(const FChordProgressionMap& Other, int32 StartTick, int32 EndTick)
{
	TicksPerQuarterNote = Other.TicksPerQuarterNote;
	ChordTrackIndex = Other.ChordTrackIndex;
	FMusicMapUtl::Copy(Other.Points, Points, StartTick, EndTick);
}

bool FChordProgressionMap::IsEmpty() const
{
	return Points.IsEmpty();
}

void FChordProgressionMap::AddChord(FName Name, int32 InStartTick, bool SortNow)
{
	FMusicMapUtl::AddPoint<FChordMapPoint,FName>(Name, Points, InStartTick, SortNow);
}

const FChordMapPoint* FChordProgressionMap::GetPointInfoForTick(int32 Tick) const
{
	return FMusicMapUtl::GetPointInfoForTick(Points, Tick);
}

const void FChordProgressionMap::GetChordListCopy(TArray<FChordMapPoint>& ChordList) const
{
	ChordList = Points;
}

void FChordProgressionMap::Empty()
{
	Points.Empty();
	ChordTrackIndex = -1;
}

FName FChordProgressionMap::GetChordNameAtTick(int32 Tick) const
{
	const FChordMapPoint* Chord = FMusicMapUtl::GetPointInfoForTick(Points, Tick);
	return Chord ? Chord->Name : FName();
}
