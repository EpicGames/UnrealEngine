// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/SongMaps.h"

DEFINE_LOG_CATEGORY(LogMIDI);

namespace Harmonix::Midi::Constants
{
	int32 SubdivisionToMidiTicks(const EMidiClockSubdivisionQuantization Division, const FTimeSignature& TimeSignature, const int32 TicksPerQuarterNote)
	{
		switch (Division)
		{
		case EMidiClockSubdivisionQuantization::None:                   return 1;
		case EMidiClockSubdivisionQuantization::Bar: 					return TimeSignature.Numerator * ((TicksPerQuarterNote * 4) / TimeSignature.Denominator);
		case EMidiClockSubdivisionQuantization::Beat:					return (TicksPerQuarterNote * 4) / TimeSignature.Denominator;
		case EMidiClockSubdivisionQuantization::ThirtySecondNote:		return TicksPerQuarterNote / 8;
		case EMidiClockSubdivisionQuantization::SixteenthNote:			return TicksPerQuarterNote / 4;
		case EMidiClockSubdivisionQuantization::EighthNote:				return TicksPerQuarterNote / 2;
		case EMidiClockSubdivisionQuantization::QuarterNote:			return TicksPerQuarterNote;
		case EMidiClockSubdivisionQuantization::HalfNote:				return TicksPerQuarterNote * 2;
		case EMidiClockSubdivisionQuantization::WholeNote:				return TicksPerQuarterNote * 4;
		case EMidiClockSubdivisionQuantization::DottedSixteenthNote:	return (TicksPerQuarterNote / 4) + (TicksPerQuarterNote / 8);
		case EMidiClockSubdivisionQuantization::DottedEighthNote:		return (TicksPerQuarterNote / 2) + (TicksPerQuarterNote / 4);
		case EMidiClockSubdivisionQuantization::DottedQuarterNote:		return (TicksPerQuarterNote)+(TicksPerQuarterNote / 2);
		case EMidiClockSubdivisionQuantization::DottedHalfNote:			return (TicksPerQuarterNote * 2) + (TicksPerQuarterNote);
		case EMidiClockSubdivisionQuantization::DottedWholeNote:		return (TicksPerQuarterNote * 4) + (TicksPerQuarterNote * 2);
		case EMidiClockSubdivisionQuantization::SixteenthNoteTriplet:   return (TicksPerQuarterNote / 2) / 3;
		case EMidiClockSubdivisionQuantization::EighthNoteTriplet:		return TicksPerQuarterNote / 3;
		case EMidiClockSubdivisionQuantization::QuarterNoteTriplet:		return (TicksPerQuarterNote * 2) / 3;
		case EMidiClockSubdivisionQuantization::HalfNoteTriplet:        return (TicksPerQuarterNote * 4) / 3;
		default:	                                             		checkNoEntry();	return 1;
		}
	}

	float SubdivisionToBeats(EMidiClockSubdivisionQuantization Subdivision, const FTimeSignature& TimeSignature)
	{
		// Easy cases first
		if (Subdivision == EMidiClockSubdivisionQuantization::Bar)
		{
			return TimeSignature.Numerator;
		}

		if (Subdivision == EMidiClockSubdivisionQuantization::Beat)
		{
			return 1;
		}

		const float BeatsPerQuarter = TimeSignature.Denominator / 4.0f;

		switch (Subdivision)
		{
		case EMidiClockSubdivisionQuantization::ThirtySecondNote:
			return BeatsPerQuarter / 8;
		case EMidiClockSubdivisionQuantization::SixteenthNote:
			return BeatsPerQuarter / 4;
		case EMidiClockSubdivisionQuantization::EighthNote:
			return BeatsPerQuarter / 2;
		case EMidiClockSubdivisionQuantization::QuarterNote:
			return BeatsPerQuarter;
		case EMidiClockSubdivisionQuantization::HalfNote:
			return BeatsPerQuarter * 2;
		case EMidiClockSubdivisionQuantization::WholeNote:
			return BeatsPerQuarter * 4;
		case EMidiClockSubdivisionQuantization::DottedSixteenthNote:
			return BeatsPerQuarter / 4 + BeatsPerQuarter / 8;
		case EMidiClockSubdivisionQuantization::DottedEighthNote:
			return BeatsPerQuarter / 2 + BeatsPerQuarter / 4;
		case EMidiClockSubdivisionQuantization::DottedQuarterNote:
			return BeatsPerQuarter + BeatsPerQuarter / 2;
		case EMidiClockSubdivisionQuantization::DottedHalfNote:
			return BeatsPerQuarter * 3;
		case EMidiClockSubdivisionQuantization::DottedWholeNote:
			return BeatsPerQuarter * 6;
		case EMidiClockSubdivisionQuantization::SixteenthNoteTriplet:
			return (BeatsPerQuarter / 4) * 2 / 3;
		case EMidiClockSubdivisionQuantization::EighthNoteTriplet:
			return (BeatsPerQuarter / 2) * 2 / 3;
		case EMidiClockSubdivisionQuantization::QuarterNoteTriplet:
			return BeatsPerQuarter * 2 / 3;
		case EMidiClockSubdivisionQuantization::HalfNoteTriplet:
			return BeatsPerQuarter * 4 / 3;
		default:
			return 0;
		}
	}

	FString GetTextTypeName(uint8 TextType)
	{
		switch (TextType)
		{
		case GMeta_Text: return TEXT("Text");
		case GMeta_Copyright: return TEXT("Copyright");
		case GMeta_TrackName: return TEXT("Track Name");
		case GMeta_InstrumentName: return TEXT("Instrument Name");
		case GMeta_Lyric: return TEXT("Lyric");
		case GMeta_Marker: return TEXT("Marker");
		case GMeta_CuePoint: return TEXT("Cue Point");
		default:
			checkNoEntry();
			break;
		}
		return TEXT("");
	}

	float RoundToStandardBeatPrecision(float InBeat, int TimeSignatureDenominator)
	{
		// this is the precision to which the "rounded" elapsed beats get presented.
		const double kQuarterNotePrecision = 0.01 * ((double)TimeSignatureDenominator / 4.0); // account for beat being a quarter note, eighth note, etc.
		double x = (double)InBeat;
		x += 0.5 * kQuarterNotePrecision;
		x *= 1.0 / kQuarterNotePrecision;
		x = FMath::Floor(x);
		x *= kQuarterNotePrecision;
		return x;
	}

	FString GetControllerName(EControllerID ControllerId)
	{
		switch (ControllerId)
		{
		case EControllerID::BankSelection:       return FString::Format(TEXT("({0}) BankSelection"), { static_cast<uint8>(ControllerId) });
		case EControllerID::ModWheel:			 return FString::Format(TEXT("({0}) ModWheel"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Breath:				 return FString::Format(TEXT("({0}) Breath"), { static_cast<uint8>(ControllerId) });
		case EControllerID::PortamentoTime:		 return FString::Format(TEXT("({0}) PortamentoTime"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DataCoarse:			 return FString::Format(TEXT("({0}) DataCoarse"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Volume:				 return FString::Format(TEXT("({0}) Volume"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Balance:			 return FString::Format(TEXT("({0}) Balance"), { static_cast<uint8>(ControllerId) });
		case EControllerID::PanRight:			 return FString::Format(TEXT("({0}) Pan (0 = left, 1 = right)"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Expression:			 return FString::Format(TEXT("({0}) Expression"), { static_cast<uint8>(ControllerId) });
		case EControllerID::BitCrushWetMix:		 return FString::Format(TEXT("({0}) BitCrushWetMix"), { static_cast<uint8>(ControllerId) });
		case EControllerID::BitCrushLevel:		 return FString::Format(TEXT("({0}) BitCrushLevel"), { static_cast<uint8>(ControllerId) });
		case EControllerID::BitCrushSampleHold:	 return FString::Format(TEXT("({0}) BitCrushSampleHold"), { static_cast<uint8>(ControllerId) });
		case EControllerID::LFO0Frequency:		 return FString::Format(TEXT("({0}) LFO0Frequency"), { static_cast<uint8>(ControllerId) });
		case EControllerID::LFO1Frequency:		 return FString::Format(TEXT("({0}) LFO1Frequency"), { static_cast<uint8>(ControllerId) });
		case EControllerID::LFO0Depth:			 return FString::Format(TEXT("({0}) LFO0Depth"), { static_cast<uint8>(ControllerId) });
		case EControllerID::LFO1Depth:			 return FString::Format(TEXT("({0}) LFO1Depth"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayTime:			 return FString::Format(TEXT("({0}) DelayTime"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayDryGain:		 return FString::Format(TEXT("({0}) DelayDryGain"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayWetGain:		 return FString::Format(TEXT("({0}) DelayWetGain"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayFeedback:		 return FString::Format(TEXT("({0}) DelayFeedback"), { static_cast<uint8>(ControllerId) });
		case EControllerID::CoarsePitchBend:	 return FString::Format(TEXT("({0}) CoarsePitchBend"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SampleStartTime:	 return FString::Format(TEXT("({0}) SampleStartTime"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DataFine:			 return FString::Format(TEXT("({0}) DataFine"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol1:		 return FString::Format(TEXT("({0}) SubStreamVol1"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol2:		 return FString::Format(TEXT("({0}) SubStreamVol2"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol3:		 return FString::Format(TEXT("({0}) SubStreamVol3"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol4:		 return FString::Format(TEXT("({0}) SubStreamVol4"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol5:		 return FString::Format(TEXT("({0}) SubStreamVol5"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol6:		 return FString::Format(TEXT("({0}) SubStreamVol6"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol7:		 return FString::Format(TEXT("({0}) SubStreamVol7"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SubStreamVol8:		 return FString::Format(TEXT("({0}) SubStreamVol8"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayEQEnabled:		 return FString::Format(TEXT("({0}) DelayEQEnabled"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayEQType:		 return FString::Format(TEXT("({0}) DelayEQType"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayEQFreq:		 return FString::Format(TEXT("({0}) DelayEQFreq"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayEQQ:			 return FString::Format(TEXT("({0}) DelayEQQ"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Hold:				 return FString::Format(TEXT("({0}) Hold"), { static_cast<uint8>(ControllerId) });
		case EControllerID::PortamentoSwitch:	 return FString::Format(TEXT("({0}) PortamentoSwitch"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Sustenuto:			 return FString::Format(TEXT("({0}) Sustenuto"), { static_cast<uint8>(ControllerId) });
		case EControllerID::SoftPedal:			 return FString::Format(TEXT("({0}) SoftPedal"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Legato:				 return FString::Format(TEXT("({0}) Legato"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Hold2:				 return FString::Format(TEXT("({0}) Hold2"), { static_cast<uint8>(ControllerId) });
		case EControllerID::FilterQ:			 return FString::Format(TEXT("({0}) FilterQ"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Release:			 return FString::Format(TEXT("({0}) Release"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Attack:				 return FString::Format(TEXT("({0}) Attack"), { static_cast<uint8>(ControllerId) });
		case EControllerID::FilterFrequency:	 return FString::Format(TEXT("({0}) FilterFrequency"), { static_cast<uint8>(ControllerId) });
		case EControllerID::TimeStretchEnvelopeOrder:		 
			return FString::Format(TEXT("({0}) TimeStretchEnvelopeOrder"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayLFOBeatSync:	 return FString::Format(TEXT("({0}) DelayLFOBeatSync"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayLFOEnabled:	 return FString::Format(TEXT("({0}) DelayLFOEnabled"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayLFORate:		 return FString::Format(TEXT("({0}) DelayLFORate"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayLFODepth:		 return FString::Format(TEXT("({0}) DelayLFODepth"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayStereoType:	 return FString::Format(TEXT("({0}) DelayStereoType"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayPanLeft:		 return FString::Format(TEXT("({0}) DelayPanLeft"), { static_cast<uint8>(ControllerId) });
		case EControllerID::DelayPanRight:		 return FString::Format(TEXT("({0}) DelayPanRight"), { static_cast<uint8>(ControllerId) });
		case EControllerID::RPNFine:			 return FString::Format(TEXT("({0}) RPNFine"), { static_cast<uint8>(ControllerId) });
		case EControllerID::RPNCourse:			 return FString::Format(TEXT("({0}) RPNCourse"), { static_cast<uint8>(ControllerId) });
		case EControllerID::AllSoundOff:		 return FString::Format(TEXT("({0}) AllSoundOff"), { static_cast<uint8>(ControllerId) });
		case EControllerID::Reset:				 return FString::Format(TEXT("({0}) Reset"), { static_cast<uint8>(ControllerId) });
		case EControllerID::AllNotesOff:		 return FString::Format(TEXT("({0}) AllNotesOff"), { static_cast<uint8>(ControllerId) });
		}
		return FString::Format(TEXT("({0}) - <unassigned>"), { static_cast<uint8>(ControllerId) });
	}

	void GetControllerNames(TArray<FString>& Names)
	{
		for (int32 i = 0; i < 128; ++i)
		{
			Names.Add(GetControllerName((EControllerID)i));
		}
	}

	int8 GetNoteNumberFromNoteName(const char* InName)
	{
		if (isdigit(*InName))
			return (int8)atoi(InName);

		char Name[6];
		size_t Index = 0;
		for (; Index < 5 && Index < strlen(InName); Index++)
			Name[Index] = (char)toupper(InName[Index]);
		Name[Index] = 0;
		char* Walk = Name;

		int32 Note = -1;
		switch (*Walk)
		{
		case 'C': Note = 0;  break;
		case 'D': Note = 2;  break;
		case 'E': Note = 4;  break;
		case 'F': Note = 5;  break;
		case 'G': Note = 7;  break;
		case 'A': Note = 9;  break;
		case 'B': Note = 11; break;
		default: break;
		}
		if (Note == -1)
			return 0;
		Walk++;
		if (*Walk == '#') 
		{
			Note++; 
			Walk++; 
		}
		else if (*Walk == 'B')
		{
			Note--;
			Walk++; 
		}

		int32 Octave = 0;
		if (*Walk != '-')
			Octave = atoi(Walk) + 1;

		return int8(Note + (Octave * GNotesPerOctave));
	}

	const char* GetNoteNameFromNoteNumber(uint8 MidiNoteNumber, ENoteNameEnharmonicStyle Style)
	{
		typedef char const* CStr;
		static CStr sSharpAndFlat[GNotesPerOctave];
		static CStr sSharp[GNotesPerOctave];
		static CStr sFlat[GNotesPerOctave];

		sSharpAndFlat[0] = "C";
		sSharpAndFlat[1] = "C#/Db";
		sSharpAndFlat[2] = "D";
		sSharpAndFlat[3] = "D#/Eb";
		sSharpAndFlat[4] = "E";
		sSharpAndFlat[5] = "F";
		sSharpAndFlat[6] = "F#/Gb";
		sSharpAndFlat[7] = "G";
		sSharpAndFlat[8] = "G#/Ab";
		sSharpAndFlat[9] = "A";
		sSharpAndFlat[10] = "A#/Bb";
		sSharpAndFlat[11] = "B";

		sSharp[0] = "C";
		sSharp[1] = "C#";
		sSharp[2] = "D";
		sSharp[3] = "D#";
		sSharp[4] = "E";
		sSharp[5] = "F";
		sSharp[6] = "F#";
		sSharp[7] = "G";
		sSharp[8] = "G#";
		sSharp[9] = "A";
		sSharp[10] = "A#";
		sSharp[11] = "B";

		sFlat[0] = "C";
		sFlat[1] = "Db";
		sFlat[2] = "D";
		sFlat[3] = "Eb";
		sFlat[4] = "E";
		sFlat[5] = "F";
		sFlat[6] = "Gb";
		sFlat[7] = "G";
		sFlat[8] = "Ab";
		sFlat[9] = "A";
		sFlat[10] = "Bb";
		sFlat[11] = "B";

		CStr const* NameTable;
		
		switch (Style)
		{
		case ENoteNameEnharmonicStyle::Flat:
			NameTable = sFlat;
			break;

		case ENoteNameEnharmonicStyle::Sharp:
			NameTable = sSharp;
			break;
		case ENoteNameEnharmonicStyle::SharpAndFlat:
		default:
			NameTable = sSharpAndFlat;
			break;
		}

		check(NameTable != nullptr);
		uint8 Index = MidiNoteNumber % GNotesPerOctave;
		return NameTable[Index];
	}

	int8 GetNoteOctaveFromNoteNumber(uint8 MidiNoteNumber)
	{
		return (int8)MidiNoteNumber / GNotesPerOctave - 1;
	}

	FString MakeStdMsgString(uint8 Status, uint8 Data1, uint8 Data2)
	{
		uint8 Channel = GetChannel(Status);
		uint8 Type = GetType(Status);

		switch (Type)
		{
		case GNoteOff:
			return FString::Printf(TEXT("Ch #%d - Note Off: %d"), Channel, Data1);
		case GNoteOn:
			return FString::Printf(TEXT("Ch #%d - Note On: %d - %d"), Channel, Data1, Data2);
		case GPolyPres:
			return FString::Printf(TEXT("Ch #%d - Polyphonic Pressure : %d - %d"), Channel, Data1, Data2);
		case GControl:
			return FString::Printf(TEXT("Ch #%d - Control: %s: %d"), Channel, *GetControllerName((EControllerID)Data1), Data2);
		case GProgram:
			return FString::Printf(TEXT("Ch #%d - Patch Change: %d"), Channel, Data1);
		case GChanPres:
			return FString::Printf(TEXT("Ch #%d - Channel Pressure: %d"), Channel, Data1);
		case GPitch:
			return FString::Printf(TEXT("Ch #%d - Pitch Wheel: %d"), Channel, (uint32)Data1 | ((uint32)Data2 << 7));
		}
		return FString(TEXT("Unrecognized MIDI Event Type!"));
	}

	FString GetMetaEventTypeName(uint8 Type)
	{
		switch (Type)
		{
		case GMeta_Text:
			return FString(TEXT("Text"));
		case GMeta_Copyright:
			return FString(TEXT("Copyright"));
		case GMeta_TrackName:
			return FString(TEXT("Track Name"));
		case GMeta_InstrumentName:
			return FString(TEXT("Instrument Name"));
		case GMeta_Lyric:
			return FString(TEXT("Lyric"));
		case GMeta_Marker:
			return FString(TEXT("Marker"));
		case GMeta_CuePoint:
			return FString(TEXT("Cur Point"));
		case GMeta_ChannelPrefix:
			return FString(TEXT("Channel Prefix"));
		case GMeta_Port:
			return FString(TEXT("Port"));
		case GMeta_EndOfTrack:
			return FString(TEXT("End Of Track"));
		case GMeta_Tempo:
			return FString(TEXT("Tempo"));
		case GMeta_SMPTE:
			return FString(TEXT("SMPTE"));
		case GMeta_TimeSig:
			return FString(TEXT("Time Signature"));
		case GMeta_KeySig:
			return FString(TEXT("Key Signature"));
		case GMeta_Special:
			return FString(TEXT("Special"));
		}
		return FString(TEXT("Unrecognized MIDI Meta Event Type!"));
	}
}
