// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMidi/MidiCursor.h"
#include "Algo/BinarySearch.h"

void FMidiCursor::Prepare(UMidiFile* InMidiFile, int32 TrackIndex, bool bResetState)
{
	if (!InMidiFile)
	{
		MidiFile = nullptr;
		return;
	}
	Prepare(InMidiFile->GetOrCreateRenderableCopy(), TrackIndex, bResetState);
}

void FMidiCursor::Prepare(TSharedPtr<FMidiFileData> InMidiFile, int32 TrackIndex, bool bResetState)
{
	WatchTrack = TrackIndex;

	if (MidiFile == InMidiFile)
	{
		if (bResetState)
		{
			SeekToNextTick(0);
		}
		return;
	}

	MidiFile = InMidiFile;
	if (MidiFile)
	{
		// these will be initialized below with one of the seeks...
		TrackNextEventIndexs.SetNum(MidiFile->Tracks.Num());
	}

	if (bResetState)
	{
		SeekToNextTick(0);
	}
	else
	{
		SeekToNextTick(NextTick);
	}
}

void FMidiCursor::SeekToNextTick(int32 NewNextTick, int32 PrerollBars, FReceiver* PrerollReceiver)
{
	if (!MidiFile)
	{
		NextTick = NewNextTick;
		return;
	}

	if (NewNextTick > 0 && PrerollBars > 0 && PrerollReceiver)
	{
		int32 BarIndex = FMath::FloorToInt32(MidiFile->SongMaps.GetBarIncludingCountInAtTick(NewNextTick)) - PrerollBars;
		if (BarIndex < 0)
		{
			BarIndex = 0;
		}
		int32 PrerollTick = MidiFile->SongMaps.BarIncludingCountInToTick(BarIndex);
		SeekToNextTick(PrerollTick);
		Preroll(PrerollTick, NewNextTick - 1, *PrerollReceiver);
		return;
	}

	const TArray<FMidiTrack>& Tracks = MidiFile->Tracks;

	int32 StartTrackIndex = (WatchTrack < 0) ? 0 : WatchTrack;
	int32 EndTrackIndex = (WatchTrack < 0) ? Tracks.Num() : (WatchTrack + 1);
	if (EndTrackIndex > Tracks.Num())
	{
		EndTrackIndex = Tracks.Num();
	}

	for (int32 TrackIndex = StartTrackIndex; TrackIndex < EndTrackIndex; ++TrackIndex)
	{
		const FMidiTrack& Track = Tracks[TrackIndex];
		const FMidiEventList& Events = Track.GetEvents();
		TrackNextEventIndexs[TrackIndex] = Algo::LowerBoundBy(Events, NewNextTick,&FMidiEvent::GetTick);
		if (TrackNextEventIndexs[TrackIndex] >= Events.Num())
		{
			TrackNextEventIndexs[TrackIndex] = -1;
		}
	}

	UpdateNextTick(NewNextTick);
}

int32 FMidiCursor::SeekToMs(float NewPositionMs, int32 PrerollBars, FReceiver* PrerollReceiver)
{
	if (!MidiFile)
	{
		SeekToNextTick(0);
		return 0;
	}

	int32 NewNextTick = MidiFile->SongMaps.MsToTick(NewPositionMs);
	SeekToNextTick(NewNextTick, PrerollBars, PrerollReceiver);
	return NewNextTick;
}

void FMidiCursor::Process(int32 FirstTickToProcess, int32 LastTickToProcess, FReceiver& Receiver)
{
	if (!MidiFile)
	{
		NextTick = LastTickToProcess + 1;
		return;
	}

	if (FirstTickToProcess != NextTick)
	{
		SeekToNextTick(FirstTickToProcess);
	}

	const TArray<FMidiTrack>& Tracks = MidiFile->Tracks;

	int32 StartTrackIndex = (WatchTrack < 0) ? 0 : WatchTrack;
	int32 EndTrackIndex = (WatchTrack < 0) ? Tracks.Num() : (WatchTrack + 1);
	if (EndTrackIndex > Tracks.Num())
	{
		EndTrackIndex = Tracks.Num();
	}

	for (int32 TrackIndex = StartTrackIndex; TrackIndex < EndTrackIndex; ++TrackIndex)
	{
		const FMidiTrack& Track = Tracks[TrackIndex];
		const FMidiEventList& Events = Track.GetEvents();
		while (TrackNextEventIndexs[TrackIndex] != -1 && LastTickToProcess >= Events[TrackNextEventIndexs[TrackIndex]].GetTick())
		{
			if (!Receiver.HandleMessage(TrackIndex, Track, Events[TrackNextEventIndexs[TrackIndex]], false))
			{
				const FMidiEvent& Event = Events[TrackNextEventIndexs[TrackIndex]];
				UE_LOGF(LogMIDI, Error, "Unknown MIDI message type %d on track %d at tick %d, file %ls", int(Event.GetMsg().MsgType()), TrackIndex, Event.GetTick(), *MidiFile->MidiFileName);
			}
			TrackNextEventIndexs[TrackIndex]++;
			if (TrackNextEventIndexs[TrackIndex] == Events.Num())
			{
				TrackNextEventIndexs[TrackIndex] = -1;
			}
		}
	}

	UpdateNextTick(LastTickToProcess + 1);
}

void FMidiCursor::Preroll(int32 FirstTickToProcess, int32 LastTickToProcess, FReceiver& Receiver)
{
	if (!MidiFile)
	{
		NextTick = LastTickToProcess + 1;
		return;
	}

	if (FirstTickToProcess != NextTick)
	{
		SeekToNextTick(FirstTickToProcess);
	}

	const TArray<FMidiTrack>& Tracks = MidiFile->Tracks;

	// We only want to broadcast note-on messages that don't have a corresponding note-off message
	// between the preroll start tick and the current tick.
	int32 StartTrackIndex = (WatchTrack < 0) ? 0 : WatchTrack;
	int32 EndTrackIndex = (WatchTrack < 0) ? Tracks.Num() : (WatchTrack + 1);
	if (EndTrackIndex > Tracks.Num())
	{
		EndTrackIndex = Tracks.Num();
	}

	TMap<uint32, const FMidiEvent*> DeferredNoteOnMsgs;
	DeferredNoteOnMsgs.Reserve(32);
	auto MakeMapKey = [](const FMidiMsg* Message) 
	{
		return  static_cast<uint32>(Message->GetStdStatus() & 0xf) << 8 | static_cast<uint32>(Message->GetStdData1()); 
	};

	float FileMsAfterProcess = MidiFile->SongMaps.TickToMs(LastTickToProcess + 1);

	for (int32 TrackIndex = StartTrackIndex; TrackIndex < EndTrackIndex; ++TrackIndex)
	{
		const FMidiTrack& Track = Tracks[TrackIndex];
		const FMidiEventList& Events = Track.GetEvents();
		while (TrackNextEventIndexs[TrackIndex] != -1 && LastTickToProcess >= Events[TrackNextEventIndexs[TrackIndex]].GetTick())
		{
			const FMidiMsg& Msg = Events[TrackNextEventIndexs[TrackIndex]].GetMsg();
			bool Defer = false;
			if (Msg.MsgType() == FMidiMsg::EType::Std)
			{
				if (Msg.IsNoteOff())
				{
					DeferredNoteOnMsgs.Remove(MakeMapKey(&Msg));
					Defer = true;
				}
				else if (Msg.IsNoteOn())
				{
					DeferredNoteOnMsgs.Add(MakeMapKey(&Msg), &Events[TrackNextEventIndexs[TrackIndex]]);
					Defer = true;
				}
			}
			if (!Defer)
			{
				if (!Receiver.HandleMessage(TrackIndex, Track, Events[TrackNextEventIndexs[TrackIndex]], true))
				{
					const FMidiEvent& Event = Events[TrackNextEventIndexs[TrackIndex]];
					UE_LOGF(LogMIDI, Error, "Unknown MIDI message type %d on track %d at tick %d, file %ls", int(Event.GetMsg().MsgType()), TrackIndex, Event.GetTick(), *MidiFile->MidiFileName);
				}
			}
			TrackNextEventIndexs[TrackIndex]++;
			if (TrackNextEventIndexs[TrackIndex] == Events.Num())
			{
				TrackNextEventIndexs[TrackIndex] = -1;
			}
		}

		for (auto Iterator : DeferredNoteOnMsgs)
		{
			const FMidiEvent* MidiEvent = Iterator.Value;
			const FMidiMsg& Msg = MidiEvent->GetMsg();
			int32 EventTick = MidiEvent->GetTick();
			float EventMs = MidiFile->SongMaps.TickToMs(EventTick);
			Receiver.OnPreRollNoteOn(TrackIndex, MidiEvent->GetTick(), LastTickToProcess + 1, FileMsAfterProcess - EventMs, Msg.GetStdStatus(), Msg.GetStdData1(), Msg.GetStdData2());
		}
		DeferredNoteOnMsgs.Reset();
	}
		
	UpdateNextTick(LastTickToProcess + 1);
}

bool FMidiCursor::PassedEnd() const
{
	if (!MidiFile)
	{
		return true;
	}

	int32 StartTrackIndex = (WatchTrack < 0) ? 0 : WatchTrack;
	int32 EndTrackIndex = (WatchTrack < 0) ? MidiFile->Tracks.Num() : (WatchTrack + 1);
	if (EndTrackIndex > MidiFile->Tracks.Num())
	{
		EndTrackIndex = MidiFile->Tracks.Num();
	}

	for (int32 TrackIndex = StartTrackIndex; TrackIndex < EndTrackIndex; ++TrackIndex)
	{
		if (TrackNextEventIndexs[TrackIndex] != -1)
		{
			return false;
		}
	}
	return true;
}

void FMidiCursor::UpdateNextTick(int32 NewNextTick)
{
	NextTick = NewNextTick;
	CurrentFileMs = MidiFile->SongMaps.TickToMs(NextTick);
}

bool FMidiCursor::FReceiver::HandleMessage(int32 TrackIndex, const FMidiTrack& Track, const FMidiEvent& Event, bool bIsPreroll)
{
	const FMidiMsg& Msg = Event.GetMsg();
	//Call callbacks for each event that the cursor is interested in
	switch (Msg.MsgType())
	{
	case FMidiMsg::EType::Std:
		OnMidiMessage(TrackIndex, Event.GetTick(), Msg.GetStdStatus(), Msg.GetStdData1(), Msg.GetStdData2(), bIsPreroll);
		break;
	case FMidiMsg::EType::Tempo:
		OnTempo(TrackIndex, Event.GetTick(), Msg.GetMicrosecPerQuarterNote(), bIsPreroll);
		break;
	case FMidiMsg::EType::Text:
		OnText(TrackIndex, Event.GetTick(), Msg.GetTextIndex(), Track.GetTextAtIndex(Msg.GetTextIndex()), Msg.GetTextType(), bIsPreroll);
		break;
	case FMidiMsg::EType::TimeSig:
		OnTimeSig(TrackIndex, Event.GetTick(), Msg.GetTimeSigNumerator(), Msg.GetTimeSigDenominator(), bIsPreroll);
		break;
	default:
		return false;
	}
	return true;
}
