// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiEvent.h"
#include "HarmonixMidi/MidiMsg.h"
#include "HarmonixMidi/MidiWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiEvent)

void FMidiEvent::WriteStdMidi(FMidiWriter& Writer, const FMidiTrack& Track) const
{
	Message.WriteStdMidi(Tick, Writer, Track);
}

bool FMidiEvent::Serialize(FArchive& Archive)
{
	Archive << Tick;
	Message.Serialize(Archive);
	return true;
}

FMidiEvent::FMidiEvent(int32 InTick, const FMidiMsg& InMessage) 
	: Tick(InTick)
	, Message(InMessage)
{
}
