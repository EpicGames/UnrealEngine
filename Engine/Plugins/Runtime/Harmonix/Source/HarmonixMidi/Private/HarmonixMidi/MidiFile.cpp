// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiFile.h"
#include "HarmonixMidi/MidiTrack.h"
#include "HarmonixMidi/MidiReader.h"
#include "HarmonixMidi/MidiReceiver.h"
#include "HarmonixMidi/MidiMsg.h"
#include "HarmonixMidi/MidiEvent.h"
#include "HarmonixMidi/MidiWriter.h"
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMidi/SongMapReceiver.h"
#include "FrameBasedMusicMap.h"
#include "Misc/Paths.h"
#include "Algo/ForEach.h"
#include "UObject/AssetRegistryTagsContext.h"
#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif
#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

//////////////////////////////////////////////////////////////
//
// private helper class for reading a std midi file into a UMidiFile
//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiFile)

#define LOCTEXT_NAMESPACE "Harmonix_MidiFile"

class FMidiEventReceiver : public IMidiReceiver
{
public:
	FMidiEventReceiver(UMidiFile& target);
	int32 GetLastTick() const { return LastTick; }

private:
	virtual bool Reset() override
	{
		LastTick = 0;
		CurrentTrackHasName = false;
		return true;
	}
	virtual bool OnNewTrack(int32 NewTrackIndex) override;
	virtual bool OnEndOfTrack(int32 InLastTick) override;
	virtual bool OnAllTracksRead() override { return true; }
	virtual bool OnMidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2) override;
	virtual bool OnTempo(int32 Tick, int32 Tempo) override;
	virtual bool OnText(int32 Tick, const FString& Str, uint8 Type) override;
	virtual bool OnTimeSignature(int32 Tick, int32 Numerator, int32 Denominator, bool FailOnError = true) override;

	UMidiFile& File;
	int32  LastTick = 0;
	bool   CurrentTrackHasName = false;
};


////////////////////////////////////////
UMidiFile::UMidiFile()
{
	// add a single conductor track as the 0th track:
	TheMidiData.Tracks.Emplace("Conductor");
}

bool UMidiFile::operator==(const UMidiFile& Other) const
{
	return TheMidiData == Other.TheMidiData;
}

void UMidiFile::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	
	Context.AddTag(UObject::FAssetRegistryTag("NumTracks", FString::FromInt(GetNumTracks()), UObject::FAssetRegistryTag::TT_Numerical));
	
	FString Tempo = FString::Printf(TEXT("%.2f BPM"), TheMidiData.SongMaps.GetTempoAtTick(0));
	Context.AddTag(UObject::FAssetRegistryTag("InitialTempo",  Tempo, UObject::FAssetRegistryTag::TT_Numerical));
	
	FString Length = FString::Printf(TEXT("%.5f Bars ( %s )"), TheMidiData.SongMaps.GetSongLengthData().LengthFractionalBars, *TheMidiData.SongMaps.GetSongLengthString());
	Context.AddTag(UObject::FAssetRegistryTag("Length", Length, UObject::FAssetRegistryTag::TT_Numerical));

#if WITH_EDITORONLY_DATA
	// this seems useful
	// this would allow us to inspect import data about this asset without fully loading the asset
	// storing it as a JSON is something that is done in other places (Like in StaticMesh)
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(
			GET_MEMBER_NAME_CHECKED(UMidiFile, AssetImportData),
			AssetImportData->GetSourceData().ToJson(),
			UObject::FAssetRegistryTag::TT_Hidden)
		);
	}
#endif

}

#if WITH_EDITORONLY_DATA
FString UMidiFile::GetImportedSrcFilePath() const
{
	return AssetImportData->GetFirstFilename();
}

#endif

TSharedPtr<FMidiFileData> UMidiFile::GetOrCreateRenderableCopy()
{
	if (!RenderableCopyOfMidiFileData)
	{
		RenderableCopyOfMidiFileData = MakeShared<FMidiFileData>(TheMidiData);
	}
	return RenderableCopyOfMidiFileData;
}

TSharedPtr<Audio::IProxyData> UMidiFile::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	TSharedPtr<FMidiFileData> Renderable = GetOrCreateRenderableCopy();
	TSharedPtr<FMidiFileProxy> Proxy = MakeShared<FMidiFileProxy>(Renderable);
	return Proxy;
}

void UMidiFile::BeginDestroy()
{
	RenderableCopyOfMidiFileData = nullptr;
	Super::BeginDestroy();
}

void UMidiFile::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (!AssetImportData && !HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
}

UMidiFile::ELoadResult UMidiFile::LoadStdMidiFile(
	const FString& FilePath,
	int32 DesiredTicksPerQuarterNote,
	Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding,
	bool EnsureNotFailed)
{
	FString Filename = FPaths::GetCleanFilename(FilePath);
	IPlatformFile& PlatformFileApi = FPlatformFileManager::Get().GetPlatformFile();
	IFileHandle* FileHandle = PlatformFileApi.OpenRead(*FilePath);
	TSharedPtr<FArchiveFileReaderGeneric> Archive = MakeShared<FArchiveFileReaderGeneric>(FileHandle, *Filename, FileHandle->Size());
	const ELoadResult Result = LoadStdMidiFile(Archive, Filename, DesiredTicksPerQuarterNote, InTextEncoding, EnsureNotFailed);

	if (Result != ELoadResult::Success)
	{
		return Result;
	}
	
#if WITH_EDITORONLY_DATA
	if (!AssetImportData)
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
	AssetImportData->Update(FilePath);
#endif

	return Result;
}

UMidiFile::ELoadResult UMidiFile::LoadStdMidiFile(
	void* Buffer,
	int32 BufferSize,
	const FString& Filename,
	int32 DesiredTicksPerQuarterNote,
	Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding,
	bool EnsureNotFailed)
{
	TSharedPtr<FBufferReader> BufferArchive = MakeShared<FBufferReader>(Buffer, BufferSize, false);
	return LoadStdMidiFile(BufferArchive, Filename, DesiredTicksPerQuarterNote, InTextEncoding, EnsureNotFailed);
}

UMidiFile::ELoadResult UMidiFile::LoadStdMidiFile(
	TSharedPtr<FArchive> Archive,
	const FString& Filename,
	int32 DesiredTicksPerQuarterNote,
	Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding,
	bool EnsureNotFailed)
{
	
	TheMidiData.TicksPerQuarterNote = DesiredTicksPerQuarterNote;
	TheMidiData.SongMaps.Init(TheMidiData.TicksPerQuarterNote);
	TheMidiData.Tracks.Empty();
	// Create a receiver list...
	FMidiReceiverList AllReceivers;
	// Add the event receiver that will populate the midi tracks with midi events...
	FMidiEventReceiver EventReceiver(*this);
	AllReceivers.Add(&EventReceiver);
	// Add the map receiver that will populate the song maps...
	FSongMapReceiver MapReceiver(&TheMidiData.SongMaps);
	AllReceivers.Add(&MapReceiver);

	// Create the reader and read all the data...
	FStdMidiFileReader reader(Archive, Filename, &AllReceivers, TheMidiData.TicksPerQuarterNote, InTextEncoding);
	reader.ReadAllTracks();

	if (reader.IsFailed())
	{
		if (EnsureNotFailed)
		{
			ensureAlwaysMsgf(false, TEXT("%s: MIDI import failed. Midi data is malformed. Check the log for details."), *Filename);
		}
		else
		{
			UE_LOGF(LogMIDI, Error, "%ls MIDI import failed. Midi data is malformed. Check the log for details.", *Filename);
		}
		TheMidiData.Empty();
		return ELoadResult::Failed;
	}
	
	// we are now "dirty"... so make sure any new requests for renderable data 
	// get a new copy...
	RenderableCopyOfMidiFileData = nullptr;
	return ELoadResult::Success;
}

void UMidiFile::SaveStdMidiFile(const FString& FilePath) const
{
	FString Filename = FPaths::GetCleanFilename(FilePath);
	IPlatformFile& PlatformFileApi = FPlatformFileManager::Get().GetPlatformFile();
	IFileHandle* FileHandle = PlatformFileApi.OpenWrite(*FilePath);
	if (ensureAlwaysMsgf(FileHandle, TEXT("Failed to open file \"%s\" for writing."), *FilePath))
	{
		TSharedPtr<FArchiveFileWriterGeneric> Archive = MakeShared<FArchiveFileWriterGeneric>(FileHandle, *Filename, 0);
		SaveStdMidiFile(Archive, Filename);
	}
}

void UMidiFile::SaveStdMidiFile(TSharedPtr<FArchive> Archive, const FString& Filename) const
{
	FMidiWriter Writer(*Archive, TheMidiData.TicksPerQuarterNote);
	Algo::ForEach(TheMidiData.Tracks, [&](const FMidiTrack& track) { track.WriteStdMidi(Writer); });
}

void UMidiFile::CreateFrameBasedMusicMap(UFrameBasedMusicMap* Map) const
{
	TheMidiData.SongMaps.FillInFrameBasedMusicMap(Map);
}

FMarkerProviderResults UMidiFile::GatherMarkers(const UFrameBasedMusicMap* Map) const
{
	FMarkerProviderResults Results;

	if (TheMidiData.Tracks.Num() > 0)
	{
		const FMidiTrack& ConductorTrack = TheMidiData.Tracks[0];
		const FMidiEventList& Events = ConductorTrack.GetEvents();
		for (const FMidiEvent& Event : Events)
		{
			const FMidiMsg& Msg = Event.GetMsg();
			if (Msg.IsText() && (Msg.GetTextType() == Harmonix::Midi::Constants::GMeta_CuePoint || Msg.GetTextType() == Harmonix::Midi::Constants::GMeta_Marker))
			{
				if (Results.Channels.IsEmpty())
				{
					Results.Channels.Emplace(TEXT("Conductor"));
				}
				float Ms = TheMidiData.SongMaps.TickToMs(static_cast<float>(Event.GetTick()));
				FFrameNumber Location = Map->FrameResolution.AsFrameNumber(static_cast<double>(Ms) / 1000.0);;
				Results.Channels[0].Markers.Emplace(ConductorTrack.GetTextForMsg(Msg), Location);
			}
		}
	}
	return Results;
}

FMidiTrack* UMidiFile::AddTrack(const FString& Name)
{
	TheMidiData.Tracks.Emplace(Name);
	return &TheMidiData.Tracks.Last();
}

void UMidiFile::Empty()
{
	TheMidiData.Empty();
}

bool UMidiFile::IsEmpty() const 
{
	return TheMidiData.IsEmpty();
}

//void UMidiFile::SetConductorTrack(const FTempoMap* TempoMap, const FBarMap* BarMap)
void UMidiFile::BuildConductorTrack()
{
	if (TheMidiData.Tracks.IsEmpty())
	{
		TheMidiData.Tracks.Emplace("Conductor");
	}
	else if (TheMidiData.Tracks[0].GetName()->Compare("Conductor", ESearchCase::IgnoreCase))
	{
		TheMidiData.Tracks.Insert(FMidiTrack("Conductor"), 0);
	}
	else
	{
		// use assignment operator to properly clear string table, events, and other data.
		TheMidiData.Tracks[0] = FMidiTrack("Conductor");
	}

	int32 numTempoChanges = TheMidiData.SongMaps.GetNumTempoChanges();

	int32 i;
	for (i = 0; i < numTempoChanges; ++i)
	{
		const FTempoInfoPoint* Point = TheMidiData.SongMaps.GetTempoInfoPoint(i);
		check(Point);
		int32 tick = Point->StartTick;
		float msPerQuarterNote = Point->GetMsPerQuarterNote();
		int32 usecPerQuarterNote = int32(msPerQuarterNote * 1000);
		TheMidiData.Tracks[0].AddEvent(FMidiEvent(tick, FMidiMsg(usecPerQuarterNote)));
	}

	int32 numTimeSigChanges = TheMidiData.SongMaps.GetNumTimeSignatureChanges();

	for (i = 0; i < numTimeSigChanges; ++i)
	{
		const FTimeSignaturePoint* sig = TheMidiData.SongMaps.GetTimeSignaturePoint(i);
		check(sig);
		TheMidiData.Tracks[0].AddEvent(FMidiEvent(sig->StartTick, FMidiMsg(sig->TimeSignature.Numerator, sig->TimeSignature.Denominator)));
	}

	TheMidiData.Tracks[0].Sort();
}

void UMidiFile::SortAllTracks()
{
	for (auto& Track : TheMidiData.Tracks)
	{
		Track.Sort();
	}
}

const FMidiTrack* UMidiFile::FindTrackByName(const FString& Name) const
{
	int32 trackIndex = FindTrackIndexByName(Name);
	if (trackIndex == INDEX_NONE)
	{
		return nullptr;
	}
	return &TheMidiData.Tracks[trackIndex];
}

int32 UMidiFile::FindTrackIndexByName(const FString& Name) const
{
	for (int32 i = 0; i < TheMidiData.Tracks.Num(); ++i)
	{
		const FMidiTrack& Track = TheMidiData.Tracks[i];
		if (Track.GetName() && *Track.GetName() == Name)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

int32 UMidiFile::FindTextEvent(const FString& eventText, const FString* TrackName)
{
	if (TrackName)
	{
		int32 TrackIndex = FindTrackIndexByName(*TrackName);
		if (TrackIndex == INDEX_NONE)
		{
			return INDEX_NONE;
		}
		return FindTextEvent(eventText, TrackIndex);
	}
	else
	{
		for (int32 i = 0; i < TheMidiData.Tracks.Num(); ++i)
		{
			int32 FoundIndex = FindTextEvent(eventText, i);
			if (FoundIndex >= 0)
			{
				return FoundIndex;
			}
		}
	}
	return INDEX_NONE;
}

int32 UMidiFile::FindTextEvent(const FString& EventText, int32 TrackIndex)
{
	if (TrackIndex >= TheMidiData.Tracks.Num())
	{
		return INDEX_NONE;
	}
	if (!TheMidiData.Tracks[TrackIndex].GetTextRepository())
	{
		return INDEX_NONE;
	}

	// first lets see if the request text is even in the track's repository...
	int32 StringIndex = 0;
	FMidiTextRepository* TextRepo = TheMidiData.Tracks[TrackIndex].GetTextRepository();
	for (; StringIndex < TextRepo->Num(); StringIndex++)
	{
		if ((*TextRepo)[StringIndex] == EventText)
		{
			break;
		}
	}
	if (StringIndex == TextRepo->Num())
	{
		return INDEX_NONE;
	}

	// it's in the list, so now let's find the event...
	for (const FMidiEvent& m : TheMidiData.Tracks[TrackIndex].GetEvents())
	{
		if (m.GetMsg().MsgType() == FMidiMsg::EType::Text && m.GetMsg().GetTextIndex() == StringIndex)
		{
			return m.GetTick();
		}
	}
	// not found? How can this be if it was in the textRepo?
	return INDEX_NONE;
}

TArray<int32> UMidiFile::FindAllTextEvents(const FString& EventText, const FString* TrackName)
{
	TArray<int32> ticks;

	if (TrackName)
	{
		int32 TrackIndex = FindTrackIndexByName(*TrackName);
		if (TrackIndex != INDEX_NONE)
		{
			FindAllTextEvents(EventText, TrackIndex, ticks);
		}
	}
	else
	{
		for (int32 i = 0; i < TheMidiData.Tracks.Num(); ++i)
		{
			FindAllTextEvents(EventText, i, ticks);
		}
	}
	return ticks;
}

void UMidiFile::FindAllTextEvents(const FString& EventText, int32 TrackIndex, TArray<int32> OutIndexes)
{
	// first lets see if the request text is even in the track's repository...
	int32 StringIndex = 0;
	FMidiTextRepository* TextRepo = TheMidiData.Tracks[TrackIndex].GetTextRepository();
	for (; StringIndex < TextRepo->Num(); StringIndex++)
	{
		if ((*TextRepo)[StringIndex] == EventText)
		{
			break;
		}
	}

	// if we found it, then look through the events...
	if (StringIndex != TextRepo->Num())
	{
		for (const FMidiEvent& m : TheMidiData.Tracks[TrackIndex].GetEvents())
		{
			if (m.GetMsg().MsgType() == FMidiMsg::EType::Text && m.GetMsg().GetTextIndex() == StringIndex)
			{
				OutIndexes.Add(m.GetTick());
			}
		}
	}
}

void UMidiFile::ScanTracksForSongLengthChange()
{
	TheMidiData.ScanTracksForSongLengthChange();
	MarkPackageDirty();
}

bool UMidiFile::LengthIsAPerfectSubdivision() const
{
	return TheMidiData.LengthIsAPerfectSubdivision();
}

void UMidiFile::QuantizeLengthToNearestPerfectSubdivision(const EMidiFileQuantizeDirection Direction)
{
	EMidiClockSubdivisionQuantization Subdivision = EMidiClockSubdivisionQuantization::None;
	int32 QuantizedLength = TheMidiData.SongMaps.QuantizeTickToAnyNearestSubdivision(TheMidiData.SongMaps.GetSongLengthData().LengthTicks, Direction, Subdivision);
	if (QuantizedLength == 0)
	{
		// If we were specifically asked to go down and going down gets
		// us a zero length file, then warn and don't quantize...
		if (Direction == EMidiFileQuantizeDirection::Down)
		{
			UE_LOGF(LogMIDI, Warning, "QuantizeLengthToNearestPerfectSubdivision: Asked to Quantize file length DOWN, but that would result in a zero length midi file. NOT ALLOWED! Skipping quantization!");
			return;
		}

		// If we were ask to go up and we STILL got a zero length
		// file that is super weird! Warn the user and skip quantizing...
		if (Direction == EMidiFileQuantizeDirection::Up)
		{
			UE_LOGF(LogMIDI, Error, "This is odd. Asked to quantize MIDI file length UP to the nearest subdivision, but returned length is zero! Skipping.");
			return;
		}

		// We MUST have been asked to quantize to nearest, but nearest must be
		// down and result in a zero length file. So... this time force up, as
		// that will be the only valid "nearest"!
		QuantizedLength = TheMidiData.SongMaps.QuantizeTickToAnyNearestSubdivision(TheMidiData.SongMaps.GetSongLengthData().LengthTicks, EMidiFileQuantizeDirection::Up, Subdivision);
		check (QuantizedLength > 0);
	}
	TheMidiData.ConformToLength(QuantizedLength);
	MarkPackageDirty();
}

void UMidiFile::QuantizeLengthToSubdivision(const EMidiFileQuantizeDirection Direction, const EMidiClockSubdivisionQuantization Subdivision)
{
	int32 QuantizedLength = TheMidiData.SongMaps.QuantizeTickToNearestSubdivision(TheMidiData.SongMaps.GetSongLengthData().LengthTicks, Direction, Subdivision);
	if (QuantizedLength == 0)
	{
		// If we were specifically asked to go down and going down gets
		// us a zero length file, then warn and don't quantize...
		if (Direction == EMidiFileQuantizeDirection::Down)
		{
			UE_LOGF(LogMIDI, Warning, "QuantizeLengthToNearestPerfectSubdivision: Asked to Quantize file length DOWN, but that would result in a zero length midi file. NOT ALLOWED! Skipping quantization!");
			return;
		}

		// If we were ask to go up and we STILL got a zero length
		// file that is super weird! Warn the user and skip quantizing...
		if (Direction == EMidiFileQuantizeDirection::Up)
		{
			UE_LOGF(LogMIDI, Error, "This is odd. Asked to quantize MIDI file length UP to the nearest subdivision, but returned length is zero! Skipping.");
			return;
		}

		// We MUST have been asked to quantize to nearest, but nearest must be
		// down and result in a zero length file. So... this time force up, as
		// that will be the only valid "nearest"!
		QuantizedLength = TheMidiData.SongMaps.QuantizeTickToNearestSubdivision(TheMidiData.SongMaps.GetSongLengthData().LengthTicks, EMidiFileQuantizeDirection::Up, Subdivision);
		check(QuantizedLength > 0);
	}
	TheMidiData.ConformToLength(QuantizedLength);
	MarkPackageDirty();
}

bool UMidiFile::ConformToLength(int32 NewLengthTicks)
{
	if (TheMidiData.ConformToLength(NewLengthTicks))
	{
		MarkPackageDirty();
		return true;
	}
	return false;
}

bool UMidiFile::ConformToLengthGivenLastEventTick(int32 NewLastEventTick)
{
	if (TheMidiData.ConformToLengthGivenLastEventTick(NewLastEventTick))
	{
		MarkPackageDirty();
		return true;
	}
	return false;
}

#if WITH_EDITORONLY_DATA
void UMidiFile::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{	
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName StartBarPropertyName = GET_MEMBER_NAME_CHECKED(UMidiFile, StartBar);
	
	if (PropertyChangedEvent.Property->GetName() == StartBarPropertyName)
	{
		TheMidiData.SongMaps.SetStartBar(StartBar);
	}
}
#endif

#if WITH_EDITOR

EDataValidationResult UMidiFile::IsDataValid(FDataValidationContext& Context) const
{
	if (TheMidiData.IsEmpty())
	{
		Context.AddError(LOCTEXT("MidiFile_DatIsEmpty", "MidiFile data is empty"));
		return EDataValidationResult::Invalid;
	}
	return EDataValidationResult::Valid;
}

#endif

FMidiEventReceiver::FMidiEventReceiver(UMidiFile& file)
	: File(file)
{
}

bool FMidiEventReceiver::OnNewTrack(int32 newTrackIndex)
{
	File.GetTracks().Emplace();
	CurrentTrackHasName = false;
	return true;
}

bool FMidiEventReceiver::OnEndOfTrack(int InLastTick)
{
	if (InLastTick > LastTick)
	{
		LastTick = InLastTick;
	}
	if (!CurrentTrackHasName)
	{
		if (File.GetTracks().IsEmpty())
		{
			return false;
		}

		FMidiTrack& Track = File.GetTracks().Last();
		uint16 StringIndex = Track.AddText("track_1");

		Track.AddEvent(FMidiEvent(0, FMidiMsg::CreateText(StringIndex, Harmonix::Midi::Constants::GMeta_TrackName)));
		Track.Sort();
		CurrentTrackHasName = true;
	}
	return true;
}

bool FMidiEventReceiver::OnMidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2)
{
	if (File.GetTracks().IsEmpty())
	{
		return false;
	}

	File.GetTracks().Last().AddEvent(FMidiEvent(Tick, FMidiMsg(Status, Data1, Data2)));

	return true;
}

bool FMidiEventReceiver::OnTempo(int32 Tick, int32 Tempo)
{
	if (File.GetTracks().IsEmpty())
	{
		return false;
	}

	File.GetTracks().Last().AddEvent(FMidiEvent(Tick, FMidiMsg(Tempo)));

	return true;
}

bool FMidiEventReceiver::OnText(int32 Tick, const FString& Str, uint8 Type)
{
	if (File.GetTracks().IsEmpty())
	{
		return false;
	}

	FMidiTrack& Track = File.GetTracks().Last();
	uint16 StringIndex = Track.AddText(Str);

	if (Type == Harmonix::Midi::Constants::GMeta_TrackName)
	{
		// Track name should always appear at tick 0!
		if (Tick != 0 && !CurrentTrackHasName)
		{
			UE_LOGF(LogMIDI, Warning, "MIDI track name (\"%ls\") event found at tick %d but none found at tick 0! The first track name event on a midi track should be on tick 0. Shifting it to tick 0.",
				*Str, Tick);
			Tick = 0;
		}
		else if (CurrentTrackHasName && Str != *Track.GetName())
		{
			UE_LOGF(LogMIDI, Warning, "2nd Track Name event found on MIDI track %ls --> %ls at tick %d",
				**Track.GetName(), *Str , Tick);
		}
		CurrentTrackHasName = true;
	}

	Track.AddEvent(FMidiEvent(Tick, FMidiMsg::CreateText(StringIndex, Type)));

	return true;
}

bool FMidiEventReceiver::OnTimeSignature(int32 Tick, int32 Numerator, int32 Denominator, bool FailOnError)
{
	if (File.GetTracks().IsEmpty())
	{
		return false;
	}

	File.GetTracks().Last().AddEvent(FMidiEvent(Tick, FMidiMsg(Numerator, Denominator)));

	return true;
}

bool FMidiFileData::operator==(const FMidiFileData& Other) const
{
	if (Tracks.Num() != Other.Tracks.Num())
	{
		return false;
	}
	for (int32 TrackIndex = 0; TrackIndex < Tracks.Num(); ++TrackIndex)
	{
		if (Tracks[TrackIndex] != Other.Tracks[TrackIndex])
		{
			return false;
		}
	}

	return MidiFileName == Other.MidiFileName &&
		TicksPerQuarterNote == Other.TicksPerQuarterNote &&
		SongMaps == Other.SongMaps;
}

int32 FMidiFileData::GetLastEventTick() const
{
	return SongMaps.GetSongLengthData().LastTick;
}

int32 FMidiFileData::FindTrackIndexByName(const FString& TrackName)
{
	for (int32 i = 0; i < Tracks.Num(); ++i)
	{
		const FMidiTrack& Track = Tracks[i];
		if (Track.GetName() && *Track.GetName() == TrackName)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

void FMidiFileData::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && SongMaps.BarMapIsEmpty() && Tracks.Num() > 0)
	{
		UE_LOGF(LogMIDI, Warning, "Empty bar map. Rebuilding.");
		const FMidiEventList& Events = Tracks[0].GetEvents();
		SongMaps.SetStartBar(1);
		for (auto& Event : Events)
		{
			if (Event.GetMsg().MsgType() == FMidiMsg::EType::TimeSig)
			{
				int32 Tick = Event.GetTick();
				int32 BarIndex;
				check(Tick == 0 || SongMaps.GetNumTimeSignatureChanges() > 0);
				if (Tick == 0)
				{
					BarIndex = 0;
				}
				else
				{
					BarIndex = SongMaps.TickToBarIncludingCountIn(Tick);
				}
				SongMaps.AddTimeSignatureAtBarIncludingCountIn(BarIndex, Event.GetMsg().GetTimeSigNumerator(), Event.GetMsg().GetTimeSigDenominator());
			}
		}
		SongMaps.FinalizeBarMap(GetLastEventTick());
	}
}

bool FMidiFileData::ConformToLength(int32 NewLengthTicks)
{
	return ConformToLengthGivenLastEventTick(NewLengthTicks - 1);
}

bool FMidiFileData::ConformToLengthGivenLastEventTick(int32 NewLastEventTick)
{
	if (NewLastEventTick < 0)
	{
		NewLastEventTick = 0;
		UE_LOGF(LogMIDI, Warning, "Asked to comform length such that the last event tick would be less than 0... Not possible! Conforming length to 1 tick, with last event being placed on tick 0!");
	}

	if (SongMaps.GetSongLengthData().LastTick == NewLastEventTick && SongMaps.GetSongLengthData().LengthTicks != 0)
	{
		// nothing to change...
		return false;
	}

	if (SongMaps.GetSongLengthData().LastTick < NewLastEventTick)
	{
		// easy... just change the length data...
		SongMaps.SetSongLengthTicks(NewLastEventTick + 1);
		// we changed something so report it...
		return true;
	}

	//keep track of the events that are moved to the last tick and excessive events removed after moving
	int32 NumEventsMovedToLastTick = 0;
	int32 TotalNumEventsRemoved = 0;
	int32 NumNoteOnNoteOffPairsRemoved = 0;
	int32 NumControlEventsRemoved = 0;

	//go through midi events in each midi track 
	for (int32 TrackIndex = 0; TrackIndex < Tracks.Num(); ++TrackIndex)
	{
		//Get the raw midi events and the last tick of the last event
		FMidiTrack& Track = Tracks[TrackIndex];
		FMidiEventList& Events = Track.GetRawEvents();
		int32 LastEventTick = Events.Last().GetTick();

		//if the last tick of this event is less than the tick of the conformed last tick,
		//this midi track does not need to be conformed, go to the next midi track
		if (LastEventTick <= NewLastEventTick) continue;

		//move all events with tick > ConfromedLastEventTick to ConformedLastTick
		for (int32 EventIndex = Events.Num() - 1; EventIndex >= 0 && Events[EventIndex].GetTick() > NewLastEventTick; --EventIndex)
		{
			// NOTE: It is safe for us to do this BECAUSE...
			// We do not have to re-sort the midi data on the tracks because while we may have slid events earlier,
			// their relative position to each other MUST be unchanged!
			Events[EventIndex].Tick = NewLastEventTick;
			NumEventsMovedToLastTick++;
		}

		//filter the midi events (at the new conformed last tick) after moving them, remove note on/note off pairs on the last tick, and other excessive events
		int32 NumItemsRemoved = 0;
		int32 CurrentEventIndex = Events.Num() - 1;

		while (CurrentEventIndex >= 0 && Events[CurrentEventIndex].GetTick() == NewLastEventTick)
		{
			using namespace Harmonix::Midi::Constants;

			FMidiEvent& Event = Events[CurrentEventIndex];
			FMidiMsg& Msg = Event.GetMsg();

			if (Msg.IsStd())
			{
				uint8 MsgType = Msg.GetStdStatusType();

				//filter Note On/Note off pairs on the last tick
				if (MsgType == GNoteOff)
				{
					int32 NoteOnEventIndex = -1;
					for (int32 i = CurrentEventIndex - 1; i > 0 && Events[i].GetTick() == NewLastEventTick; --i)
					{
						//check equality of note on/note off events' midi note number (data1) 
						if (Events[i].GetMsg().IsNoteOn() && Events[i].GetMsg().GetStdData1() == Msg.GetStdData1())
						{
							NoteOnEventIndex = i;
							break;
						}
					}
					//remove note on/note off pairs on the conformed last tick
					if (NoteOnEventIndex != -1)
					{
						Events.RemoveAt(CurrentEventIndex);
						NumItemsRemoved++;
						Events.RemoveAt(NoteOnEventIndex);
						NumItemsRemoved++;
						NumNoteOnNoteOffPairsRemoved++;
						TotalNumEventsRemoved += 2;
					}
				}

				//filter chan press/pitch bend/program change events
				else if (MsgType == GChanPres || MsgType == GPitch || MsgType == GProgram)
				{
					TArray<int32> OtherEventIndexesToRemove;
					//check if there exist multiple events with same status on the last tick
					for (int32 i = CurrentEventIndex - 1; i >= 0 && Events[i].GetTick() == NewLastEventTick; --i)
					{
						if (Events[i].GetMsg().IsStd() &&
							Events[i].GetMsg().Status == Msg.Status) 
						{
							OtherEventIndexesToRemove.Add(i);
						}
					}
					//remove excessive events on the conformed last tick
					if (!OtherEventIndexesToRemove.IsEmpty())
					{
						for (int32 i : OtherEventIndexesToRemove)
						{
							Events.RemoveAt(i);
						}
						NumItemsRemoved += OtherEventIndexesToRemove.Num();
						NumControlEventsRemoved += OtherEventIndexesToRemove.Num();
						TotalNumEventsRemoved += OtherEventIndexesToRemove.Num();
					}
				}

				//filter Control Change events and Poly Pres events
				else if (MsgType == GControl || MsgType == GPolyPres)
				{
					TArray<int32> ControlEventIndexesToRemove;
					for (int32 i = CurrentEventIndex - 1; i >= 0 && Events[i].GetTick() == NewLastEventTick; --i)
					{
						//check control change events for identical controller ID (data1) on the same tick and remove the later one
						//check poly pres events for the same note number (data1) on the same tick and remove the later one
						if (Events[i].GetMsg().IsStd() &&
							Events[i].GetMsg().GetStdStatus() == Msg.GetStdStatus() && 
							Events[i].GetMsg().Data1 == Msg.Data1)
						{
							ControlEventIndexesToRemove.Add(i);
						}
					}
					//remove excessive control changes or poly pressure on conformed last tick
					if (!ControlEventIndexesToRemove.IsEmpty())
					{
						for (int32 i : ControlEventIndexesToRemove)
						{
							Events.RemoveAt(i);
						}
						NumItemsRemoved += ControlEventIndexesToRemove.Num();
						NumControlEventsRemoved += ControlEventIndexesToRemove.Num();
						TotalNumEventsRemoved += ControlEventIndexesToRemove.Num();
					}
				}

				//update indices after removing excessive events
				if (NumItemsRemoved == 0)
				{
					CurrentEventIndex -= 1;
				}
				else 
				{
					CurrentEventIndex -= NumItemsRemoved;
					NumItemsRemoved = 0;
				}
			} 
			else
			{
				CurrentEventIndex--;
			}
		}
	}

	// TracksChanged may have cause the song length data to
	// shrink down because we have have removed all of the 
	// "past-the-end" events above, and be left with an 
	// unquantized length. So here will will force the song
	// length to the proper valyue.
	SongMaps.SetSongLengthTicks(NewLastEventTick + 1);

	//Log conform summary 
	FString MidiFileLengthConformSummary = FString::Printf(
		TEXT("Midi file '%s.mid' length conformance summary: moved %d Midi event(s) to tick %d"),
		*MidiFileName,
		NumEventsMovedToLastTick,
		NewLastEventTick);
	if (TotalNumEventsRemoved > 0)
	{
		MidiFileLengthConformSummary.Append(FString::Printf(TEXT("\n\tRemoved %d Midi event(s):"), TotalNumEventsRemoved));
		if (NumNoteOnNoteOffPairsRemoved > 0) MidiFileLengthConformSummary.Append(FString::Printf(TEXT("\n\t\t%d Note On/Note off event pair(s)"), NumNoteOnNoteOffPairsRemoved));
		if (NumControlEventsRemoved > 0) MidiFileLengthConformSummary.Append(FString::Printf(TEXT("\n\t\t%d Aftertouch / Program / Pitch Wheel / CC event(s)"), NumControlEventsRemoved));
		MidiFileLengthConformSummary.Append(FString::Printf(TEXT("\n\tat tick %d "), NewLastEventTick));
	}
	UE_LOGF(LogMIDI, Log, "%ls", *MidiFileLengthConformSummary);
	
	return true;
}

void FMidiFileData::AddTempoChange(int32 TrackIdx, int32 Tick, float TempoBPM)
{
	check(Tracks.IsValidIndex(TrackIdx));

	int32 MidiTempo = Harmonix::Midi::Constants::BPMToMidiTempo(TempoBPM);
	Tracks[TrackIdx].AddEvent(FMidiEvent(Tick, FMidiMsg(MidiTempo)));
	Tracks[TrackIdx].Sort();
	SongMaps.AddTempoInfoPoint(MidiTempo, Tick);
}

void FMidiFileData::AddTimeSigChange(int32 TrackIdx, int32 Tick, int32 InTimeSigNum, int32 InTimeSigDenom)
{
	check(Tracks.IsValidIndex(TrackIdx));

	// Time signature changes can only happen at the beginning of a bar,
	// so round up to the next bar boundary...
	int32 AbsoluteBar = FMath::CeilToInt32(SongMaps.GetBarIncludingCountInAtTick(Tick));
	Tick = SongMaps.BarBeatTickIncludingCountInToTick(AbsoluteBar, 1, 0);
	int32 TimeSigNum = FMath::Clamp(InTimeSigNum, 1, 64);
	int32 TimeSigDenom = FMath::Clamp(InTimeSigDenom, 1, 64);
	Tracks[TrackIdx].AddEvent(FMidiEvent(Tick, FMidiMsg((uint8)TimeSigNum, (uint8)TimeSigDenom)));
	Tracks[TrackIdx].Sort();
	SongMaps.AddTimeSignatureAtBarIncludingCountIn(AbsoluteBar, TimeSigNum, TimeSigDenom);
}

void FMidiFileData::ScanTracksForSongLengthChange()
{
	int32 NewLastEventTick = 0;
	bool bIsFileEmpty = true;
	for (auto& Track : Tracks)
	{
		FMidiEventList& Events = Track.GetRawEvents();
		if (bIsFileEmpty && !Events.IsEmpty())
		{
			bIsFileEmpty = false;
		}
		int32 LastTickOnTrack = Events.IsEmpty() ? 0 : Events.Last().GetTick();
		if (NewLastEventTick < LastTickOnTrack)
		{
			NewLastEventTick = LastTickOnTrack;
		}
	}

	FSongLengthData& LengthData = SongMaps.GetSongLengthData();

	//update length data in song length data if needed 
	LengthData.LengthTicks = NewLastEventTick + 1;
	LengthData.LastTick = NewLastEventTick;
	LengthData.LengthFractionalBars = SongMaps.TickToFractionalBarIncludingCountIn(LengthData.LengthTicks);
}

bool FMidiFileData::LengthIsAPerfectSubdivision() const
{
	return SongMaps.LengthIsAPerfectSubdivision();
}


#undef LOCTEXT_NAMESPACE // Harmonix_MidiFile
