// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiMsg.h"
#include "HarmonixMidi/MidiWriter.h"
#include "HarmonixMidi/MidiTrack.h"
#include "HarmonixMidi/MidiConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiMsg)

FMidiMsg FMidiMsg::CreateNoteOn(int32 Channel, int32 Note, int32 Velocity)
{
	return FMidiMsg(0x90 | (uint8)(Channel & 0xF), (uint8)Note, (uint8)Velocity);
}

FMidiMsg FMidiMsg::CreateNoteOff(int32 Channel, int32 Note)
{
	return FMidiMsg(0x80 | (uint8)(Channel & 0xF), (uint8)Note, 0);
}

FMidiMsg FMidiMsg::CreateControlChange(uint8 Channel, uint8 ControlNumber, uint8 Value)
{
	const uint8 StatusByte = Harmonix::Midi::Constants::GControl | (Channel & 0xf);
	return FMidiMsg{ StatusByte, ControlNumber, Value };
}

FMidiMsg FMidiMsg::CreateText(uint16 InTextIndex, uint8 InTextType)
{
	FMidiMsg Msg;
	Msg.Type = EType::Text;
	Msg.TextIndex = InTextIndex;
	Msg.TextType = InTextType;
	return Msg;
}

float FMidiMsg::GetPitchBendFromData(int8 InData1, int8 InData2)
{
	static const int16 kMaxPitch = 0x3FFF;
	static const int16 kZeroPitch = 0x2000;
	static const int16 kRange = kMaxPitch - kZeroPitch;
	static const float kPitchNormalizer = 1.0f / (float)kRange;

	unsigned char LSB = InData1;
	unsigned char MSB = InData2;
	int16 val = (MSB << 7) | LSB;
	val -= kZeroPitch;

	float fval = (float)val * kPitchNormalizer;

	// because of the asymmetry in the 14 bit data
	// (no _real_ center since there are an even number
	// of valid pitch value)
	// there are two 14-bit values that we map to -1
	if (fval < -1.0f)
		fval = -1.0f;

	return fval;
}

float FMidiMsg::GetPitchBendFromData() const
{
	return GetPitchBendFromData(Data1, Data2);
}

FMidiMsg::FMidiMsg(uint8 InStatus, uint8 InData1, uint8 InData2)
	: Type(EType::Std)
	, Status(InStatus)
	, Data1(InData1)
	, Data2(InData2)
{}

FMidiMsg::FMidiMsg(int32 MicrosecPerQuarterNote)
	: Type(EType::Tempo)
{
	MicsPerQuarterNoteH = ((uint32)MicrosecPerQuarterNote >> 16) & 0xFF;
	MicsPerQuarterNoteL = (uint32)MicrosecPerQuarterNote & 0xFFFF;
}

FMidiMsg::FMidiMsg(uint8 InNumerator, uint8 InDenominator)
	: Type(EType::TimeSig)
	, Numerator(InNumerator)
	, Denominator(InDenominator)
	, ts_pad(0)
{}

void FMidiMsg::SetNoteOnVelocity(uint8 Velocity)
{
	if (!IsNoteOn())
	{
		return;
	}
	Data2 = Velocity;
}

void FMidiMsg::WriteStdMidi(int32 Tick, FMidiWriter& Writer, const FMidiTrack& Track) const
{
	switch (Type)
	{
	case EType::Std:
		Writer.MidiMessage(Tick, Status, Data1, Data2);
		break;
	case EType::Tempo:
		Writer.Tempo(Tick, GetMicrosecPerQuarterNote());
		break;
	case EType::TimeSig:
		Writer.TimeSignature(Tick, Numerator, Denominator);
		break;
	case EType::Text:
		Writer.Text(Tick, *Track.GetTextAtIndex(GetTextIndex()), GetTextType());
		break;
	case EType::Runtime:
		// Runtime messages are non-standard midi and are not saved.
		break;
	default:
		checkNoEntry();
		break;
	}
}

FString FMidiMsg::ToString(const FMidiMsg& Message, const FMidiTrack* Track /*= nullptr*/)
{
	using namespace Harmonix::Midi::Constants;
	
	FStringFormatOrderedArguments Args;
	switch (Message.Type)
	{
	case EType::Std:
		switch (Message.GetStdStatusType())
		{
		case GNoteOff:
			Args.Add(Message.GetStdChannel() + 1);
			Args.Add(Message.GetStdData1());
			return FString::Format(TEXT("Note Off: Channel {0}, Note {1}"), Args);
		case GNoteOn:
			Args.Add(Message.GetStdChannel() + 1);
			Args.Add(Message.GetStdData1());
			Args.Add(Message.GetStdData2());
			return FString::Format(TEXT("Note On: Channel {0}, Note {1}, Velocity {2}"), Args);
		case GPolyPres:
			Args.Add(Message.GetStdChannel() + 1);
			Args.Add(Message.GetStdData1());
			Args.Add(Message.GetStdData2());
			return FString::Format(TEXT("Poly Pressure: Channel {0}, Note {1}, Pressure {2}"), Args);
		case GControl:
			Args.Add(Message.GetStdChannel() + 1);
			Args.Add(GetControllerName((EControllerID)Message.GetStdData1()));
			Args.Add(Message.GetStdData2());
			return FString::Format(TEXT("Controller: Channel {0}, {1}, Value {2}"), Args);
		case GProgram:
			Args.Add(Message.GetStdChannel() + 1);
			Args.Add(Message.GetStdData1());
			return FString::Format(TEXT("Program Change: Channel {0}, Program {1}"), Args);
		case GChanPres:
			Args.Add(Message.GetStdChannel() + 1);
			Args.Add(Message.GetStdData1());
			return FString::Format(TEXT("Aftertouch: Channel {0}, Value {1}"), Args);
		case GPitch:
			Args.Add(Message.GetStdChannel() + 1);
			Args.Add(Message.GetPitchBendFromData());
			return FString::Format(TEXT("Pitchbend: Channel {0}, Value {1}"), Args);
		case GSystem:
			return TEXT("!! SYstem Exclusive !!");
		}
	case EType::Tempo:
		Args.Add(MidiTempoToBPM(Message.GetMicrosecPerQuarterNote()));
		return FString::Format(TEXT("Tempo Change: {0} bpm"), Args);
	case EType::TimeSig:
		Args.Add(Message.GetTimeSigNumerator());
		Args.Add(Message.GetTimeSigDenominator());
		return FString::Format(TEXT("Time Signature: {0}/{1}"), Args);
	case EType::Text:
		Args.Add(GetTextTypeName(Message.GetTextType()));
		if (Track)
		{
			Args.Add(Track->GetTextAtIndex(Message.GetTextIndex()));
		}
		else
		{
			Args.Add(TEXT("!<unknown>!"));
		}
		return FString::Format(TEXT("Text Message: Type {0}, Text = {1}"), Args);
	case EType::Runtime:
		switch (Message.Status)
		{
			case GRuntimeAllNotesOffStatus:
				return TEXT("Runtime: All Notes Off (allow ADSR releases.)");
			case GRuntimeAllNotesKillStatus:
				return TEXT("Runtime: All Notes Kill (No ADSR releases. Kill all rendering.)");
			default:
				checkNoEntry();
				break;
		}
	default:
		return TEXT("!<Unknown midi msg type>!");
	}
}

bool FMidiMsg::Serialize(FArchive& Archive)
{
	static_assert(sizeof(FMidiMsg) == 4);
	Archive << Type;
	switch (Type)
	{
	case FMidiMsg::EType::Std:
		Archive << Status;
		Archive << Data1;
		Archive << Data2;
		break;
	case FMidiMsg::EType::Tempo:
		Archive << MicsPerQuarterNoteH;
		Archive << MicsPerQuarterNoteL;
		break;
	case FMidiMsg::EType::TimeSig:
		Archive << Numerator;
		Archive << Denominator;
		break;
	case FMidiMsg::EType::Text:
		Archive << TextType;
		Archive << TextIndex;
		break;
	case FMidiMsg::EType::Runtime:
		// runtime messages are not serialized!
		break;
	default:
		checkNoEntry();
		break;
	}
	return true;
}

