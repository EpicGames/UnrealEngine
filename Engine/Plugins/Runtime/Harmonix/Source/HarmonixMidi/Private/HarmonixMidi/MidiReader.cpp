// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiReader.h"
#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/MidiReceiver.h"
#include "HarmonixMidi/VarLenNumber.h"
#include "HarmonixMidi/MusicTimeSpecifier.h"
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/TempoMap.h"
#include "Misc/Paths.h"

// A 4-character IFF file chunk ID.  (Only "MThd" and "MTrk" are recognized
// by the SMF file format.)
const int32 kMidiChunkIDSize = 4;

#define MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(expected, ret_val_on_fail, ...)	\
	do                                                                          \
	{                                                                           \
		if (!(expected))                                                        \
		{                                                                       \
			UE_LOG(LogMIDI, Error, __VA_ARGS__);                                \
			State = EState::Failed;                                             \
			return ret_val_on_fail;                                             \
		}                                                                       \
	} while (0)

#define MIDIREADER_EXPECT_WITH_ERROR(expected, ...)	\
	do                                              \
	{                                               \
		if (!(expected))                            \
		{                                           \
			UE_LOG(LogMIDI, Error, __VA_ARGS__);    \
			State = EState::Failed;                 \
			return;                                 \
		}                                           \
	} while (0)

class FMidiChunkID
{
public:
	FMidiChunkID(const char* InStr) { memcpy(Str, InStr, kMidiChunkIDSize); }
	FMidiChunkID(FArchive& Archive) { Archive.Serialize(Str, 4); }

	friend FArchive& operator<<(FArchive& Archive, FMidiChunkID& Id);

	bool operator==(const FMidiChunkID& rhs) const
	{
		return !FCStringAnsi::Strncmp(Str, rhs.Str, kMidiChunkIDSize);
	}

	bool operator!=(const FMidiChunkID& rhs) const { return !operator==(rhs); }

	static const FMidiChunkID kMThd; // "MThd": MIDI file header chunk
	static const FMidiChunkID kMTrk; // "MTrk": MIDI track chunk

private:
	char Str[kMidiChunkIDSize]; // storage for the 4 bytes
};

const FMidiChunkID FMidiChunkID::kMThd("MThd");
const FMidiChunkID	FMidiChunkID::kMTrk("MTrk");

FArchive& operator<<(FArchive& Archive, FMidiChunkID& Id)
{
	Archive.Serialize(Id.Str, 4); 
	return Archive;
}

// Contains a chunk ID and a length.  The length is the number of bytes of
// data in the chunk AFTER the header.

class FMidiChunkHeader
{
public:
	FMidiChunkHeader(FArchive& Archive)
		: ID(Archive)
		, Length(0)
	{
		Archive << Length;
	}
	FMidiChunkHeader(const FMidiChunkID& InId, uint32 InLength) :
		ID(InId), Length(InLength) {}

	// Access chunk-header information
	uint32 GetLength() const { return Length; } // length of data following
	const FMidiChunkID& GetID() const { return ID; } // the chunk's 4-byte ID

	friend FArchive& operator<<(FArchive& Archive, FMidiChunkHeader& Header);

private:
	FMidiChunkID ID; // Chunk's 4-byte id
	uint32 Length; // length of data part of this chunk (not including header)
};

FArchive& operator<<(FArchive& Archive, FMidiChunkHeader& Header)
{
	Archive << Header.ID;
	Archive << Header.Length;
	return Archive;
}

///////////////////////////////////////////////////////////////////////////////
// FStdMidiFileReader
FStdMidiFileReader::FStdMidiFileReader(
	void* Buffer,
	int32 BufferSize,
	 const FString& InFileName,
	 IMidiReceiver* InReceiver,
	 int32 TicksPerQuarterNote,
	 Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding)
	: Filename(InFileName)
	, Receiver(InReceiver)
	, TextEncoding(InTextEncoding)
	, DestinationTicksPerQuarterNote(TicksPerQuarterNote)
{
	check(Receiver);
	Archive = MakeShared<FBufferReader>(Buffer, BufferSize, false);
	Init();
}

FStdMidiFileReader::FStdMidiFileReader(
	const FString& FilePath,
	IMidiReceiver* InReceiver,
	int32 TicksPerQuarterNote,
	Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding)
	: Receiver(InReceiver)
	, TextEncoding(InTextEncoding)
	, DestinationTicksPerQuarterNote(TicksPerQuarterNote)
{
	check(Receiver);
	Filename = FPaths::GetCleanFilename(FilePath);
	IPlatformFile& PlatformFileApi = FPlatformFileManager::Get().GetPlatformFile();
	IFileHandle* FileHandle = PlatformFileApi.OpenRead(*FilePath);
	Archive = MakeShared<FArchiveFileReaderGeneric>(FileHandle, *Filename, FileHandle->Size());
	Init();
}

FStdMidiFileReader::FStdMidiFileReader(
	TSharedPtr<FArchive> Archive,
	const FString& InFilename,
	IMidiReceiver* InReceiver,
	int32 TicksPerQuarterNote,
	Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding)
	: Filename(InFilename)
	, Archive(Archive.ToSharedRef())
	, Receiver(InReceiver)
	, TextEncoding(InTextEncoding)
	, DestinationTicksPerQuarterNote(TicksPerQuarterNote)
{
	check(Receiver);
	Init();
}

void FStdMidiFileReader::ReadAllTracks()
{
	// read from the top of the stream
	Archive->Seek(0);

	UE_LOGF(LogMIDI, VeryVerbose, "READING ALL MIDI TRACKS:\n========================");

	bool TrackReadSuccessfully = false;
	do 
	{
		TrackReadSuccessfully = ReadTrack();
	} while (State != EState::Failed && TrackReadSuccessfully);

	if (State != EState::Failed)
	{
		if (!Receiver->Finalize(LastTick))
		{
			State = EState::Failed;
		}
	}
}

bool FStdMidiFileReader::ReadSomeEvents(int32 NumEvents)
{
	for (int32 i = 0; i < NumEvents; ++i)
	{
		ReadNextEvent();
		if (State == EState::End || State == EState::Failed)
		{
			return true;
		}
	}
	return false;
}

bool FStdMidiFileReader::ReadTrack()
{
	do
	{
		ReadNextEvent();
	} while (State != EState::End && State != EState::NewTrack && State != EState::Failed);

	return (State == EState::NewTrack);
}

bool FStdMidiFileReader::ReadEvents(int32 Count)
{
	for (int32 i = 0; i < Count; ++i)
	{
		if (State == EState::End || State == EState::Failed)
		{
			return false;
		}
		ReadNextEvent();
	}
	return true;
}

// skips the processing of the current track and moves on to the next track.
// Has no effect if called from within EndOfTrack().
void FStdMidiFileReader::SkipCurrentTrack()
{
	UE_LOGF(LogMIDI, VeryVerbose, "Skipping Current Track (%d)", CurrentTrackIndex);

	if (State == EState::InTrack)
	{
		if (CurrentTrackIndex == NumTracks - 1)
		{
			State = EState::End;
			if (!Receiver->OnEndOfTrack(CurrentTick) || !Receiver->OnAllTracksRead())
			{
				State = EState::Failed;
			}
		}
		else
		{
			State = EState::NewTrack;
			if (!Receiver->OnEndOfTrack(CurrentTick))
			{
				State = EState::Failed;
			}
			else
			{
				Archive->Seek(TrackEndPos);
			}
		}
	}
}

void FStdMidiFileReader::Init()
{
	// make oursleces a "local" barmap as it can be used to 
	// provide better error messages...
	BarMap = MakeShared<FBarMap>();
	// midi files are big-endian. If this is a little endian platform we need to byteswap!
	Archive->ArForceByteSwapping = PLATFORM_LITTLE_ENDIAN == 1;
	Receiver->SetMidiReader(this);
}

void FStdMidiFileReader::ReadNextEvent()
{
	ReadNextEventImpl();
}

void FStdMidiFileReader::ReadNextEventImpl()
{
	switch (State)
	{
	case EState::InTrack:
		ReadEvent();
		break;
	case EState::NewTrack:
		if (Format == 0 && CurrentTrackIndex == 0)
		{
			UE_LOGF(LogMIDI, VeryVerbose, "Type 0 file so rewinding to re-read track in 'non-conductor' mode.");
			// we already ready the one and only track as the "conductor track".
			// we have to read it again as "track 1", so...
			Archive->Seek(LastTracksFilePosition);
			TrackFilteringMode = ETrackFilteringMode::NonConductorEvents;
		}
		else 
		{
			LastTracksFilePosition = Archive->Tell();
		}
		ReadTrackHeader();
		break;
	case EState::Start:
		ReadFileHeader();
		break;
	case EState::End:
	case EState::Failed:
		break;
	}
}

// Read the standard Midi file header chunk (MThd)
void FStdMidiFileReader::ReadFileHeader()
{
	check(State == EState::Start);

	UE_LOGF(LogMIDI, VeryVerbose, "Reading file header...");

	FMidiChunkHeader Header(*Archive);

	// Verify the MIDI header is properly formed.
	MIDIREADER_EXPECT_WITH_ERROR(Header.GetID() == FMidiChunkID::kMThd && Header.GetLength() == 6, TEXT("%s: MIDI file header is corrupt"), *Filename);

	// SMF format:
	//   0: A single multi-channel track
	//   1: 1 or more simultaneous tracks
	//   2: 1 or more sequentially independent tracks
	// we only support type 0 & 1.
	*Archive << Format;
	MIDIREADER_EXPECT_WITH_ERROR(Format == 1 || Format == 0, TEXT("%s: Only type 0 or 1 MIDI files are supported; this file is type %d"), *Filename, Format);

	*Archive << NumTracks;

	MIDIREADER_EXPECT_WITH_ERROR(NumTracks > 0, TEXT("%s: MIDI file has no tracks"), *Filename);

	if (Format == 0)
	{
		MIDIREADER_EXPECT_WITH_ERROR(NumTracks == 1, TEXT("%s: Format 0 file expected only one track but found %d"), *Filename, NumTracks);

		// We are starting to read a format 0 midi file, so on the first pass
		// over the one and only track in the file gather the conductor type events...
		TrackFilteringMode = ETrackFilteringMode::ConductorEvents;
		// we are going to convert it to a format 1 file, with a conductor track
		// and one track for non-conductor events...
		NumTracks = 2;
	}
	else
	{
		TrackFilteringMode = ETrackFilteringMode::None;
	}

	MIDIREADER_EXPECT_WITH_ERROR(Receiver->SetNumTracks(NumTracks), TEXT("%s: Failed to set the number of tracks (%d)"), *Filename, NumTracks);

	// Hi bit set on ticksPerQuarter indicates a SMPTE division.
	// We don't support that.
	int16 TicksPerQuarter;
	*Archive << TicksPerQuarter;
	MIDIREADER_EXPECT_WITH_ERROR((TicksPerQuarter & 0x8000) == 0, TEXT("%s: MIDI file uses SMPTE time division. This is not supported"), *Filename);

	// check for rational TicksPerQuarter...
	MIDIREADER_EXPECT_WITH_ERROR((TicksPerQuarter % 48) == 0, TEXT("%s: MIDI file Ticks Per Quarter Note is not rational (%d). Must be divisible by 48"), *Filename, TicksPerQuarter);

	TickConversionFactor = float(DestinationTicksPerQuarterNote) / (float)TicksPerQuarter;

	State = EState::NewTrack;

	UE_LOGF(LogMIDI, VeryVerbose, "... Done reading file header. Format = %d, File PPQ = %d, NumTracks = %d", Format, TicksPerQuarter, Format == 0 ? 1 : NumTracks);
}

// Read the standard Midi track chunk header (MTrk)
void FStdMidiFileReader::ReadTrackHeader()
{
	check(State == EState::NewTrack);

	UE_LOG(LogMIDI, VeryVerbose, TEXT("Reading track header (starting %" INT64_FMT " bytes into file)..."), Archive->Tell());

	FMidiChunkHeader TrackHeader(*Archive);

	MIDIREADER_EXPECT_WITH_ERROR(TrackHeader.GetID() == FMidiChunkID::kMTrk, TEXT("%s: MIDI track header for track %d is corrupt"), *Filename, CurrentTrackIndex + 1);

	// check for rational track data size...
	MIDIREADER_EXPECT_WITH_ERROR((TrackHeader.GetLength() + (uint32)Archive->Tell()) <= (uint32)std::numeric_limits<int32>::max(),
		TEXT("%s: MIDI track data length exceeds maximum length!"), *Filename);

	TrackEndPos = int32(Archive->Tell() + TrackHeader.GetLength());
	MIDIREADER_EXPECT_WITH_ERROR(TrackEndPos <= (int32)Archive->TotalSize(), TEXT("%s: MIDI track %d data length as recorded in the header exceeds the amount of data in the file!"), *Filename, CurrentTrackIndex + 1);

	++CurrentTrackIndex; // update track num

	// Let the midi receiver know that this is a new track.
	MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnNewTrack(CurrentTrackIndex), TEXT("%s: Failed to create new Track (%d)"), *Filename, CurrentTrackIndex);

	// reset state for reading a new track
	PrevStatus = 0;
	CurrentFileTick = 0;
	CurrentTick = 0;    // units of file's Tick
	MidiListTick = -1;  // units of desired Tick (for sorting)
	State = EState::InTrack;

	if (Format == 0)
	{
		if (CurrentTrackIndex == 0)
		{
			CurrentTrackName = TEXT("Conductor");
		}
		else
		{
			CurrentTrackName = TEXT("Track-0");
		}

		MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnText(0, CurrentTrackName, Harmonix::Midi::Constants::GMeta_TrackName), TEXT("%s: Failed to add track name to Track (%d)"), *Filename, CurrentTrackIndex);
	}

	UE_LOGF(LogMIDI, VeryVerbose, "... Done reading track header. It contains %d bytes of midi events.", TrackHeader.GetLength());
}

// Read an event from the track
void FStdMidiFileReader::ReadEvent()
{
	check(State == EState::InTrack);

	if (FailIfReadPositionIsPastTrackEnd())
	{
		return;
	}

	uint8 Status, Data1 = 0;    // MIDI message bytes
	bool RunningStatus;     // true if running status is detected

	CurrentFileTick += Midi::VarLenNumber::Read(*Archive); // accumulate tick

	if (FailIfReadPositionIsPastTrackEnd())
	{
		return;
	}

	CurrentTick = int32(CurrentFileTick * TickConversionFactor);  // convert to kTicksPerQuarterNoteInt

	if (CurrentTick > LastTick)
	{
		LastTick = CurrentTick;
	}

	if (CurrentTick != MidiListTick)
	{
		// we have a new tick; sort all midi from previous tick and send it out
		ProcessMidiList();

		if (State != EState::InTrack)
		{
			return;
		}

		MidiListTick = CurrentTick;
	}

	*Archive << Status;
	if (Harmonix::Midi::Constants::IsStatus(Status))  // this byte is truly a status byte
	{
		RunningStatus = false;
		if (!Harmonix::Midi::Constants::IsSystem(Status))
		{
			// save the current status byte, in case the next message uses running
			// status.  Note that this is only for midi events; running status can
			// run across meta-events and system exclusive events.
			PrevStatus = Status;
		}
	}
	else
	{
		// whoops, that wasn't actually a status byte we read, but a data byte;
		// the status byte comes from running status.
		RunningStatus = true;
		Data1 = Status;
		Status = PrevStatus;
	}

	MIDIREADER_EXPECT_WITH_ERROR(Harmonix::Midi::Constants::IsStatus(Status), TEXT("%s (%s): Found invalid MIDI status byte (%d). MIDI data appears to be currupt."), *Filename, *CurrentTrackName, Status);

	// Check if this is a system (ie, non-channel) message
	if (Harmonix::Midi::Constants::IsSystem(Status))
	{
		ReadSystemEvent(CurrentTick, Status);
	}                   // All other messages are channel (non-system) messages
	else
	{
		// Running status means that data1 has already been read above
		if (!RunningStatus)
		{
			if (FailIfBytesNotAvailableInTrackData(1)) // 1 byte needed for status byte
			{
				return;
			}
			*Archive << Data1;
		}
		ReadMidiEvent(CurrentTick, Status, Data1);
	}
}

void FStdMidiFileReader::ReadMidiEvent(int32 Tick, uint8 Status, uint8 Data1)
{
	using namespace Harmonix::Midi::Constants;
	uint8 Data2 = 0;
	bool ValidMidiEvent = false;
	switch (Status & GMessageTypeMask)
	{
		// special case for NoteOn (3 byte message)
	case GNoteOn:
		if (FailIfBytesNotAvailableInTrackData(1)) // 1 byte needed for velocity
		{
			return;
		}
		*Archive << Data2;
		ValidMidiEvent = true;
		// convert note-on vel=0 -> note offs
		if (Data2 == 0)
		{
			Status = GNoteOff | (Status & GChannelMask);
		}
		break;

		// other 3 byte messages
	case GNoteOff:
	case GControl:
	case GPitch:
	case GPolyPres:
		if (FailIfBytesNotAvailableInTrackData(1)) // 1 byte needed for controller value
		{
			return;
		}
		*Archive << Data2;
		ValidMidiEvent = true;
		break;

		// 2 byte messages
	case GProgram:
	case GChanPres:
		Data2 = 0;
		ValidMidiEvent = true;
		break;

	default:
		UE_LOGF(LogMIDI, Error, "%ls (%ls): Cannot parse event %i", *Filename, *CurrentTrackName, (Status & GMessageTypeMask));
		State = EState::Failed;
		return;
	}

	if (ValidMidiEvent && TrackFilteringMode != ETrackFilteringMode::ConductorEvents)
	{
		// At this time, we only care about Note On / Note Off events.
		QueueChannelMsg(Tick, Status, Data1, Data2);
		UE_LOGF(LogMIDI, VeryVerbose, "TICK %d :: %ls", Tick, *Harmonix::Midi::Constants::MakeStdMsgString(Status, Data1, Data2));
	}
}

// Read system event from the stream.  Status byte has already been read.
void FStdMidiFileReader::ReadSystemEvent(int32 Tick, uint8 Status)
{
	using namespace Harmonix::Midi::Constants;
	
	switch (Status)
	{
	case GFile_SysEx:
	case GFile_Escape:
		{
			// All system events start with length. We'll
			// seek past that length and ignore these events
			// entirely.
			int32 PacketLength = Midi::VarLenNumber::Read(*Archive);
			UE_LOGF(LogMIDI, VeryVerbose, "Seeking %d bytes past SysEx or Escape system event.", PacketLength);
			if (FailIfBytesNotAvailableInTrackData((int64)PacketLength))
			{
				return;
			}
			Archive->Seek(Archive->Tell() + PacketLength);
		}
		break;
	case GFile_Meta:
		{
			if (FailIfBytesNotAvailableInTrackData(1)) // 1 byte needed for meta event type
			{
				return;
			}
			uint8 type;
			*Archive << type;
			ReadMetaEvent(Tick, type);
		}
		break;
	default:
		UE_LOGF(LogMIDI, Error, "%ls (%ls): Cannot parse system event %i", *Filename, *CurrentTrackName, Status);
		State = EState::Failed;
		break;
	}
}

// Read a Meta Event from stream.
// Status byte and event type have already been read.
void FStdMidiFileReader::ReadMetaEvent(int32 Tick, uint8 Type)
{
	using namespace Harmonix::Midi::Constants;
	
	if (FailIfReadPositionIsPastTrackEnd())
	{
		return;
	}

	int32 Length = Midi::VarLenNumber::Read(*Archive);
	int32 StartPos = int32(Archive->Tell());
	FString WorkingString;

	switch (Type)
	{
	case GMeta_TrackName:
		CurrentTrackName = ReadText(Length);
		if (CurrentTrackIndex == 0)
		{
			// This is the conductor track. Different DAWs use different names for this hidden track,
			// so we ignore the name we got from the file and normalize it here to "conductor"...
			CurrentTrackName = TEXT("Conductor");
		}
		else if (CurrentTrackName.IsEmpty())
		{
			CurrentTrackName = FString::Format(TEXT("Track-{0}"), {CurrentTrackIndex - 1});
			UE_LOGF(LogMIDI, VeryVerbose, "TICK %d :: Got track name meta type but an empty string. Using a default track name...", Tick);
		}
		UE_LOGF(LogMIDI, VeryVerbose, "Got track name event... '%ls'", *CurrentTrackName);
		MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnText(Tick, CurrentTrackName, Type), TEXT("%s (%s): Failed to add track name to Track"), *Filename, *CurrentTrackName);
		break;
	case GMeta_Copyright:
	case GMeta_Marker:
	case GMeta_CuePoint:
		WorkingString = ReadText(Length);
		if (TrackFilteringMode != ETrackFilteringMode::NonConductorEvents)
		{
			MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnText(Tick, WorkingString, Type), TEXT("%s (%s): Failed to add copyright, marker, or cuepoint to Track"), *Filename, *CurrentTrackName);
			UE_LOGF(LogMIDI, VeryVerbose, "TICK %d :: Got Copyright, Marker, or CuePoint event... '%ls'", Tick, *WorkingString);
		}
		break;
	case GMeta_Text:
	case GMeta_Lyric:
		WorkingString = ReadText(Length);
		if (TrackFilteringMode != ETrackFilteringMode::ConductorEvents)
		{
			MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnText(Tick, WorkingString, Type), TEXT("%s (%s): Failed to add text or lyric to Track"), *Filename, *CurrentTrackName);
			UE_LOGF(LogMIDI, VeryVerbose, "TICK %d :: Got Text or Lyric event... '%ls'", Tick, *WorkingString);
		}
		break;
	case GMeta_Tempo:
		{
			if (FailIfBytesNotAvailableInTrackData(3)) // three bytes needed for tempo
			{
				return;
			}
			uint8 Byte1, Byte2, Byte3;
			*Archive << Byte1 << Byte2 << Byte3;
			int32 Tempo = (Byte1 << 16) + (Byte2 << 8) + Byte3;
			if (TrackFilteringMode != ETrackFilteringMode::NonConductorEvents)
			{
				MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnTempo(Tick, Tempo), TEXT("%s (%s): Failed to add tempo to Track"), *Filename, *CurrentTrackName);
				UE_LOGF(LogMIDI, VeryVerbose, "TICK %d :: Got Tempo event... %f bpm", Tick, Harmonix::Midi::Constants::MidiTempoToBPM(Tempo));
			}
		}
		break;

	case GMeta_EndOfTrack:
		ProcessMidiList();  // in case there is anything left to send out.
		MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnEndOfTrack(Tick), TEXT("%s (%s): Failed ending current track read"), *Filename, *CurrentTrackName);
		UE_LOGF(LogMIDI, VeryVerbose, "TICK %d :: Got End Of Trackevent.", Tick);
		if (CurrentTrackIndex == NumTracks - 1)
		{
			State = EState::End;
			MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnAllTracksRead(), TEXT("%s: Failure telling MIDI reader all tracks are read"), *Filename);
		}
		else
		{
			State = EState::NewTrack;
		}
		break;

	case GMeta_TimeSig:
		{
			if (FailIfBytesNotAvailableInTrackData(2)) // 2 bytes needed for time signature
			{
				return;
			}
			uint8 Numerator, DenominatorExp;
			*Archive << Numerator << DenominatorExp;

			MIDIREADER_EXPECT_WITH_ERROR(DenominatorExp <= 6, TEXT("%s (%s): Time signature at %s has invalid denominator (2^%d); max is 64 (2^6)"), *Filename, *CurrentTrackName, *MidiTickFormat(Tick, BarMap.Get(), Midi::EMusicTimeStringFormat::Position), DenominatorExp);

			// time signature denominator is given as a power of 2
			//  e.g. for 6/8 time, numerator=6 and denominator=3
			int32 Denominator = int32(FMath::Pow(2.0f, DenominatorExp));

			MIDIREADER_EXPECT_WITH_ERROR(Numerator != 0, TEXT("%s (%s): Time signature %d/%d at %s has invalid numerator (%d)"), *Filename, *CurrentTrackName, Numerator, Denominator,
				*MidiTickFormat(Tick, BarMap.Get(), Midi::EMusicTimeStringFormat::Position), Numerator);

			if (TrackFilteringMode != ETrackFilteringMode::NonConductorEvents)
			{
				// before telling any receivers about the time signature we update our own local tempo map
				// so we can print better error and warning messages...
				check(Tick == 0 || BarMap->GetNumTimeSignaturePoints() > 0);
				int32 BarIndex = BarMap->TickToBarIncludingCountIn(Tick);
				UE_LOGF(LogMIDI, VeryVerbose, "TICK %d :: Got Time Signature Event. %d/%d", Tick, Numerator, Denominator);
				MIDIREADER_EXPECT_WITH_ERROR(BarMap->AddTimeSignatureAtBarIncludingCountIn(BarIndex, Numerator, Denominator, true, false),
					TEXT("%s (%s): Time signature %d/%d at %s overlaps or conflicts with nearby time signatures"),
					*Filename, *CurrentTrackName, Numerator, Denominator, *MidiTickFormat(Tick, BarMap.Get(), Midi::EMusicTimeStringFormat::Position));
				// now the 'clients' view of the world...
				MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnTimeSignature(Tick, Numerator, Denominator, false),
					TEXT("%s (%s): Failed adding time signature change to MIDI (%d/%d)"), *Filename, *CurrentTrackName, Numerator, Denominator);
			}

			// skip over next two bytes, "MIDI clocks per metronome click"
			// and "32nd notes per MIDI quarter note"; they're not too useful
			if (FailIfBytesNotAvailableInTrackData(2))
			{
				return;
			}
			Archive->Seek(Archive->Tell() + 2);
		}
		break;

	case GMeta_ChannelPrefix:
	case GMeta_Port:
	case GMeta_KeySig:
	case GMeta_SMPTE:
	case GMeta_InstrumentName:
		// these are valid events, but not currently supported by MidiReceiver
		UE_LOGF(LogMIDI, Warning, "Track '%ls', TICK = %d :: Skipping unsupported MIDI event -> %d - %ls", *CurrentTrackName, Tick, Type, *Harmonix::Midi::Constants::GetMetaEventTypeName(Type));

		break;

	default:
		UE_LOGF(LogMIDI, Error, "%ls (%ls): Cannot parse meta event %i", *Filename, *CurrentTrackName, Type);
		State = EState::Failed;
		return;
	}

	// 
	if (State != EState::InTrack)
	{
		return;
	}

	// skip to end of meta event
	Archive->Seek(StartPos + Length);
}

FString FStdMidiFileReader::ReadText(int32 Length)
{
	// make sure length is rational...
	MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(Length >= 0 && Length <= kMaxSupportedMidiStringSize, FString(),
		TEXT("%s (%s): %d byte long string found in MIDI data. Maximum supported string length is %d."), *Filename, *CurrentTrackName, Length, kMaxSupportedMidiStringSize);

	// make sure there are enough bytes in the file...
	if (Length == 0 || FailIfBytesNotAvailableInTrackData(Length))
	{
		return FString();
	}

	unsigned char* AsUtf8 = nullptr;
	unsigned char* AsUtf8End = nullptr;
	if (TextEncoding == Harmonix::Midi::Constants::EMidiTextEventEncoding::Latin1)
	{
		unsigned char* AsLatin = (unsigned char*)FMemory::Malloc(Length + 2);
		MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(AsLatin, FString(),
			TEXT("%s (%s): Couldn't allocate memory for %d byte string."), *Filename, *CurrentTrackName, Length);

		Archive->Serialize(AsLatin, Length);
		// force some null terminators...
		AsLatin[Length] = 0;
		AsLatin[Length+1] = 0;
		unsigned char* AsLatinEnd = AsLatin + Length + 2;
		AsUtf8 = (unsigned char*)FMemory::Malloc(Length * 2 + 2);
		if (!AsUtf8)
		{
			FMemory::Free(AsLatin);
			MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(false, FString(),
				TEXT("%s (%s): Couldn't allocate memory for %d byte string."), *Filename, *CurrentTrackName, Length);
		}
		FMemory::Memset(AsUtf8, 0, Length * 2 + 2);
		AsUtf8End = AsUtf8 + (Length * 2 + 2);

		unsigned char* in = AsLatin;
		unsigned char* out = AsUtf8;
		while (*in)
		{
			if (in >= AsLatinEnd)
			{
				break;
			}
			if (*in < 128)
			{
				if (out >= AsUtf8End)
				{
					break;
				}
				*out++ = *in++;
			}
			else
			{
				if (out >= AsUtf8End)
				{
					break;
				}
				*out++ = 0xc0 | *in >> 6;
				if (out >= AsUtf8End)
				{
					break;
				}
				*out++ = 0x80 | (*in++ & 0x3f);
			}
		}
		if (out < AsUtf8End)
		{
			*out = 0;
		}
		else
		{
			*(AsUtf8End - 1) = 0;
		}

		FMemory::Free(AsLatin);
	}
	else
	{
		AsUtf8 = (unsigned char*)FMemory::Malloc(Length + 2);
		MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(AsUtf8, FString(),
			TEXT("%s (%s): Couldn't allocate memory for %d byte string."), *Filename, *CurrentTrackName, Length);
		Archive->Serialize(AsUtf8, Length);
		// force some null terminators...
		AsUtf8[Length] = 0;
		AsUtf8[Length + 1] = 0;
	}

	if (Archive->IsError())
	{
		FMemory::Free(AsUtf8);
		MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(false, FString(),
			TEXT("%s: MIDI track %d ... Error reading string!"), *Filename, CurrentTrackIndex);
	}

	FString AsFString = StringCast<TCHAR>((const UTF8CHAR*)AsUtf8).Get();
	FMemory::Free(AsUtf8);
	return AsFString;

}

// Queue all midi that is on the same tick, for sorting. 
// Note that tick is already in terms of desired ticks per quarter note.
void FStdMidiFileReader::QueueChannelMsg(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2)
{
	// stick this message in the MidiList
	EventsOnSameTick.Add({Status, Data1, Data2});
}

// Check if MidiList has anything in it. If so, sort and send out.
void FStdMidiFileReader::ProcessMidiList()
{
	if (TrackFilteringMode != ETrackFilteringMode::ConductorEvents)
	{
		EventsOnSameTick.StableSort(RawMidiLess());
		for (auto& MidiItem : EventsOnSameTick)
		{
			MIDIREADER_EXPECT_WITH_ERROR(Receiver->OnMidiMessage(MidiListTick, MidiItem.Status, MidiItem.Data1, MidiItem.Data2),
				TEXT("%s (%s): Failed to MIDI message to Track"), *Filename, *CurrentTrackName);

			// state changed because SkipCurrentTrack was called. Stop processing.
			if (State != EState::InTrack)
			{
				break;
			}
		}
	}
	EventsOnSameTick.Empty();
}

bool FStdMidiFileReader::FailIfReadPositionIsPastTrackEnd()
{
	MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(Archive->Tell() < (int64)TrackEndPos, true,
		TEXT("%s: MIDI track %d ... Finding events on track passed track data block! (ie missing 'end-of-track' event)"), *Filename, CurrentTrackIndex);
	return false; // no error
}

bool FStdMidiFileReader::FailIfBytesNotAvailableInTrackData(int64 NumBytes)
{
	int64 BytesAvailable = (int64)TrackEndPos - Archive->Tell();
	MIDIREADER_EXPECT_WITH_ERROR_RETURN_VAL(BytesAvailable >= NumBytes, true,
		TEXT("%s: MIDI track %d ... Track data underrun. Not enough bytes in track block."), *Filename, CurrentTrackIndex);
	return false; // no error
}

