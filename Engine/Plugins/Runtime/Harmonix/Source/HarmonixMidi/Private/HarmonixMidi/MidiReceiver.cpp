// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiReceiver.h"
#include "HarmonixMidi/MidiReader.h"
#include "HarmonixMidi/MusicTimeSpecifier.h"
#include "HarmonixMidi/MidiConstants.h"

void IMidiReceiver::SkipCurrentTrack()
{
	check(Reader);
	Reader->SkipCurrentTrack();
}
