// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/MusicTimeSpecifier.h"
#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/SongMaps.h"

// -1 = less than, 1 = greater than, 0 = equal to

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiSongPos)
int32 SongPosCmp(const FMidiSongPos& lhs, const FMidiSongPos& rhs)
{
	// can't use bar alone because of floating point. 
	// so use bar first and then "rounded" beat.
		
	// bar
	if (lhs.Timestamp.Bar < rhs.Timestamp.Bar)
	{
		return -1;
	}
	else if (lhs.Timestamp.Bar > rhs.Timestamp.Bar)
	{
		return 1;
	}

	// beat
	if (Harmonix::Midi::Constants::RoundToStandardBeatPrecision(lhs.Timestamp.Beat, lhs.TimeSigDenominator) ==
		Harmonix::Midi::Constants::RoundToStandardBeatPrecision(rhs.Timestamp.Beat, rhs.TimeSigDenominator))
	{
		return 0;
	}
	if (lhs.Timestamp.Beat < rhs.Timestamp.Beat)
	{
		return -1;
	}
	return 1;
}

// operators
bool FMidiSongPos::operator<(const FMidiSongPos& rhs) const
{
	return SongPosCmp(*this, rhs) == -1;
}

bool FMidiSongPos::operator<=(const FMidiSongPos& rhs) const
{
	int32 cmp = SongPosCmp(*this, rhs);
	return cmp == -1 || cmp == 0;
}

bool FMidiSongPos::operator>(const FMidiSongPos& rhs) const
{
	return SongPosCmp(*this, rhs) == 1;
}

bool FMidiSongPos::operator>=(const FMidiSongPos& rhs) const
{
	int32 cmp = SongPosCmp(*this, rhs);
	return cmp == 1 || cmp == 0;
}

FMidiSongPos FMidiSongPos::Lerp(const FMidiSongPos& A, const FMidiSongPos& B, float Alpha)
{
	FMidiSongPos Result;
	Result.SecondsIncludingCountIn = FMath::Lerp(A.SecondsIncludingCountIn, B.SecondsIncludingCountIn, Alpha);
	Result.SecondsFromBarOne = FMath::Lerp(A.SecondsFromBarOne, B.SecondsFromBarOne, Alpha);
	Result.TimeSigNumerator = A.TimeSigNumerator;
	Result.TimeSigDenominator = A.TimeSigDenominator;
	Result.Tempo = A.Tempo;
	Result.CurrentSongSection = (Alpha < 0.5f) ? A.CurrentSongSection : B.CurrentSongSection;
	Result.BarsIncludingCountIn = FMath::Lerp(A.BarsIncludingCountIn, B.BarsIncludingCountIn, Alpha);
	Result.BeatsIncludingCountIn = FMath::Lerp(A.BeatsIncludingCountIn, B.BeatsIncludingCountIn, Alpha);
	
	// infer start bar...
	int32 StartBar = A.Timestamp.Bar - FMath::FloorToInt32(A.BarsIncludingCountIn);
	
	Result.Timestamp.Bar = FMath::FloorToInt32(Result.BarsIncludingCountIn) + StartBar;
	Result.Timestamp.Beat = FMath::Fractional(Result.BarsIncludingCountIn) * Result.TimeSigNumerator + 1.0f;
	if (FMath::FloorToInt32(Result.BeatsIncludingCountIn) == FMath::FloorToInt32(A.BeatsIncludingCountIn))
	{
		Result.BeatType = A.BeatType;
	}
	else
	{
		Result.BeatType = B.BeatType;
	}
	// Could also imagine using &&, but I was worried about propagating not set forever -jwf 
	Result.IsSet = A.IsSet || B.IsSet;
	return Result;
}

bool FMidiSongPos::operator==(const FMidiSongPos& rhs) const
{
	return SongPosCmp(*this, rhs) == 0;
}

float FMidiSongPos::GetBeatsPerMinute() const
{
	// if we have a tempo set, we should have a time signature.
	ensure(TimeSigDenominator != 0 || Tempo == 0);
	return Tempo * TimeSigDenominator / 4.0f;
}


void FMidiSongPos::SetByTime(float InElapsedMs, float InBpm, int32 InTimeSigNumerator, int32 InTimeSigDenominator, int32 StartBar)
{
	// first some numbers we will need...
	float QuarterNotesPerSecond = InBpm / 60.0f;
	float BeatsPerSecond       = QuarterNotesPerSecond * (InTimeSigDenominator / 4);
	float TotalBeats           = BeatsPerSecond * (InElapsedMs / 1000.0f);
	float BeatsPerBar          = (float)InTimeSigNumerator; // simple assumption given lack of SongMaps!
	int32 BeatsOfCountIn       = (1 - StartBar) * InTimeSigNumerator;
	float CountInSeconds       = (float)BeatsOfCountIn / BeatsPerSecond;

	SecondsIncludingCountIn = InElapsedMs / 1000.0f;
	SecondsFromBarOne       = SecondsIncludingCountIn - CountInSeconds;
	TimeSigNumerator        = InTimeSigNumerator;
	TimeSigDenominator      = InTimeSigDenominator;
	Tempo                   = InBpm;
	BarsIncludingCountIn    = TotalBeats / TimeSigNumerator;
	BeatsIncludingCountIn   = TotalBeats;
	Timestamp.Bar = FMath::FloorToInt32(BarsIncludingCountIn) + StartBar;
	Timestamp.Beat = FMath::Fractional(BarsIncludingCountIn) * InTimeSigNumerator + 1.0f;
	IsSet = true;
}

void FMidiSongPos::SetByTime(float InMs, const ISongMapEvaluator& Map)
{
	float Tick = Map.MsToTick(InMs);
	SetByTimeAndTick(InMs, Tick, Map);
}

void FMidiSongPos::SetByTick(float InTick, const ISongMapEvaluator& Map)
{
	float Ms = Map.TickToMs(InTick);
	SetByTimeAndTick(Ms, InTick, Map);
}

void FMidiSongPos::SetByTimeAndTick(float InMs, float InTick, const ISongMapEvaluator& Map)
{
	Timestamp = Map.TickToMusicTimestamp(InTick);

	const FTimeSignature* TimeSig = Map.GetTimeSignatureAtTick(int32(InTick));
	SecondsIncludingCountIn = InMs / 1000.0f;
	SecondsFromBarOne       = SecondsIncludingCountIn - Map.GetCountInSeconds();
	TimeSigNumerator        = TimeSig ? TimeSig->Numerator : 4;
	TimeSigDenominator      = TimeSig ? TimeSig->Denominator : 4;
	Tempo                   = Map.GetTempoAtTick(int32(InTick));
	BarsIncludingCountIn    = Map.GetBarIncludingCountInAtTick(InTick);
	IsSet = true;

	const FSongSection* SongSection = Map.GetSectionAtTick(int32(InTick));
	CurrentSongSection = SongSection ? FSongSection(SongSection->Name, SongSection->StartTick, SongSection->LengthTicks) : FSongSection();

	int32 BeatPointIndex = 0;
	const FBeatMapPoint* BeatPoint = Map.GetBeatPointInfoAtTick(int32(InTick), &BeatPointIndex);
	if (BeatPoint != nullptr)
	{
		float TickInBeat = InTick - BeatPoint->StartTick;
		float FractionalPart = TickInBeat / BeatPoint->LengthTicks;
		BeatsIncludingCountIn = BeatPointIndex + FractionalPart;
		BeatType = BeatPoint->Type;
	}
	else
	{
		const FTimeSignaturePoint* TimeSigPoint = Map.GetTimeSignaturePointAtTick(int32(InTick));
		if (TimeSigPoint)
		{
			float BarAtTimeSig = BarsIncludingCountIn - TimeSigPoint->BarIndex;
			float FractPart = FMath::Fractional(BarAtTimeSig);
			BeatsIncludingCountIn = TimeSigPoint->BeatIndex + (BarAtTimeSig * TimeSigPoint->TimeSignature.Numerator);
		}
		else
		{
			BeatsIncludingCountIn = BarsIncludingCountIn * 4; // 4/4
		}
		BeatType = FMath::IsNearlyEqual(Timestamp.Beat, 1.0f) ? EMusicalBeatType::Downbeat : EMusicalBeatType::Normal;
	}
}
