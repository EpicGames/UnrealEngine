// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiTrack.h"
#include "HarmonixMidi/MidiMsg.h"
#include "HarmonixMidi/MidiWriter.h"
#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiTrack)

FMidiTrack::FMidiTrack()
	: Sorted(true)
	, PrimaryMidiChannel(-1)
{
	// Don't add a default track in here! 
	// It just slows down loading.

	// If you need a FMidiTrack with a default track... 
	// Create a FMidiTrack using the constructor below 
	// with something like "Conductor" as the track name!
}

FMidiTrack::FMidiTrack(const FString& Name)
	: Sorted(true)
	, PrimaryMidiChannel(-1)
{
	uint16 stringIndex = AddText(Name);
	AddEvent(FMidiEvent(0, FMidiMsg::CreateText(stringIndex, Harmonix::Midi::Constants::GMeta_TrackName)));
}

bool FMidiTrack::operator==(const FMidiTrack& Other) const
{
	if (Events.Num() != Other.Events.Num() || Strings.Num() != Other.Strings.Num())
	{
		return false;
	}
	for (int32 EventIndex = 0; EventIndex < Events.Num(); ++EventIndex)
	{
		if (Events[EventIndex] != Other.Events[EventIndex])
		{
			return false;
		}
	}
	for (int32 StringIndex = 0; StringIndex < Strings.Num(); ++StringIndex)
	{
		if (Strings[StringIndex] != Other.Strings[StringIndex])
		{
			return false;
		}
	}
	return Sorted == Other.Sorted && PrimaryMidiChannel == Other.PrimaryMidiChannel;
}

const FMidiEventList& FMidiTrack::GetEvents() const
{
	check(Sorted);
	return Events;
}

const FMidiEventList& FMidiTrack::GetUnsortedEvents() const
{
	return Events;
}

FMidiEventList& FMidiTrack::GetRawEvents()
{
	Sort();
	return Events;
}

void FMidiTrack::SetName(const FString& InName)
{
	for (auto& Event : Events)
	{
		if (Event.GetMsg().MsgType() == FMidiMsg::EType::Text)
		{
			FMidiMsg& Message = Event.GetMsg();
			if (Message.GetTextType() == Harmonix::Midi::Constants::GMeta_TrackName)
			{
				Strings[Message.GetTextIndex()] = InName;
				return;
			}
		}
	}
	UE_LOGF(LogMIDI, Warning, "Cannot set name of track to %ls: track does not contain a track name event", *InName);
}

const FString* FMidiTrack::GetName() const
{
	for (auto& Event : Events)
	{
		if (Event.GetMsg().MsgType() == FMidiMsg::EType::Text)
		{
			const FMidiMsg& Message = Event.GetMsg();
			if (Message.GetTextType() == Harmonix::Midi::Constants::GMeta_TrackName)
			{
				return &GetTextAtIndex(Message.GetTextIndex());
			}
		}
	}
	return nullptr;
}

void FMidiTrack::AddEvent(const FMidiEvent& Event)
{
	if (!Events.IsEmpty() && Event < Events.Last())
	{
		Sorted = false;
	}

	if (PrimaryMidiChannel == -1 && Event.GetMsg().MsgType() == FMidiMsg::EType::Std)
	{
		PrimaryMidiChannel = Event.GetMsg().GetStdChannel();
	}

	// Last name event added at tick zero wins.
	if (Event.GetTick() == 0 && Event.GetMsg().MsgType() == FMidiMsg::EType::Text && Event.GetMsg().GetTextType() == Harmonix::Midi::Constants::GMeta_TrackName)
	{
		for (auto& ExistingEvent : Events)
		{
			if (ExistingEvent.GetTick() != 0)
			{
				break;
			}
			const FMidiMsg& Message = ExistingEvent.GetMsg();
			if (Message.MsgType() == FMidiMsg::EType::Text && Message.GetTextType() == Harmonix::Midi::Constants::GMeta_TrackName)
			{
				ExistingEvent = Event;
				return;
			}
		}
	}

	Events.Add(Event);
}

void FMidiTrack::ChangeTick(FMidiEventList::TIterator Iterator, int32 NewTick)
{
	// find the event in our list (so as to get a non-const version):
	check(Iterator.GetIndex() < Events.Num());

	// adjust the location and flag for sorting.
	Iterator->Tick = NewTick;
	Sorted = false;
}

void FMidiTrack::WriteStdMidi(FMidiWriter& Writer) const
{
	ensureAlwaysMsgf(Sorted, TEXT("Midi events on track \"%s\" are not sorted! Code that built this midi track should call Sort() before attempting to export Standard MIDI file."), GetName() ? **GetName() : TEXT("<unnamed>"));

	for (auto& Event : Events)
	{
		Event.WriteStdMidi(Writer, *this);
	}

	Writer.EndOfTrack();
}

void FMidiTrack::Sort()
{
	if (Sorted)
	{
		return;
	}

	Events.StableSort();

	Sorted = true;
}

void FMidiTrack::ClearEventsAfter(int32 Tick, bool IncludeTick)
{
	// remove operation...
	for (int32 i = Events.Num() - 1; i > 0 && Events[i].GetTick() >= Tick; --i)
	{
		FMidiEvent& Event = Events[i];
		if (Event.GetTick() > Tick || (Event.GetTick() == Tick && IncludeTick))
		{
			Events.RemoveAt(i, EAllowShrinking::No);
		}
	}
	Events.Shrink();
}

void FMidiTrack::ClearEventsBefore(int32 Tick, bool IncludeTick)
{
	// assumes sorted!
	while (Events.Num() > 0 && (Events[0].GetTick() < Tick || (Events[0].GetTick() == Tick && IncludeTick)))
	{
		Events.RemoveAt(0, EAllowShrinking::No);
	}
	Events.Shrink();
}

int32 FMidiTrack::CopyEvents(FMidiTrack& SourceTrack,	int32 FromTick, int32 ThruTick, int32 TickOffset,
	bool Clear,	int32 MinNote, int32 MaxNote, int32 NoteTranspose, bool FilterTrackName)
{
	if (Clear)
	{
		Events.Empty();
	}

	int32 NumCopied = 0;
	const FMidiEventList& SourceEvents = SourceTrack.GetRawEvents();
	for (auto& SourceEvent : SourceEvents)
	{
		if (SourceEvent.GetTick() < FromTick)
		{
			continue;
		}
		if (SourceEvent.GetTick() > ThruTick)
		{
			break;
		}

		const FMidiMsg& SourceMessage = SourceEvent.GetMsg();
		if (SourceMessage.MsgType() == FMidiMsg::EType::Std &&
			(SourceMessage.GetStdStatusType() == Harmonix::Midi::Constants::GNoteOn ||
				SourceMessage.GetStdStatusType() == Harmonix::Midi::Constants::GNoteOff))
		{
			// we may have to filter or transpose...
			int32 NoteNum = SourceMessage.GetStdData1();
			if (NoteNum < MinNote || NoteNum > MaxNote)
			{
				continue; // filter
			}
			FMidiMsg NewMessage(SourceMessage.GetStdStatus(), SourceMessage.GetStdData1() + NoteTranspose, SourceMessage.GetStdData2());
			AddEvent(FMidiEvent(SourceEvent.GetTick() + TickOffset, NewMessage));
		}
		else if (SourceMessage.MsgType() == FMidiMsg::EType::Text)
		{
			// we have to do something a little special for text messages...

			// Is this the track name, and are we filtering the track name?
			if (SourceMessage.GetTextType() == Harmonix::Midi::Constants::GMeta_TrackName && FilterTrackName)
			{
				continue;
			}

			// get a const char* to the text in the src track...
			const FString& EventText = SourceTrack.GetTextAtIndex(SourceMessage.GetTextIndex());

			// copy string to this track and capture new index...
			int32 TextIndex = AddText(EventText);

			// make a new msg with this new text index...
			FMidiMsg NewMessage = FMidiMsg::CreateText(TextIndex, SourceMessage.GetTextType());

			// now add the event to this track...
			AddEvent(FMidiEvent(SourceEvent.GetTick() + TickOffset, NewMessage));
		}
		else
		{
			AddEvent(FMidiEvent(SourceEvent.GetTick() + TickOffset, SourceEvent.GetMsg()));
		}
		NumCopied++;
	}

	Sort();

	return NumCopied;
}

SIZE_T FMidiTrack::GetAllocatedSize() const
{
	SIZE_T AllocatedSize = sizeof(FMidiTrack) + Events.GetAllocatedSize() + Strings.GetAllocatedSize();
	for (const FString& String : Strings)
	{
		AllocatedSize += String.GetAllocatedSize();
	}
	return AllocatedSize;
}
