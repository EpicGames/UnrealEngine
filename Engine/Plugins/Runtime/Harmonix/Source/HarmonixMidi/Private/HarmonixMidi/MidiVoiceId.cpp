// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiVoiceId.h"


FCriticalSection FMidiVoiceGeneratorBase::GeneratorIdLock;
uint32 FMidiVoiceGeneratorBase::NextGeneratorId = 1;

FMidiVoiceGeneratorBase::FMidiVoiceGeneratorBase()
{
	FScopeLock Lock(&GeneratorIdLock);
	IdBits = NextGeneratorId++;
	if (NextGeneratorId >= (1L << IdWidth))
	{
		NextGeneratorId = 1;
	}
	IdBits <<= (32 - IdWidth);
}

