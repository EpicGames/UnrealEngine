// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MidiWriter.h"
#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/VarLenNumber.h"

#define _WRITE_TRACK_MIDI_BYTE(b) do{ uint8 _writeByte = b; *CurrentWriter << _writeByte; }while(false)
#define _WRITE_FILE_MIDI_BYTE(b) do{ uint8 _writeByte = b; Archive << _writeByte; }while(false)

FMidiWriter::FMidiWriter(FArchive& OutArchive, int32 InTicksPerQuarterNote) 
	: Archive(OutArchive)
	, TicksPerQuarterNote(InTicksPerQuarterNote)
	, CurTick(0)
	, Closed(false)
{
	check (!OutArchive.IsLoading());
	OutArchive.ArForceByteSwapping = PLATFORM_LITTLE_ENDIAN == 1;

	TSharedPtr<FBufferWriter> Writer = MakeShared<FBufferWriter>(FMemory::Malloc(1024), 1024, EBufferWriterFlags::TakeOwnership | EBufferWriterFlags::AllowResize);
	TrackWriters.Add(Writer);
	CurrentWriter = TrackWriters.Last();
}

FMidiWriter::~FMidiWriter()
{
	Close();
}

void FMidiWriter::Close()
{
	if (Closed)
	{
		return;
	}
	Write();
	TrackWriters.Empty();
	CurrentWriter = nullptr;
	Closed = true;
}

void FMidiWriter::EndOfTrack()
{
	ProcessTick(CurTick);
	_WRITE_TRACK_MIDI_BYTE(Harmonix::Midi::Constants::GFile_Meta);
	_WRITE_TRACK_MIDI_BYTE(Harmonix::Midi::Constants::GMeta_EndOfTrack);
	_WRITE_TRACK_MIDI_BYTE(0);
	TSharedPtr<FBufferWriter> Writer = MakeShared<FBufferWriter>(FMemory::Malloc(1024), 1024, EBufferWriterFlags::TakeOwnership | EBufferWriterFlags::AllowResize);
	TrackWriters.Add(Writer);
	CurrentWriter = TrackWriters.Last();
	CurTick = 0;
}

void FMidiWriter::MidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2)
{
	using namespace Harmonix::Midi::Constants;
	ProcessTick(Tick);

	*CurrentWriter << Status;

	// messages are of varying length:
	switch (Status & GMessageTypeMask)
	{
	case GNoteOn: // 3-byte
	case GNoteOff:
	case GControl:
	case GPitch:
	case GPolyPres:
		*CurrentWriter << Data1 << Data2;
		break;

		// 2 byte messages
	case GProgram:
	case GChanPres:
		*CurrentWriter << Data1;
		break;

	default:
		checkNoEntry();
		break;
	}
}

void FMidiWriter::Tempo(int32 Tick, int32 Tempo)
{
	using namespace Harmonix::Midi::Constants;
	check(Tempo <= 0xFFFFFF);
	ProcessTick(Tick);
	_WRITE_TRACK_MIDI_BYTE(GFile_Meta);  // meta-event
	_WRITE_TRACK_MIDI_BYTE(GMeta_Tempo); // tempo tag
	_WRITE_TRACK_MIDI_BYTE(0x03);							// size of following
	_WRITE_TRACK_MIDI_BYTE(uint8((Tempo >> 16) & 0xFF));  // 3 bytes of tempo
	_WRITE_TRACK_MIDI_BYTE(uint8((Tempo >> 8) & 0xFF));
	_WRITE_TRACK_MIDI_BYTE(uint8(Tempo & 0xFF));
}

void FMidiWriter::Text(int32 Tick, const TCHAR* Str, uint8 Type)
{
	check(Type >= 0x01 && Type <= 0x07);
	ProcessTick(Tick);
	_WRITE_TRACK_MIDI_BYTE(Harmonix::Midi::Constants::GFile_Meta); // meta-event
	_WRITE_TRACK_MIDI_BYTE(Type);						   // tag

	auto TextAsUtf8 = StringCast<UTF8CHAR>(Str);
	int32 Length = TextAsUtf8.Length();
	Midi::VarLenNumber::Write(*CurrentWriter, (int32)Length);  // length of string
	CurrentWriter->Serialize(const_cast<FGenericPlatformTypes::UTF8CHAR*>(TextAsUtf8.Get()), Length);        // string data
}

void FMidiWriter::TimeSignature(int32 Tick, int32 Numerator, int32 Denominator)
{
	using namespace Harmonix::Midi::Constants;
	check(Numerator > 0);
	check(Denominator > 0);
	ProcessTick(Tick);
	_WRITE_TRACK_MIDI_BYTE(GFile_Meta);    // meta-event
	_WRITE_TRACK_MIDI_BYTE(GMeta_TimeSig); // tag
	_WRITE_TRACK_MIDI_BYTE(0x04);							  // size of following
	_WRITE_TRACK_MIDI_BYTE(uint8(Numerator));               // numerator of time signature

	// denominator is stored as power of 2:
	_WRITE_TRACK_MIDI_BYTE(uint8(FMath::Log2(float(Denominator))));

	// XXX just using sensible defaults for these values
	_WRITE_TRACK_MIDI_BYTE(24); // clocks per click 
	_WRITE_TRACK_MIDI_BYTE(8); // number of 32nd notes per quarter note 
}

// store the delta to the new tick, and update the value
void FMidiWriter::ProcessTick(int32 Tick)
{
	check(!Closed);
	check(Tick >= CurTick);
	Midi::VarLenNumber::Write(*CurrentWriter, uint32(Tick - CurTick));
	CurTick = Tick;
}

void FMidiWriter::Write()
{
	// get rid of the last buffer (which should be empty, if the last function
	// called was EndOfTrack())
	check(TrackWriters.Last()->Tell() == 0);
	TrackWriters.SetNum(TrackWriters.Num()-1);

	WriteFileHeader();

	for (auto& Writer : TrackWriters)
	{
		WriteTrack(*Writer);
	}
}

void FMidiWriter::WriteFileHeader()
{
	_WRITE_FILE_MIDI_BYTE('M');
	_WRITE_FILE_MIDI_BYTE('T');
	_WRITE_FILE_MIDI_BYTE('h');
	_WRITE_FILE_MIDI_BYTE('d');
	uint32 Uint32Buffer = 6; // length of this header
	Archive << Uint32Buffer; 
	uint16 Uint16Buffer = 1; // Format 1
	Archive << Uint16Buffer;
	Uint16Buffer = TrackWriters.Num(); 
	Archive << Uint16Buffer;
	Uint16Buffer = TicksPerQuarterNote; // pulses per quarter-note
	Archive << Uint16Buffer;
}

void FMidiWriter::WriteTrack(FBufferWriter& TrackArchive)
{
	_WRITE_FILE_MIDI_BYTE('M');
	_WRITE_FILE_MIDI_BYTE('T');
	_WRITE_FILE_MIDI_BYTE('r');
	_WRITE_FILE_MIDI_BYTE('k');
	uint32 TrackBytes = TrackArchive.Tell();
	Archive << TrackBytes;
	Archive.Serialize(TrackArchive.GetWriterData(), TrackBytes);
}
#undef WRITE_BYTE
