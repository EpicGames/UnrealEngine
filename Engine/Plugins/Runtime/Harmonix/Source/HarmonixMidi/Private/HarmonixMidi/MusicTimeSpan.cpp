// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMidi/MusicTimeSpan.h"
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMidi/MidiSongPos.h"
#include "HarmonixMidi/MidiConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicTimeSpan)

float FMusicalTimeSpan::CalcPositionInSpan(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const
{
	if (Position.TimeSigDenominator == 0)
	{
		return 0.0f;
	}

	if (Offset != 0)
	{
		return CalcPositionInSpanWithOffset(Position, Maps);
	}

	return CalcPositionInSpanNoOffset(Position, Maps);
}

float FMusicalTimeSpan::CalcPositionInSpanWithOffset(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const
{
	using namespace Harmonix::Midi::Constants;
	FMidiSongPos OffsetPosition;

	int32 UnadjustedTick = Maps.MsToTick(Position.SecondsIncludingCountIn * 1000.0f);
	int32 AdjustedTick = -1;
	switch (OffsetUnits)
	{
	case EMusicTimeSpanOffsetUnits::Ms:
		{
			OffsetPosition.SetByTime(Position.SecondsIncludingCountIn * 1000.0f - Offset, Maps);
		}
		break;
	case EMusicTimeSpanOffsetUnits::Bars:
		{
			float FractionalBar = Position.BarsIncludingCountIn + Maps.GetStartBar() - Offset;
			int32 Tick = Maps.FractionalBarIncludingCountInToTick(FractionalBar);
			OffsetPosition.SetByTick((float)Tick, Maps);
		}
		break;
	case EMusicTimeSpanOffsetUnits::Beats:
		{
			float Beat = Position.BeatsIncludingCountIn - Offset;
			OffsetPosition.SetByTime(Maps.GetMsAtBeat(Beat), Maps);
		}
		break;
	case EMusicTimeSpanOffsetUnits::ThirtySecondNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset / 8;
		break;
	case EMusicTimeSpanOffsetUnits::SixteenthNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset / 4;
		break;
	case EMusicTimeSpanOffsetUnits::EighthNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset / 2;
		break;
	case EMusicTimeSpanOffsetUnits::QuarterNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset;
		break;
	case EMusicTimeSpanOffsetUnits::HalfNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 2;
		break;
	case EMusicTimeSpanOffsetUnits::WholeNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 4;
		break;
	case EMusicTimeSpanOffsetUnits::DottedSixteenthNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 3 / 8;
		break;
	case EMusicTimeSpanOffsetUnits::DottedEighthNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 3 / 4;
		break;
	case EMusicTimeSpanOffsetUnits::DottedQuarterNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 3 / 2;
		break;
	case EMusicTimeSpanOffsetUnits::DottedHalfNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 3;
		break;
	case EMusicTimeSpanOffsetUnits::DottedWholeNotes:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 6;
		break;
	case EMusicTimeSpanOffsetUnits::SixteenthNoteTriplets:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset / 6;
		break;
	case EMusicTimeSpanOffsetUnits::EighthNoteTriplets:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset / 3;
		break;
	case EMusicTimeSpanOffsetUnits::QuarterNoteTriplets:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 2 / 3;
		break;
	case EMusicTimeSpanOffsetUnits::HalfNoteTriplets:
		AdjustedTick = UnadjustedTick - GTicksPerQuarterNoteInt * Offset * 4 / 3;
		break;
	default:
		checkNoEntry();
		break;
	}

	if (AdjustedTick > 0)
	{
		OffsetPosition.SetByTick(AdjustedTick, Maps);
	}

	return CalcPositionInSpanNoOffset(OffsetPosition, Maps);
}

float FMusicalTimeSpan::CalcPositionInSpanNoOffset(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const
{
	if (LengthUnits == EMusicTimeSpanLengthUnits::Bars || LengthUnits == EMusicTimeSpanLengthUnits::Beats)
	{
		return CalculateEnclosingVariableSizeSpanExtents(Position, Maps);
	}
	return CalculateEnclosingFixedSizeSpanExtents(Position, Maps);
}


float FMusicalTimeSpan::CalcPositionInSpan(float Ms, const ISongMapEvaluator& Maps) const
{
	FMidiSongPos Position;
	Position.SetByTime(Ms, Maps);
	return CalcPositionInSpan(Position, Maps);
}

float FMusicalTimeSpan::CalculateEnclosingVariableSizeSpanExtents(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const
{
	switch (LengthUnits)
	{
	case EMusicTimeSpanLengthUnits::Bars:
		{
			float Progress = FMath::Fractional(Position.BarsIncludingCountIn / Length);
			if (Progress < 0.0f)
			{
				Progress += 1.0f;
			}
			return Progress;
		}
	case EMusicTimeSpanLengthUnits::Beats:
		{
			float Progress = FMath::Fractional(Position.BeatsIncludingCountIn / Length);
			if (Progress < 0.0f)
			{
				Progress += 1.0f;
			}
			return Progress;
		}
	default:
		checkNoEntry();
		break;
	}
	return 0.0f;
}

float FMusicalTimeSpan::CalculateEnclosingFixedSizeSpanExtents(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const
{
	int32 Tick = Maps.MsToTick(Position.SecondsIncludingCountIn * 1000.0f);
	int32 MidiTTQ = Maps.GetTicksPerQuarterNote();
	int32 GridUnitTicks = 1;
	switch (LengthUnits)
	{
	case EMusicTimeSpanLengthUnits::ThirtySecondNotes:
		GridUnitTicks = MidiTTQ / 8;
		break;
	case EMusicTimeSpanLengthUnits::SixteenthNotes:
		GridUnitTicks = MidiTTQ / 4;
		break;
	case EMusicTimeSpanLengthUnits::EighthNotes:
		GridUnitTicks = MidiTTQ / 2;
		break;
	case EMusicTimeSpanLengthUnits::QuarterNotes:
		GridUnitTicks = MidiTTQ;
		break;
	case EMusicTimeSpanLengthUnits::HalfNotes:
		GridUnitTicks = MidiTTQ * 2;
		break;
	case EMusicTimeSpanLengthUnits::WholeNotes:
		GridUnitTicks = MidiTTQ * 4;
		break;
	case EMusicTimeSpanLengthUnits::DottedSixteenthNotes:
		GridUnitTicks = MidiTTQ * 3 / 8;
		break;
	case EMusicTimeSpanLengthUnits::DottedEighthNotes:
		GridUnitTicks = MidiTTQ * 3 / 4;
		break;
	case EMusicTimeSpanLengthUnits::DottedQuarterNotes:
		GridUnitTicks = MidiTTQ * 3 / 2;
		break;
	case EMusicTimeSpanLengthUnits::DottedHalfNotes:
		GridUnitTicks = MidiTTQ * 3;
		break;
	case EMusicTimeSpanLengthUnits::DottedWholeNotes:
		GridUnitTicks = MidiTTQ * 6;
		break;
	case EMusicTimeSpanLengthUnits::SixteenthNoteTriplets:
		GridUnitTicks = MidiTTQ / 6;
		break;
	case EMusicTimeSpanLengthUnits::EighthNoteTriplets:
		GridUnitTicks = MidiTTQ / 3;
		break;
	case EMusicTimeSpanLengthUnits::QuarterNoteTriplets:
		GridUnitTicks = MidiTTQ * 2 / 3;
		break;
	case EMusicTimeSpanLengthUnits::HalfNoteTriplets:
		GridUnitTicks = MidiTTQ * 4 / 3;
		break;
	default:
		checkNoEntry();
		break;
	}
	int32 CellSizeTicks = GridUnitTicks * Length;
	float InCell = (float) Tick / CellSizeTicks;
	float Progress = FMath::Fractional(InCell);
	return Progress < 0.0f ? 1.0f + Progress : Progress;
}

namespace MusicalTimeSpan
{
	int32 CalculateMusicalSpanLengthTicks(EMusicTimeSpanLengthUnits Unit, const ISongMapEvaluator& Maps, int32 AtTick)
	{
		int32 MidiTTQ = Maps.GetTicksPerQuarterNote();
		int32 GridUnitTicks = 1;
		switch (Unit)
		{
		case EMusicTimeSpanLengthUnits::Bars:
			{
				const FTimeSignature* TimeSig = Maps.GetTimeSignatureAtTick(AtTick);
				check(TimeSig);
				int32 TicksInBeat = MidiTTQ * 4 / TimeSig->Denominator;
				GridUnitTicks = TicksInBeat * TimeSig->Numerator;
			}
			break;
		case EMusicTimeSpanLengthUnits::Beats:
			{
				const FTimeSignature* TimeSig = Maps.GetTimeSignatureAtTick(AtTick);
				check(TimeSig);
				GridUnitTicks = MidiTTQ * 4 / TimeSig->Denominator;
			}
			break;
		case EMusicTimeSpanLengthUnits::ThirtySecondNotes:
			GridUnitTicks = MidiTTQ / 8;
			break;
		case EMusicTimeSpanLengthUnits::SixteenthNotes:
			GridUnitTicks = MidiTTQ / 4;
			break;
		case EMusicTimeSpanLengthUnits::EighthNotes:
			GridUnitTicks = MidiTTQ / 2;
			break;
		case EMusicTimeSpanLengthUnits::QuarterNotes:
			GridUnitTicks = MidiTTQ;
			break;
		case EMusicTimeSpanLengthUnits::HalfNotes:
			GridUnitTicks = MidiTTQ * 2;
			break;
		case EMusicTimeSpanLengthUnits::WholeNotes:
			GridUnitTicks = MidiTTQ * 4;
			break;
		case EMusicTimeSpanLengthUnits::DottedSixteenthNotes:
			GridUnitTicks = MidiTTQ * 3 / 8;
			break;
		case EMusicTimeSpanLengthUnits::DottedEighthNotes:
			GridUnitTicks = MidiTTQ * 3 / 4;
			break;
		case EMusicTimeSpanLengthUnits::DottedQuarterNotes:
			GridUnitTicks = MidiTTQ * 3 / 2;
			break;
		case EMusicTimeSpanLengthUnits::DottedHalfNotes:
			GridUnitTicks = MidiTTQ * 3;
			break;
		case EMusicTimeSpanLengthUnits::DottedWholeNotes:
			GridUnitTicks = MidiTTQ * 6;
			break;
		case EMusicTimeSpanLengthUnits::SixteenthNoteTriplets:
			GridUnitTicks = MidiTTQ / 6;
			break;
		case EMusicTimeSpanLengthUnits::EighthNoteTriplets:
			GridUnitTicks = MidiTTQ / 3;
			break;
		case EMusicTimeSpanLengthUnits::QuarterNoteTriplets:
			GridUnitTicks = MidiTTQ * 2 / 3;
			break;
		case EMusicTimeSpanLengthUnits::HalfNoteTriplets:
			GridUnitTicks = MidiTTQ * 4 / 3;
			break;
		default:
			checkNoEntry();
			break;
		}
		return GridUnitTicks;
	}
}
