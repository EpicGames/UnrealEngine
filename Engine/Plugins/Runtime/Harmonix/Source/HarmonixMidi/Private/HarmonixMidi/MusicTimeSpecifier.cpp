// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/MusicTimeSpecifier.h"

#include "Containers/UnrealString.h"

#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/MidiConstants.h"

namespace Midi
{
	FMusicTimestamp HARMONIXMIDI_API ParseMusicTimestamp(const FString& Str, EMusicTimeStringFormat Format)
	{
		FString MutableStr = Str;
		MutableStr.ReplaceCharInline(':','.');
		int32 NumDelimiters = 0;
		for (int32 i = 0; i < MutableStr.Len(); ++i)
		{
			if (MutableStr[i] == '.')
			{
				NumDelimiters++;
			}
		}
		TArray<FString> Parts;
		MutableStr.ParseIntoArray(Parts, TEXT("."));

		ensureAlwaysMsgf(NumDelimiters <= 2, TEXT("Enexpected number of delimiters in music time string %s."), *Str);
		for (const FString& P : Parts)
		{
			ensureAlwaysMsgf(P.IsNumeric(), TEXT("Enexpected character in music time string %s (%s)."), *Str, *P);
		}

		FMusicTimestamp Result;

		if (Parts.Num() == 0)
		{
			if (Format == EMusicTimeStringFormat::Duration)
				return {0, 0.0f};
			return {1, 1.0f};
		}

		int32 PartIndex = 0;
		if (Parts.Num() > 2)
		{
			Result.Bar = FCString::Atoi(*Parts[PartIndex++]);
			if (Result.Bar < 0 && Format == EMusicTimeStringFormat::Duration)
			{
				UE_LOGF(LogMIDI, Warning, "Unexpected bar value specified in midi duration string \"%ls\". Bar in duration string should be >= 0!",*Str);
				Result.Bar = 0;
			}
		}
		else if (Format == EMusicTimeStringFormat::Duration)
		{
			Result.Bar = 0;
		}
		else
		{
			Result.Bar = 1;
		}

		if (Parts.Num() > 1 || (Parts.Num() == 1 && NumDelimiters == 0))
		{
			Result.Beat = (float)FCString::Atoi(*Parts[PartIndex++]);
			if (Result.Beat < 1.0f && Format == EMusicTimeStringFormat::Position)
			{
				UE_LOGF(LogMIDI, Warning, "Unexpected beat value specified in midi position string %ls. Beat in position string should be >= 1!", *Str);
				Result.Beat = 1;
			}
			else if (Result.Beat < 0 && Format == EMusicTimeStringFormat::Duration)
			{
				UE_LOGF(LogMIDI, Warning, "Unexpected beat value specified in midi duration string %ls. Beat in duration string should be >= 0!", *Str);
				Result.Beat = 0;
			}
		}

		if (PartIndex < Parts.Num())
		{
			float div = FMath::Pow(10.0f, (float)Parts[PartIndex].Len());
			float num = (float)FCString::Atoi(*Parts[PartIndex]);
			if (num < 0)
			{
				UE_LOGF(LogMIDI, Warning, "Unexpected negative value found in fractional beat portion of midi time specifier string %ls. Using 0 for fractional portion!", *Str);
				num = 0.0f;
			}
			Result.Beat +=  num / div;
		}

		return Result;
	}

	FString HARMONIXMIDI_API FormatBarBeat(const FMusicTimestamp& Timestamp, EMusicTimeStringFormat Format)
	{
		TStringBuilder<32> Builder;
		Builder << Timestamp.Bar << ":";
		Builder.Appendf(TEXT("%.3f"), Timestamp.Beat);
		return Builder.ToString();
	}

	int32 HARMONIXMIDI_API BarBeatStringtoMidiTick(const FString& Str, const FBarMap* BarMap, EMusicTimeStringFormat Format)
	{
		FMusicTimestamp Timestamp = ParseMusicTimestamp(Str, Format);
		return BarMap->MusicTimestampToTick(Timestamp);
	}

	FString HARMONIXMIDI_API MidiTickFormat(int32 InTick, const FBarMap* Map, EMusicTimeStringFormat Format)
	{
		if (InTick >= 0 && Map)
		{
			FMusicTimestamp Timestamp = Map->TickToMusicTimestamp(InTick);
			return FormatBarBeat(Timestamp, Format);
		}

		TStringBuilder<32> Builder;
		Builder << TEXT("tick ") << InTick;
		return Builder.ToString();
	}
}