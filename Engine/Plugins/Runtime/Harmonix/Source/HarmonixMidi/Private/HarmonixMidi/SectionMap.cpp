// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/SectionMap.h"
#include "HarmonixMidi/BeatMap.h"
#include "Misc/RuntimeErrors.h"
#include "Algo/BinarySearch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SectionMap)

bool FSectionMap::operator==(const FSectionMap& Other) const
{
	if (TicksPerQuarterNote != Other.TicksPerQuarterNote || Points.Num() != Other.Points.Num())
	{
		return false;
	}

	for (int32 PointIndex = 0; PointIndex < Points.Num(); ++PointIndex)
	{
		if (Points[PointIndex] != Other.Points[PointIndex])
		{
			return false;
		}
	}
	return true;
}

void FSectionMap::Finalize(int32 LastTick)
{
	FMusicMapUtl::Finalize(Points, LastTick);
}

void FSectionMap::Empty()
{
	Points.Empty();
}

void FSectionMap::Copy(const FSectionMap& Other, int32 StartTick, int32 EndTick)
{
	TicksPerQuarterNote = Other.TicksPerQuarterNote;
	FMusicMapUtl::Copy(Other.Points, Points, StartTick, EndTick);
}

bool FSectionMap::IsEmpty() const
{
	return Points.IsEmpty();
}

bool FSectionMap::AddSection(const FString& Name, int32 InStartTick, int32 InLengthTicks, bool SortNow)
{
	FMusicMapUtl::AddPoint<FSongSection, FString>(Name, Points, InStartTick, SortNow, InLengthTicks);
	return true;
}

int32 FSectionMap::TickToSectionIndex(int32 Tick) const
{
	return FMusicMapUtl::GetPointIndexForTick(Points, Tick);
}

const FSongSection* FSectionMap::TickToSection(int32 Tick) const
{
	int32 SectionIndex = FMusicMapUtl::GetPointIndexForTick(Points, Tick);
	if (SectionIndex != -1)
	{
		return &Points[SectionIndex];
	}
	return nullptr;
}

int32 FSectionMap::GetSectionStartTick(const FString& Name) const
{
	const FSongSection* Section = FindSectionInfo(Name);
	if (!Section)
	{
		return -1;
	}
	return Section->StartTick;
}

int32 FSectionMap::GetSectionStartTick(int32 SectionIndex) const
{
	if (SectionIndex < 0 || SectionIndex >= Points.Num())
	{
		return -1;
	}
	return Points[SectionIndex].StartTick;
}

const FSongSection* FSectionMap::GetSection(int32 SectionIndex) const
{
	if (SectionIndex < 0 || SectionIndex >= Points.Num())
	{
		return nullptr;
	}
	return &Points[SectionIndex];
}

FString FSectionMap::GetSectionName(int32 SectionIndex) const
{
	if (SectionIndex < 0 || SectionIndex >= Points.Num())
	{
		return FString();
	}
	return Points[SectionIndex].Name;
}

FString FSectionMap::GetSectionNameAtTick(int32 Tick) const
{
	const FSongSection* Section = TickToSection(Tick);
	if (!Section)
	{
		return FString();
	}
	return Section->Name;
}

void FSectionMap::GetSectionNames(TArray<FString>& Names) const
{
	for (auto& Section : Points)
	{
		Names.Emplace(Section.Name);
	}
}

int32 FSectionMap::FindSectionIndex(const FString& Name) const
{
	for (int32 i = 0; i < Points.Num(); ++i)
	{
		if (Points[i].Name == Name)
		{
			return i;
		}
	}
	return -1;
}

const FSongSection* FSectionMap::FindSectionInfo(const FString& Name) const
{
	int32 i = FindSectionIndex(Name);
	if (i != -1)
	{
		return &Points[i];
	}
	return nullptr;
}

