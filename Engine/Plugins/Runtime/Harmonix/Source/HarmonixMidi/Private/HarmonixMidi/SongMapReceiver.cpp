// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/SongMapReceiver.h"
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMidi/MidiReader.h"
#include "HarmonixMidi/MusicTimeSpecifier.h"

#define _LOG_SONGMAP_WARNING(t,...) UE_LOG(LogMIDI,Warning,TEXT("%s (%s): " t), *Reader->GetFilename(), *Reader->GetCurrentTrackName(), ##__VA_ARGS__)
#define _LOG_SONGMAP_ERROR(t,...) UE_LOG(LogMIDI,Error,TEXT("%s (%s): " t), *Reader->GetFilename(), *Reader->GetCurrentTrackName(), ##__VA_ARGS__)

FSongMapReceiver::FSongMapReceiver(FSongMaps* Maps)
	: SongMaps(Maps)
{
	check(Maps);
	Reset();
}

bool FSongMapReceiver::Reset()
{
	CurrentTrack      = EMidiTrack::UnknownTrack;
	LastTick          = 0;
	TrackIndex        = -1;
	LastBeatTick      = -1;
	LastBeatType      = EMidiNoteAssignments::Invalid;
	bHaveBeatFailure  = false;
	SongMaps->EmptyAllMaps();
	return true;
}

bool FSongMapReceiver::Finalize(int32 InLastFileTick)
{
	if (InLastFileTick > LastTick)
	{
		LastTick = InLastFileTick;
	}
	return SongMaps->FinalizeRead(Reader);
}

bool FSongMapReceiver::OnNewTrack(int32 NewTrackIndex)
{
	TrackIndex = NewTrackIndex;
	if (NewTrackIndex == 0)
	{
		CurrentTrack = EMidiTrack::FirstTrack;
	}
	return true;
}

bool FSongMapReceiver::OnEndOfTrack(int32 InLastTick)
{
	if (InLastTick > LastTick)
	{
		LastTick = InLastTick;
	}
	TrackIndex = -1;
	CurrentTrack = EMidiTrack::UnknownTrack;
	return true;
}

bool FSongMapReceiver::OnMidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2)
{
	if (CurrentTrack != EMidiTrack::BeatTrack)
	{
		return true;
	}
	
	using namespace Harmonix::Midi::Constants;

	if (GetType(Status) != GNoteOn)
	{
		return true;
	}

	if ((EMidiNoteAssignments)Data1 != EMidiNoteAssignments::NormalBeatPitch && (EMidiNoteAssignments)Data1 != EMidiNoteAssignments::StrongBeatPitch)
	{
		return true;
	}


	// check for errors in the beat map:
	if ((LastBeatTick != -1) && !bHaveBeatFailure)
	{
		// Don't allow beat tracks faster than 16th note denominators.
		if ((Tick - LastBeatTick) < GTicksPerQuarterNoteInt / 4)
		{
			_LOG_SONGMAP_ERROR("Beat track cannot be faster than 16th notes; beats are less than %d ticks apart at %s", GTicksPerQuarterNoteInt / 4, *FmtTick(Tick));
			bHaveBeatFailure = true;
		}

		// We don't want two downbeats back to back...
		if ((LastBeatType == EMidiNoteAssignments::StrongBeatPitch) &&
			((EMidiNoteAssignments)Data1 == EMidiNoteAssignments::StrongBeatPitch))
		{
			_LOG_SONGMAP_ERROR("Two strong beats occur back to back at %s and %s", *FmtTick(LastBeatTick), *FmtTick(Tick));
			bHaveBeatFailure = true;
		}
	}

	// add the beat
	EMusicalBeatType BeatType = ((EMidiNoteAssignments)Data1 == EMidiNoteAssignments::StrongBeatPitch) ? EMusicalBeatType::Strong : EMusicalBeatType::Normal;
	if (BeatType == EMusicalBeatType::Strong && FMath::IsNearlyZero(FMath::Frac(SongMaps->GetBarMap().TickToFractionalBarIncludingCountIn(Tick)), UE_KINDA_SMALL_NUMBER))
	{
		BeatType = EMusicalBeatType::Downbeat;
	}
	SongMaps->BeatMap.AddBeat(BeatType, Tick, false);
	LastBeatTick = Tick;
	LastBeatType = (EMidiNoteAssignments)Data1;
	return true;
}

bool FSongMapReceiver::OnText(int32 Tick, const FString& Str, uint8 Type)
{
	// we only care about track name
	if (Type == Harmonix::Midi::Constants::GMeta_TrackName)
	{
		OnTrackName(Str);
	}
	else if (Type == Harmonix::Midi::Constants::GMeta_Text)
	{
		switch (CurrentTrack)
		{
		case EMidiTrack::SectionTrack:
			ReadSectionTrackText(Tick, Str);
			break;
		case EMidiTrack::ChordTrack:
			ReadChordTrackText(Tick, Str);
			break;
		}
	}
	return true;
}

bool FSongMapReceiver::OnTempo(int32 Tick, int32 Tempo)
{
	using namespace Harmonix::Midi::Constants;

	// Filter for reasonable tempos...
	float Bpm = MidiTempoToBPM(Tempo);
	if (Bpm < GMinMidiFileTempo || Bpm > GMaxMidiFileTempo)
	{
		_LOG_SONGMAP_ERROR("Unsupported MIDI tempo encountered (%f). Tempo must be between %f and %f.", Bpm, GMinMidiFileTempo, GMaxMidiFileTempo);
		return false;
	}

	if (!SongMaps->TempoMap.AddTempoInfoPoint(Tempo, Tick))
	{
		_LOG_SONGMAP_WARNING("Tempo marker at %s (%.f bpm) conflicts with other tempo markers", *FmtTick(Tick), (60000000.0f / Tempo));
		return false;
	}
	return true;
}

bool FSongMapReceiver::OnTimeSignature(int32 Tick, int32 Numerator, int32 Denominator, bool FailOnError)
{
	check(Tick == 0 || SongMaps->BarMap.GetNumTimeSignaturePoints() > 0);
	int32 BarIndex = SongMaps->BarMap.TickToBarIncludingCountIn(Tick);

	if (!SongMaps->BarMap.AddTimeSignatureAtBarIncludingCountIn(BarIndex, Numerator, Denominator, true, FailOnError))
	{
		_LOG_SONGMAP_WARNING("Time signature %d/%d at %s overlaps or conflicts with nearby time signatures", Numerator, Denominator, *FmtTick(Tick));
		return false;
	}
	return true;
}

void FSongMapReceiver::ReadSectionTrackText(int32 Tick, const FString& Str)
{
	SongMaps->SectionMap.AddSection(*Str, Tick, 1, false);
}

void FSongMapReceiver::ReadChordTrackText(int32 Tick, const FString& Str)
{
	SongMaps->ChordMap.AddChord(*Str, Tick, false);
}

void FSongMapReceiver::OnTrackName(const FString& TrackName)
{
	TArray<FString>& TrackNames = SongMaps->TrackNames;
	if (TrackIndex >= TrackNames.Num())
	{
		TrackNames.SetNum(TrackIndex + 1);
	}
	TrackNames[TrackIndex] = TrackName;

	if (TrackName == "BEAT" || TrackName == "PULSE")
	{
		OnStartBeatTrack();
		return;
	}
	else if (TrackName == "SECTION" || TrackName == "SECTIONS")
	{
		OnStartSectionTrack();
		return;
	}
	else if (TrackName == "CHORD" || TrackName == "CHORDS")
	{
		OnStartChordTrack();
		return;
	}
}

void FSongMapReceiver::OnStartSectionTrack()
{
	CurrentTrack = EMidiTrack::SectionTrack;
	ensureAlwaysMsgf(SongMaps->SectionMapIsEmpty(), TEXT("Multiple section tracks in MIDI file %s?"), *GetReader()->GetFilename());
	SongMaps->EmptySectionMap();
	return;
}

void FSongMapReceiver::OnStartChordTrack()
{
	CurrentTrack = EMidiTrack::ChordTrack;
	ensureAlwaysMsgf(SongMaps->ChordMapIsEmpty(), TEXT("Multiple chord tracks in MIDI file (%s)?"), *GetReader()->GetFilename());
	SongMaps->EmptyChordMap();
	SongMaps->ChordMap.SetTrack(TrackIndex);
	return;
}

void FSongMapReceiver::OnStartBeatTrack()
{
	CurrentTrack = EMidiTrack::BeatTrack;
	ensureAlwaysMsgf(SongMaps->BeatMapIsEmpty(), TEXT("Multiple beat map tracks in MIDI file (%s)?"), *GetReader()->GetFilename());
	SongMaps->EmptyBeatMap();
	return;
}

FString FSongMapReceiver::FmtTick(int32 Tick) const
{
	return MidiTickFormat(Tick, &SongMaps->BarMap, Midi::EMusicTimeStringFormat::Position);
}

#undef _LOG_SONGMAP_WARNING
#undef _LOG_SONGMAP_ERROR
