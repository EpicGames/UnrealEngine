// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MidiReceiver.h"

struct FSongMaps;

class FSongMapReceiver : public IMidiReceiver
{
public:
	FSongMapReceiver(FSongMaps* Maps);
	virtual ~FSongMapReceiver()	= default;

	virtual bool Reset() override;
	virtual bool Finalize(int32 InLastFileTick) override;

	// Track begin/end:
	virtual bool OnNewTrack(int32 NewTrackIndex) override;
	virtual bool OnEndOfTrack(int32 InLastTick) override;
	virtual bool OnAllTracksRead() override { return true; };

	// Standard 1- or 2-byte MIDI message:
	virtual bool OnMidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2) override;
	virtual bool OnText(int32 Tick, const FString& Str, uint8 Type) override;
	virtual bool OnTempo(int32 Tick, int32 Tempo) override;
	virtual bool OnTimeSignature(int32 Tick, int32 Numerator, int32 Denominator, bool FailOnError = true) override;

	int32 GetLastTick() { return LastTick; }

private:
	void ReadSectionTrackText(int32 Tick, const FString& Str);
	void ReadChordTrackText(int32 Tick, const FString& Str);
	void OnTrackName(const FString& TrackName);
	void OnStartSectionTrack();
	void OnStartChordTrack();
	void OnStartBeatTrack();

	FString FmtTick(int32 Tick) const;

	enum class EMidiNoteAssignments {
		Invalid = -1,
		DetailedGridPitch = 11,
		StrongBeatPitch = 12,
		NormalBeatPitch = 13,
	};

	enum class EMidiTrack {
		UnknownTrack,
		FirstTrack,
		BeatTrack,
		SectionTrack,
		ChordTrack,
	};

	EMidiTrack CurrentTrack = EMidiTrack::UnknownTrack;
	int32 TrackIndex        = -1;
	int32 LastBeatTick     = -1;
	bool bHaveBeatFailure  = false;
	int32 LastTick          = 0;
	FSongMaps* SongMaps     = nullptr;
	EMidiNoteAssignments LastBeatType = EMidiNoteAssignments::Invalid;
};
