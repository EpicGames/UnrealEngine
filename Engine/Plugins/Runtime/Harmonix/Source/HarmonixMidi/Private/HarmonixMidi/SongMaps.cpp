// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMidi/SongMapReceiver.h"
#include "HarmonixMidi/MidiFile.h"
#include "HarmonixMidi/MidiReader.h"
#include "Misc/RuntimeErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SongMaps)

FSongMaps::FSongMaps()
{
}

FSongMaps::FSongMaps(float Bpm, int32 TimeSigNumerator, int32 TimeSigDenominator)
{
	TempoMap.AddTempoInfoPoint(Harmonix::Midi::Constants::BPMToMidiTempo(Bpm), 0);
	BarMap.AddTimeSignatureAtBarIncludingCountIn(0, TimeSigNumerator == 0 ? 4 : TimeSigNumerator, TimeSigDenominator == 0 ? 4 : TimeSigDenominator);
}

FSongMaps::FSongMaps(const ISongMapEvaluator& Other)
{
	TicksPerQuarterNote = Other.GetTicksPerQuarterNote();
	TempoMap = Other.GetTempoMap();
	BarMap = Other.GetBarMap();
	BeatMap = Other.GetBeatMap();
	SectionMap = Other.GetSectionMap();
	ChordMap = Other.GetChordMap();
	LengthData = Other.GetSongLengthData();
}

bool FSongMaps::operator==(const FSongMaps& Other) const
{
	if (TrackNames.Num() != Other.TrackNames.Num())
	{
		return false;
	}
	for (int32 TrackIndex = 0; TrackIndex < TrackNames.Num(); ++TrackIndex)
	{
		if (TrackNames[TrackIndex] != Other.TrackNames[TrackIndex])
		{
			return false;
		}
	}
	return	TicksPerQuarterNote == Other.TicksPerQuarterNote &&
		TempoMap == Other.TempoMap &&
		BarMap == Other.BarMap &&
		BeatMap == Other.BeatMap &&
		SectionMap == Other.SectionMap &&
		ChordMap == Other.ChordMap &&
		LengthData == Other.LengthData;
}

void FSongMaps::Init(int32 InTicksPerQuarterNote)
{
	TicksPerQuarterNote = InTicksPerQuarterNote;
	TempoMap.SetTicksPerQuarterNote(InTicksPerQuarterNote);
	TempoMap.Empty();
	BarMap.SetTicksPerQuarterNote(InTicksPerQuarterNote);
	BarMap.Empty();
	BeatMap.SetTicksPerQuarterNote(InTicksPerQuarterNote);
	BeatMap.Empty();
	SectionMap.SetTicksPerQuarterNote(InTicksPerQuarterNote);
	SectionMap.Empty();
	ChordMap.SetTicksPerQuarterNote(InTicksPerQuarterNote);
	ChordMap.Empty();
}

void FSongMaps::Copy(const ISongMapEvaluator& Other, int32 StartTick, int32 EndTick)
{
	EmptyAllMaps();

	TicksPerQuarterNote = Other.GetTicksPerQuarterNote();

	TempoMap.Copy(Other.GetTempoMap(), StartTick, EndTick);
	BarMap.Copy(Other.GetBarMap(), StartTick, EndTick);
	BeatMap.Copy(Other.GetBeatMap(), StartTick, EndTick);
	SectionMap.Copy(Other.GetSectionMap(), StartTick, EndTick);
	ChordMap.Copy(Other.GetChordMap(), StartTick, EndTick);

	int32 LastTick = EndTick == -1 ? Other.GetSongLengthData().LastTick : EndTick;
	TempoMap.Finalize(LastTick);
	BarMap.Finalize(LastTick);
	BeatMap.Finalize(LastTick);
	SectionMap.Finalize(LastTick);
	ChordMap.Finalize(LastTick);

	if (EndTick != std::numeric_limits<int32>::max())
	{
		memset(&LengthData, 0, sizeof(LengthData));
		EMidiClockSubdivisionQuantization Division = EMidiClockSubdivisionQuantization::None;
		LengthData.LastTick = QuantizeTickToAnyNearestSubdivision(EndTick, EMidiFileQuantizeDirection::Nearest, Division) - 1;
		LengthData.LengthTicks = LengthData.LastTick + 1;
		LengthData.LengthFractionalBars = BarMap.TickToFractionalBarIncludingCountIn(LengthData.LengthTicks);
	}
	else
	{
		LengthData = Other.GetSongLengthData();
	}
}

bool FSongMaps::LoadFromStdMidiFile(const FString& FilePath)
{
	FSongMapReceiver MapReceiver(this);
	FStdMidiFileReader Reader(FilePath, &MapReceiver);
	return ReadWithReader(Reader);
}

bool FSongMaps::LoadFromStdMidiFile(void* Buffer, int32 BufferSize, const FString& Filename)
{
	FSongMapReceiver MapReceiver(this);
	FStdMidiFileReader Reader(Buffer, BufferSize, Filename, &MapReceiver);
	return ReadWithReader(Reader);
}

bool FSongMaps::LoadFromStdMidiFile(TSharedPtr<FArchive> Archive, const FString& Filename)
{
	FSongMapReceiver MapReceiver(this);
	FStdMidiFileReader Reader(Archive, Filename, &MapReceiver);
	return ReadWithReader(Reader);
}

void FSongMaps::FillInFrameBasedMusicMap(UFrameBasedMusicMap* Map) const
{
	Map->Clear();

	auto TempoMapIterator = TempoMap.GetTempoPoints().begin();
	auto BarMapIterator = BarMap.GetTimeSignaturePoints().begin();

	check(TempoMapIterator != TempoMap.GetTempoPoints().end());
	check(BarMapIterator != BarMap.GetTimeSignaturePoints().end());

	int32 NextTempoPointTick = (*TempoMapIterator).StartTick;
	int32 NextBarPointTick = (*BarMapIterator).StartTick;

	while (NextTempoPointTick != std::numeric_limits<int32>::max() ||
		NextBarPointTick != std::numeric_limits<int32>::max())
	{
		if (NextBarPointTick <= NextTempoPointTick)
		{
			Map->AddTimeSignature(NextBarPointTick, (*BarMapIterator).BarIndex, (*BarMapIterator).TimeSignature.Numerator, (*BarMapIterator).TimeSignature.Denominator);
			++BarMapIterator;
			NextBarPointTick = (BarMapIterator != BarMap.GetTimeSignaturePoints().end()) ? (*BarMapIterator).StartTick : std::numeric_limits<int32>::max();
			continue;
		}
		Map->AddTempo(NextTempoPointTick, TempoMap.TickToMs(NextTempoPointTick),  (*TempoMapIterator).GetBPM());
		++TempoMapIterator;
		NextTempoPointTick = (TempoMapIterator != TempoMap.GetTempoPoints().end()) ? (*TempoMapIterator).StartTick : std::numeric_limits<int32>::max();
	}
}

bool FSongMaps::FinalizeRead(IMidiReader* Reader)
{
	int32 LastTick = Reader->GetLastTick();

	if (Reader->IsFailed())
	{
		EmptyAllMaps();
		return false;
	}

	TempoMap.Finalize(LastTick);
	BarMap.Finalize(LastTick);
	BeatMap.Finalize(LastTick);
	SectionMap.Finalize(LastTick);
	ChordMap.Finalize(LastTick);

	memset(&LengthData, 0, sizeof(LengthData));
	LengthData.LastTick = LastTick;
	LengthData.LengthTicks = LengthData.LastTick + 1;
	LengthData.LengthFractionalBars = BarMap.TickToFractionalBarIncludingCountIn(LengthData.LengthTicks);
	return true;
}

bool FSongMaps::ReadWithReader(FStdMidiFileReader& Reader)
{
	Reader.ReadAllTracks();
	return FinalizeRead(&Reader);
}

void FSongMaps::EmptyAllMaps()
{
	TempoMap.Empty();
	BarMap.Empty();
	BeatMap.Empty();
	SectionMap.Empty();
	ChordMap.Empty();
	TrackNames.Empty();
	LengthData.LastTick = 0;
	LengthData.LengthFractionalBars = 0.0f;
	LengthData.LengthTicks = 0;
}

bool FSongMaps::IsEmpty() const
{
	return 	TempoMap.IsEmpty() &&
			BarMap.IsEmpty() &&
			BeatMap.IsEmpty() &&
			SectionMap.IsEmpty() &&
			ChordMap.IsEmpty() &&
			TrackNames.IsEmpty() &&
			LengthData.LastTick == 0 &&
			LengthData.LengthFractionalBars == 0 &&
			LengthData.LengthTicks == 0;
}

void FSongMaps::SetStartBar(int32 StartBar)
{
	BarMap.SetStartBar(StartBar);
}

void FSongMaps::SetSongLengthTicks(int32 NewLengthTicks)
{
	if (NewLengthTicks < 1)
	{
		UE_LOGF(LogMIDI, Warning, "SetSongLengthTicks: Asked to set length less than 1. That is not possible. Setting to length 1!");
		NewLengthTicks = 1;
	}
	LengthData.LengthTicks = NewLengthTicks;
	LengthData.LastTick = NewLengthTicks - 1;
	LengthData.LengthFractionalBars = BarMap.TickToFractionalBarIncludingCountIn(LengthData.LengthTicks);
}

void FSongMaps::FinalizeBarMap(int32 InLastTick)
{
	BarMap.Finalize(InLastTick);
}

bool FSongMaps::AddTempoInfoPoint(int32 MicrosecondsPerQuarterNote, int32 Tick, bool SortNow)
{
	return TempoMap.AddTempoInfoPoint(MicrosecondsPerQuarterNote, Tick, SortNow);
}

bool FSongMaps::AddTimeSignatureAtBarIncludingCountIn(int32 BarIndex, int32 InNumerator, int32 InDenominator, bool SortNow, bool FailOnError)
{
	return BarMap.AddTimeSignatureAtBarIncludingCountIn(BarIndex, InNumerator, InDenominator, SortNow, FailOnError);
}

FTimeSignaturePoint* FSongMaps::GetMutableTimeSignaturePoint(int32 PointIndex)
{
	if (!BarMap.GetTimeSignaturePoints().IsValidIndex(PointIndex))
	{
		return nullptr;
	}
	return &BarMap.GetTimeSignaturePoint(PointIndex);
}

void FSongMaps::StringLengthToMT(const FString& LengthString, int32& OutBars, int32& OutTicks)
{
	if (BarMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Bar Map.");
		return;
	}

	// ASSUMES WE DON'T HAVE A COMPLICATED TIME SIGNATURE.
	// IF WE DID, THEN WE WOULD NEVER GET HERE, AS WE WOULD
	// HAVE DETERMINED THE LENGTH FROM THE MIDI FILE!

	// ALSO: LengthString, being a 'length', is specifying 0 based bars and beats!
	//       And, the return values are lengths, so also 0 based.
	const TCHAR* Walk = *LengthString;
	OutBars = 0;
	int32 Beats = 0;
	OutTicks = 0;

	// get bars...
	int32 Count = 0;
	while (*Walk != ':' && *Walk != 0 && Count < 3)
	{
		OutBars = (OutBars * 10) + ((int32)(*Walk) - (int32)'0');
		Walk++;
		Count++;
	}
	if (*Walk == ':')
	{
		// get beats...
		Count = 0;
		Walk++;
		while (*Walk != ':' && *Walk != 0 && Count < 3)
		{
			Beats = (Beats * 10) + ((int32)(*Walk) - (int32)'0');
			Walk++;
			Count++;
		}
		if (*Walk == ':')
		{
			// get ticks...
			Count = 0;
			Walk++;
			while (*Walk != ':' && *Walk != 0 && Count < 3)
			{
				OutTicks = (OutTicks * 10) + ((int32)(*Walk) - (int32)'0');
				Walk++;
				Count++;
			}
		}
	}
	const FTimeSignaturePoint* TimeSignaturePoint = &BarMap.GetTimeSignaturePoint(0);
	if (!TimeSignaturePoint)
	{
		UE_LOGF(LogMIDI, Log, "No Time Signature found in SongMaps.");
		return;
	}
	OutBars += (Beats == 0 && OutTicks == 0) ? 0 : 1;
	int32 TicksPerBeat = TicksPerQuarterNote / (TimeSignaturePoint->TimeSignature.Denominator / 4);
	int32 TicksPerBar = TicksPerBeat * TimeSignaturePoint->TimeSignature.Numerator;
	OutTicks = (TicksPerBar * OutBars)
		+ (TicksPerBeat * Beats)
		+ OutTicks;
}

FString FSongMaps::GetTrackName(int32 Index) const
{
	if (Index < 0 || Index >= TrackNames.Num())
		return FString();

	return TrackNames[Index];
}

int32 ISongMapEvaluator::GetTicksPerQuarterNote() const
{
	return GetTempoMap().GetTicksPerQuarterNote();
}

float ISongMapEvaluator::TickToMs(float Tick) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return 0.0f;
	}
	return TempoMap.TickToMs(Tick);
}

float ISongMapEvaluator::MsToTick(float Ms) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return 0;
	}
	return TempoMap.MsToTick(Ms);
}

float ISongMapEvaluator::GetCountInSeconds() const
{
	const FBarMap& BarMap = GetBarMap();
	const FTempoMap& TempoMap = GetTempoMap();
	float BarOneBeatOneTick = BarMap.MusicTimestampToTick({ 1, 1.0f });
	return TempoMap.TickToMs(BarOneBeatOneTick) / 1000.f;
}

float ISongMapEvaluator::GetSongLengthMs() const
{
	return TickToMs(float(GetSongLengthData().LengthTicks));
}

int32 ISongMapEvaluator::GetSongLengthBeats() const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (!BeatMap.IsEmpty())
	{
		return BeatMap.GetNumMapPoints();
	}

	return FMath::Floor(GetFractionalBeatAtTick(GetSongLengthData().LastTick) - GetFractionalBeatAtTick(0));
}

float ISongMapEvaluator::GetSongLengthFractionalBars() const
{
	const FSongLengthData& LengthData = GetSongLengthData();
	return LengthData.LengthFractionalBars;
}

bool ISongMapEvaluator::LengthIsAPerfectSubdivision() const
{
	const FBarMap& BarMap = GetBarMap();
	const FSongLengthData& LengthData = GetSongLengthData();
	const int32 TicksPerQuarterNote = BarMap.GetTicksPerQuarterNote();
	int32 BarIndex = 0;
	int32 BeatInBar = 0;
	int32 TickIndexInBeat = 0;
	BarMap.TickToBarBeatTickIncludingCountIn(LengthData.LengthTicks, BarIndex, BeatInBar, TickIndexInBeat);
	// The smallest subdivision we will consider is a 64th note triplet. 
	// A sixty fourth note triplet divides a quarter note into 24 parts. 
	int32 TicksPer64thTriplet = TicksPerQuarterNote / 24;
	int32 TicksPer64th = TicksPerQuarterNote / 16;
	return (TickIndexInBeat % TicksPer64thTriplet) == 0 || (TickIndexInBeat % TicksPer64th) == 0;
}

int32 ISongMapEvaluator::QuantizeTickToAnyNearestSubdivision(int32 InTick, EMidiFileQuantizeDirection Direction, EMidiClockSubdivisionQuantization& Division) const
{
	const FBarMap& BarMap = GetBarMap();
	const int32 TicksPerQuarterNote = BarMap.GetTicksPerQuarterNote();
	int32 BarIndex = 0;
	int32 BeatInBar = 0;
	int32 TickIndexInBeat = 0;
	int32 TicksPerBar = 0;
	int32 TicksPerBeat = 0;
	BarMap.TickToBarBeatTickIncludingCountIn(InTick, BarIndex, BeatInBar, TickIndexInBeat, &TicksPerBar, &TicksPerBeat);
	int32 BeatIndex = BeatInBar - 1; // BeatInBar is 1 bases!

	if (BeatIndex == 0 && TickIndexInBeat == 0)
	{
		Division = EMidiClockSubdivisionQuantization::Bar;
		return InTick;
	}
	if (TickIndexInBeat == 0)
	{
		Division = EMidiClockSubdivisionQuantization::Beat;
		return InTick;
	}

	// Not so simple. Now we need to know the time signature...
	int32 ZeroPoint = 0;
	FTimeSignature TimeSignature(4, 4);
	int32 BarMapPointIndex = BarMap.GetPointIndexForTick(InTick);
	if (BarMapPointIndex >= 0)
	{
		const FTimeSignaturePoint& TimeSignaturePoint = BarMap.GetTimeSignaturePoint(BarMapPointIndex);
		TimeSignature = TimeSignaturePoint.TimeSignature;
		ZeroPoint = TimeSignaturePoint.StartTick;
	}

	// We "start" the quantization grid at the nearest proceeding time signature change...
	int32 TickAtTimeSignature = InTick - ZeroPoint;
	EMidiClockSubdivisionQuantization BestDivision = EMidiClockSubdivisionQuantization::None;
	int32 BestDistanceFromDivision = std::numeric_limits<int32>::max();
	
	auto TryDivision = [this, &TickAtTimeSignature, &BestDivision, &BestDistanceFromDivision, &TimeSignature, &Division, &Direction, &TicksPerQuarterNote](EMidiClockSubdivisionQuantization TryDivision)
		{
			int32 TicksPerDivision = Harmonix::Midi::Constants::SubdivisionToMidiTicks(TryDivision, TimeSignature, TicksPerQuarterNote);
			int32 DistanceFromDivision = TickAtTimeSignature % TicksPerDivision;
			switch (Direction)
			{
			case EMidiFileQuantizeDirection::Up:
				DistanceFromDivision -= TicksPerDivision;
				break;
			case EMidiFileQuantizeDirection::Down:
				// nothing to do here
				break;
			default:
			case EMidiFileQuantizeDirection::Nearest:
				if (DistanceFromDivision > (TicksPerDivision / 2))
					DistanceFromDivision -= TicksPerDivision;
				break;
			}
			if (DistanceFromDivision == 0)
			{
				Division = TryDivision;
				return true;
			}
			else if (FMath::Abs(BestDistanceFromDivision) > FMath::Abs(DistanceFromDivision))
			{
				BestDivision = TryDivision;
				BestDistanceFromDivision = DistanceFromDivision;
			}
			return false;
		};

	if (TryDivision(EMidiClockSubdivisionQuantization::Bar))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::Beat))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::QuarterNote))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::QuarterNoteTriplet))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::EighthNote))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::EighthNoteTriplet))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::SixteenthNote))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::SixteenthNoteTriplet))
		return InTick;
	if (TryDivision(EMidiClockSubdivisionQuantization::ThirtySecondNote))
		return InTick;

	Division = BestDivision;
	return ZeroPoint + (TickAtTimeSignature - BestDistanceFromDivision);
}

int32 ISongMapEvaluator::QuantizeTickToNearestSubdivision(int32 InTick, EMidiFileQuantizeDirection Direction, EMidiClockSubdivisionQuantization Division) const
{
	int32 LowerTick = 0;
	int32 UpperTick = 0;
	GetTicksForNearestSubdivision(InTick, Division, LowerTick, UpperTick);
	if (Direction == EMidiFileQuantizeDirection::Down)
		return LowerTick;
	if (Direction == EMidiFileQuantizeDirection::Up)
		return UpperTick;
	if ((InTick - LowerTick) < (UpperTick - InTick))
		return LowerTick;
	return UpperTick;
}

void ISongMapEvaluator::GetTicksForNearestSubdivision(int32 InTick, EMidiClockSubdivisionQuantization Division, int32& LowerTick, int32& UpperTick) const
{
	const FBarMap& BarMap = GetBarMap();
	const int32 TicksPerQuarterNote = BarMap.GetTicksPerQuarterNote();

	int32 TicksInSubdivision = 0;
	int32 TickError = 0;
	if (BarMap.IsEmpty())
	{
		FTimeSignature TimeSignature(4, 4);
		TicksInSubdivision = Harmonix::Midi::Constants::SubdivisionToMidiTicks(Division, TimeSignature, TicksPerQuarterNote);
		TickError = InTick % TicksInSubdivision;
		LowerTick = InTick - TickError;
		UpperTick = LowerTick + TicksInSubdivision;
		return;
	}

	int32 BarIndex = 0;
	int32 BeatInBar = 0;
	int32 TickIndexInBeat = 0;
	int32 BeatsPerBar = 0;
	int32 TicksPerBeat = 0;
	BarMap.TickToBarBeatTickIncludingCountIn(InTick, BarIndex, BeatInBar, TickIndexInBeat, &BeatsPerBar, &TicksPerBeat);
	int32 TicksInBar = BeatsPerBar * TicksPerBeat;

	if (Division == EMidiClockSubdivisionQuantization::Bar)
	{
		LowerTick = BarMap.BarBeatTickIncludingCountInToTick(BarIndex, 1, 0);
		UpperTick = LowerTick + TicksInBar;
		return;
	}

	if (Division == EMidiClockSubdivisionQuantization::Beat)
	{
		LowerTick = BarMap.BarBeatTickIncludingCountInToTick(BarIndex, BeatInBar, 0);
		UpperTick = LowerTick + TicksPerBeat;
		return;
	}

	// Not so simple. Now we need to know the time signature...
	int32 ZeroPoint = 0;
	FTimeSignature TimeSignature(4, 4);
	int32 BarMapPointIndex = BarMap.GetPointIndexForTick(InTick);
	if (BarMapPointIndex >= 0)
	{
		const FTimeSignaturePoint& TimeSignaturePoint = BarMap.GetTimeSignaturePoint(BarMapPointIndex);
		TimeSignature = TimeSignaturePoint.TimeSignature;
		ZeroPoint = TimeSignaturePoint.StartTick;
	}

	// We "start" the quantization grid at the nearest proceeding time signature change...
	int32 TickAtTimeSignature = InTick - ZeroPoint;
	TicksInSubdivision = SubdivisionToMidiTicks(Division, InTick);
	TickError = TickAtTimeSignature % TicksInSubdivision;
	// Now that we know the tick error we can apply it to our original input tick...
	LowerTick = InTick - TickError;
	UpperTick = LowerTick + TicksInSubdivision;
}

FMusicalTime ISongMapEvaluator::GetMusicalTimeAtSeconds(double Seconds) const
{
	float InMs = static_cast<float>(Seconds * 1000.0);
	int32 Tick = FMath::FloorToInt32(MsToTick(InMs));
	int32 OutBarIndex;  // 0 based
	int32 OutBeatInBar; // 1 based
	int32 OutTickIndex; // 0 based
	int32 OutBeatsPerBar;
	int32 OutTicksPerBeat;
	TickToBarBeatTickIncludingCountIn(Tick, OutBarIndex, OutBeatInBar, OutTickIndex, &OutBeatsPerBar, &OutTicksPerBeat);
	OutBeatInBar--; // NOW 0 based

	FMusicalTime Result;
	Result.Bar = OutBarIndex;
	Result.TicksPerBeat = OutTicksPerBeat * MusicalTime::TicksPerQuarterNote / GetTicksPerQuarterNote();
	Result.TicksPerBar = Result.TicksPerBeat * OutBeatsPerBar;
	Result.TickInBar = OutBeatInBar * Result.TicksPerBeat + (OutTickIndex * MusicalTime::TicksPerQuarterNote / GetTicksPerQuarterNote());
	return Result;
}

FMusicalTime ISongMapEvaluator::GetMusicalTimeAtFractionalBar(float Bars) const
{
	int32 Tick = FMath::FloorToInt32(FractionalBarIncludingCountInToTick(Bars));
	int32 OutBarIndex;  // 0 based
	int32 OutBeatInBar; // 1 based
	int32 OutTickIndex; // 0 based
	int32 OutBeatsPerBar;
	int32 OutTicksPerBeat;
	TickToBarBeatTickIncludingCountIn(Tick, OutBarIndex, OutBeatInBar, OutTickIndex, &OutBeatsPerBar, &OutTicksPerBeat);
	OutBeatInBar--; // NOW 0 based

	FMusicalTime Result;
	Result.Bar = OutBarIndex;
	Result.TicksPerBeat = OutTicksPerBeat * MusicalTime::TicksPerQuarterNote / GetTicksPerQuarterNote();
	Result.TicksPerBar = Result.TicksPerBeat * OutBeatsPerBar;
	Result.TickInBar = OutBeatInBar * Result.TicksPerBeat + (OutTickIndex * MusicalTime::TicksPerQuarterNote / GetTicksPerQuarterNote());
	return Result;
}

FString ISongMapEvaluator::GetSongLengthString() const
{
	const FBarMap& BarMap = GetBarMap();
	const FSongLengthData& LengthData = GetSongLengthData();
	int32 BarIndex;
	int32 BeatInBar;
	int32 TickIndexInBeat;
	int32 BeatsPerBar;
	int32 TicksPerBeat;
	BarMap.TickToBarBeatTickIncludingCountIn(LengthData.LengthTicks, BarIndex, BeatInBar, TickIndexInBeat, &BeatsPerBar, &TicksPerBeat);
	int32 BeatIndex = BeatInBar - 1; // BeatInBar is 1 based
	return FString::Printf(TEXT("%d | %.3f"), BarIndex, (float)BeatIndex + (float)TickIndexInBeat / (float)TicksPerBeat );
}

///////////////////////////////////////////////////////////////////////////////////
// TEMPO
const FTempoInfoPoint* ISongMapEvaluator::GetTempoInfoForMs(float Ms) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return nullptr;
	}
	int32 Tick = int32(MsToTick(Ms));
	return TempoMap.GetTempoPointAtTick(Tick);
}

const FTempoInfoPoint* ISongMapEvaluator::GetTempoInfoForTick(int32 Tick) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return nullptr;
	}
	return TempoMap.GetTempoPointAtTick(Tick);
}

int32 ISongMapEvaluator::GetTempoPointIndexForTick(int32 Tick) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return -1;
	}
	return TempoMap.GetTempoPointIndexAtTick(Tick);
}

const FTempoInfoPoint* ISongMapEvaluator::GetTempoInfoPoint(int32 PointIndex) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (!TempoMap.GetTempoPoints().IsValidIndex(PointIndex))
	{
		return nullptr;
	}
	return &TempoMap.GetTempoPoints()[PointIndex];
}

int32 ISongMapEvaluator::GetNumTempoChanges() const
{
	return GetTempoMap().GetNumTempoChangePoints();
}

int32 ISongMapEvaluator::GetTempoChangePointTick(int32 PointIndex) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (!TempoMap.GetTempoPoints().IsValidIndex(PointIndex))
	{
		return 0;
	}
	return TempoMap.GetTempoChangePointTick(PointIndex);
}

float ISongMapEvaluator::GetTempoAtMs(float Ms) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return 0.0f;
	}
	int32 Tick = int32(MsToTick(Ms));
	return TempoMap.GetTempoAtTick(Tick);
}

float ISongMapEvaluator::GetTempoAtTick(int32 Tick) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return 0.0f;
	}
	return TempoMap.GetTempoAtTick(Tick);
}

bool ISongMapEvaluator::TempoMapIsEmpty() const
{
	return GetTempoMap().GetNumTempoChangePoints() == 0; 
}

///////////////////////////////////////////////////////////////////////////////////
// BEAT
const FBeatMapPoint* ISongMapEvaluator::GetBeatAtMs(float Ms) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (BeatMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Beat Map.");
		return nullptr;
	}
	int32 Tick = int32(MsToTick(Ms));
	return BeatMap.GetPointInfoForTick(Tick);
}

float ISongMapEvaluator::GetMsAtBeat(float Beat) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	const FBarMap& BarMap = GetBarMap();
	const int32 TicksPerQuarterNote = BarMap.GetTicksPerQuarterNote();

	if (!BeatMap.IsEmpty() && Beat < BeatMap.GetNumMapPoints())
	{
		float Tick = BeatMap.GetFractionalTickAtBeat(Beat);
		return TickToMs(Tick);
	}

	if (BarMap.IsEmpty())
	{
		return 0.0f;
	}

	const FTimeSignaturePoint& Point1 = BarMap.GetTimeSignaturePoint(0);
	if (BarMap.GetNumTimeSignaturePoints() == 1)
	{
		float Tick = Beat * TicksPerQuarterNote / (Point1.TimeSignature.Denominator / 4);
		return TickToMs(Tick);
	}

	const FTimeSignaturePoint* FromPoint = &Point1;
	for (int i = 1; i < BarMap.GetNumTimeSignaturePoints(); ++i)
	{
		const FTimeSignaturePoint& Point2 = BarMap.GetTimeSignaturePoint(i);
		if (Beat < Point2.BeatIndex)
		{
			break;
		}
		FromPoint = &Point2;
	}

	float BeatsAtTimeSignature = Beat - (float)FromPoint->BeatIndex;
	float Bar = (float)FromPoint->BarIndex + (BeatsAtTimeSignature / (float)FromPoint->TimeSignature.Numerator);
	return TickToMs(BarMap.FractionalBarIncludingCountInToTick(Bar));
}

const FBeatMapPoint* ISongMapEvaluator::GetBeatAtTick(int32 Tick) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (BeatMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Beat Map.");
		return nullptr;
	}
	return BeatMap.GetPointInfoForTick(Tick);
}

float ISongMapEvaluator::GetMsPerBeatAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetMsPerBeatAtTick(Tick);
}

float ISongMapEvaluator::GetMsPerBeatAtTick(int32 Tick) const
{
	const FBeatMapPoint* BeatInfo = GetBeatAtTick(Tick);
	if (!BeatInfo)
	{
		// In midi, tempo is always quarter notes per minute. Without a beat map,
		// a beat is the divisor in the time signature, which might not be a 
		// quarter note. So we will convert here...
		float QuarterNotesPerMinute = GetTempoAtTick(Tick);
		const FTimeSignature* TimeSignature = GetTimeSignatureAtTick(Tick);
		float BeatsPerMinute = QuarterNotesPerMinute / ((float)TimeSignature->Denominator / 4.0f /* quarter note / 4 == 1 */);
		return 60000.0f / BeatsPerMinute;
	}
	return TickToMs(float(BeatInfo->StartTick + BeatInfo->LengthTicks)) - TickToMs(float(BeatInfo->StartTick));
}

float ISongMapEvaluator::GetFractionalBeatAtMs(float Ms) const
{
	float Tick = MsToTick(Ms);
	return GetFractionalBeatAtTick(Tick);
}

float ISongMapEvaluator::GetFractionalBeatAtTick(float Tick) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	const FBarMap& BarMap = GetBarMap();
	if (BeatMap.IsEmpty() && BarMap.IsEmpty())
	{
		return 1.0f;
	}
	
	if (BeatMap.IsEmpty())
	{
		if (BarMap.GetStartBar() == 1)
		{
			return BarMap.TickToFractionalBeatIncludingCountIn(Tick) + 1.0f;
		}
		else
		{
			return BarMap.TickToFractionalBeatIncludingCountIn(Tick) - BarMap.TickToFractionalBeatIncludingCountIn(BarMap.GetTickOfBarOne()) + 1.0f;
		}
	}

	int32 BeatIndex = GetBeatIndexAtTick(int32(Tick));
	if (BeatIndex < 0 )
	{
		return 1.0f; // 1 based position!
	}
	const FBeatMapPoint* BeatInfo = &BeatMap.GetBeatPointInfo(BeatIndex);
	check(BeatInfo);
	float TickInBeat = Tick - BeatInfo->StartTick;
	float FractionalPart = TickInBeat / BeatInfo->LengthTicks;
	return BeatIndex + FractionalPart + 1.0f; // +1 for musical position
}

int32 ISongMapEvaluator::GetBeatIndexAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetBeatIndexAtTick(Tick);
}

int32 ISongMapEvaluator::GetBeatIndexAtTick(int32 Tick) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (!BeatMap.IsEmpty())
	{
		return BeatMap.GetPointIndexForTick(Tick);

	}
	const FBarMap& BarMap = GetBarMap();
	if (!BarMap.IsEmpty())
	{
		return FMath::Floor(BarMap.TickToFractionalBeatIncludingCountIn(Tick));
	}
	return INDEX_NONE;
}

EMusicalBeatType ISongMapEvaluator::GetBeatTypeAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetBeatTypeAtTick(Tick);
}

EMusicalBeatType ISongMapEvaluator::GetBeatTypeAtTick(int32 Tick) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (BeatMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Beat Map.");
		return EMusicalBeatType::Normal;
	}
	return BeatMap.GetBeatTypeAtTick(Tick);
}

const FBeatMapPoint* ISongMapEvaluator::GetBeatPointInfoAtTick(int32 Tick, int32* PointIndex) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (PointIndex)
	{
		*PointIndex = BeatMap.GetPointIndexForTick(Tick);
		return *PointIndex >= 0 ? &BeatMap.GetBeatPointInfo(*PointIndex) : nullptr;		
	}
	return BeatMap.GetPointInfoForTick(Tick);
}

float ISongMapEvaluator::GetBeatInPulseBarAtMs(float Ms) const
{
	float Tick = MsToTick(Ms);
	return GetBeatInPulseBarAtTick(Tick);
}

float ISongMapEvaluator::GetBeatInPulseBarAtTick(float Tick) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (BeatMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Beat Map.");
		return 0.0f;
	}
	return BeatMap.GetBeatInPulseBarAtTick(Tick);
}

int32 ISongMapEvaluator::GetNumBeatsInPulseBarAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetNumBeatsInPulseBarAtTick(Tick);
}

int32 ISongMapEvaluator::GetNumBeatsInPulseBarAtTick(int32 Tick) const
{
	const FBeatMap& BeatMap = GetBeatMap();
	if (BeatMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Beat Map.");
		return 0;
	}
	return BeatMap.GetNumBeatsInPulseBarAt(Tick);
}

bool ISongMapEvaluator::BeatMapIsEmpty() const
{
	return GetBeatMap().GetNumMapPoints() == 0; 
}

///////////////////////////////////////////////////////////////////////////////////
// Time Signature

int32 ISongMapEvaluator::GetStartBar() const
{
	const FBarMap& BarMap = GetBarMap();
	return BarMap.GetStartBar();
}

int32 ISongMapEvaluator::GetNumTimeSignatureChanges() const
{
	return GetBarMap().GetNumTimeSignaturePoints();
}

const FTimeSignature* ISongMapEvaluator::GetTimeSignatureAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetTimeSignatureAtTick(Tick);
}

const FTimeSignature* ISongMapEvaluator::GetTimeSignatureAtTick(int32 Tick) const
{
	const FBarMap& BarMap = GetBarMap();
	if (BarMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Bar Map.");
		return nullptr;
	}
	return &BarMap.GetTimeSignatureAtTick(Tick);
}

int32 ISongMapEvaluator::GetTimeSignaturePointIndexForTick(int32 Tick) const
{
	const FBarMap& BarMap = GetBarMap();
	if (BarMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Bar Map.");
		return -1;
	}
	return BarMap.GetPointIndexForTick(Tick);
}

const FTimeSignature* ISongMapEvaluator::GetTimeSignatureAtBar(int32 Bar) const
{
	const FBarMap& BarMap = GetBarMap();
	if (Bar < 1)
	{
		UE_LOGF(LogMIDI, Warning, "Bar < 1 (%d) specified as a musical position! Bars are '1' based in musical positions. Using bar 1!", Bar);
	}

	if (BarMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Bar Map.");
		return nullptr;
	}
	return &BarMap.GetTimeSignatureAtBar(Bar);
}

const FTimeSignaturePoint* ISongMapEvaluator::GetTimeSignaturePointAtTick(int32 Tick) const
{
	return GetBarMap().GetTimeSignaturePointForTick(Tick);
}

const FTimeSignaturePoint* ISongMapEvaluator::GetTimeSignaturePoint(int32 PointIndex) const
{
	return &GetBarMap().GetTimeSignaturePoint(PointIndex);
}

int32 ISongMapEvaluator::GetTimeSignatureChangePointTick(int32 PointIndex)
{
	const FBarMap& BarMap = GetBarMap();
	if (!BarMap.GetTimeSignaturePoints().IsValidIndex(PointIndex))
	{
		return 0;
	}
	return BarMap.GetTimeSignatureChangePointTick(PointIndex);
}

float ISongMapEvaluator::GetBarIncludingCountInAtMs(float Ms) const
{
	float Tick = MsToTick(Ms);
	return GetBarIncludingCountInAtTick(Tick);
}

float ISongMapEvaluator::GetBarIncludingCountInAtTick(float Tick) const
{
	return GetBarMap().TickToFractionalBarIncludingCountIn(Tick);
}

float ISongMapEvaluator::GetMsAtFractionalBarIncludingCountIn(float BarIncludingCountIn) const
{
	const float Tick = GetBarMap().FractionalBarIncludingCountInToTick(BarIncludingCountIn);
	return TickToMs(Tick);
}

float ISongMapEvaluator::GetMsPerBarAtMs(float Ms) const
{
	float Tick = MsToTick(Ms);
	return GetMsPerBarAtTick(Tick);
}

float ISongMapEvaluator::GetMsPerBarAtTick(float Tick) const
{
	const FTempoMap& TempoMap = GetTempoMap();
	const FBarMap& BarMap = GetBarMap();
	if (TempoMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Tempo Map.");
		return 0.0f;
	}
	if (BarMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Bar Map.");
		return 0.0f;
	}
	float Bpm = TempoMap.GetTempoAtTick(int32(Tick)); // quarter notes per minute
	const FTimeSignature* TimeSignature = GetTimeSignatureAtTick(int32(Tick));
	float QuarterNotesInBar;
	if (TimeSignature)
	{
		QuarterNotesInBar = (float)TimeSignature->Numerator / ((float)TimeSignature->Denominator / 4.0f);
	}
	else
	{
		QuarterNotesInBar = 4.0f;
	}
	float MsPerQuarterNote = 1000.0f / (Bpm / 60.0f);
	return QuarterNotesInBar * MsPerQuarterNote;
}

bool ISongMapEvaluator::BarMapIsEmpty() const
{
	return GetBarMap().GetNumTimeSignaturePoints() == 0; 
}

FMusicTimestamp ISongMapEvaluator::TickToMusicTimestamp(float Tick, int32* OutBeatsPerBar) const
{
	return GetBarMap().TickToMusicTimestamp(Tick, OutBeatsPerBar);
}

int32 ISongMapEvaluator::BarIncludingCountInToTick(int32 BarIndex, int32* OutBeatsPerBar, int32* OutTicksPerBeat) const
{
	return GetBarMap().BarIncludingCountInToTick(BarIndex, OutBeatsPerBar, OutTicksPerBeat);
}

int32 ISongMapEvaluator::BarBeatTickIncludingCountInToTick(int32 BarIndex, int32 BeatInBar, int32 TickInBeat) const
{
	return GetBarMap().BarBeatTickIncludingCountInToTick(BarIndex, BeatInBar, TickInBeat);
}

float ISongMapEvaluator::FractionalBarIncludingCountInToTick(float FractionalBarIndex) const
{
	return GetBarMap().FractionalBarIncludingCountInToTick(FractionalBarIndex);
}

float ISongMapEvaluator::FractionalBeatIncludingCountInToTick(float FractionalBeatIndex) const
{
	return GetBarMap().FractionalBeatIncludingCountInToTick(FractionalBeatIndex);
}

int32 ISongMapEvaluator::TickToBarIncludingCountIn(int32 Tick) const
{
	return GetBarMap().TickToBarIncludingCountIn(Tick);
}

float ISongMapEvaluator::TickToFractionalBarIncludingCountIn(float Tick) const
{
	return GetBarMap().TickToFractionalBarIncludingCountIn(Tick);
}

void ISongMapEvaluator::TickToBarBeatTickIncludingCountIn(int32 RawTick, int32& OutBarIndex, int32& OutBeatInBar, int32& OutTickIndexInBeat, int32* OutBeatsPerBar, int32* OutTicksPerBeat) const
{
	GetBarMap().TickToBarBeatTickIncludingCountIn(RawTick, OutBarIndex, OutBeatInBar, OutTickIndexInBeat, OutBeatsPerBar, OutTicksPerBeat);
}

void FSongMaps::SetLengthTotalBars(int32 Bars)
{
	check(Bars >= 0);
	memset(&LengthData, 0, sizeof(LengthData));
	LengthData.LengthTicks = BarMap.BarIncludingCountInToTick(Bars);
	LengthData.LastTick = LengthData.LengthTicks - 1;
	LengthData.LengthFractionalBars = Bars;
}

void FSongMaps::AddTempoChange(int32 Tick, float TempoBPM)
{
	int32 MidiTempo = Harmonix::Midi::Constants::BPMToMidiTempo(TempoBPM);
	AddTempoInfoPoint(MidiTempo, Tick);
}

void FSongMaps::AddTimeSigChange(int32 Tick, int32 InTimeSigNum, int32 InTimeSigDenom)
{
	// Time signature changes can only happen at the beginning of a bar,
	// so round up to the next bar boundary...
	int32 AbsoluteBar = FMath::CeilToInt32(GetBarIncludingCountInAtTick(Tick));
	Tick = BarBeatTickIncludingCountInToTick(AbsoluteBar, 1, 0);
	int32 TimeSigNum = FMath::Clamp(InTimeSigNum, 1, 64);
	int32 TimeSigDenom = FMath::Clamp(InTimeSigDenom, 1, 64);
	AddTimeSignatureAtBarIncludingCountIn(AbsoluteBar, TimeSigNum, TimeSigDenom);
}

int32 ISongMapEvaluator::CalculateMidiTick(const FMusicTimestamp& Timestamp, const EMidiClockSubdivisionQuantization Quantize) const
{
	const FBarMap& BarMap = GetBarMap();
	return BarMap.CalculateMidiTick(Timestamp, Quantize);
}

int32 ISongMapEvaluator::SubdivisionToMidiTicks(const EMidiClockSubdivisionQuantization Division, const int32 AtTick) const
{
	const FBarMap& BarMap = GetBarMap();
	return BarMap.SubdivisionToMidiTicks(Division, AtTick);
}

float ISongMapEvaluator::MusicTimestampToTick(const FMusicTimestamp& Timestamp) const
{
	const FBarMap& BarMap = GetBarMap();
	return BarMap.MusicTimestampToTick(Timestamp);
}

int32 ISongMapEvaluator::MusicTimestampBarToTick(int32 BarNumber, int32* OutBeatsPerBar, int32* OutTicksPerBeat) const
{
	const FBarMap& BarMap = GetBarMap();
	return BarMap.MusicTimestampBarToTick(BarNumber, OutBeatsPerBar, OutTicksPerBeat);
}

///////////////////////////////////////////////////////////////////////////////////
// Sections

const TArray<FSongSection>& ISongMapEvaluator::GetSections() const
{
	const FSectionMap& SectionMap = GetSectionMap();
	return SectionMap.GetSections();
}

int32 ISongMapEvaluator::GetNumSections() const
{
	const FSectionMap& SectionMap = GetSectionMap();
	return SectionMap.GetNumSections();
}

const FSongSection* ISongMapEvaluator::GetSectionAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetSectionAtTick(Tick);
}

const FSongSection* ISongMapEvaluator::GetSectionAtTick(int32 Tick) const
{
	const FSectionMap& SectionMap = GetSectionMap();
	if (SectionMap.IsEmpty())
	{
		return nullptr;
	}
	return SectionMap.TickToSection(Tick);
}

int32 ISongMapEvaluator::GetSectionIndexAtTick(int32 Tick) const
{
	const FSectionMap& SectionMap = GetSectionMap();
	if (SectionMap.IsEmpty())
	{
		return -1;
	}
	return SectionMap.TickToSectionIndex(Tick);
}


const FSongSection* ISongMapEvaluator::GetSectionWithName(const FString& Name) const
{
	const FSectionMap& SectionMap = GetSectionMap();
	if (SectionMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Section Map.");
		return nullptr;
	}
	return SectionMap.FindSectionInfo(Name);
}

FString ISongMapEvaluator::GetSectionNameAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetSectionNameAtTick(Tick);
}

FString ISongMapEvaluator::GetSectionNameAtTick(int32 Tick) const
{
	const FSectionMap& SectionMap = GetSectionMap();
	if (SectionMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Section Map.");
		return FString();
	}
	return SectionMap.GetSectionNameAtTick(Tick);
}

float ISongMapEvaluator::GetSectionLengthMsAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetSectionLengthMsAtTick(Tick);
}

float ISongMapEvaluator::GetSectionStartMsAtMs(float Ms) const
{
	const int32 Tick = int32(MsToTick(Ms));
	const FSongSection* SectionAtTick = GetSectionAtTick(Tick);
	if (!SectionAtTick)
	{
		return 0.0f;
	}

	return TickToMs(float(SectionAtTick->StartTick));
}

float ISongMapEvaluator::GetSectionEndMsAtMs(float Ms) const
{
	const int32 Tick = int32(MsToTick(Ms));
	const FSongSection* SectionAtTick = GetSectionAtTick(Tick);
	if (!SectionAtTick)
	{
		return 0.0f;
	}

	return TickToMs(float(SectionAtTick->EndTick()));
}

float ISongMapEvaluator::GetSectionLengthMsAtTick(int32 Tick) const
{
	const FSongSection* SectionAtTick = GetSectionAtTick(Tick);
	if (!SectionAtTick)
	{
		return 0.0f;
	}

	float StartMs = TickToMs(float(SectionAtTick->StartTick));
	float EndMs = TickToMs(float(SectionAtTick->EndTick()));
	return EndMs - StartMs;
}

bool ISongMapEvaluator::SectionMapIsEmpty() const
{
	return GetSectionMap().GetNumSections() == 0; 
}


///////////////////////////////////////////////////////////////////////////////////
// Chords

const FChordMapPoint* ISongMapEvaluator::GetChordAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetChordAtTick(Tick);
}

const FChordMapPoint* ISongMapEvaluator::GetChordAtTick(int32 Tick) const
{
	const FChordProgressionMap& ChordMap = GetChordMap();
	if (ChordMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Chord Map.");
		return nullptr;
	}
	return ChordMap.GetPointInfoForTick(Tick);
}

FName ISongMapEvaluator::GetChordNameAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetChordNameAtTick(Tick);
}

FName ISongMapEvaluator::GetChordNameAtTick(int32 Tick) const
{
	const FChordProgressionMap& ChordMap = GetChordMap();
	if (ChordMap.IsEmpty())
	{
		UE_LOGF(LogMIDI, Log, "SongMaps does not contain a Chord Map.");
		return FName();
	}
	return ChordMap.GetChordNameAtTick(Tick);
}

float ISongMapEvaluator::GetChordLengthMsAtMs(float Ms) const
{
	int32 Tick = int32(MsToTick(Ms));
	return GetChordLengthMsAtTick(Tick);
}

float ISongMapEvaluator::GetChordLengthMsAtTick(int32 Tick) const
{
	const FChordMapPoint* ChordInfo = GetChordAtTick(Tick);
	if (!ChordInfo)
	{
		return 0.0f;
	}
	float ChordStartMs = TickToMs(float(ChordInfo->StartTick));
	float ChordEndMs = TickToMs(float(ChordInfo->EndTick()));
	return ChordEndMs - ChordStartMs;
}

bool ISongMapEvaluator::ChordMapIsEmpty() const
{
	return GetChordMap().GetNumChords() == 0; 
}

const FTempoMap& FSongMapsWithAlternateTempoSource::GetTempoMap() const
{
	return SongMapsWithTempoMap->GetTempoMap();
}

const FBeatMap& FSongMapsWithAlternateTempoSource::GetBeatMap() const
{
	return SongMapsWithOtherMaps->GetBeatMap();
}

const FBarMap& FSongMapsWithAlternateTempoSource::GetBarMap() const
{
	return SongMapsWithOtherMaps->GetBarMap();
}

const FSectionMap& FSongMapsWithAlternateTempoSource::GetSectionMap() const
{
	return SongMapsWithOtherMaps->GetSectionMap();
}

const FChordProgressionMap& FSongMapsWithAlternateTempoSource::GetChordMap() const
{
	return SongMapsWithOtherMaps->GetChordMap();
}

const FSongLengthData& FSongMapsWithAlternateTempoSource::GetSongLengthData() const
{
	return SongMapsWithOtherMaps->GetSongLengthData();
}
