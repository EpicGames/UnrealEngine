// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMidi/TempoMap.h"
#include "HarmonixMidi/MidiConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TempoMap)

float FTempoMap::GetMsPerQuarterNoteAtTick(int32 Tick) const
{
	if (Points.IsEmpty())
	{
		return 800.0f;
	}

	int32 Index = FMusicMapUtl::GetPointIndexForTick(Points, Tick);
	if (Index == -1)
	{
		Index = 0;
	}
	return Points[Index].MidiTempo / 1000.0f;
}

int32 FTempoMap::GetMicrosecondsPerQuarterNoteAtTick(int32 Tick) const
{
	if (Points.IsEmpty())
	{
		return 800000;
	}

	int32 Index = FMusicMapUtl::GetPointIndexForTick(Points, Tick);
	if (Index == -1)
	{
		Index = 0;
	}
	return Points[Index].MidiTempo;
}

float FTempoMap::GetTempoAtTick(int32 Tick) const
{
	return Harmonix::Midi::Constants::MidiTempoToBPM(GetMicrosecondsPerQuarterNoteAtTick(Tick));
}

bool FTempoMap::operator==(const FTempoMap& Other) const
{
	if (TicksPerQuarterNote != Other.TicksPerQuarterNote || Points.Num() != Other.Points.Num())
	{
		return false;
	}
	for (int32 PointIndex = 0; PointIndex < Points.Num(); ++PointIndex)
	{
		if (Points[PointIndex] != Other.Points[PointIndex])
		{
			return false;
		}
	}
	return true;
}

void FTempoMap::Empty()
{
	Points.Empty();
}

void FTempoMap::Copy(const FTempoMap& Other, int32 StartTick, int32 EndTick)
{
	TicksPerQuarterNote = Other.TicksPerQuarterNote;
	FMusicMapUtl::Copy(Other.Points, Points, StartTick, EndTick);
}

bool FTempoMap::IsEmpty() const
{
	return Points.IsEmpty();
}

float FTempoMap::TickToMs(float Tick) const
{
	if (Tick == 0.f || Points.IsEmpty())
	{
		return 0.f;
	}

	int32 Index = FMusicMapUtl::GetPointIndexForTick(Points, Tick);
	if (Index == -1)
	{
		if (Tick < 0)
		{
			Index = 0;
		}
		else
		{
			return 0.0f;
		}
	}

	return TickToMsInternal(Tick, Points[Index]);
}

float FTempoMap::TickToMs(float Tick, int32 InIndex, int32* OutIndex) const
{
	if (Tick < 0.0f)
	{
		if (!Points.IsEmpty())
		{
			if (OutIndex)
			{
				*OutIndex = 0;
			}
			return TickToMsInternal(Tick, Points[0]);
		}
		if (OutIndex)
		{
			*OutIndex = -1;
		}
		return 0.0f;
	}

	//Linear search forward
	while (InIndex + 1 < Points.Num() && Points[InIndex + 1].StartTick <= Tick)
	{
		++InIndex;
	}

	//or backward
	while (InIndex && Points[InIndex - 1].StartTick > Tick)
	{
		--InIndex;
	}

	if (InIndex >= 0 && InIndex < Points.Num())
	{
		const FTempoInfoPoint& Point = Points[InIndex];
		if (OutIndex)
		{
			*OutIndex = InIndex;
		}
		return TickToMsInternal(Tick, Point);
	}
	if (OutIndex)
	{
		*OutIndex = -1;
	}
	return 0.0f;
}

float FTempoMap::TickToMsInternal(float Tick, const FTempoInfoPoint& PrevTempoInfoPoint) const
{
	if (Tick == 0.0f)
	{
		return 0;
	}
	float MsPerTick = PrevTempoInfoPoint.MidiTempo / ((float)TicksPerQuarterNote * 1000.0f);
	float DeltaTick = Tick - PrevTempoInfoPoint.StartTick;
	return PrevTempoInfoPoint.Ms + (DeltaTick * MsPerTick);
}

float FTempoMap::MsToTick(const float TimeMs) const
{
	if (TimeMs == 0.f)
	{
		return 0.f;
	}
	int32 Index = PointIndexForTime(TimeMs);
	if (Index == -1)
	{
		return 0.f;
	}
	return MsToTickInternal(TimeMs, Points[Index]);
}

float FTempoMap::MsToTick(const float TimeMs, int32 InIndex, int32* OutIndex) const
{
	// Linear search forward
	while (InIndex + 1 < Points.Num() && Points[InIndex + 1].Ms <= TimeMs)
	{
		++InIndex;
	}

	// or backward
	while (InIndex && Points[InIndex - 1].Ms > TimeMs)
	{
		--InIndex;
	}

	if (InIndex >= 0 && InIndex < Points.Num())
	{
		const FTempoInfoPoint& Point = Points[InIndex];
		if (OutIndex)
		{
			*OutIndex = InIndex;
		}
		return MsToTickInternal(TimeMs, Point);
	}
	if (OutIndex)
	{
		*OutIndex = -1;
	}
	return 0.0f;
}

float FTempoMap::MsToTickInternal(float TimeMs, const FTempoInfoPoint& PrevTempoInfoPoint) const
{
	if (TimeMs == 0.0f)
	{
		return 0.f;
	}
	return (PrevTempoInfoPoint.StartTick +
			(TimeMs - PrevTempoInfoPoint.Ms)  // ms
			* 1000.0f                         // us/ms
			/ PrevTempoInfoPoint.MidiTempo    // quarter note/us
			* (float)TicksPerQuarterNote);    // ticks/quarter note
}

bool FTempoMap::AddTempoInfoPoint(int32 Tempo, int32 Tick, bool SortNow)
{
	if (Points.Num() == 0)
	{
		ensureMsgf(Tick == 0, TEXT("FTempoMap::AddTempoInfoPoint(): tried to add point (%d, %d) to an empty tempo map! First tempo point must be at tick 0."), Tick, Tempo);
		if (Tick != 0)
		{
			return false;
		}
	}
	else
	{
		const FTempoInfoPoint& LastPoint = Points.Last();
		ensureMsgf(Tick >= LastPoint.StartTick, TEXT("FTempoMap::AddTempoInfoPoint(): tried to add point (%d, %d), but the current last point has a higher tick value (%d, %d)!"), Tick, Tempo, LastPoint.StartTick, LastPoint.MidiTempo);
		if (Tick < LastPoint.StartTick)
		{
			return false;
		}
	}
	FMusicMapUtl::AddPoint<FTempoInfoPoint,float,int32>(TickToMs(Tick), Tempo, Points, Tick, SortNow);
	return true;
}

void FTempoMap::WipeTempoInfoPoints(const int32 Tick)
{
	FMusicMapUtl::RemovePointsOnAndAfterTick(Points, Tick);
}

const FTempoInfoPoint* FTempoMap::GetTempoPointAtTick(int32 Tick) const
{
	return FMusicMapUtl::GetPointInfoForTick(Points, Tick);
}

int32 FTempoMap::GetTempoPointIndexAtTick(int32 Tick) const
{
	return FMusicMapUtl::GetPointIndexForTick(Points, Tick);
}

int32 FTempoMap::GetNumTempoChangePoints() const
{
	return Points.Num();
}

int32 FTempoMap::GetTempoChangePointTick(const int32 Index) const
{
	return Points[Index].StartTick;
}

void FTempoMap::Finalize(int32 LastTick)
{
	if (Points.IsEmpty())
	{
		SupplyDefault();
	}
	FMusicMapUtl::Finalize(Points, LastTick);
}

int32 FTempoMap::PointIndexForTime(const float TimeMs) const
{
	if (Points.IsEmpty())
	{
		return -1;
	}

	int32 Index = Algo::UpperBound(Points, TimeMs, FTempoInfoPoint::TimeLessThan());
	return (Index == 0) ? 0 : Index - 1;
}

void FTempoMap::SupplyDefault()
{
	// 500,000 us = 0.5 s = 1 quarter-note at 120bpm
	if (Points.IsEmpty())
	{
		AddTempoInfoPoint(500000, 0);
	}
}

