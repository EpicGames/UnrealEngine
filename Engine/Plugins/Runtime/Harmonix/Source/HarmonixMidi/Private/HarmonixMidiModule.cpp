// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarmonixMidiModule.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogHarmonixMidiModule, Log, Log)

void FHarmonixMidiModule::StartupModule()
{
}

void FHarmonixMidiModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHarmonixMidiModule, HarmonixMidi);
