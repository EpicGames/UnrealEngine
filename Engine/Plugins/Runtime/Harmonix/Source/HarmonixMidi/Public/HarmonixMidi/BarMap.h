// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MusicMapBase.h"
#include "HarmonixMidi/MidiConstants.h"
#include "Math/UnrealMathUtility.h"
#include <limits>

#include "BarMap.generated.h"

#define UE_API HARMONIXMIDI_API

class UTempoMap;

USTRUCT(BlueprintType)
struct FMusicTimestamp
{
	GENERATED_BODY()

public:
	UPROPERTY(Category = "Music", EditAnywhere, BlueprintReadWrite)
	int32 Bar = 1;
	UPROPERTY(Category = "Music", EditAnywhere, BlueprintReadWrite)
	float Beat = 1.0f;

	FMusicTimestamp() = default;
	FMusicTimestamp(int32 InBar, float InBeat)
		: Bar(InBar)
		, Beat(InBeat)
	{}


	bool IsValid() const
	{
		return Beat > 1.0f || FMath::IsNearlyEqual(Beat, 1.0f);
	}

	void Reset()
	{
		Bar = 1;
		Beat = 1.0f;
	}

	bool operator>=(const FMusicTimestamp& Other) const
	{
		return Bar > Other.Bar ||
			(Bar == Other.Bar && Beat >= Other.Beat);
	}

	bool operator>(const FMusicTimestamp& Other) const
	{
		return Bar > Other.Bar ||
			(Bar == Other.Bar && Beat > Other.Beat);
	}

	bool operator==(const FMusicTimestamp& Other) const
	{
		return Bar == Other.Bar && FMath::IsNearlyEqual(Beat, Other.Beat, 0.01f);
	}

	bool operator!=(const FMusicTimestamp& Other) const
	{
		return !operator==(Other);
	}

	bool operator<(const FMusicTimestamp& Other) const
	{
		return Bar < Other.Bar ||
			(Bar == Other.Bar && Beat < Other.Beat);
	}

	bool operator<=(const FMusicTimestamp& Other) const
	{
		return Bar < Other.Bar ||
			(Bar == Other.Bar && Beat <= Other.Beat);
	}

	friend FORCEINLINE uint32 GetTypeHash(const FMusicTimestamp& InTimestamp)
	{
		return HashCombineFast(GetTypeHash(InTimestamp.Bar), GetTypeHash(InTimestamp.Beat));
	}
};

/** A simple container for a musical time signature (numerator and denominator.) */
USTRUCT(BlueprintType)
struct FTimeSignature
{
	GENERATED_BODY()

public:
	FTimeSignature()
		: Numerator(4)
		, Denominator(4)
	{}
	FTimeSignature(int32 InNumerator, int32 InDenominator)
		: Numerator(InNumerator)
		, Denominator(InDenominator)
	{}

	bool operator==(const FTimeSignature& Other) const
	{
		return Other.Numerator == Numerator && Other.Denominator == Denominator;
	}

	bool operator!=(const FTimeSignature& Other) const
	{
		return Other.Numerator != Numerator || Other.Denominator != Denominator;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="TimeSignature", meta=(ClampMin=1, ClampMax=99))
	int32 Numerator = 4;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="TimeSignature", meta=(ClampMin=1, ClampMax=99))
	int32 Denominator = 4;

	friend FORCEINLINE uint32 GetTypeHash(const FTimeSignature& InTimeSignature)
	{
		return HashCombineFast(GetTypeHash(InTimeSignature.Numerator), GetTypeHash(InTimeSignature.Denominator));
	}
};

/** A time signature associated with a specific tick in a midi file. */
USTRUCT()
struct FTimeSignaturePoint : public FMusicMapTimespanBase
{
	GENERATED_BODY()

public:
	static constexpr bool DefinedAsRegions = false;

	FTimeSignaturePoint()
		: BarIndex(0)
		, BeatIndex(0)
		, TimeSignature(4, 4) 
	{}
	FTimeSignaturePoint(int32 InBarIndex, int32 InBeatIndex, const FTimeSignature& InTimeSignature, int32 InStartTick, int32 InLengthTicks = 1)
		: FMusicMapTimespanBase(InStartTick, InLengthTicks)
		, BarIndex(InBarIndex)
		, BeatIndex(InBeatIndex)
		, TimeSignature(InTimeSignature)
	{}

	bool operator==(const FTimeSignaturePoint& Other) const
	{
		return	BarIndex == Other.BarIndex &&
				BeatIndex == Other.BeatIndex &&
				TimeSignature == Other.TimeSignature;
	}

	UPROPERTY()
	int32 BarIndex; // 0 based since it is internal!
	UPROPERTY()
	int32 BeatIndex; // 0 based since it is internal!
	UPROPERTY()
	FTimeSignature TimeSignature;

	struct BarLessThan
	{
		bool operator()(int32 InBarIndex, const FTimeSignaturePoint& Point) const { return InBarIndex < Point.BarIndex; }
		bool operator()(const FTimeSignaturePoint& Point, int32 InBarIndex) const { return Point.BarIndex < InBarIndex; }
		bool operator()(const FTimeSignaturePoint& PointA, const FTimeSignaturePoint& PointB) const { return PointA.BarIndex < PointB.BarIndex; }
	};

	struct BeatLessThan
	{
		bool operator()(int32 InBeatIndex, const FTimeSignaturePoint& Point) const { return InBeatIndex < Point.BeatIndex; }
		bool operator()(const FTimeSignaturePoint& Point, int32 InBeatIndex) const { return Point.BeatIndex < InBeatIndex; }
		bool operator()(const FTimeSignaturePoint& PointA, const FTimeSignaturePoint& PointB) const { return PointA.BeatIndex < PointB.BeatIndex; }
	};
};

/** A map of time signatures changes in a song. */
USTRUCT()
struct FBarMap
{
	GENERATED_BODY()

public:
	FBarMap()
		: TicksPerQuarterNote(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt)
	{}
	UE_API bool operator==(const FBarMap& Other) const;

	UE_API void Empty();
	UE_API void Copy(const FBarMap& Other, int32 StartTick = 0, int32 EndTick = std::numeric_limits<int32>::max());
	UE_API bool IsEmpty() const;

	UE_API int32 CalculateMidiTick(const FMusicTimestamp& Timestamp, const EMidiClockSubdivisionQuantization Quantize) const;

	UE_API int32 SubdivisionToMidiTicks(const EMidiClockSubdivisionQuantization Division, const int32 AtTick) const;

	/** Called by the midi file importer before map points are added to this map */
	void SetTicksPerQuarterNote(int32 InTicksPerQuarterNote)
	{
		TicksPerQuarterNote = InTicksPerQuarterNote;
	}

	int32 GetTicksPerQuarterNote() const { return TicksPerQuarterNote; }

	void SupplyDefault() { AddTimeSignatureAtBarIncludingCountIn(0, 4, 4); }

	/** 
	 * Takes an FMusicTimestamp and computes the absolute (raw) total tick from the beginning of the 
	 * midi data. Based on the time signature(s), StartBar, and TicksPerQuarterNote set when the
	 * map was created.
	 * @param Timestamp
	 * @return The 0-based absolute tick in the midi data.
	 */
	UE_API float MusicTimestampToTick(const FMusicTimestamp& Timestamp) const;

	/**
	 * Takes an absolute tick and calculates a music timestamp. Uses the collection of time signatures,
	 * the setting of TicksPerQuarterNote, and the setting of StartBar to do its calculations. It will
	 * also optionally return the number of beats in the found bar, which is the numerator of the 
	 * current time signature.
	 * @param Tick 0-based 'raw total ticks'
	 * @param OutBeatPerBar Is not nullptr it will be filled in with the number of total beats in the described bar.
	 * @return The FMusicTimestamp where Bar 1 Beat 1 is the "beginning of the song" after count-in and pickup bars.
	 */
	UE_API FMusicTimestamp TickToMusicTimestamp(float Tick, int32* OutBeatsPerBar = nullptr) const;

	/**
	 * Takes an tick relative to "bar 1 beat 1" and calculates a music timestamp. Uses the collection of 
	 * time signatures, the setting of TicksPerQuarterNote, and the setting of StartBar to do its calculations.
	 * It will also optionally return the number of beats in the found bar, which is the numerator of the 
	 * current time signature.
	 * @param Tick 0-based tick offset from Bar 1. So Tick 0 will always be Bar 1 Beat 1 regarless of the length of 
	 *        the count-in or pickup bars.
	 * @param OutBeatPerBar Is not nullptr it will be filled in with the number of total beats in the described bar.
	 * @return The FMusicTimestamp where Bar 1 Beat 1 is the "beginning of the song" after count-in and pickup bars.
	 */
	UE_API FMusicTimestamp TickFromBarOneToMusicTimestamp(float Tick, int32* OutBeatsPerBar = nullptr) const;

	/**
	 * Takes a "music timestamp" bar and computes the absolute (raw) total tick from the beginning of the
	 * midi data. Optionally returns the number of beats in that bar and the number of ticks per beat.
	 * @param BarNumber As would be returned in a MusicTimestamp, where Bar 1 is the "beginning of the music" 
	 *            after count-in and pickups.
	 * @param optional OutBeatsPerBar Number of beats in this bar (numerator of the time signature)
	 * @param optional OutTicksPerBeat Number of ticks per beat (based on the time signature denominator and the system's TTPQ)
	 * @return The 0-based absolute tick in the midi data.
	 */
	UE_API int32 MusicTimestampBarToTick(int32 BarNumber, int32* OutBeatsPerBar = nullptr, int32* OutTicksPerBeat = nullptr) const;

	/**
	 * Takes a bar index and computes the absolute (raw) total tick from the beginning of the midi data. 
	 * This is a '0' based bar from the beginning of the raw midi data with no reference to count-in or pickup 
	 * data. Optionally returns the number of beats in that bar and the number of ticks per beat.
	 * @param BarIndex 0 based index of the bar
	 * @param optional OutBeatsPerBar Number of beats in this bar (numerator of the time signature)
	 * @param optional OutTicksPerBeat Number of ticks per beat (based on the time signature denominator and the system's TTPQ)
	 * @return The 0-based absolute tick in the midi data.
	 */
	UE_API int32 BarIncludingCountInToTick(int32 BarIndex, int32* OutBeatsPerBar = nullptr, int32* OutTicksPerBeat = nullptr) const;

	/**
	 * Take a bar, beat (1 based) within that bar, and tick (0 based) within that beat and computes the
	 * absolute (raw) total tick from the beginning of the  midi data. Based on the time signature(s), 
	 * StartBar, and TicksPerQuarterNote. 
	 * 
	 * NOTE: Here, bar and beat are as one would find them in a FMusicTimestamp structure... bar 1 
	 * beat 1 is the beginning of the song. Bars before '1' are count-in or pickup bars.
	 * 
	 * @param BarNumber Bar in question. Remember... songs start at bar 1 beat 1!
	 * @param BeatInBar 1-based beat in the bar. Given a time signature of x/y, there are 'x' beats in a bar, and each beat is 'y' in length.
	 * @param TickInBeat 0-based tick in the Beat. NOTE: The number of ticks in a beat depends on
	 *        the length of the beat (the denominator in the time signature) and the map's ticks per quarter note.
	 * @return The 0-based absolute tick in the midi data.
	 */
	UE_API int32 MusicTimestampBarBeatTickToTick(int32 BarNumber, int32 BeatInBar, int32 TickInBeat) const;

	/**
	 * Take a bar, beat (1 based) within that bar, and tick (0 based) within that beat and computes the
	 * absolute (raw) total tick from the beginning of the  midi data. Based on the time signature(s),
	 * and TicksPerQuarterNote. Here, bar is a 'raw' bar from the beginning of the midi data. There is no 
	 * consideration of count-in or pickup bars.
	 * 
	 * @param BarIndex Bar in question. The beginning of the midi data is bar 0!
	 * @param BeatInBar 1-based beat in the bar. Given a time signature of x/y, there are 'x' beats in a bar, and each beat is 'y' in length.
	 * @param TickInBeat 0-based tick in the Beat. NOTE: The number of ticks in a beat depends on
	 *        the length of the beat (the denominator in the time signature) and the map's ticks per quarter note.
	 * @return The 0-based absolute tick in the midi data.
	 */
	UE_API int32 BarBeatTickIncludingCountInToTick(int32 BarIndex, int32 BeatInBar, int32 TickInBeat) const;

	/**
	 * Take "raw" tick (tick from the beginning of the midi data and in units of TicksPerQuarterNote)
	 * and return a bar (0 based), beat in bar (1 based) within that bar, and tick (0 based) within that beat. 
	 * Here, the returned position is a 'raw' position from the beginning of the midi data. There is
	 * no consideration of count-in or pickup bars.
	 *
	 * @param RawTick from the beginning of the midi data.
	 * @param OutBarIndex (output)
	 * @param OutBeatInBar (1 based!) (output)
	 * @param OutTickIndexInBeat (output)
	 * @param OutBeatsPerBar (output - optional)
	 * @param OutTicksPerBeat (output - optional)
	 */
	UE_API void TickToBarBeatTickIncludingCountIn(int32 RawTick, int32& OutBarIndex, int32& OutBeatInBar, int32& OutTickIndexInBeat, int32* OutBeatsPerBar = nullptr, int32* OutTicksPerBeat = nullptr) const;

	/**
	 * Takes an absolute tick (from the beginning of the midi data) and calculates the fractional bar,
	 * where bar 0.0 is the beginning of the raw midi data.
	 *
	 * Uses the collection of time signatures and the setting of kTicksPerQuarterNoteInt
	 *
	 * @param Tick 0-based 'raw total ticks'
	 * @return The 0-based fractional bar index
	 */
	UE_API float TickToFractionalBarIncludingCountIn(float Tick) const;

	/**
	 * Takes an absolute tick (from the beginning of the midi data) and calculates the fractional beat,
	 * where beat 0.0 is the beginning of the raw midi data.
	 *
	 * Uses the collection of time signatures and the setting of kTicksPerQuarterNoteInt
	 *
	 * @param Tick 0-based 'raw total ticks'
	 * @return The 0-based fractional beat index
	 */
	UE_API float TickToFractionalBeatIncludingCountIn(float Tick) const;

	/**
	 * Takes an absolute tick (from the beginning of the midi data) and calculates the bar index,
	 * where bar 0 is the beginning of the raw midi data.
	 *
	 * Uses the collection of time signatures and the setting of kTicksPerQuarterNoteInt
	 *
	 * @param Tick 0-based 'raw total ticks'
	 * @return The 0-based integrer bar index
	 */
	UE_API int32 TickToBarIncludingCountIn(int32 Tick) const;

	/**
	 * Takes an absolute tick (from the beginning of the midi data) and calculates the beat index,
	 * where beat 0 is the beginning of the raw midi data.
	 *
	 * Uses the collection of time signatures and the setting of kTicksPerQuarterNoteInt
	 *
	 * @param Tick 0-based 'raw total ticks'
	 * @return The 0-based integral beat index
	 */
	UE_API int32 TickToBeatIncludingCountIn(float Tick) const;

	/**
	* Takes a fractional bar and returns the absolute tick. Note: Bar 0.0 is the beginning
	* of raw midi data. Uses the collection of time signatures and the setting of kTicksPerQuarterNoteInt
	* to do its calculations.
	*
	* Note: If the returned tick is negative it is BEFORE any authored midi content.
	* 
	* @param FractionalBarIndex 0-based fractional bar
	* @return The 0-based raw tick
	*/
	UE_API float FractionalBarIncludingCountInToTick(float FractionalBarIndex) const;

	/**
	* Takes a fractional beat and returns the absolute tick. Note: Beat 0.0 is the beginning
	* of raw midi data. Uses the collection of time signatures and the setting of kTicksPerQuarterNoteInt
	* to do its calculations.
	*
	* Note: If the returned tick is negative it is BEFORE any authored midi content.
	* 
	* @param FractionalBeatIndex 0-based fractional bar
	* @return The 0-based raw tick
	*/
	UE_API float FractionalBeatIncludingCountInToTick(float FractionalBeatIndex) const;

	/**
	 * Adds a time signature at the specified position, in this case a bar number as one would find
	 * in an FMusicTimestamp. In other words, a BarNumber before '1' would be in the count-in or pickup.
	 * @param BarNumber Bar 1 is the "beginning of the song". Bars before '1' are count-in or pickup bars.
	 * @param Numerator TIme Signature Numerator
	 * @param Denominator Time Signature Denominator
	 * @param SortNow Whether to sort the time signature points immediately or wait. 
	 * @param FailOnError If an error is detected halt execution.
	 */
	UE_API bool AddTimeSignatureAtMusicTimestampBar(int32 BarNumber, int32 Numerator, int32 Denominator, bool SortNow = true, bool FailOnError = true);

	/**
	 * Adds a time signature at the specified position, in this case a bar index indicating the bar index
	 * from the beginning of the raw midi data.
	 * @param BarIndex '0' based index of the bar where the time signature belongs.
	 * @param Numerator TIme Signature Numerator
	 * @param Denominator Time Signature Denominator
	 * @param SortNow Whether to sort the time signature points immediately or wait.
	 * @param FailOnError If an error is detected halt execution.
	 */
	UE_API bool AddTimeSignatureAtBarIncludingCountIn(int32 BarIndex, int32 InNumerator, int32 InDenominator, bool SortNow = true, bool FailOnError = true);

	/** Remove every point from the given tick onward */
	UE_API void WipeTimeSignaturePoints(int32 Tick);

	UE_API int32 GetNumTimeSignaturePoints() const;

	UE_API int32 GetPointIndexForTick(int32 Tick) const;

	UE_API const FTimeSignaturePoint& GetTimeSignaturePoint(int32 Index) const;
	UE_API FTimeSignaturePoint& GetTimeSignaturePoint(int32 Index);

	UE_API const FTimeSignature& GetTimeSignatureAtTick(int32 InTick) const;

	UE_API const FTimeSignaturePoint* GetTimeSignaturePointForTick(int32 InTick) const;

	UE_API const FTimeSignature& GetTimeSignatureAtBar(int32 InBar) const;

	FORCEINLINE int32 GetTicksInBeatAfterPoint(int32 Index) const
	{
		if (!Points.IsValidIndex(Index) || Points[Index].TimeSignature.Denominator == 0)
		{
			return TicksPerQuarterNote;
		}
		return (TicksPerQuarterNote * 4) / Points[Index].TimeSignature.Denominator;
	}

	FORCEINLINE int32 GetTicksInBarAfterPoint(int32 Index) const
	{
		if (!Points.IsValidIndex(Index))
		{
			return 4 * TicksPerQuarterNote;
		}
		return Points[Index].TimeSignature.Numerator * GetTicksInBeatAfterPoint(Index);
	}

	UE_API void Finalize(int32 InLastTick);

	int32 GetTickOfBarOne() const { return MusicTimestampBarBeatTickToTick(1, 1, 0); }

	void SetStartBar(int32 InStartBar) { StartBar = InStartBar; }
	int32 GetStartBar() const { return StartBar; }

	// Returns the time signature points for inspection.
	const TArray<FTimeSignaturePoint>& GetTimeSignaturePoints() const { return Points; }

	UE_API int32 GetTimeSignatureChangePointTick(int32 PointIndex) const;

protected:
	UPROPERTY()
	int32 StartBar = 1;
	UPROPERTY()
	int32 TicksPerQuarterNote;
	UPROPERTY()
	TArray<FTimeSignaturePoint> Points;

	static UE_API const FTimeSignaturePoint sDefaultTimeSignature;
};

#undef UE_API
