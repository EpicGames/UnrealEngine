// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MusicMapBase.h"
#include "HarmonixMidi/MidiConstants.h"
#include <limits>

#include "BeatMap.generated.h"

#define UE_API HARMONIXMIDI_API

UENUM(BlueprintType)
enum class EMusicalBeatType : uint8
{
	Downbeat = 0,
	Strong = 1,
	Normal = 2,
};

FString HARMONIXMIDI_API MusicalBeatTypeToString(EMusicalBeatType BeatType);

/**
 * A point in the music representing a "beat".
 * 
 * Type may be 'Downbeat', 'Strong', or 'Normal' beat. 
 * 
 * This is very useful for odd time signatures like 5/8 where the music's
 * beat might be on the 1st and 3rd eighth notes or the 1st and 4th eighth notes
 */
USTRUCT()
struct FBeatMapPoint : public FMusicMapTimespanBase
{
	GENERATED_BODY()

public:
	static constexpr bool DefinedAsRegions = true;

	FBeatMapPoint()
		: PulseBar(-1)
		, Type(EMusicalBeatType::Normal)
	{}

	FBeatMapPoint(EMusicalBeatType InType, int32 InStartTick, int32 InLengthTicks = 1)
		: FMusicMapTimespanBase(InStartTick, InLengthTicks)
		, PulseBar(0)
		, Type(InType)
	{}

	bool operator==(const FBeatMapPoint& Other) const
	{
		return PulseBar == Other.PulseBar && Type == Other.Type;
	}

	UPROPERTY()
	int32 PulseBar;
	UPROPERTY()
	EMusicalBeatType Type;
};

/**
 * Pulse Bars are groupings of beats where the first beat in the group has been marked up as a
 * 'Downbeat' type.
 */
USTRUCT()
struct FPulseBar
{
	GENERATED_BODY()

public:
	FPulseBar()
		: StartTick(0)
		, LengthTicks(0)
		, FirstIncludedBeatIndex(0)
		, LastIncludedBeatIndex(0)
	{}
	FPulseBar(int32 InStartTick, int32 InLengthTicks, int32 InFirstIncludedBeatIndex, int32 InLastIncludedBeatIndex)
		: StartTick(InStartTick)
		, LengthTicks(InLengthTicks)
		, FirstIncludedBeatIndex(InFirstIncludedBeatIndex)
		, LastIncludedBeatIndex(InLastIncludedBeatIndex)
	{}

	bool operator==(const FPulseBar& Other) const
	{
		return StartTick == Other.StartTick && LengthTicks == Other.LengthTicks && FirstIncludedBeatIndex == Other.FirstIncludedBeatIndex && LastIncludedBeatIndex == Other.LastIncludedBeatIndex;
	}

	UPROPERTY()
	int32 StartTick;
	UPROPERTY()
	int32 LengthTicks;
	UPROPERTY()
	int32 FirstIncludedBeatIndex;
	UPROPERTY()
	int32 LastIncludedBeatIndex;
};

/**
 * A map of 'beats' in a piece of music.
 */
USTRUCT()
struct FBeatMap
{
	GENERATED_BODY()

public:
	FBeatMap()
		: TicksPerQuarterNote(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt)
	{}
	UE_API bool operator==(const FBeatMap& Other) const;

	UE_API void Empty();
	UE_API void Copy(const FBeatMap& Other, int32 StartTick = 0, int32 EndTick = std::numeric_limits<int32>::max());
	UE_API bool IsEmpty() const;
	UE_API int32 GetNumMapPoints() const;

	/** Called by the midi file importer before map points are added to this map */
	void SetTicksPerQuarterNote(int32 InTicksPerQuarterNote)
	{
		TicksPerQuarterNote = InTicksPerQuarterNote;
	}

	UE_API void AddBeat(EMusicalBeatType Type, int32 Tick, bool SortNow = true);

	UE_API const FBeatMapPoint* GetPointInfoForTick(int32 Tick) const;

	UE_API int32 GetPointIndexForTick(int32 Tick) const;

	UE_API float GetFractionalBeatAtTick(float Tick) const;

	UE_API float GetFractionalTickAtBeat(float InBeat) const;

	UE_API EMusicalBeatType GetBeatTypeAtTick(int32 Tick) const;

	UE_API float GetBeatInPulseBarAtTick(float Tick) const;

	UE_API int32 GetNumBeatsInPulseBarAt(int32 Tick) const;

	/** Is the specified "from count in" beat the first beat in its bar? */
	UE_API bool IsDownbeat(float Beat) const;

	/** Retrieve the first downbeat AFTER the specified "from count in" beat */
	UE_API int32 FindDownbeatIndexAfterBeat(float Beat) const;

	/** Retrieve the first downbeat ON OR AFTER the specified "from count in"  beat */
	UE_API int32 FindDownbeatIndexAtOrAfterBeat(float Beat) const;

	UE_API const FBeatMapPoint& GetBeatPointInfo(int32 index) const;
	
	UE_API void Finalize(int32 LastTick);

protected:
	UPROPERTY()
	int32 TicksPerQuarterNote;
	UPROPERTY()
	TArray<FBeatMapPoint> Points;

	UPROPERTY()
	TArray<FPulseBar> Bars;
};

#undef UE_API
