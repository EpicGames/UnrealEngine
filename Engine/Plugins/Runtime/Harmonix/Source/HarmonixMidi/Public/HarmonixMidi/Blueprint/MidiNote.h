// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MidiNote.generated.h"

#define UE_API HARMONIXMIDI_API

/**
* Helper struct for getting and assigning MidiNote values
* Uses custom detail customization and custom pins for convenience in Editor
* 
* has a one property
* uint8 NoteNumber
* ranging from Midi::kMinNote to Midi::kMaxNote [0 - 127]
*/
USTRUCT(BlueprintType, Meta = (DisplayName = "MIDI Note"))
struct FMidiNote
{
	GENERATED_BODY()

	FMidiNote() : NoteNumber(60) { };

	FMidiNote(uint8 inValue) : NoteNumber(inValue) { };
	FMidiNote(int inValue) : NoteNumber(inValue < 0 ? 0 : inValue) { };
	FMidiNote(int8 inValue) : NoteNumber(inValue < 0 ? 0 : inValue) { };

	operator uint8() { return NoteNumber; }
	operator int8() { return NoteNumber; }
	operator int() { return NoteNumber; }

	bool operator==(const FMidiNote& rhs) const { return NoteNumber == rhs.NoteNumber; }
	bool operator!=(const FMidiNote& rhs) const { return NoteNumber != rhs.NoteNumber; }
	bool operator< (const FMidiNote& rhs) const { return NoteNumber <  rhs.NoteNumber; }
	bool operator> (const FMidiNote& rhs) const { return NoteNumber >  rhs.NoteNumber; }
	bool operator<=(const FMidiNote& rhs) const { return NoteNumber <= rhs.NoteNumber; }
	bool operator>=(const FMidiNote& rhs) const { return NoteNumber >= rhs.NoteNumber; }
	
	

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Midi", meta=(ClampMin="0", ClampMax="127"))
	uint8 NoteNumber = 60;

	UE_API FText GetDisplayText() const;

	UE_API FName GetDisplayName() const;

	// note names formated enharmonic stile sharp with octave number (eg. C#4)
	UE_API FString ToString() const;

	// takes a note name (eg C#4) and makes a FMidiNote struct from it
	static UE_API FMidiNote FromString(const FString& NoteName);

	static UE_API uint8 NoteNumberFromString(const FString& NoteName);

#ifdef WITH_EDITOR

	UE_API FString ToEditorString() const;

	static UE_API FMidiNote FromEditorString(const FString& EditorName);

	static UE_API uint8 NoteNumberFromEditorString(const FString& EditorName);
#endif
};

#undef UE_API
