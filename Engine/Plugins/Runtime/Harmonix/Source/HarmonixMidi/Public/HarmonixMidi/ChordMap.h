// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MusicMapBase.h"
#include "HarmonixMidi/MidiConstants.h"
#include <limits>

#include "ChordMap.generated.h"

#define UE_API HARMONIXMIDI_API

/**
 * Specifies the tonality of a given section of music.
 */
USTRUCT()
struct FChordMapPoint : public FMusicMapTimespanBase
{
	GENERATED_BODY()

public:
	static constexpr bool DefinedAsRegions = false;

	FChordMapPoint()
	{}

	FChordMapPoint(FName ChordName, int32 InStartTick, int32 InLengthTicks = 1)
		: FMusicMapTimespanBase(InStartTick, InLengthTicks)
		, Name(ChordName)
	{}

	bool operator==(const FChordMapPoint& Other) const
	{
		return Name == Other.Name;
	}

	UPROPERTY()
	FName  Name;
};

/**
 * A collection of chords.
 * 
 * Constructed when a standard midi file is imported and is seen to
 * contain a 'chord track' that conforms to the Harmonix chord markup 
 * convention.
 */
USTRUCT()
struct FChordProgressionMap 
{
	GENERATED_BODY()

public:
	FChordProgressionMap()
		: TicksPerQuarterNote(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt)
	{}
	UE_API bool operator==(const FChordProgressionMap& Other) const;

	UE_API void Finalize(int32 LastTick);

	UE_API void Copy(const FChordProgressionMap& Other, int32 StartTick = 0, int32 EndTick = std::numeric_limits<int32>::max());
	UE_API bool IsEmpty() const;

	/** Called by the midi file importer before map points are added to this map */
	void SetTicksPerQuarterNote(int32 InTicksPerQuarterNote)
	{
		TicksPerQuarterNote = InTicksPerQuarterNote;
	}

	/**
	 * Adds a chord at the specified position. Note: It DOES NOT
	 * take a duration or end tick! It is assumed to endure until 
	 * the next existing chord or for the remainder of the song.
	 */
	UE_API void AddChord(FName Name, int32 StartTick, bool SortNow = false);

	UE_API const FChordMapPoint* GetPointInfoForTick(int32 Tick) const;

	/**
	 * Returns the chord that applies to the specified Tick
	 */
	UE_API FName GetChordNameAtTick(int32 Tick) const;

	int32 GetNumChords() const { return Points.Num(); }
	const TArray<FChordMapPoint>& GetChordList() const { return Points; }
	UE_API const void GetChordListCopy(TArray<FChordMapPoint>& ChordList) const;

	/**
	 * When the midi file importer finds a chord track, and constructs this map,
	 * it will set this map's ChordTrackIndex. This could be useful if you would
	 * like to parse the chord track yourself using the FMidiFile & FMidiTrack
	 * APIs, etc.
	 */
	int32  GetTrack() const { return ChordTrackIndex; }

	/**
	 * Called by the standard midi file importer when it finds a chord track in
	 * the data. If you construct the chord track yourself you may choose to set this
	 * index, but it isn't required.
	 */
	void SetTrack(int32 t) { ChordTrackIndex = t; }

	UE_API void Empty();

protected:
	UPROPERTY()
	int32 TicksPerQuarterNote;
	UPROPERTY()
	TArray<FChordMapPoint> Points;

private:
	int32 ChordTrackIndex = -1;
};

#undef UE_API
