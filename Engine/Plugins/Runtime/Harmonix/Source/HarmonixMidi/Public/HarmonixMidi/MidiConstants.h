// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

HARMONIXMIDI_API DECLARE_LOG_CATEGORY_EXTERN(LogMIDI, Log, All);

enum class EMidiClockSubdivisionQuantization : uint8;
struct FTimeSignature;

namespace Harmonix::Midi::Constants
{
	enum class EMidiTextEventEncoding
	{
		Latin1,
		UTF8
	};

	/////////////////////////////////////////////////////////////////////////////// 
	// Constants and functions for MIDI 1.0 standard
	//
	// Useful web resources:
	//  http://www.midi.org/about-midi/table1.shtml
	//  http://www.borg.com/~jglatt/tech/midispec.htm

	inline constexpr uint8 GNumChannels    = 16;
	inline constexpr uint8 GNotesPerOctave = 12;
	inline constexpr uint8 GMinNote        = 0;
	inline constexpr uint8 GMaxNote        = 127;
	inline constexpr uint8 GMaxNumNotes    = 128;
	inline constexpr uint8 GMinVelocity    = 0;
	inline constexpr uint8 GMaxVelocity    = 127;

	inline constexpr int32 GTicksPerQuarterNoteInt  = 960;
	inline constexpr float GTicksPerQuarterNote = 960.0f;
	inline constexpr float GQuarterNotesPerTick = 1.0f / GTicksPerQuarterNote;

	inline constexpr float GMinMidiFileTempo = 10.0f;
	inline constexpr float GMaxMidiFileTempo = 960.0;

	//////////////////////////////////////////////////////////////////////////
	// Constants for the SMF (Standard Midi File) format.

	// Codes for special handling of System Exclusive messages:
	inline constexpr uint8 GFile_Escape      = 0xf7;
	inline constexpr uint8 GFile_SysEx       = 0xf0; 

	// Status code for SMF meta-events:
	inline constexpr uint8 GFile_Meta        = 0xff;

	// Meta-event IDs:
	inline constexpr uint8 GMeta_Text            = 0x01;
	inline constexpr uint8 GMeta_Copyright       = 0x02;
	inline constexpr uint8 GMeta_TrackName       = 0x03;
	inline constexpr uint8 GMeta_InstrumentName  = 0x04;
	inline constexpr uint8 GMeta_Lyric           = 0x05;
	inline constexpr uint8 GMeta_Marker          = 0x06;
	inline constexpr uint8 GMeta_CuePoint        = 0x07;
	inline constexpr uint8 GMeta_ChannelPrefix   = 0x20;
	inline constexpr uint8 GMeta_Port            = 0x21; // obsolete
	inline constexpr uint8 GMeta_EndOfTrack      = 0x2f;
	inline constexpr uint8 GMeta_Tempo           = 0x51;
	inline constexpr uint8 GMeta_SMPTE           = 0x54;
	inline constexpr uint8 GMeta_TimeSig         = 0x58;
	inline constexpr uint8 GMeta_KeySig          = 0x59;
	inline constexpr uint8 GMeta_Special         = 0x7f;

	HARMONIXMIDI_API FString GetMetaEventTypeName(uint8 Type);

	// if FMidiMsg type is "Runtime" these are the possible status byte.
	// note: runtime messages are not serialized!
	inline constexpr uint8 GRuntimeAllNotesOffStatus = 0x01;  // Receiver should feel free to allow notes to "adsr release"
	inline constexpr uint8 GRuntimeAllNotesKillStatus = 0x02; // Receiver is expected to immediately free resources associated with any sounding notes and stop playing ASAP!

	HARMONIXMIDI_API FString GetTextTypeName(uint8 TextType);

	///////////////////////////////////////////////////////////////////////////////
	// Status bytes indicate the type of message (3 bytes long except where noted).

	// Masks for disassembling MIDI Status bytes
	inline constexpr uint8 GStatusBitMask   = 0x80; // hi bit always set on Status byte
	inline constexpr uint8 GMessageTypeMask = 0xf0; // mask for message type
	inline constexpr uint8 GChannelMask     = 0x0f; // mask for channel
	inline constexpr uint8 GRealTimeMask    = 0xf8; // indicates real-time Status message

	inline constexpr uint8 GNoteOff  = 0x80; // Note Off
	inline constexpr uint8 GNoteOn   = 0x90; // Note On
	inline constexpr uint8 GPolyPres = 0xa0; // Polyphonic Key Pressure 
	inline constexpr uint8 GControl  = 0xb0; // Control Change
	inline constexpr uint8 GProgram  = 0xc0; // Program Change (2 bytes)
	inline constexpr uint8 GChanPres = 0xd0; // Channel Pressure (2 bytes)
	inline constexpr uint8 GPitch    = 0xe0; // Pitch Wheel Change
	inline constexpr uint8 GSystem   = 0xf0; // System

	inline bool  IsStatus(uint8 Byte)     { return (Byte & GStatusBitMask) > 0;  }
	inline uint8 GetType(uint8 Status)    { check(IsStatus(Status)); return Status & GMessageTypeMask; }

	inline bool  IsNoteOff(uint8 Status)  { return GetType(Status) == GNoteOff;  }
	inline bool  IsNoteOn(uint8 Status)   { return GetType(Status) == GNoteOn;   }
	inline bool  IsPolyPres(uint8 Status) { return GetType(Status) == GPolyPres; }
	inline bool  IsControl(uint8 Status)  { return GetType(Status) == GControl;  }
	inline bool  IsProgram(uint8 Status)  { return GetType(Status) == GProgram;  }
	inline bool  IsChanPres(uint8 Status) { return GetType(Status) == GChanPres; }
	inline bool  IsPitch(uint8 Status)    { return GetType(Status) == GPitch;    }
	inline bool  IsSystem(uint8 Status)   { return GetType(Status) == GSystem;   }

	inline uint8 GetChannel(uint8 Status) { check(IsStatus(Status));	check(!IsSystem(Status)); return Status & GChannelMask; }

	// turns MIDI style tempo (microseconds per quarter note) into BPM...
	inline float MidiTempoToBPM(int32 UsPerQuarterNote) { return UsPerQuarterNote== 0 ? 0 : 60000000.0f/(float)UsPerQuarterNote; }
	inline int32 BPMToMidiTempo(float Bpm)       { return Bpm == 0.0 ? 0 : (int32)(60000000.0f / (float)Bpm); }
	HARMONIXMIDI_API int32 SubdivisionToMidiTicks(const EMidiClockSubdivisionQuantization Division, const FTimeSignature& TimeSignature, const int32 TicksPerQuarterNote);
	HARMONIXMIDI_API float SubdivisionToBeats(EMidiClockSubdivisionQuantization Subdivision, const FTimeSignature& TimeSignature);

	HARMONIXMIDI_API float RoundToStandardBeatPrecision(float InBeat, int TimeSignatureDenominator);

	///////////////////////////////////////////////////////////////////////////////
	// Controllers
	// 
	// In a Control Change message, data byte 1 is the controller ID, and data
	// byte 2 is the controller value.
	enum class EControllerID : uint8
	{ // from midi specification...
	   // NOTE: If you add any here also add them to the GetControllerName function!
		BankSelection = 0,
		ModWheel = 1,
		Breath = 2,

		PortamentoTime = 5,
		DataCoarse = 6,
		Volume = 7,
		Balance = 8,
		PanRight = 10,
		Expression = 11,

		BitCrushWetMix = 14,
		BitCrushLevel = 15,
		BitCrushSampleHold = 16,

		LFO0Frequency = 22,
		LFO1Frequency = 23,
		LFO0Depth = 24,
		LFO1Depth = 25,

		DelayTime = 26,
		DelayDryGain = 27,
		DelayWetGain = 28,
		DelayFeedback = 29,

		CoarsePitchBend = 30,
		SampleStartTime = 31,
		DataFine = 38,

		SubStreamVol1 = 52,
		SubStreamVol2 = 53,
		SubStreamVol3 = 54,
		SubStreamVol4 = 55,
		SubStreamVol5 = 56,
		SubStreamVol6 = 57,
		SubStreamVol7 = 58,
		SubStreamVol8 = 59,

		DelayEQEnabled = 60,
		DelayEQType = 61,
		DelayEQFreq = 62,
		DelayEQQ = 63,

		Hold = 64,
		PortamentoSwitch = 65,
		Sustenuto = 66,
		SoftPedal = 67,
		Legato = 68,
		Hold2 = 69,
		FilterQ = 71,
		Release = 72,
		Attack = 73,
		FilterFrequency = 74,

		TimeStretchEnvelopeOrder = 79,

		DelayLFOBeatSync = 80,
		DelayLFOEnabled = 81,
		DelayLFORate = 82,
		DelayLFODepth = 83,

		DelayStereoType = 84,
		DelayPanLeft = 85,
		DelayPanRight = 86,

		RPNFine = 100,
		RPNCourse = 101,
		AllSoundOff = 120,
		Reset = 121,
		AllNotesOff = 123
	};

	HARMONIXMIDI_API FString GetControllerName(EControllerID ControllerId);
	HARMONIXMIDI_API void GetControllerNames(TArray<FString>& Names);

	HARMONIXMIDI_API FString MakeStdMsgString(uint8 Status, uint8 Data1, uint8 Data2);

	// allows choice between various naming conventions for enharmonic notes
	enum class ENoteNameEnharmonicStyle : uint8
	{
		Sharp,
		Flat,
		SharpAndFlat
	};

	HARMONIXMIDI_API int8 GetNoteNumberFromNoteName(const char* InName);
	HARMONIXMIDI_API const char* GetNoteNameFromNoteNumber(uint8 MidiNoteNumber, ENoteNameEnharmonicStyle Style = ENoteNameEnharmonicStyle::SharpAndFlat);
	HARMONIXMIDI_API int8 GetNoteOctaveFromNoteNumber(uint8 MidiNoteNumber);

	///////////////////////////////////////////////////////////////////////////////
	// System messages:

	// System Common:
	inline constexpr uint8 GSys_Mtc         = 0xf1; // 2 byte message
	inline constexpr uint8 GSys_SongPos     = 0xf2; // 3 byte message
	inline constexpr uint8 GSys_SongSelect  = 0xf3; // 2 byte message
	inline constexpr uint8 GSys_TuneRequest = 0xf6; // 1 byte message
	inline constexpr uint8 GSys_Eox         = 0xf7; // 1 byte message
	
	// System RealTime messages (all 1 byte)
	inline constexpr uint8 GSys_TimingClock = 0xf8;
	inline constexpr uint8 GSys_Start       = 0xfa;
	inline constexpr uint8 GSys_Continue    = 0xfb;
	inline constexpr uint8 GSys_Stop        = 0xfc;
	inline constexpr uint8 GSys_ActiveSense = 0xfe;
	inline constexpr uint8 GSys_Reset       = 0xff;
};
