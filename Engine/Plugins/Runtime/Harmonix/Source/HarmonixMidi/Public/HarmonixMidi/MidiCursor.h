// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MidiFile.h"

#define UE_API HARMONIXMIDI_API

class FMidiCursor
{
public:
	static const int32 kAllTracks = -1;

	UE_API void Prepare(UMidiFile* InMidiFile, int32 TrackIndex = kAllTracks, bool bResetState = true);
	UE_API void Prepare(TSharedPtr<FMidiFileData> InMidiFile, int32 TrackIndex = kAllTracks, bool bResetState = true);

	int32 GetNextTick() const { return NextTick; }

	class FReceiver
	{
	public:

		virtual void OnMidiMessage(int32 TrackIndex, int32 Tick, uint8 Status, uint8 Data1, uint8 Data2, bool bIsPreroll) {}
		virtual void OnPreRollNoteOn(int32 TrackIndex, int32 EventTick, int32 Tick, float PreRollMs, uint8 Status, uint8 Data1, uint8 Data2) {}
		virtual void OnText(int32 TrackIndex, int32 Tick, int32 TextIndex, const FString& Str, uint8 Type, bool bIsPreroll) {}
		virtual void OnTempo(int32 TrackIndex, int32 Tick, int32 Tempo, bool bIsPreroll) {}
		virtual void OnTimeSig(int32 TrackIndex, int32 Tick, int32 Numerator, int32 Denominator, bool bIsPreroll) {}
		UE_API virtual bool HandleMessage(int32 TrackIndex, const FMidiTrack& Track, const FMidiEvent& Event, bool bIsPreroll);
	};

	UE_API void  SeekToNextTick(int32 NewNextTick, int32 PrerollBars = 0, FReceiver* PrerollReceiver = nullptr);
	UE_API int32 SeekToMs(float NewPositionMs, int32 PrerollBars = 0, FReceiver* PrerollReceiver = nullptr);

	UE_API void Process(int32 FirstTickToProcess, int32 LastTickToProcess, FReceiver& Receiver);
	
	// Preroll is like process except it filters meaningless messages that happen in the provided
	// span of ticks. For example:
	//    - If the span includes both a note-on AND a note-off for the same note on the same track
	//      those events are 'eaten' and not provided to the receiver. 
	//    - Only the last time signature and tempo in the span will be sent.
	//    - etc. 
	UE_API void Preroll(int32 FirstTickToProcess, int32 LastTickToProcess, FReceiver& Receiver);

	UE_API bool PassedEnd() const;

private:
	UE_API void UpdateNextTick(int32 NewNextTick);

	TSharedPtr<FMidiFileData> MidiFile;
	TArray<int32> TrackNextEventIndexs;
	float CurrentFileMs = 0.0f;
	int32 NextTick = 0;
	int32 LastTick = -1;
	int32 WatchTrack = -1;
};

#undef UE_API
