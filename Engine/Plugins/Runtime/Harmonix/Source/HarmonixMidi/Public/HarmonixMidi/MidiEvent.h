// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MidiMsg.h"

#include "MidiEvent.generated.h"

#define UE_API HARMONIXMIDI_API

class FMidiWriter;
struct FMidiTrack;

#pragma pack(push,1)
/**
 * An FMidiEvent is a container for a tick and a midi message. 
 */
USTRUCT()
struct FMidiEvent
{
	GENERATED_BODY()

public:
	// default construct for USTRUCT
	FMidiEvent()
	: Tick(0)
	{}
	
	UE_API FMidiEvent(int32 Tick, const FMidiMsg& InMessage);

	bool operator==(const FMidiEvent& Other) const
	{
		return Tick == Other.Tick && Message == Other.Message;
	}

	/** Save the event out in standard midi file format */
	UE_API void WriteStdMidi(FMidiWriter& Writer, const FMidiTrack& Track) const;

	UE_API bool Serialize(FArchive& Archive);

	int32  GetTick() const { return Tick;         }
	bool IsText() const { return Message.IsText(); }
	const FMidiMsg& GetMsg() const { return Message; }
	FMidiMsg& GetMsg() { return Message; }
	void SetMsg(const FMidiMsg& InMessage) { Message = InMessage; }

	bool operator < (const FMidiEvent& rhs) const { return Tick < rhs.Tick; }

	struct LessThan
	{
		bool operator()(const FMidiEvent& Event, float InTick) const { return Event.GetTick() < InTick; }
		bool operator()(float InTick, const FMidiEvent& Event) const { return InTick < Event.GetTick(); }
		bool operator()(const FMidiEvent& EventA, const FMidiEvent& EventB) const { return EventA.GetTick() < EventB.GetTick(); }
	};

	explicit FORCEINLINE FMidiEvent(EForceInit FI)
		: Tick(0)
		, Message(FI)
	{
	}

private:
	friend struct FMidiTrack;
	friend struct FMidiFileData;
	friend class  UMidiFile;

	UPROPERTY()
	int32		Tick;
	UPROPERTY()
	FMidiMsg	Message;
};
#pragma pack(pop)

template <>
struct TStructOpsTypeTraits<FMidiEvent> : public TStructOpsTypeTraitsBase2<FMidiEvent>
{
	enum
	{
		WithSerializer = true,
		WithNoInitConstructor = true,
	};
};

#undef UE_API
