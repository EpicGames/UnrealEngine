// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HarmonixMidi/TempoMap.h"
#include "HarmonixMidi/MidiMsg.h"
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/SongMaps.h"
#include "HarmonixMidi/MidiTrack.h"
#include "IAudioProxyInitializer.h"
#include "MusicTypes/MusicMapSource.h"
#include "MidiFile.generated.h"

#define UE_API HARMONIXMIDI_API

class UAssetImportData;

class FMidiFileProxy;
using FMidiFileProxyPtr = TSharedPtr<FMidiFileProxy, ESPMode::ThreadSafe>;

USTRUCT(BlueprintType, Meta = (DisplayName = "MIDI File Data"))
struct FMidiFileData
#if CPP
	: public ISongMapEvaluator
#endif
{
	GENERATED_BODY()

	FMidiFileData()
		: TicksPerQuarterNote(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt)
	{}

	virtual ~FMidiFileData() = default;

	UE_API bool operator==(const FMidiFileData& Other) const;

	void Empty()
	{
		MidiFileName.Empty();
		TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt;
		SongMaps.EmptyAllMaps();
		Tracks.Empty();
		Tracks.Emplace("Conductor");
	}

	bool IsEmpty() const 
	{
		return MidiFileName.IsEmpty() &&
			TicksPerQuarterNote == Harmonix::Midi::Constants::GTicksPerQuarterNoteInt &&
			SongMaps.IsEmpty() &&
			(Tracks.IsEmpty() || (Tracks.Num() == 1 && Tracks[0].GetNumEvents() == 1 && Tracks[0].GetName()->Compare("Conductor", ESearchCase::IgnoreCase) == 0));
	}

	UPROPERTY(BlueprintReadOnly, Category = "MidiFile")
	FString MidiFileName;
	UPROPERTY(BlueprintReadOnly, Category = "MidiFile")
	int32 TicksPerQuarterNote;
	UPROPERTY(BlueprintReadOnly, Category = "MidiFile")
	FSongMaps SongMaps;
	UPROPERTY(BlueprintReadOnly, Category = "MidiFile")
	TArray<FMidiTrack> Tracks;

	UE_API int32 GetLastEventTick() const;

	UE_API int32 FindTrackIndexByName(const FString& TrackName);
	
	UE_API void PostSerialize(const FArchive& Ar);

	UE_API bool ConformToLength(int32 NewLengthTicks);
	UE_API bool ConformToLengthGivenLastEventTick(int32 NewLastEventTick);

	/**
	* Adds a tempo change to the midi data at the given tick for the given track idx
	* 
	* asserts valid TrackIdx
	*/
	UE_API void AddTempoChange(int32 TrackIdx, int32 Tick, float TempoBPM);
	
	/**
	* Adds a Time Signature change to the next bar boundary from the input tick for the given track
	* 
	* Time signature changes can only happen at the beginning of a bar
	* So input Tick will be rounded UP to the next bar boundary.
	* 
	* asserts valid TrackIdx
	*/
	UE_API void AddTimeSigChange(int32 TrackIdx, int32 Tick, int32 TimeSigNum, int32 TimeSigDenom);

	UE_API void ScanTracksForSongLengthChange();

	UE_API bool LengthIsAPerfectSubdivision() const;

	//** BEGIN ISongMapEvaluator Overrides ***********
	virtual const FSongLengthData& GetSongLengthData() const override { return SongMaps.GetSongLengthData(); }
	virtual const FTempoMap& GetTempoMap() const override { return SongMaps.GetTempoMap(); }
	virtual const FBeatMap& GetBeatMap() const override { return SongMaps.GetBeatMap(); }
	virtual const FBarMap& GetBarMap() const override { return SongMaps.GetBarMap(); }
	virtual const FSectionMap& GetSectionMap() const override { return SongMaps.GetSectionMap(); }
	virtual const FChordProgressionMap& GetChordMap() const override { return SongMaps.GetChordMap(); }
	//** END ISongMapEvaluator Overrides ***********
};

template<>
struct TStructOpsTypeTraits<FMidiFileData> : public TStructOpsTypeTraitsBase2<FMidiFileData>
{
	enum
	{
		WithPostSerialize = true,
	};
};

/**
 * An FMidFile is primarily a container for FMidiTracks. 
 * 
 * This class can handle loading and saving standard midi files, as well
 * as serializing itself to standard Unreal Engine FArchives.
 */
UCLASS(MinimalAPI, BlueprintType, Category="Music", Meta = (DisplayName = "Standard MIDI File"))
class UMidiFile : public UObject, public IAudioProxyDataFactory, public IMusicMapSource
{
	GENERATED_BODY()

public:
	using FMidiTrackList = TArray<FMidiTrack>;

	enum class ELoadResult
	{
		Failed = 0,
		Success
	};

	UE_API UMidiFile();

	// A comparison operator. This allows for differences in members related to how
	// a midi file was generated/imported, but compares the underlying "renderable"
	// midi data.
	UE_API bool operator==(const UMidiFile& Other) const;

	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_API virtual void PostInitProperties() override;

	/** A method for importing a standard midi file, with the option of providing a pointer to an FSongMaps instance that will be populated during the load. */
	UE_API ELoadResult LoadStdMidiFile(
		const FString& FilePath,
		int32 DesiredTicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt,
		Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding = Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8,
		bool EnsureNotFailed = true);
	/** A method for importing a standard midi file, with the option of providing a pointer to an FSongMaps instance that will be populated during the load. */
	UE_API ELoadResult LoadStdMidiFile(
		void* Buffer,
		int32 BufferSize,
		const FString& FileName,
		int32 DesiredTicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt,
		Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding = Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8,
		bool EnsureNotFailed = true);
	/** A method for importing a standard midi file, with the option of providing a pointer to an FSongMaps instance that will be populated during the load. */
	UE_API ELoadResult LoadStdMidiFile(
		TSharedPtr<FArchive> Archive,
		const FString& Filename,
		int32 DesiredTicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt,
		Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding = Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8,
		bool EnsureNotFailed = true);

	/** A method for exporting the midi track data to a standard midi file. */
	UE_API void SaveStdMidiFile(const FString& FilePath) const;
	/** A method for exporting the midi track data to a standard midi file. */
	UE_API void SaveStdMidiFile(TSharedPtr<FArchive> Archive, const FString& Filename = FString()) const;

	//** BEGIN IMusicMapSource
	UE_API virtual void CreateFrameBasedMusicMap(UFrameBasedMusicMap* Map) const override;
	UE_API virtual FMarkerProviderResults GatherMarkers(const UFrameBasedMusicMap* Map) const override;
	//** END IMusicMapSource

	UE_API FMidiTrack* AddTrack(const FString& Name);

	UE_API void Empty();
	UE_API bool IsEmpty() const;

	UE_API void SortAllTracks();

	int32 GetNumTracks() const { return TheMidiData.Tracks.Num(); }
	const FMidiTrack* GetTrack(int32 Index) const { return &TheMidiData.Tracks[Index]; }
	FMidiTrack* GetTrack(int32 Index) { return &TheMidiData.Tracks[Index]; }
	UE_API const FMidiTrack* FindTrackByName(const FString& TrackName) const;
	UE_API int32 FindTrackIndexByName(const FString& TrackName) const;

	// Find the text event and return the tick or -1 if not found.
	UE_API int32 FindTextEvent(const FString& EventText, const FString* TrackName = nullptr);
	// Find the text event and return the tick or -1 if not found.
	UE_API int32 FindTextEvent(const FString& EventText, int32 TrackIndex);

	// Find all the ticks at which the text event occurs
	UE_API TArray<int32> FindAllTextEvents(const FString& EventText, const FString* TrackName = nullptr);
	UE_API void FindAllTextEvents(const FString& EventText, int32 TrackIndex, TArray<int32> OutIndexes);

	FMidiTrackList& GetTracks() { return TheMidiData.Tracks; }
	const FMidiTrackList& GetTracks() const { return TheMidiData.Tracks; }

	UE_API void BuildConductorTrack();

	/**
	 * This function must be called if any changes are made to any of the tracks of this midi file. It will 
	 * do various internal tasks necessary to assure the midi data is consistent and "playable".
	 */
	UE_API void ScanTracksForSongLengthChange();

	int32  GetLastEventTick() const { return TheMidiData.GetLastEventTick(); }

#if WITH_EDITORONLY_DATA
	// Import data for this MidiFileAsset
	// updated during Import in MidiFile
	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<UAssetImportData> AssetImportData;
	UE_API FString GetImportedSrcFilePath() const;

	//The Start Bar of a Midi File
	UPROPERTY(EditDefaultsOnly,BlueprintReadOnly,Category = "Midi File Start Bar")
	int32 StartBar = 1;

	//StartBar UPROPERTY callback 
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
#if WITH_EDITOR

	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

#endif

	int32 GetStartBar() { return TheMidiData.SongMaps.GetStartBar(); };

	// This does a quick and dirty to check to see if the length of the midi is 
	// on a musical subdivision. It is used primarily during asset importing, and
	// if it returns true the importer will do further checking to see how far off
	// it is, and suggest either conforming the length via QuantizeLengthToNearestPerfectSubdivision
	// (for "off-by-one" type small errors) or QuantizeLengthToSubdivision (when the
	// error is large and the user should choose a specific subdivision and direction
	// for conforming the file.
	UE_API bool LengthIsAPerfectSubdivision() const;

	UE_API void QuantizeLengthToNearestPerfectSubdivision(const EMidiFileQuantizeDirection Direction);

	UE_API void QuantizeLengthToSubdivision(const EMidiFileQuantizeDirection Direction, const EMidiClockSubdivisionQuantization Subdivision);

	UE_API bool ConformToLength(int32 NewLengthTicks);

	UE_API bool ConformToLengthGivenLastEventTick(int32 NewLastEventTick);

	const FSongMaps* GetSongMaps() const { return &TheMidiData.SongMaps; }
	FSongMaps* GetSongMaps() { return &TheMidiData.SongMaps; }

	UE_API TSharedPtr<FMidiFileData> GetOrCreateRenderableCopy();

	// IAudioProxyDataFactory
	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

	// UObject
	UE_API void BeginDestroy() override;

protected:
	UPROPERTY()
	FMidiFileData TheMidiData;

	TSharedPtr<FMidiFileData> RenderableCopyOfMidiFileData;
};

class FMidiFileProxy : public Audio::TProxyData<FMidiFileProxy>
{
public:
	IMPL_AUDIOPROXY_CLASS(FMidiFileProxy);

	explicit FMidiFileProxy(TSharedPtr<FMidiFileData>& Data)
		: MidiFileDataPtr(Data)
	{}

	TSharedPtr<FMidiFileData> GetMidiFile()
	{
		return MidiFileDataPtr;
	}

private:
	TSharedPtr<FMidiFileData> MidiFileDataPtr;
};

#undef UE_API
