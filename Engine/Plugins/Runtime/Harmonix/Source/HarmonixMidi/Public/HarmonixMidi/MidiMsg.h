// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MidiConstants.h"

#include "MidiMsg.generated.h"

#define UE_API HARMONIXMIDI_API

class FMidiWriter;
struct FMidiTrack;

using FMidiTextRepository = TArray<FString>;

#pragma pack(push,1)

/**
	* A class representing a single standard midi message. This class is tiny and tightly packed
	* to keep the midi data footprint small on disk and in memory. 
	*/
USTRUCT()
struct FMidiMsg
{
	GENERATED_BODY()

public:
	enum class EType : uint8
	{
		Std     = 1,
		Tempo   = 2,
		TimeSig = 4,
		Text    = 8,
		Runtime = 16
	};

	static UE_API FMidiMsg CreateNoteOn(int32 Channel, int32 Note, int32 Velocity);
	static UE_API FMidiMsg CreateNoteOff(int32 Channel, int32 Note);
	static UE_API FMidiMsg CreateControlChange(uint8 Channel, uint8 ControlNumber, uint8 Value);
	static UE_API FMidiMsg CreateText(uint16 InTextIndex, uint8 InTextType);
	static UE_API float GetPitchBendFromData(int8 Data1, int8 Data2);

	EType MsgType() const   { return Type;                   }
	bool  IsStd() const     { return Type == EType::Std;     }
	bool  IsRuntime() const { return Type == EType::Runtime; }
	bool  IsText() const    { return Type == EType::Text;    }
	bool  IsNoteMessage() const { return IsNoteOn() || IsNoteOff() || IsAllNotesOff() || IsAllNotesKill(); }
	bool  IsNoteOn() const  { return Type == EType::Std && Harmonix::Midi::Constants::IsNoteOn(Status);  }
	bool  IsNoteOff() const
	{
		return Type == EType::Std && (Harmonix::Midi::Constants::IsNoteOff(Status) || (Harmonix::Midi::Constants::IsNoteOn(Status) && Data2 == 0));
	}
	bool  IsAllNotesOff() const { return Type == EType::Runtime && Status == Harmonix::Midi::Constants::GRuntimeAllNotesOffStatus; }
	bool  IsAllNotesKill() const { return Type == EType::Runtime && Status == Harmonix::Midi::Constants::GRuntimeAllNotesKillStatus; }
	bool  IsControlChange() const { return IsStd() && Harmonix::Midi::Constants::IsControl(Status); }
	bool  IsTempo() const { return Type == EType::Tempo; }
	bool  IsTimeSignature() const { return Type == EType::TimeSig; }
	bool  IsLyric() const { return Type == EType::Text && TextType == Harmonix::Midi::Constants::GMeta_Lyric; }
	bool  IsSystemMessage() const { return Type == EType::Std && Harmonix::Midi::Constants::IsSystem(Status); }

	/** Construct a standard (std) midi message */
	UE_API FMidiMsg(uint8 InStatus, uint8 InData1, uint8 InData2);
	uint8 GetStdStatus()     const { check(Type == EType::Std); return Status; }
	uint8 GetStdData1()      const { check(Type == EType::Std); return Data1; }
	uint8 GetStdData2()      const { check(Type == EType::Std); return Data2; }
	uint8 GetStdChannel()    const { check(Type == EType::Std); return Harmonix::Midi::Constants::GetChannel(Status); }
	uint8 GetStdStatusType() const { check(Type == EType::Std); return Harmonix::Midi::Constants::GetType(Status); }
	UE_API float GetPitchBendFromData() const;

	/** Construct a midi tempo message */
	UE_API FMidiMsg(int32 MicrosecPerQuarterNote);
	int32 GetMicrosecPerQuarterNote() const
	{
		check(Type == EType::Tempo);
		return int32((uint32)MicsPerQuarterNoteH << 16 | (uint32)MicsPerQuarterNoteL);
	}

	/** Construct a midi time signature message */
	UE_API FMidiMsg(uint8 Numerator, uint8 Denominator);
	uint8 GetTimeSigNumerator()  const { check(Type == EType::TimeSig); return Numerator;   }
	uint8 GetTimeSigDenominator()  const { check(Type == EType::TimeSig); return Denominator; }

	//Text
	// Note: messages of this type are constructed using the subclass below.
	uint16 GetTextIndex()   const { check(Type == EType::Text); return TextIndex; }
	uint8  GetTextType()    const { check(Type == EType::Text); return TextType;  }

	UE_API void SetNoteOnVelocity(uint8 Velocity);

	UE_API void WriteStdMidi(int32 Tick, FMidiWriter& writer, const FMidiTrack& track) const;
		
	static UE_API FString ToString(const FMidiMsg& Message, const FMidiTrack* Track = nullptr);

	UE_API bool Serialize(FArchive& Archive);

	explicit FORCEINLINE FMidiMsg(EForceInit)
		: Type(EType::Std)
		, Status(0)
		, Data1(0)
		, Data2(0)
	{
	}

	// Runtime Messages
	static FMidiMsg CreateAllNotesOff()
	{
		FMidiMsg NewMsg;
		NewMsg.Type = EType::Runtime;
		NewMsg.Status = Harmonix::Midi::Constants::GRuntimeAllNotesOffStatus;
		return NewMsg;
	}
	static FMidiMsg CreateAllNotesKill()
	{
		FMidiMsg NewMsg;
		NewMsg.Type = EType::Runtime;
		NewMsg.Status = Harmonix::Midi::Constants::GRuntimeAllNotesKillStatus;
		return NewMsg;
	}

	FMidiMsg()
		: Type(EType::Std)
		, Status(0)
		, Data1(0)
		, Data2(0)
	{}

	bool operator==(const FMidiMsg& Other) const
	{
		return Type == Other.Type && Status == Other.Status && Data1 == Other.Data1 && Data2 == Other.Data2;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FMidiMsg& InMidiMsg)
	{
		return HashCombineFast(
			HashCombineFast(GetTypeHash(InMidiMsg.Type), GetTypeHash(InMidiMsg.Status)),
			HashCombineFast(GetTypeHash(InMidiMsg.Data1), GetTypeHash(InMidiMsg.Data2)));
	}

	EType Type;
	// Use a union instead of subclassing MidiMsg so that MidiMsg is 
	// constant size and therefore you can have a vector<MidiMsg>.
	//
	// All structs in this unit should be 3 bytes long. 
	//
	// All members of those structs (including padding) must be 
	// initialized so that no random data appears when these are
	// serialized!
	union
	{
		//Std
		struct
		{
			uint8 Status;
			uint8 Data1;
			uint8 Data2;
		};
		// Tempo
		struct
		{
			uint8  MicsPerQuarterNoteH;
			uint16 MicsPerQuarterNoteL;
		};
		// Time Sig
		struct
		{
			uint8  Numerator;
			uint8  Denominator;
			uint8  ts_pad;
		};
		//Text
		struct
		{
			uint8  TextType;
			uint16 TextIndex;
		};
	};
};

static_assert(sizeof(FMidiMsg) == 4);

template <>
struct TStructOpsTypeTraits<FMidiMsg> : public TStructOpsTypeTraitsBase2<FMidiMsg>
{
	enum
	{
		WithSerializer = true,
		WithNoInitConstructor = true,
	};
};

#pragma pack(pop)

#undef UE_API
