// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Serialization/BufferReader.h"
#include "HAL/FileManagerGeneric.h"
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/MidiConstants.h"

/**
* MidiReaders are helper classes for reading/parsing Standard MIDI Files.
* they read midi bytes, turn them in to more appropriate data representations,
* and then passes the data to an IMidiReceivers
 */

class IMidiReceiver;

/**
 * A base class for Midi Readers so that IMidiReceivers can have a generic
 * "back pointer" to the reader that is calling them.
 */
class IMidiReader
{
public:
	virtual ~IMidiReader() = default;
	/** Called by a receiver when it wants to format a tick into an MBT for error messaging */
	virtual const TSharedPtr<FBarMap> GetBarMap() const = 0; 
	virtual const FString& GetCurrentTrackName() const = 0;
	virtual int32 GetCurrentTrackIndex() const = 0;
	virtual const FString& GetFilename() const = 0;
	virtual int32 GetLastTick() const = 0;
	/** Called by a receiver when it wants to skip to the next track */
	virtual void SkipCurrentTrack() = 0;           
	virtual bool IsFailed() const = 0;
};

/**
 * A concrete IMidiReader implementation that can read standard midi files.
 * 
 * You rarely have to interact with this class, as FMidiFile uses it internally when 
 * you use its LoadStdMidiFile functions. 
 * 
 * @see FMidiFile
 */
class FStdMidiFileReader : public  IMidiReader
{
public:
	FStdMidiFileReader(
		TSharedPtr<FArchive> Archive,
		const FString& InFilename,
		IMidiReceiver* InReceiver, 
		int32 TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt,
		Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding = Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8);
	FStdMidiFileReader(
		const FString& FilePath,
		IMidiReceiver* Receiver,
		int32 TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt,
		Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding = Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8);
	FStdMidiFileReader(
		void* Buffer,
		int32 BufferSize,
		const FString& FileName,
		IMidiReceiver* Receiver,
		int32 TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt,
		Harmonix::Midi::Constants::EMidiTextEventEncoding InTextEncoding = Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8);

	virtual ~FStdMidiFileReader() = default;

	void ReadAllTracks();                // Read all tracks
	bool ReadSomeEvents(int32 NumEvents);
	bool ReadTrack();                    // Read the next track, return false if no more
	bool ReadEvents(int32 Count);          // Read some events, return false if no more
	void SkipCurrentTrack() override;    // call this at any point to skip to the next track

	// used by error reporting system
	virtual const FString& GetFilename() const override { return Filename; }
	virtual int32 GetCurrentTrackIndex() const override { return CurrentTrackIndex; }
	virtual const FString& GetCurrentTrackName() const override { return CurrentTrackName; }
	virtual const TSharedPtr<FBarMap> GetBarMap() const override { return BarMap; }

	virtual int32 GetLastTick() const override { return LastTick; }

	virtual bool IsFailed() const override { return State == EState::Failed; }

private:
	void Init();
	void ReadNextEvent();
	void ReadNextEventImpl();

	void ReadFileHeader();
	void ReadTrackHeader();
	void ReadEvent();
	void ReadMidiEvent(int32 Tick, uint8 Status, uint8 Data1);
	void ReadSystemEvent(int32 Tick, uint8 Status);
	void ReadMetaEvent(int32 Tick, uint8 Type);
	FString ReadText(int32 Length);

	// functions for sorting the midi before sending it out to the receivers
	void QueueChannelMsg(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2);
	void ProcessMidiList();
	bool FailIfReadPositionIsPastTrackEnd();
	bool FailIfBytesNotAvailableInTrackData(int64 NumBytes);

	enum class EState
	{
		Failed,
		Start,     // at the very beginning (before MThd chunk)
		NewTrack,  // at the beginning of a track (before MTrk chunk)
		InTrack,   // in the middle of reading a track
		End,        // done
	};

	FString Filename;
	TSharedRef<FArchive> Archive;

	IMidiReceiver* Receiver = nullptr;
	Harmonix::Midi::Constants::EMidiTextEventEncoding TextEncoding = Harmonix::Midi::Constants::EMidiTextEventEncoding::Latin1;
	EState State = EState::Start;

	int32 DestinationTicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt;
	float TickConversionFactor = 1.0f; // factor to convert from file's tick-per-quarternote to kTicksPerQuarterNote 

	int16 Format = 1;             // Format 0 or Format 1 standard midi file?
	int16 NumTracks = 0;          // Number of tracks.
	int64 LastTracksFilePosition; // Position in the file of the current track's first byte.
	int32 TrackEndPos = 0;        // end byte position of the current track
	int32 CurrentFileTick = 0;    // Current tick in units of File's ticks-per-quarter.
	int32 CurrentTick = 0;        // Current tick count for current track where one quarter note is TicksPerQuarterNote.
	int32 LastTick = 0;
	uint8 PrevStatus = 0;         // Previous status byte (for running status).
	int32 CurrentTrackIndex = -1; // What track are we currently reading?
	FString CurrentTrackName;

	inline static constexpr int32 kMaxSupportedMidiStringSize = 2048;

	enum class ETrackFilteringMode
	{
		None,
		ConductorEvents,
		NonConductorEvents,
	};
	ETrackFilteringMode TrackFilteringMode = ETrackFilteringMode::None;

	// stuff for sorting Midi messages on the same tick
	struct FRawMidiMsg
	{
		uint8 Status;
		uint8 Data1;
		uint8 Data2;
	};
	using FRawMidiList = TArray<FRawMidiMsg>;
	FRawMidiList EventsOnSameTick;         // a list of midi msgs all at the same tick (to be sorted)
	int32          MidiListTick = 0; // the tick of the messages in mMidiList
	struct RawMidiLess
	{
		inline int32 MidiRank(uint8 Status) const
		{
			using namespace Harmonix::Midi::Constants;
			switch (Status & GMessageTypeMask)
			{
			case GNoteOff:  return 1;
			case GControl:  return 2;
			case GProgram:  return 3;
			case GChanPres: return 4;
			case GPitch:    return 5;
			case GPolyPres: return 6;
			case GNoteOn:   return 7;
			default:        return 8;
			}
		}
		bool operator()(const FStdMidiFileReader::FRawMidiMsg& lhs, const FStdMidiFileReader::FRawMidiMsg& rhs) const
		{
			return MidiRank(lhs.Status) < MidiRank(rhs.Status);
		}
	};

	// we'll keep track of our own bar map so we can display
	// more meaningful error/warning messages 
	TSharedPtr<FBarMap> BarMap;
};
