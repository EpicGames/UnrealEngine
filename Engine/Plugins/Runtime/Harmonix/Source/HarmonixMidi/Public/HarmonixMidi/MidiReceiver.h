// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Algo/ForEach.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"

/**
 * MidiReceivers are used in conjunction with a IMidiReaders to read data from a
 * MIDI file. As the IMidiReader processes the file, the appropriate member 
 * functions on the receiver are called.
 * 
 * FSongMaps has a type specific IMidiReceiver that takes callbacks from the IMidiReader
 * and constructs its internal maps. 
 * 
 * Games can implement their own midi receivers if they want to process midi messages 
 * when midi files are read. In this way, the game doesn't need to worry about the 
 * low level byte wrangling and can respond to callbacks that receive well formed 
 * midi messages
 */

class IMidiReader;

class IMidiReceiver
{
public:
	virtual ~IMidiReceiver() = default;

	/** The reader is about to read a new MIDI file. Now is your chance to reset your state. */
	virtual bool Reset() = 0;

	/** Called when the reader has determined how many tracks are in the midi data */
	virtual bool SetNumTracks(int32 Num) { return true; }

	/** called when all tracks have been read. */
	virtual bool Finalize(int32 InLastFileTick) { return true; }

	virtual bool OnNewTrack(int32 NewTrackIndex) = 0;
	virtual bool OnEndOfTrack(int32 LastTick) = 0;
	virtual bool OnAllTracksRead() = 0;

	/** For standard 1- or 2-byte MIDI message */
	virtual bool OnMidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2) = 0;

	/**
	 * Text, Copyright, TrackName, InstName, Lyric, Marker, CuePoint meta-event:
	 * "type" is the type of meta-event (constants defined in MidiConstants.h) 
	 */
	virtual bool OnText(int32 Tick, const FString& Str, uint8 Type) = 0;

	/**
	 * Tempo Change meta - event:
	 * tempo is in microseconds per quarter-note
	 */
	virtual bool OnTempo(int32 Tick, int32 Tempo) { return true; }

	/** Time Signature meta - event:
	 *   time signature is numerator/denominator
	 */
	virtual bool OnTimeSignature(int32 Tick, int32 Numerator, int32 Denominator, bool FailOnError = true) { return true; }

	/**
	 * Called by the reader before it starts processing the midi data stream.
	 * This saves off a back pointer that is very useful for things like writing 
	 * good error messages when a receiver encounters unexpected data, or for 
	 * telling the reader to skip the current track and move on.
	 */
	virtual void SetMidiReader(IMidiReader* InReader) { Reader = InReader; }

	/** Tells the reader to Skip the rest of this track and go to the next one */
	void SkipCurrentTrack();

	const IMidiReader* GetReader() const { return Reader; }

protected:
	IMidiReader* Reader = nullptr;
};

/**
 * An implementation of IMidiReceiver that allows one to parse a midi file
 * and send the found data to many receivers at the same time.
 * 
 * Again, this can be very useful if a game has multiple systems that want to 
 * build their own runtime data based on data found in a standard midi file.
 *
 * @see IMidiReceiver
 */
class FMidiReceiverList : public IMidiReceiver
{
#define _FOR_EACH_MIDI_RECEIVER(func_body) Algo::ForEach(ReceiverList, [&](IMidiReceiver* recvr){ if (Result) func_body });
public:
	virtual bool Reset()
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->Reset(); });
		return Result;
	}

	virtual bool Finalize(int32 LastFileTick)
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result =  recvr->Finalize(LastFileTick); });
		return Result;
	}

	virtual bool OnNewTrack(int32 NewTrackIndex)
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->OnNewTrack(NewTrackIndex); });
		return Result;
	}

	virtual bool OnEndOfTrack(int32 LastTick)
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->OnEndOfTrack(LastTick); });
		return Result;
	}

	virtual bool OnAllTracksRead()
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->OnAllTracksRead(); });
		return Result;
	}

	virtual bool OnMidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2)
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->OnMidiMessage(Tick, Status, Data1, Data2); });
		return Result;
	}

	virtual bool OnText(int32 Tick, const FString& Str, uint8 Type)
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->OnText(Tick, Str, Type); });
		return Result;
	}

	virtual bool OnTempo(int32 Tick, int32 Tempo)
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->OnTempo(Tick, Tempo); });
		return Result;
	}

	virtual bool OnTimeSignature(int32 Tick, int32 Numerator, int32 Denominator, bool FailOnError = true)
	{
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ Result = recvr->OnTimeSignature(Tick, Numerator, Denominator, FailOnError); });
		return Result;
	}

	virtual void SetMidiReader(IMidiReader* InReader)
	{
		IMidiReceiver::SetMidiReader(InReader);
		bool Result = true;
		_FOR_EACH_MIDI_RECEIVER({ recvr->SetMidiReader(InReader); });
	}

	void Add(IMidiReceiver* recvr) { ReceiverList.Add(recvr); }

	void Empty() { ReceiverList.Empty(); }

	int32 Num() { return ReceiverList.Num(); }

private:
	TArray<IMidiReceiver*> ReceiverList;
#undef _FOR_EACH_MIDI_RECEIVER
};
