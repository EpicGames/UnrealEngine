// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/BeatMap.h"
#include "HarmonixMidi/MidiConstants.h"
#include "Math/UnrealMathUtility.h"
#include "SectionMap.h"

#include "MidiSongPos.generated.h"

#define UE_API HARMONIXMIDI_API

struct ISongMapEvaluator;

/////////////////////////////////////////////////////////////////////////////
 // Position within a song (midi info)
//
USTRUCT(BlueprintType, Meta = (DisplayName = "MIDI Song Position"))
struct FMidiSongPos
{
	GENERATED_BODY()

public:
	/** Total seconds from bar 1 beat 1. (negative for count-in and/or pickup bars) */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	float SecondsFromBarOne = 0.0f;

	/** Total seconds from the beginning of the musical content (ie. includes all count-in and pickup bars) */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	float SecondsIncludingCountIn = 0.0f;

	/** Time Signature Numerator (Indicates Number of Beats Per Bar) */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	int32 TimeSigNumerator = 0;

	/** Time Signature Numerator (Indicates Subdivision of Bar that counts as 1 beat (4 means Quarter Note = One Beat)) */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	int32 TimeSigDenominator = 0;

	/** Tempo in Quarter Notes Per Minute (BPM) */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	float Tempo = 0.0f;
	/** Total bars from the beginning of the song. */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	float BarsIncludingCountIn = 0.0f;

	/** Total beats from the beginning of the song. */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	float BeatsIncludingCountIn = 0.0f;

	/**
	 * Whether this beat is a Downbeat (beat 1 of the current bar) or otherwise
	 * This value is determined by the authored content
	 * Defaults to "Normal"
	 */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	EMusicalBeatType BeatType = EMusicalBeatType::Normal;

	/** The Musical Timestamp: Bar Number and Beat Number within the bar (Bar 1 Beat 1.0f) */
	UPROPERTY(BlueprintReadOnly, Category = "MidiSongPosition")
	FMusicTimestamp Timestamp;

	FSongSection CurrentSongSection;

	// This version is for use when song maps are unavailable. StartBar can be used if you know the content
	// has, or will have, etc. a pickup or count-in. So for example... 
	// If tempo = 60 bpm, time signature = 4/4, and StartBar = -1 (two bars or count-in, bars -1 and 0), then...
	//     InElapsedMs = 0000.0 --> Seconds = -8, ElapsedBars = 0.00, MusicPosition.Bar = -1, MusicPosition.Beat = 1
	//     InElapsedMs = 1000.0 --> Seconds = -7, ElapsedBars = 0.25, MusicPosition.Bar = -1, MusicPosition.Beat = 2
	//     InElapsedMs = 2000.0 --> Seconds = -6, ElapsedBars = 0.50, MusicPosition.Bar = -1, MusicPosition.Beat = 3
	//     InElapsedMs = 3000.0 --> Seconds = -5, ElapsedBars = 0.75, MusicPosition.Bar = -1, MusicPosition.Beat = 4
	//     InElapsedMs = 4000.0 --> Seconds = -4, ElapsedBars = 1.00, MusicPosition.Bar =  0, MusicPosition.Beat = 1
	//     InElapsedMs = 5000.0 --> Seconds = -3, ElapsedBars = 1.25, MusicPosition.Bar =  0, MusicPosition.Beat = 2
	//     InElapsedMs = 6000.0 --> Seconds = -2, ElapsedBars = 1.50, MusicPosition.Bar =  0, MusicPosition.Beat = 3
	//     InElapsedMs = 7000.0 --> Seconds = -1, ElapsedBars = 1.75, MusicPosition.Bar =  0, MusicPosition.Beat = 4
	//     InElapsedMs = 8000.0 --> Seconds =  0, ElapsedBars = 2.00, MusicPosition.Bar =  1, MusicPosition.Beat = 1
	// etc.
	UE_API void SetByTime(float InElapsedMs, float Bpm, int32 TimeSigNum = 4, int32 TimeSigDenom = 4, int32 StartBar = 1);

	UE_API void SetByTime(float InElapsedMs, const ISongMapEvaluator& Map);

	// Low-level version for midi players / parsers that use the low-level
	// "midi tick system" for advancing song position.
	UE_API void SetByTick(float Tick, const ISongMapEvaluator& Map);

	void Reset()
	{
		SecondsFromBarOne = 0.0f;
		SecondsIncludingCountIn = 0.0f;
		TimeSigNumerator = 0;
		TimeSigDenominator = 0;
		Tempo = 0.0f;
		Timestamp.Reset();
		BarsIncludingCountIn = 0.0f;
		BeatsIncludingCountIn = 0.0f;
		BeatType = EMusicalBeatType::Normal;
		CurrentSongSection = FSongSection();
		IsSet = false;
	}

	/** Tempo converted from Quarter Notes to our Beats, only valid if TimeSigDenominator is set. */
	UE_API float GetBeatsPerMinute() const;

	float GetBeatsPerSecond() const { return GetBeatsPerMinute() / 60.0f; }

	// Has this been set by a system yet, or is it in the default state? 
	bool IsValid() const { return IsSet; }

	// operators
	UE_API bool operator<(const FMidiSongPos& rhs) const;
	UE_API bool operator<=(const FMidiSongPos& rhs) const;
	UE_API bool operator>(const FMidiSongPos& rhs) const;
	UE_API bool operator>=(const FMidiSongPos& rhs) const;
	UE_API bool operator==(const FMidiSongPos& rhs) const;

	static UE_API FMidiSongPos Lerp(const FMidiSongPos& A, const FMidiSongPos& B, float Alpha);

	UE_API void SetByTimeAndTick(float Ms, float Tick, const ISongMapEvaluator& Map);

private:
	// Has anyone explicitly set this, or is this just the defaults.
	bool IsSet = false;
};

#undef UE_API
