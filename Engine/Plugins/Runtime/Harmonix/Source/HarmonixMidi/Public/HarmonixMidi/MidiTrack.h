// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MidiEvent.h"
#include "HarmonixMidi/MidiMsg.h"

#include "MidiTrack.generated.h"

#define UE_API HARMONIXMIDI_API

using FMidiEventList = TArray<FMidiEvent>;
class FMidiWriter;

/**
	* An FMidiTrack is a collection of FMidiEvents in chronological order.  
	* 
	* It can be created dynamically or be the end result of importing a 
	* standard midi file.
	*/
USTRUCT(BlueprintType, Meta = (DisplayName = "MIDI Track"))
struct FMidiTrack
{
	GENERATED_BODY()

public:
	UE_API FMidiTrack();
	UE_API FMidiTrack(const FString& name);

	UE_API bool operator==(const FMidiTrack& Other) const;

	UE_API const FMidiEventList& GetEvents() const;  		// this will ASSERT if events aren't sorted!
	UE_API const FMidiEventList& GetUnsortedEvents() const; // this won't sort

	/** Do not call unless you know what you're doing! */
	UE_API FMidiEventList& GetRawEvents();

	int32 GetNumEvents() const { return (int32)Events.Num(); }
	const FMidiEvent* GetEvent(int32 index) const { return &Events[index]; }

	UE_API void SetName(const FString& InName);
	UE_API const FString* GetName() const;

	UE_API void AddEvent(const FMidiEvent& Event);

	/**
		* This will not move the event's location in the list so that iterators into
		* Events do note get screwed up, BUT it CAN result in the midi events not being 
		* sorted. So if you call this function one or more times you should call Sort()
		* after doing so.
		* @see Sort
		*/
	UE_API void ChangeTick(FMidiEventList::TIterator Iterator, int32 NewTick);

	UE_API void WriteStdMidi(FMidiWriter& writer) const;

	UE_API void Sort();

	void Empty() { Events.Empty(); }
	UE_API void ClearEventsAfter(int32 Tick, bool IncludeTick);
	UE_API void ClearEventsBefore(int32 Tick, bool IncludeTick);

	int32 GetPrimaryMidiChannel() const { return PrimaryMidiChannel; }

	UE_API int32 CopyEvents(FMidiTrack& SourceTrack, int32 FromTick, int32 ThruTick, int32 TickOffset = 0,
		bool Clear = true, int32 MinNote = 0, int32 MaxNote = 127, int32 NoteTranspose = 0, bool FilterTrackName = true);

	const FString& GetTextAtIndex(int32 Index) const
	{
		return Strings[Index];
	}

	int32 AddText(const FString& Str)
	{
		int32 Index = INDEX_NONE;
		for (int32 Idx = 0; Idx < Strings.Num(); ++Idx)
		{
			if (Strings[Idx].Equals(Str, ESearchCase::CaseSensitive))
			{
				Index = Idx;
				break;
			}
		}
		
		if (Index != INDEX_NONE)
		{
			return Index;
		}
		
		return Strings.Add(Str);
	}

	FMidiTextRepository* GetTextRepository() { return &Strings; }
	const FMidiTextRepository* GetTextRepository() const { return &Strings; }
	FString GetTextForMsg(const FMidiMsg& Message) const { check(Message.MsgType() == FMidiMsg::EType::Text); return GetTextAtIndex(Message.GetTextIndex()); }

	UE_API SIZE_T GetAllocatedSize() const;

private:
	UPROPERTY()
	TArray<FMidiEvent> Events; // All the midi events
	UPROPERTY()
	bool Sorted;               // Are the events sorted.
	UPROPERTY()
	int32 PrimaryMidiChannel;  // The midi channel of the first event on the track!
	UPROPERTY()
	TArray<FString> Strings;
};

#undef UE_API
