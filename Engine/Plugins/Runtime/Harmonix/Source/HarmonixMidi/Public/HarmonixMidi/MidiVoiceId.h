// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/MidiMsg.h"

#define UE_API HARMONIXMIDI_API

class FMidiVoiceGeneratorBase
{
public:
	UE_API FMidiVoiceGeneratorBase();
	uint32 GetIdBits() const { return IdBits; }

	static constexpr uint32 IdWidth = 20;

private:
	uint32 IdBits;
	static UE_API FCriticalSection GeneratorIdLock;
	static UE_API uint32 NextGeneratorId;
};

class FMidiVoiceId
{
public:
	FMidiVoiceId()
		: Id(0)
	{}

	FMidiVoiceId(const uint32 GeneratorId, const FMidiMsg& FromMessage)
	{
		static_assert(FMidiVoiceGeneratorBase::IdWidth <= 20);
		uint8 NotePart = (FromMessage.IsNoteOn() || FromMessage.IsNoteOff()) ? FromMessage.GetStdData1() : 0;
		uint8 ChPart = FromMessage.IsStd() && !FromMessage.IsSystemMessage() ? FromMessage.GetStdChannel() : 0;
		Id = GeneratorId | ((uint32)(ChPart & 0xF) << 8) | (uint32)NotePart;
	}

	FMidiVoiceId(const FMidiVoiceId& Other)
		: Id (Other.Id)
	{}

	FMidiVoiceId(FMidiVoiceId&& Other) noexcept
		: Id(Other.Id)
	{
		Other.Id = 0;
	}

	void ReassignGenerator(uint32 GeneratorId)
	{
		uint8 Ch;
		uint8 Note;
		GetChannelAndNote(Ch, Note);
		Id = (GeneratorId | ((uint32)(Ch & 0xF) << 8) | (uint32)Note);
	}

	void GetGeneratorChannelAndNote(uint32& GeneratorOut, uint8& ChOut, uint8& NoteOut) const
	{
		GeneratorOut = Id & 0xFFFFFC00;
		GetChannelAndNote(ChOut, NoteOut);
	}

	void GetChannelAndNote(uint8& ChOut, uint8& NoteOut) const
	{
		ChOut = (Id >> 8) & 0xF;
		NoteOut = Id & 0xFF;
	}

	uint32 GetGeneratorId() const
	{
		return Id & 0xFFFFFC00;
	}

	FMidiVoiceId& operator=(const FMidiVoiceId& Other)
	{
		Id = Other.Id;
		return *this;
	}

	FMidiVoiceId& operator=(uint32 RawId)
	{
		Id = RawId;
		return *this;
	}

	operator bool() const
	{
		return Id != 0;
	}

	bool operator==(const FMidiVoiceId& Other) const
	{
		return Other.Id == Id;
	}

	bool operator==(uint32 RawId) const
	{
		return RawId == Id;
	}

	static FMidiVoiceId Any()
	{
		return FMidiVoiceId(std::numeric_limits<uint32>::max());
	}

	static FMidiVoiceId None()
	{
		return FMidiVoiceId();
	}

	friend uint32 GetTypeHash(const FMidiVoiceId& VoiceId);

private:
	FMidiVoiceId(uint32 RawId)
		: Id(RawId)
	{}

	uint32 Id;
};

FORCEINLINE uint32 GetTypeHash(const FMidiVoiceId& VoiceId)
{
	return VoiceId.Id;
}

static_assert(sizeof(FMidiVoiceId) == 4);

#undef UE_API
