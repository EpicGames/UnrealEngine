// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Serialization/BufferWriter.h"
#include "HarmonixMidi/MidiConstants.h"

#define UE_API HARMONIXMIDI_API

/**
	* Writes MIDI data to an SMF (Standard MIDI Format) file.
	*
	* Does not support all meta-events (no SysEx events, for example). Only writes Format 1 files.
	* Does not take advantage of running status.
	* 
	* You rarely have to interact with this class, as FMidiFile uses it internally when
	* you use its SaveStdMidiFile functions.
	* @see FMidiFile
	*/
class FMidiWriter
{
public:
	UE_API FMidiWriter(FArchive& OutputArhive, int32 TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt);
	UE_API ~FMidiWriter();

	/** Flushes data to file; also called by dtor */
	UE_API void Close();

	UE_API void EndOfTrack();

	/** Writes a standard 1- or 2-byte MIDI message */
	UE_API void MidiMessage(int32 Tick, uint8 Status, uint8 Data1, uint8 Data2);

	/**
		* Writes a Tempo Change meta-event:
		* Tempo is in microseconds per quarter-note */
	UE_API void Tempo(int32 Tick, int32 Tempo);

	/**
		* Writes Text, Copyright, TrackName, InstName, Lyric, Marker, CuePoint meta-event:
		* "type" is the type of meta-event (constants defined in
		* MidiConstants.h)
		*/
	UE_API void Text(int32 Tick, const TCHAR* Str, uint8 Type);

	/**
		* Writes a Time Signature meta-event:
		* time signature is numer/denom
		*/
	UE_API void TimeSignature(int32 Tick, int32 Numerator, int32 Denominator);

private:
	FArchive& Archive;

	TArray<TSharedPtr<FBufferWriter>> TrackWriters; // one per track
	TSharedPtr<FBufferWriter> CurrentWriter;
	int32 TicksPerQuarterNote;
	int32 CurTick;
	bool  Closed;

	void ProcessTick(int32 Tick);
	void WriteFileHeader();
	void WriteTrack(FBufferWriter& OFile);
	void Write();

	FMidiWriter(const FMidiWriter&);
	FMidiWriter& operator=(const FMidiWriter&);
};

#undef UE_API
