// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MusicMapBase.generated.h"

/**
 * FMusicMapPointBase represents a span in a midi file defined by a start tick and a length.
 * This is a base class for many types of map points we recognize in midi files (e.g. chord,
 * section, tempo, pulse, etc.)
 */
USTRUCT(BlueprintType)
struct FMusicMapTimespanBase
{
	GENERATED_BODY();
public:
	FMusicMapTimespanBase()
		: StartTick(0)
		, LengthTicks(1)
	{}
	FMusicMapTimespanBase(int32 InStartTick, int32 InLengthTicks = 1)
		: StartTick(InStartTick)
		, LengthTicks(InLengthTicks)
	{}

	UPROPERTY(BlueprintReadOnly, Category = "Timespan")
	int32 StartTick;
	UPROPERTY(BlueprintReadOnly, Category = "Timespan")
	int32 LengthTicks;

	FORCEINLINE int32 EndTick() const { return StartTick + LengthTicks; };
	FORCEINLINE bool ContainsTick(float Tick) const { return Tick >= StartTick && Tick < StartTick + LengthTicks; }
	FORCEINLINE float TickInPoint(float Tick) const { return FMath::Clamp(Tick - StartTick, 0, LengthTicks - 1); }
	FORCEINLINE float Progress(float Tick) const { return LengthTicks < 2 ? 1.0f : TickInPoint(Tick) / (LengthTicks - 1); }
	// friend FArchive& operator<<(FArchive& Archive, FMusicMapTimespanBase& Info);
	struct LessThan
	{
		bool operator()(const FMusicMapTimespanBase& Point, float Tick) const { return Point.StartTick < Tick; }
		bool operator()(float Tick, const FMusicMapTimespanBase& Point) const { return Tick < Point.StartTick; }
		bool operator()(const FMusicMapTimespanBase& Point1, const FMusicMapTimespanBase& Point2) const { return Point1.StartTick < Point2.StartTick; }
	};
};

/*
USTRUCT()
struct HARMONIXMIDI_API FMusicMapBaseInterface
{
	GENERATED_BODY()
public:
	virtual int32 GetPointIndexForTick(int32 Tick) const = 0;
	virtual float GetFractionalPointForTick(int32 Tick) const = 0;
	virtual float GetFractionalTickForFractionalPoint(float Point) const = 0;
	virtual void Finalize(int32 LastTick = 0) = 0;
	virtual void RemovePointsOnAndAfterTick(int32 Tick) = 0;

	template<typename... T0toN>
	void AddPoint(T0toN... constructor_args, int32 StartTick, bool SortNow = true, int32 LengthTicks = 1);
	template<typename T>
	const T* GetPointInfoForTick(int32 Tick) const;
};
*/

/**
	* This template class is a base for many different types of "maps" of musical position/regions
	* within a midi file. 
	* 
	* @tparam T This must be a class derived from FMusicMapPointBase.
	* @tparam DEFINED_REGIONS If false, the map point that corresponds to a given tick is simply the nearest 
	* map point PRESEDING the tick. If true, a tick must fall between a point's start and end to be considered
	* to be "in" that point. So consider GetPointIndexForTick. If CONTIGUOUS_REGIONS is true, it will return -1 
	* if the specified tick does not fall IN a map point. If false, the specified tick only need fall after some
	* map point to be considered part of that point.
	*/
//template<typename T, bool DEFINED_REGIONS>
struct FMusicMapUtl
{
public:
	template<typename T>
	static int32 GetPointIndexForTick(const TArray<T>& Points, int32 Tick)
	{
		if (Points.IsEmpty() || Tick < Points[0].StartTick || (T::DefinedAsRegions && Tick >= Points.Last().EndTick()))
		{
			return -1;
		}

		int32 Index = Algo::UpperBound(Points, Tick, FMusicMapTimespanBase::LessThan()) - 1;
		if (T::DefinedAsRegions && Tick >= Points[Index].EndTick())
		{
			// fell in a gap...
			return -1; 
		}

		return Index;
	}

	template<typename T>
	static float GetFractionalPointForTick(const TArray<T>& Points, float Tick)
	{
		int32 Index = GetPointIndexForTick(Points, int32(Tick));
		if (Index == -1)
		{
			return 0.0f;
		}
		return Index + Points[Index].Progress(Tick);
	}

	template<typename T>
	static const T* GetPointInfoForTick(const TArray<T>& Points, int32 Tick)
	{
		int32 Index = GetPointIndexForTick(Points, Tick);
		return Index < 0 ? nullptr : &Points[Index];
	}

	template<typename T>
	static float GetFractionalTickForFractionalPoint(const TArray<T>& Points, float Point)
	{
		int32 Index = FMath::FloorToInt(Point);
		if (!Points.IsValidIndex(Index))
		{
			return 0.0f;
		}
		return Points[Index].StartTick + Points[Index].LengthTicks * FMath::Frac(Point);
	}

	template<typename T>
	static void Finalize(TArray<T>& Points, int32 LastTick = 0)
	{
		Points.StableSort(FMusicMapTimespanBase::LessThan());
		if (T::DefinedAsRegions)
		{
			for (int32 i = 0; i < Points.Num(); ++i)
			{
				if (i < Points.Num() - 1)
				{
					Points[i].LengthTicks = Points[i + 1].StartTick - Points[i].StartTick;
				}
				else
				{
					Points[i].LengthTicks = LastTick - Points[i].StartTick;
				}
			}
		}
	}

	template<typename T, typename... T0toN>
	static void AddPoint(T0toN... constructor_args, TArray<T>& Points, int32 StartTick, bool SortNow = true, int32 LengthTicks = 1)
	{
		int32 ExistingIndex = GetPointIndexForTick(Points, StartTick);
		if (ExistingIndex != -1 && Points[ExistingIndex].StartTick == StartTick)
		{
			Points[ExistingIndex] = T(constructor_args..., StartTick, LengthTicks);
			return;
		}

		Points.Emplace(constructor_args..., StartTick, LengthTicks);
		if (SortNow)
		{
			Points.StableSort(FMusicMapTimespanBase::LessThan());
		}
	}

	template<typename T>
	static void RemovePointsOnAndAfterTick(TArray<T>& Points, int32 Tick)
	{
		if (Points.IsEmpty())
		{
			return;
		}
		while (Points.Num() > 0 && Tick <= Points.Last().StartTick)
		{
			Points.SetNum(Points.Num() - 1);
		}
	}

	template<typename T>
	static void Copy(const TArray<T>& FromPoints, TArray<T>& ToPoints, int32 StartTick, int32 EndTick)
	{
		if (FromPoints.IsEmpty() || EndTick < FromPoints[0].StartTick || StartTick > FromPoints.Last().StartTick)
		{
			return;
		}

		for (int i = 0; i < FromPoints.Num(); ++i)
		{
			if (EndTick != -1 && FromPoints[i].StartTick > EndTick)
			{
				break;
			}

			if (FromPoints[i].StartTick >= StartTick)
			{
				ToPoints.Add(FromPoints[i]);
			}
		}
	}

};

/*
template <class T, bool DEFINED_REGIONS>
template <typename... T0toN>
static void FMusicMapUtl::AddPoint(T0toN... constructor_args, TArray<T>& Points, int32 InStartTick, bool SortNow, int32 InLengthTicks)
*/
