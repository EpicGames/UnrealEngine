// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MusicTimeSpan.generated.h"

#define UE_API HARMONIXMIDI_API

struct ISongMapEvaluator;
struct FMidiSongPos;

UENUM()
enum class EMusicTimeSpanOffsetUnits : uint8
{
	Ms						UMETA(DisplayName = "Milliseconds", EnumDefault),
	Bars					UMETA(DisplayName = "Bars", ToolTip = "(dependent on time signature)"),
	Beats					UMETA(DisplayName = "Beats", ToolTip = "(dependent on time signature and Pulse Override)"),
	ThirtySecondNotes		UMETA(DisplayName = "1/32s"),
	SixteenthNotes			UMETA(DisplayName = "1/16s"),
	EighthNotes				UMETA(DisplayName = "1/8s"),
	QuarterNotes			UMETA(DisplayName = "1/4s"),
	HalfNotes				UMETA(DisplayName = "Halfs"),
	WholeNotes				UMETA(DisplayName = "Wholes"),
	DottedSixteenthNotes	UMETA(DisplayName = "(dotted) 1/16s"),
	DottedEighthNotes		UMETA(DisplayName = "(dotted) 1/8s"),
	DottedQuarterNotes		UMETA(DisplayName = "(dotted) 1/4s"),
	DottedHalfNotes			UMETA(DisplayName = "(dotted) Halfs"),
	DottedWholeNotes		UMETA(DisplayName = "(dotted) Wholes"),
	SixteenthNoteTriplets	UMETA(DisplayName = "1/16 (triplets)"),
	EighthNoteTriplets		UMETA(DisplayName = "1/8 (triplets)"),
	QuarterNoteTriplets		UMETA(DisplayName = "1/4 (triplets)"),
	HalfNoteTriplets		UMETA(DisplayName = "1/2 (triplets)"),
};

UENUM()
enum class EMusicTimeSpanLengthUnits : uint8
{
	Bars				  UMETA(DisplayName = "Bars", ToolTip = "(dependent on time signature)"),
	Beats				  UMETA(DisplayName = "Beats", ToolTip = "(dependent on time signature and Pulse Override)"),
	ThirtySecondNotes	  UMETA(DisplayName = "1/32s"),
	SixteenthNotes		  UMETA(DisplayName = "1/16s"),
	EighthNotes			  UMETA(DisplayName = "1/8s"),
	QuarterNotes		  UMETA(DisplayName = "1/4s"),
	HalfNotes			  UMETA(DisplayName = "Halfs"),
	WholeNotes			  UMETA(DisplayName = "Wholes"),
	DottedSixteenthNotes  UMETA(DisplayName = "(dotted) 1/16s"),
	DottedEighthNotes	  UMETA(DisplayName = "(dotted) 1/8s"),
	DottedQuarterNotes	  UMETA(DisplayName = "(dotted) 1/4s"),
	DottedHalfNotes		  UMETA(DisplayName = "(dotted) Halfs"),
	DottedWholeNotes	  UMETA(DisplayName = "(dotted) Wholes"),
	SixteenthNoteTriplets UMETA(DisplayName = "1/16 (triplets)"),
	EighthNoteTriplets	  UMETA(DisplayName = "1/8 (triplets)"),
	QuarterNoteTriplets	  UMETA(DisplayName = "1/4 (triplets)"),
	HalfNoteTriplets	  UMETA(DisplayName = "1/2 (triplets)"),
};

namespace MusicalTimeSpan
{
	UE_API int32 CalculateMusicalSpanLengthTicks(EMusicTimeSpanLengthUnits Unit, const ISongMapEvaluator& Maps, int32 AtTick);
}

USTRUCT(BlueprintType, Category = "HarmonixMidi", meta = (ScriptName = MusicalTimeSpan))
struct FMusicalTimeSpan
{
	GENERATED_BODY()
public:

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "HarmonixMidi")
	EMusicTimeSpanLengthUnits LengthUnits = EMusicTimeSpanLengthUnits::Bars;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "HarmonixMidi")
	int Length = 1;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "HarmonixMidi")
	EMusicTimeSpanOffsetUnits OffsetUnits = EMusicTimeSpanOffsetUnits::Ms;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "HarmonixMidi")
	int Offset = 0;

	FMusicalTimeSpan()
	{}

	FMusicalTimeSpan(EMusicTimeSpanLengthUnits InLengthUnits, float InLength, EMusicTimeSpanOffsetUnits InOffsetUnits, float InOffset)
		: LengthUnits(InLengthUnits)
		, Length(InLength)
		, OffsetUnits(InOffsetUnits)
		, Offset(InOffset)
	{}

	UE_API float CalcPositionInSpan(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const;
	UE_API float CalcPositionInSpan(float Ms, const ISongMapEvaluator& Maps) const;

private:
	UE_API float CalculateEnclosingVariableSizeSpanExtents(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const;
	UE_API float CalculateEnclosingFixedSizeSpanExtents(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const;

	UE_API float CalcPositionInSpanWithOffset(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const;
	UE_API float CalcPositionInSpanNoOffset(const FMidiSongPos& Position, const ISongMapEvaluator& Maps) const;
};

#undef UE_API
