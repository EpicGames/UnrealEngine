// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HarmonixMidi/BarMap.h"

struct FBarMap;

namespace Midi
{
	/**
	 * Composers, Musicians, Sound Designers, etc. (and their software tools) speak about 
	 * musical time in terms of positions and durations.
	 * 
	 * Positions are specified as 1 based numbers for bars and beats. Bar 1, 
	 * beat 1.0 is the first moment in a song. 
	 * 
	 * Durations are 0 based. A musical note might be 0 Bars and 0.33 beats long
	 * (a eighth note triplet if beats are quarter notes). 
	 * 
	 * When we are parsing a string that specifies a musical time, or converting a midi tick
	 * to musical time string we want to maintain this convention. So the parse and format
	 * functions below need to know whether they are working on a position or a duration.
	 */
	enum class EMusicTimeStringFormat
	{
		Position,
		Duration
	};

	/**
	 * Converts a music time string to a bar and a beat.
	 *
	 * @param Str A string like "3.2.25" or "3:2.25" If 3 numbers are present:
	 * (1) is assumed to be the bar, (2&3) is assumed to be the fractional beat. 
	 * If only 1 or 2 are present it is assumed to specify a beat. 
	 * 
	 * @param Format Is the string a music position or a music duration? If position
	 * 1 will be subtracted from the values found in the string.
	 *
	 * @param OutBar The found bar number. NOTE: 0-based!
	 * 
	 * @param OutBeat The found beat number. NOTE: 0-based!
	 *
	 * @see ParseBarBeatDuration
	 */
	FMusicTimestamp HARMONIXMIDI_API ParseMusicTimestamp(const FString& Str, EMusicTimeStringFormat Format);

	/**
	 * converts a bar and beat position to a string... "Bar.Beat.BeatFraction"
	 *
	 * @param Bar
	 * @param Beat
	 * @param Format If 'Position', 1 will be added to both the bar and beat values.
	 */
	FString HARMONIXMIDI_API FormatBarBeat(const FMusicTimestamp& Timestamp, EMusicTimeStringFormat Format);

	/**
	 * Converts a "midi position" string to a low-level midi tick. given
	 * the supplied time signature and the current value of the kTicksPerQuarterNote
	 * constant in MidiConstants.h
	 *
	 * @param Str A string like "3.2.25" or "3:2.25" If 3 numbers are present
	 * (1) is assumed to be the bar, (2-3) is assumed to be the fractional beat.
	 * If only 1 or 2 are present it is assumed to specify a beat. THE NUMBERS
	 * SPECIFYING A POSITION ARE ASSUMED TO BE 1-BASED!
	 * 
	 * @param Format Is the string a music position or a music duration? If position
	 * 1 will be subtracted from the values found in the string.
	 *
	 * @param TimeSignatureNumerator
	 * 
	 * @param TimeSignatureDenominator
	 * 
	 * @see BarBeatDurationStringtoMidiTick
	 */
	int32 HARMONIXMIDI_API BarBeatStringtoMidiTick(const FString& Str, const FBarMap* BarMap, EMusicTimeStringFormat Format);

	/**
	 * Given a Tick and a FBarMap it will return a formated string like 
	 * "3.2.25". In this string the first number is the bar, the second
	 * and third are the fractional beat (Reaper style). 
	 * 
	 * @param InTick "raw" midi tick
	 * @param Map Bar map to use for translation.
	 * @param Format If 'Position', 1 will be added to both the bar and beat values.
	 */
	FString HARMONIXMIDI_API MidiTickFormat(int32 InTick, const FBarMap* Map, EMusicTimeStringFormat Format);
}
