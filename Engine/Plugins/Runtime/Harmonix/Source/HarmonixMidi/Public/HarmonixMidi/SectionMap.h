// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MusicMapBase.h"
#include "HarmonixMidi/MidiConstants.h"
#include <limits>

#include "SectionMap.generated.h"

#define UE_API HARMONIXMIDI_API

/**
 * A section in a piece of music has a name, a starting point, and a length
 */
USTRUCT(BlueprintType)
struct FSongSection : public FMusicMapTimespanBase
{
	GENERATED_BODY()

public:
	static constexpr bool DefinedAsRegions = true;

	FSongSection() {}

	FSongSection(const FString& InName, int32 InStartTick, int32 InLengthTicks = 1)
		: FMusicMapTimespanBase(InStartTick, InLengthTicks)
		, Name(InName)
	{}

	bool operator==(const FSongSection& Other) const
	{
		return Name == Other.Name;
	}

	UPROPERTY(BlueprintReadOnly, Category = "SongSection")
	FString Name;
};

/**
 * A map of sections in a piece of music
 */
USTRUCT()
struct FSectionMap
{
	GENERATED_BODY()

public:
	FSectionMap()
		: TicksPerQuarterNote(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt)
	{}
	UE_API bool operator==(const FSectionMap& Other) const;

	UE_API void Finalize(int32 LastTick);

	UE_API void Empty();
	UE_API void Copy(const FSectionMap& Other, int32 StartTick = 0, int32 EndTick = std::numeric_limits<int32>::max());
	UE_API bool IsEmpty() const;

	/** Called by the midi file importer before map points are added to this map */
	void SetTicksPerQuarterNote(int32 InTicksPerQuarterNote)
	{
		TicksPerQuarterNote = InTicksPerQuarterNote;
	}

	UE_API bool AddSection(const FString& Name, int32 StartTick, int32 LengthTicks, bool SortNow = true);

	UE_API int32 TickToSectionIndex(int32 Tick) const;
	UE_API const FSongSection* TickToSection(int32 Tick) const;
	UE_API int32 GetSectionStartTick(const FString& Name) const;
	UE_API int32 GetSectionStartTick(int32 SectionIndex) const;

	const TArray<FSongSection>& GetSections() const { return Points; }
	int32 GetNumSections() const { return Points.Num(); }
	UE_API const FSongSection* GetSection(int32 SectionIndex) const;
	UE_API FString GetSectionName(int32 SectionIndex) const;
	UE_API FString GetSectionNameAtTick(int32 Tick) const;
	UE_API void  GetSectionNames(TArray<FString>& Names) const;
	UE_API int32   FindSectionIndex(const FString& Name) const;
	UE_API const FSongSection* FindSectionInfo(const FString& Name) const;

protected:
	UPROPERTY()
	int32 TicksPerQuarterNote;
	UPROPERTY()
	TArray<FSongSection> Points;
};

#undef UE_API
