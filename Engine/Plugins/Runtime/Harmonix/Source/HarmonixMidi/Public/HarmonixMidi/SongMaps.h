// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/TempoMap.h"
#include "HarmonixMidi/BarMap.h"
#include "HarmonixMidi/BeatMap.h"
#include "HarmonixMidi/ChordMap.h"
#include "HarmonixMidi/SectionMap.h"
#include "Misc/MusicalTime.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "FrameBasedMusicMap.h"
#include <limits>

#include "SongMaps.generated.h"

#define UE_API HARMONIXMIDI_API

class IMidiReader;
class FStdMidiFileReader;
class FSongMapReceiver;
class UMidiFile;
struct FMidiFileData;

namespace HarmonixMetasound 
{
	class FMidiClock;
}

UENUM(BlueprintType)
enum class EMidiFileQuantizeDirection : uint8
{
	Nearest = 0,
	Up = 1,
	Down = 2
};

UENUM(BlueprintType)
enum class EMidiClockSubdivisionQuantization : uint8
{
	Bar = static_cast<uint8>(EQuartzCommandQuantization::Bar),
	Beat = static_cast<uint8>(EQuartzCommandQuantization::Beat),
	ThirtySecondNote = static_cast<uint8>(EQuartzCommandQuantization::ThirtySecondNote),
	SixteenthNote = static_cast<uint8>(EQuartzCommandQuantization::SixteenthNote),
	EighthNote = static_cast<uint8>(EQuartzCommandQuantization::EighthNote),
	QuarterNote = static_cast<uint8>(EQuartzCommandQuantization::QuarterNote),
	HalfNote = static_cast<uint8>(EQuartzCommandQuantization::HalfNote),
	WholeNote = static_cast<uint8>(EQuartzCommandQuantization::WholeNote),
	DottedSixteenthNote = static_cast<uint8>(EQuartzCommandQuantization::DottedSixteenthNote),
	DottedEighthNote = static_cast<uint8>(EQuartzCommandQuantization::DottedEighthNote),
	DottedQuarterNote = static_cast<uint8>(EQuartzCommandQuantization::DottedQuarterNote),
	DottedHalfNote = static_cast<uint8>(EQuartzCommandQuantization::DottedHalfNote),
	DottedWholeNote = static_cast<uint8>(EQuartzCommandQuantization::DottedWholeNote),
	SixteenthNoteTriplet = static_cast<uint8>(EQuartzCommandQuantization::SixteenthNoteTriplet),
	EighthNoteTriplet = static_cast<uint8>(EQuartzCommandQuantization::EighthNoteTriplet),
	QuarterNoteTriplet = static_cast<uint8>(EQuartzCommandQuantization::QuarterNoteTriplet),
	HalfNoteTriplet = static_cast<uint8>(EQuartzCommandQuantization::HalfNoteTriplet),
	None = static_cast<uint8>(EQuartzCommandQuantization::None)
};

USTRUCT()
struct FSongLengthData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 LengthTicks = 0;
	UPROPERTY()
	float LengthFractionalBars = 0.0f;
	UPROPERTY()
	int32 LastTick = 0;

	bool operator==(const FSongLengthData& Other) const
	{
		return	LengthTicks == Other.LengthTicks &&
				LengthFractionalBars == Other.LengthFractionalBars && 
				LastTick == Other.LastTick;
	}
};

struct ISongMapEvaluator
{
public:
	UE_API int32 GetTicksPerQuarterNote() const;
	UE_API float TickToMs(float Tick) const;
	UE_API float MsToTick(float Ms) const;
	UE_API float GetCountInSeconds() const;

	// tempo
	UE_API const FTempoInfoPoint* GetTempoInfoForMs(float Ms) const;
	UE_API const FTempoInfoPoint* GetTempoInfoForTick(int32 Tick) const;
	UE_API int32 GetTempoPointIndexForTick(int32 Tick) const;
	UE_API const FTempoInfoPoint* GetTempoInfoPoint(int32 PointIndex) const;
	UE_API int32 GetNumTempoChanges() const;
	UE_API int32 GetTempoChangePointTick(int32 PointIndex) const;
	UE_API float GetTempoAtMs(float Ms) const;
	UE_API float GetTempoAtTick(int32 Tick) const;
	UE_API bool TempoMapIsEmpty() const;

	// beats
	UE_API const FBeatMapPoint* GetBeatAtMs(float Ms) const;
	UE_API float GetMsAtBeat(float Beat) const; // Beat is zero indexed.
	UE_API const FBeatMapPoint* GetBeatAtTick(int32 Tick) const;
	UE_API float GetMsPerBeatAtMs(float Ms) const;
	UE_API float GetMsPerBeatAtTick(int32 Tick) const;
	UE_API float GetFractionalBeatAtMs(float Ms) const; // Returns ONE indexed beat, beware.
	UE_API float GetFractionalBeatAtTick(float Tick) const; // Returns ONE indexed beat, beware.
	UE_API int32 GetBeatIndexAtMs(float Ms) const;
	UE_API int32 GetBeatIndexAtTick(int32 Tick) const;
	UE_API EMusicalBeatType GetBeatTypeAtMs(float Ms) const;
	UE_API EMusicalBeatType GetBeatTypeAtTick(int32 Tick) const;
	UE_API const FBeatMapPoint* GetBeatPointInfoAtTick(int32 Tick, int32* PointIndex = nullptr) const;

	UE_API float GetBeatInPulseBarAtMs(float Ms) const;
	UE_API float GetBeatInPulseBarAtTick(float Tick) const;
	UE_API int32 GetNumBeatsInPulseBarAtMs(float Ms) const;
	UE_API int32 GetNumBeatsInPulseBarAtTick(int32 Tick) const;
	UE_API bool BeatMapIsEmpty() const;

	// bars
	UE_API int32 GetStartBar() const;
	UE_API int32 GetNumTimeSignatureChanges() const;
	UE_API const FTimeSignature* GetTimeSignatureAtMs(float Ms) const;
	UE_API const FTimeSignature* GetTimeSignatureAtTick(int32 Tick) const;
	UE_API int32 GetTimeSignaturePointIndexForTick(int32 Tick) const;
	UE_API const FTimeSignature* GetTimeSignatureAtBar(int32 Bar) const;
	UE_API const FTimeSignaturePoint* GetTimeSignaturePointAtTick(int32 Tick) const;
	UE_API const FTimeSignaturePoint* GetTimeSignaturePoint(int32 PointIndex) const;
	UE_API int32 GetTimeSignatureChangePointTick(int32 PointIndex);
	UE_API float GetBarIncludingCountInAtMs(float Ms) const;
	UE_API float GetBarIncludingCountInAtTick(float Tick) const;
	UE_API float GetMsAtFractionalBarIncludingCountIn(float BarIncludingCountIn) const; // BarIncludingCountIn is zero based
	UE_API float GetMsPerBarAtMs(float Ms) const;
	UE_API float GetMsPerBarAtTick(float Tick) const;
	UE_API bool BarMapIsEmpty() const;
	UE_API FMusicTimestamp TickToMusicTimestamp(float Tick, int32* OutBeatsPerBar = nullptr) const;
	UE_API int32 BarIncludingCountInToTick(int32 BarIndex, int32* OutBeatsPerBar = nullptr, int32* OutTicksPerBeat = nullptr) const;
	UE_API int32 BarBeatTickIncludingCountInToTick(int32 BarIndex, int32 BeatInBar, int32 TickInBeat) const;
	UE_API float FractionalBarIncludingCountInToTick(float FractionalBarIndex) const;
	UE_API float FractionalBeatIncludingCountInToTick(float FractionalBeatIndex) const;
	UE_API int32 TickToBarIncludingCountIn(int32 Tick) const;
	UE_API float TickToFractionalBarIncludingCountIn(float Tick) const;
	UE_API void TickToBarBeatTickIncludingCountIn(int32 RawTick, int32& OutBarIndex, int32& OutBeatInBar, int32& OutTickIndexInBeat, int32* OutBeatsPerBar = nullptr, int32* OutTicksPerBeat = nullptr) const;
	UE_API int32 CalculateMidiTick(const FMusicTimestamp& Timestamp, const EMidiClockSubdivisionQuantization Quantize) const;
	UE_API int32 SubdivisionToMidiTicks(const EMidiClockSubdivisionQuantization Division, const int32 AtTick) const;
	UE_API float MusicTimestampToTick(const FMusicTimestamp& Timestamp) const;
	UE_API int32 MusicTimestampBarToTick(int32 BarNumber, int32* OutBeatsPerBar = nullptr, int32* OutTicksPerBeat = nullptr) const;

	// sections
	UE_API const TArray<FSongSection>& GetSections() const;
	UE_API int32 GetNumSections() const;
	UE_API float GetSectionStartMsAtMs(float Ms) const;
	UE_API float GetSectionEndMsAtMs(float Ms) const;
	UE_API const FSongSection* GetSectionAtMs(float Ms) const;
	UE_API const FSongSection* GetSectionAtTick(int32 Tick) const;
	UE_API int32 GetSectionIndexAtTick(int32 Tick) const;
	UE_API const FSongSection* GetSectionWithName(const FString& Name) const;
	UE_API FString GetSectionNameAtMs(float Ms) const;
	UE_API FString GetSectionNameAtTick(int32 Tick) const;
	UE_API float GetSectionLengthMsAtMs(float Ms) const;
	UE_API float GetSectionLengthMsAtTick(int32 Tick) const;
	UE_API bool SectionMapIsEmpty() const;

	// chords
	UE_API const FChordMapPoint* GetChordAtMs(float Ms) const;
	UE_API const FChordMapPoint* GetChordAtTick(int32 Tick) const;
	UE_API FName GetChordNameAtMs(float Ms) const;
	UE_API FName GetChordNameAtTick(int32 Tick) const;
	UE_API float GetChordLengthMsAtMs(float Ms) const;
	UE_API float GetChordLengthMsAtTick(int32 Tick) const;
	UE_API bool ChordMapIsEmpty() const;

	// length
	UE_API float GetSongLengthMs() const;
	UE_API int32 GetSongLengthBeats() const;
	UE_API float GetSongLengthFractionalBars() const;
	UE_API bool LengthIsAPerfectSubdivision() const;
	UE_API FString GetSongLengthString() const;

	UE_API int32 QuantizeTickToAnyNearestSubdivision(int32 InTick, EMidiFileQuantizeDirection Direction, EMidiClockSubdivisionQuantization& Division) const;
	UE_API int32 QuantizeTickToNearestSubdivision(int32 InTick, EMidiFileQuantizeDirection Direction, EMidiClockSubdivisionQuantization Division) const;
	UE_API void GetTicksForNearestSubdivision(int32 InTick, EMidiClockSubdivisionQuantization Division, int32& LowerTick, int32& UpperTick) const;

	virtual const FSongLengthData& GetSongLengthData() const = 0;
	
	// engine level music types
	UE_API FMusicalTime GetMusicalTimeAtSeconds(double Seconds) const;
	UE_API FMusicalTime GetMusicalTimeAtFractionalBar(float Bars) const;

protected:
	friend struct FSongMaps;
	friend struct FSongMapsWithAlternateTempoSource;
	virtual const FTempoMap& GetTempoMap() const = 0;
	virtual const FBeatMap& GetBeatMap() const = 0;
	virtual const FBarMap& GetBarMap() const = 0;
	virtual const FSectionMap& GetSectionMap() const = 0;
	virtual const FChordProgressionMap& GetChordMap() const = 0;
};


/**
 * FSongMaps encapsulates a number of other musical/midi map types
 * that are very useful for musical gameplay and interactivity. 
 * 
 * With this class and the current playback position of a piece of music you can
 * do things like determine the current Bar | Beat | Tick, song section, tempo,
 * chord, etc.
 */
USTRUCT(BlueprintType)
struct FSongMaps 
#if CPP
	: public ISongMapEvaluator
#endif
{
	GENERATED_BODY()

public:
	UE_API FSongMaps();
	UE_API FSongMaps(float Bpm, int32 TimeSigNumerator, int32 TimeSigDenominator);
	UE_API FSongMaps(const ISongMapEvaluator& Other);
	virtual ~FSongMaps() = default;

	UE_API bool operator==(const FSongMaps& Other) const;

	UE_API void Init(int32 InTicksPerQuarterNote);
	UE_API void Copy(const ISongMapEvaluator& Other, int32 StartTick = 0, int32 EndTick = std::numeric_limits<int32>::max());

	// For importing...
	UE_API bool LoadFromStdMidiFile(const FString& FilePath);
	UE_API bool LoadFromStdMidiFile(void* Buffer, int32 BufferSize, const FString& Filename);
	UE_API bool LoadFromStdMidiFile(TSharedPtr<FArchive> Archive, const FString& Filename);

	//** BEGIN Support for IMusicMapSource
	UE_API void FillInFrameBasedMusicMap(UFrameBasedMusicMap* Map) const;
	//** END Support for IMusicMapSource

	// tracks
	TArray<FString>& GetTrackNames() { return TrackNames; }
	const TArray<FString>& GetTrackNames() const { return TrackNames; }
	UE_API FString GetTrackName(int32 Index) const;
	bool TrackNamesIsEmpty() const { return TrackNames.IsEmpty(); }
	void EmptyTrackNames() { TrackNames.Empty(); }

	void EmptyTempoMap() { TempoMap.Empty(); }
	void EmptyBeatMap() { BeatMap.Empty(); }
	void EmptyBarMap() { BarMap.Empty(); }
	UE_API void SetLengthTotalBars(int32 Bars);
	void EmptySectionMap() { SectionMap.Empty(); }
	void EmptyChordMap() { ChordMap.Empty(); }

	UE_API void EmptyAllMaps();
	UE_API bool IsEmpty() const;

	// BEGIN ISongMapEvaluator overrides (Immutable access to maps)
	virtual const FTempoMap& GetTempoMap() const override { return TempoMap; }
	virtual const FBeatMap& GetBeatMap() const override { return BeatMap; }
	virtual const FBarMap& GetBarMap() const override { return BarMap; }
	virtual const FSectionMap& GetSectionMap() const override { return SectionMap; }
	virtual const FChordProgressionMap& GetChordMap() const override { return ChordMap; }
	virtual const FSongLengthData& GetSongLengthData() const override { return LengthData; }
	// END ISongMapEvaluator overrides

	FTempoMap& GetTempoMap() { return TempoMap; }
	FBeatMap& GetBeatMap() { return BeatMap; }
	FBarMap& GetBarMap() { return BarMap; }
	FSectionMap& GetSectionMap() { return SectionMap; }
	FChordProgressionMap& GetChordMap() { return ChordMap; }
	FSongLengthData& GetSongLengthData() { return LengthData; }

	int32 GetTicksPerQuarterNote() const { return TicksPerQuarterNote; }

	UE_API void SetStartBar(int32 StartBar);
	UE_API void SetSongLengthTicks(int32 NewLengthTicks);
	UE_API void FinalizeBarMap(int32 InLastTick);

	UE_API void AddTempoChange(int32 Tick, float TempoBPM);
	UE_API void AddTimeSigChange(int32 Tick, int32 TimeSigNum, int32 TimeSigDenom);

	UE_API bool AddTempoInfoPoint(int32 MicrosecondsPerQuarterNote, int32 Tick, bool SortNow = true);
	UE_API bool AddTimeSignatureAtBarIncludingCountIn(int32 BarIndex, int32 InNumerator, int32 InDenominator, bool SortNow = true, bool FailOnError = true);
	UE_API FTimeSignaturePoint* GetMutableTimeSignaturePoint(int32 PointIndex);

protected:
	friend class FSongMapReceiver;
	friend class HarmonixMetasound::FMidiClock;

	UPROPERTY()
	int32 TicksPerQuarterNote = Harmonix::Midi::Constants::GTicksPerQuarterNoteInt;
	UPROPERTY()
	FTempoMap TempoMap;
	UPROPERTY()
	FBarMap BarMap;
	UPROPERTY()
	FBeatMap BeatMap;
	UPROPERTY()
	FSectionMap SectionMap;
	UPROPERTY()
	FChordProgressionMap ChordMap;
	UPROPERTY()
	TArray<FString>	TrackNames;

private:
	UPROPERTY()
	FSongLengthData LengthData;


	UE_API void StringLengthToMT(const FString& LengthString, int32& OutBars, int32& OutTicks);
	UE_API bool ReadWithReader(FStdMidiFileReader& Reader);
	UE_API bool FinalizeRead(IMidiReader* Reader);
};

struct FSongMapsWithAlternateTempoSource : public ISongMapEvaluator
{
public:
	FSongMapsWithAlternateTempoSource(const TSharedPtr<const ISongMapEvaluator>& SongMapsWithTempo, const TSharedPtr<const ISongMapEvaluator>& SongMapsWithOthers)
		: SongMapsWithTempoMap(SongMapsWithTempo)
		, SongMapsWithOtherMaps(SongMapsWithOthers)
	{}

	FSongMapsWithAlternateTempoSource(const TSharedPtr<const ISongMapEvaluator>& SongMaps)
		: SongMapsWithTempoMap(SongMaps)
		, SongMapsWithOtherMaps(SongMaps)
	{
	}

	FSongMapsWithAlternateTempoSource& operator=(const TSharedPtr<const FSongMapsWithAlternateTempoSource>& Other)
	{
		SongMapsWithTempoMap = Other->SongMapsWithTempoMap;
		SongMapsWithOtherMaps = Other->SongMapsWithOtherMaps;
		return *this;
	}

	virtual ~FSongMapsWithAlternateTempoSource() = default;

	operator bool() const 
	{
		return SongMapsWithTempoMap.IsValid() && SongMapsWithOtherMaps.IsValid();
	}

	const TSharedPtr<const ISongMapEvaluator>& GetSongMapsWithTempoMap() const { return SongMapsWithTempoMap; }
	const TSharedPtr<const ISongMapEvaluator>& GetSongMapsWithOtherMaps() const { return SongMapsWithOtherMaps; }

	bool AllMapsHaveOneSource() const { return SongMapsWithTempoMap == SongMapsWithOtherMaps; }

protected:
	UE_API virtual const FTempoMap& GetTempoMap() const override;
	UE_API virtual const FBeatMap& GetBeatMap() const override;
	UE_API virtual const FBarMap& GetBarMap() const override;
	UE_API virtual const FSectionMap& GetSectionMap() const override;
	UE_API virtual const FChordProgressionMap& GetChordMap() const override;
	UE_API virtual const FSongLengthData& GetSongLengthData() const override;

	TSharedPtr<const ISongMapEvaluator> SongMapsWithTempoMap;
	TSharedPtr<const ISongMapEvaluator> SongMapsWithOtherMaps;
};

#undef UE_API
