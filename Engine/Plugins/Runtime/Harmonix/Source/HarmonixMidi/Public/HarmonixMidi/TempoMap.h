// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/MusicMapBase.h"
#include <limits>

#include "TempoMap.generated.h"

#define UE_API HARMONIXMIDI_API

/**
 * A position in a song where the tempo is specified (the rate at which the music plays)
 */
USTRUCT()
struct FTempoInfoPoint : public FMusicMapTimespanBase
{
	GENERATED_BODY()

public:
	static constexpr bool DefinedAsRegions = false;

	FTempoInfoPoint() 
	{}

	FTempoInfoPoint(float InMs, int32 InMidiTempo, int32 InStartTick, int32 InLengthTicks = 1)
		: FMusicMapTimespanBase(InStartTick, InLengthTicks)
		, Ms(InMs)
		, MidiTempo(InMidiTempo)
	{
	}

	bool operator==(const FTempoInfoPoint& Other) const
	{
		return Ms == Other.Ms && MidiTempo == Other.MidiTempo;
	}

	float GetBPM() const { return Harmonix::Midi::Constants::MidiTempoToBPM(MidiTempo); }

	float GetMsPerQuarterNote() const { return (float)MidiTempo / 1000.0f; }

	UPROPERTY()
	float Ms = 0;        // The time at which the tempo is changing
	UPROPERTY()
	int32 MidiTempo = 0; // microseconds per beat at that point

	struct TimeLessThan
	{
		bool operator()(float InMs, const FTempoInfoPoint& Point) const { return InMs < Point.Ms; }
		bool operator()(const FTempoInfoPoint& Point, float InMs) const { return Point.Ms < InMs; }
		bool operator()(const FTempoInfoPoint& PointA, const FTempoInfoPoint& PointB) const { return PointA.Ms < PointB.Ms; }
	};
};

/**
 * A TempoMap that can change over time; the graph of the correspondence
 * between time and tick has multiple line segments.
 *
 * The tempo changes are specified by adding (tick, tempo) pairs, 
 */

 USTRUCT()
struct FTempoMap 
{
	GENERATED_BODY()

public:

	FTempoMap() 
		: TicksPerQuarterNote(Harmonix::Midi::Constants::GTicksPerQuarterNoteInt)
	{}
	virtual ~FTempoMap() = default;

	UE_API bool operator==(const FTempoMap& Other) const;

	UE_API void Empty();
	UE_API void Copy(const FTempoMap& Other, int32 StartTick = 0, int32 EndTick = std::numeric_limits<int32>::max());
	UE_API bool IsEmpty() const;

	/** Called by the midi file importer before map points are added to this map */
	void SetTicksPerQuarterNote(int32 InTicksPerQuarterNote)
	{
		TicksPerQuarterNote = InTicksPerQuarterNote;
	}

	/** Get the time (in milliseconds) at a given tick: */
	UE_API float TickToMs(float Tick) const;

	/**
	 * Get the time (in milliseconds) at the given tick.
	 * 
	 * This flavor of the function can speed up iteration through midi data
	 * when you might be making many calls to TickToMs in a row with sequencial 
	 * tick numbers.
	 * 
	 * @param Tick Midi tick position
	 * @param Index The index of the tempo point to be checked first.
	 * @param NextIndex Filles in with the tempo point most likely to contain the next tick you are likely to request.
	 */
	UE_API float TickToMs(float Tick, int32 Index, int32* NextIndex) const;

	/** Get the tick at a given time (in milliseconds) */
	UE_API float MsToTick(float TimeMs) const;

	/**
		* Get the tick at the given millisecond.
		* 
		* This flavor of the function can speed up iteration through midi data
		* when you might be making many calls to MsToTick in a row with sequencial 
		* times.
		* 
		* @param TimeMs Millisecond
		* @param Index The index of the tempo point to be checked first.
		* @param NextIndex Fills in with the tempo point most likely to contain the next tick you are likely to request.
		*/
	UE_API float MsToTick(float TimeMs, int32 Index, int32* NextIndex) const;

	/** Get the tempo in ms/quarter-note */
	UE_API float GetMsPerQuarterNoteAtTick(int32 Tick) const;

	/** Get the tempo in us/quarter-note */
	UE_API int32 GetMicrosecondsPerQuarterNoteAtTick(int32 Tick) const;

	/** Get the tempo in beats (quarter-notes)/min */
	UE_API float GetTempoAtTick(int32 Tick) const;

	/** Get the tempo info point */
	UE_API const FTempoInfoPoint* GetTempoPointAtTick(int32 Tick) const;

	/** Get the tempo info point */
	UE_API int32 GetTempoPointIndexAtTick(int32 Tick) const;

	/** The number of 'tempo events' in the song */
	UE_API int32 GetNumTempoChangePoints() const;

	/** Return the tick of the nth tempo change point. */
	UE_API int32 GetTempoChangePointTick(int32 Index) const;

	/** Call when you're done changing it */
	UE_API void Finalize(int32 LastTick);

	UE_API bool AddTempoInfoPoint(int32 MicrosecondsPerQuarterNote, int32 Tick, bool SortNow = true);

	/** Remove every point from the given tick onward */
	UE_API void WipeTempoInfoPoints(int32 Tick);

	/** if there are no points, supply a default tempo of 120. */
	UE_API void SupplyDefault();

	// Returns the tempo info points for inspection.
	const TArray<FTempoInfoPoint>& GetTempoPoints() const { return Points; }

	int32 GetTicksPerQuarterNote() const { return TicksPerQuarterNote; }

protected:
	UPROPERTY()
	int32 TicksPerQuarterNote;
	UPROPERTY()
	TArray<FTempoInfoPoint> Points;

private:

	// Given a time, find the TempoInfoPoint at or before it.
	UE_API int32 PointIndexForTime(float TimeMs) const;

	// Assuming prevInfoPoint is the last TempoInfoPoint before tick,
	// calculate tick as time in ms
	UE_API float TickToMsInternal(float Tick, const FTempoInfoPoint& prevTempoInfoPoint) const;
	UE_API float MsToTickInternal(float TimeMs, const FTempoInfoPoint& prevTempoInfoPoint) const;

};

#undef UE_API
