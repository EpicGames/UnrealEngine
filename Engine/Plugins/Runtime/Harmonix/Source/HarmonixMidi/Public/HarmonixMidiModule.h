// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FHarmonixMidiModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();
};
