// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HarmonixMidiEditor : ModuleRules
{
    public HarmonixMidiEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		//OptimizeCode = CodeOptimization.Never;

        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[] {
				"Core",
                "CoreUObject",
				"Slate",
				"SlateCore",
				"PropertyEditor",
				"GraphEditor",
				"EditorStyle",
				"Settings",
				"BlueprintGraph",
				"EditorFramework"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			 new string[] {
					"Engine",
					"UnrealEd",
					"HarmonixMidi",
					"AssetDefinition",
					"DetailCustomizations",
					"InputCore",
					"ContentBrowser",
					"AssetTools",
					"ToolMenus",
				}
		 );
	}
}
