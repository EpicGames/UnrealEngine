// Copyright Epic Games, Inc. All Rights Reserved.
#include "AssetDefinition_MidiFile.h"

#include "Algo/AnyOf.h"
#include "ContentBrowserMenuContexts.h"
#include "HarmonixMidi/MidiFile.h"
#include "ToolMenus.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/MessageDialog.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MidiFile)

#define LOCTEXT_NAMESPACE "Harmonix_Midi"

FString UAssetDefinition_MidiFile::LastMidiExportFolder;

TSoftClassPtr<UObject> UAssetDefinition_MidiFile::GetAssetClass() const
{
	return UMidiFile::StaticClass();
}

FText UAssetDefinition_MidiFile::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "MIDIFileDefinition", "Standard MIDI File");
}

FLinearColor  UAssetDefinition_MidiFile::GetAssetColor() const
{

	return FLinearColor(1.0f, 0.5f, 0.0f);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MidiFile::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Audio / NSLOCTEXT("Harmonix", "HmxAssetCategoryName", "Harmonix") };
	return Categories;
}

bool UAssetDefinition_MidiFile::CanImport() const
{
	return true;
}

void UAssetDefinition_MidiFile::RegisterContextMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.MidiFile");
	FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");

	Section.AddDynamicEntry("MidiFile_ExportMid",
		FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const TAttribute<FText> Label = LOCTEXT("MidiFile_ExportMid", "Export Standard MIDI File (.mid)");
				const TAttribute<FText> ToolTip = LOCTEXT("MidiFile_ExportMidToolTip", "Exports standard MIDI file(s)");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MidiFile");
				const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MidiFile::ExecuteExportMidiFile);
				InSection.AddMenuEntry("MidiFile_ExportMid", Label, ToolTip, Icon, UIAction);
			})
		);
	Section.AddDynamicEntry("MidiFile_Compare",
		FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const TAttribute<FText> Label = LOCTEXT("MidiFile_Compare", "Compare Standard MIDI File Assets");
				const TAttribute<FText> ToolTip = LOCTEXT("MidiFile_CompareMidToolTip", "Compares selected Midi File Assets");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MidiFile");
				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MidiFile::ExecuteCompareMidiFiles);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_MidiFile::CanExecuteCompareMidiFiles);
				InSection.AddMenuEntry("MidiFile_CompareMid", Label, ToolTip, FSlateIcon(), UIAction);
			})
		);
	Section.AddDynamicEntry("MidiFile_OpenExtern",
		FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const TAttribute<FText> Label = LOCTEXT("MidiFile_OpenExtern", "View Standard MIDI File In External Application");
				const TAttribute<FText> ToolTip = LOCTEXT("MidiFile_OpenExternToolTip", "Opens the selected Standard MIDI File in the default viewer for '.mid' files on you PC.");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MidiFile");
				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MidiFile::ExecuteOpenMidiFileInExternalEditor);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_MidiFile::CanExecuteOpenMidiFileInExternalEditor);
				InSection.AddMenuEntry("MidiFile_OpenExtern", Label, ToolTip, FSlateIcon(), UIAction);
			})
	);
}

void UAssetDefinition_MidiFile::ExecuteExportMidiFile(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		if (Context->SelectedAssets.Num() > 1)
		{
			ExportAllMidiToFolder(Context);
		}
		else
		{
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			if (DesktopPlatform)
			{
				UMidiFile* MidiFile = Context->LoadFirstSelectedObject<UMidiFile>();
				FString	DefaultFileName = MidiFile->GetName() + TEXT(".mid");
				TArray<FString> SaveFileNames;
				const bool bFileSelected = DesktopPlatform->SaveFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					LOCTEXT("MidiFile_ExportMid_SaveFileDialogTitle", "Save Standard MIDI File as...").ToString(),
					LastMidiExportFolder.IsEmpty() ? FPaths::ProjectDir() : LastMidiExportFolder,
					DefaultFileName,
					TEXT("Standard MIDI File (*.mid)|*.mid"),
					EFileDialogFlags::None,
					SaveFileNames);

				if (!bFileSelected)
				{
					return;
				}

				if (ensure(SaveFileNames.Num() == 1))
				{
					FString OutputFileName = SaveFileNames[0];
					MidiFile->SaveStdMidiFile(OutputFileName);
					LastMidiExportFolder = FPaths::GetPath(OutputFileName);
				}
			}
		}
	}
}

void UAssetDefinition_MidiFile::ExportAllMidiToFolder(const UContentBrowserAssetContextMenuContext* Context)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		FString SelectedFolderName;
		if (DesktopPlatform->OpenDirectoryDialog(FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr), "Select destination for Standard MIDI Files...", LastMidiExportFolder.IsEmpty() ? FPaths::ProjectDir() : LastMidiExportFolder, SelectedFolderName))
		{
			LastMidiExportFolder = SelectedFolderName;
			bool bKeepWarning = true;
			for (UMidiFile* MidiFile : Context->LoadSelectedObjects<UMidiFile>())
			{
				FString OutFilePath = FPaths::Combine(LastMidiExportFolder, MidiFile->GetName() + TEXT(".mid"));
				if (bKeepWarning && FPaths::FileExists(OutFilePath))
				{
					EAppReturnType::Type AskResult = AskOverwrite(OutFilePath);
					if (AskResult == EAppReturnType::NoAll)
					{
						break;
					}
					if (AskResult == EAppReturnType::No)
					{
						continue;
					}
					if (AskResult == EAppReturnType::YesAll)
					{
						bKeepWarning = false;
					}
				}
				MidiFile->SaveStdMidiFile(OutFilePath);
			}
		}
	}
}

void UAssetDefinition_MidiFile::ExecuteCompareMidiFiles(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		TArray<UMidiFile*> AsMidiFiles = Context->LoadSelectedObjects<UMidiFile>();
		if (AsMidiFiles.Num() < 2)
		{
			return;
		}
		bool TheSame = true;
		for (int32 i = 1; i < AsMidiFiles.Num(); ++i)
		{
			if (*AsMidiFiles[0] != *AsMidiFiles[i])
			{
				TheSame = false;
				break;
			}
		}
		FText Equal = LOCTEXT("MidiFile_Compare_Equal", "The selected MIDI files are the same.");
		FText Different = LOCTEXT("MidiFile_Compare_Different", "The selected MIDI files are different.");
		FMessageDialog::Open(EAppMsgType::Ok, TheSame ? Equal : Different, LOCTEXT("MidiFile_Compare_Result_Title", "MIDI File Comparison..."));
	}
}

bool UAssetDefinition_MidiFile::CanExecuteCompareMidiFiles(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		if (Context->SelectedAssets.Num() < 2)
		{
			return false;
		}
		int32 MidiCount = 0;
		for (const FAssetData& AssetData : Context->SelectedAssets)
		{
			if (AssetData.AssetClassPath.GetAssetName() == "MidiFile")
			{
				++MidiCount;
			}
		}
		return MidiCount > 1;
	}
	return false;
}

void UAssetDefinition_MidiFile::ExecuteOpenMidiFileInExternalEditor(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		TArray<UMidiFile*> AsMidiFiles = Context->LoadSelectedObjects<UMidiFile>();

		IFileManager& FileManager = IFileManager::Get();
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		
		FString UserTempPath = FDesktopPlatformModule::Get()->GetUserTempPath();
		FString MidiTempPath = FPaths::Combine(UserTempPath, "UnrealEditor", "TempMidi");

		FileManager.DeleteDirectory(*MidiTempPath, false, true);
		FileManager.MakeDirectory(*MidiTempPath, true);

		FString DestFilePrefix = FString::Format(TEXT("VIEW_ONLY_{0}"), { *AsMidiFiles[0]->GetName() });

		FString DestFilePath = FPaths::CreateTempFilename(*MidiTempPath, *DestFilePrefix, TEXT(".mid"));
		AsMidiFiles[0]->SaveStdMidiFile(*DestFilePath);
		
		FString DestFileAsArg = FString::Format(TEXT("\"{0}\""), {*DestFilePath});

		FPlatformProcess::LaunchFileInDefaultExternalApplication(*DestFileAsArg, NULL, ELaunchVerb::Open);
	}
}

bool UAssetDefinition_MidiFile::CanExecuteOpenMidiFileInExternalEditor(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		if (Context->SelectedAssets.Num() != 1)
		{
			return false;
		}
		int32 MidiCount = 0;
		for (const FAssetData& AssetData : Context->SelectedAssets)
		{
			if (AssetData.AssetClassPath.GetAssetName() == "MidiFile")
			{
				++MidiCount;
			}
		}
		return MidiCount == 1;
	}
	return false;
}

EAppReturnType::Type UAssetDefinition_MidiFile::AskOverwrite(FString& OutPath)
{
	FText OutPathText = FText::FromString(*OutPath);
	return FMessageDialog::Open(EAppMsgCategory::Warning, EAppMsgType::YesNoYesAllNoAll, 
		FText::Format(LOCTEXT("MidiFile_ExportMid_OverwriteMessage", "{0} exists.\n\nWould you like to overwrite it?"), OutPathText));
}

#undef LOCTEXT_NAMESPACE
