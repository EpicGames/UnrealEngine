// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_MidiFile.generated.h"

struct FToolMenuContext;
class UContentBrowserAssetContextMenuContext;
namespace EAppReturnType { enum Type : int; }

UCLASS()
class UAssetDefinition_MidiFile : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override;

	static void RegisterContextMenu();
private:
	static void ExecuteExportMidiFile(const FToolMenuContext& MenuContext);
	static void ExportAllMidiToFolder(const UContentBrowserAssetContextMenuContext* Context);
	static void ExecuteCompareMidiFiles(const FToolMenuContext& MenuContext);
	static bool CanExecuteCompareMidiFiles(const FToolMenuContext& MenuContext);
	static void ExecuteOpenMidiFileInExternalEditor(const FToolMenuContext& MenuContext);
	static bool CanExecuteOpenMidiFileInExternalEditor(const FToolMenuContext& MenuContext);

	static EAppReturnType::Type AskOverwrite(FString& OutPath);
	static FString LastMidiExportFolder;
};
