// Copyright Epic Games, Inc. All Rights Reserved.
#include "HarmonixMidiEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"
#include "MidiNoteCustomization.h"
#include "Pins/MidiNotePinFactory.h"
#include "AssetDefinition_MidiFile.h"
#include "PropertyEditorModule.h"
#include "HarmonixMidi/MidiFile.h"
#include "MidiFileDetailCustomization.h"
#include "PropertyEditorDelegates.h"
#include "IDetailCustomization.h"
#include "ToolMenus.h"
#include "AssetDefinition_MidiFile.h"

#define LOCTEXT_NAMESPACE "Harmonix_Midi"

DEFINE_LOG_CATEGORY(LogHarmonixMidiEditor)

void FHarmonixMidiEditorModule::StartupModule()
{
	FPropertyEditorModule& propertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	propertyModule.RegisterCustomPropertyTypeLayout("MidiNote", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMidiNoteCustomization::MakeInstance));
	propertyModule.NotifyCustomizationModuleChanged();

	FEdGraphUtilities::RegisterVisualPinFactory(MakeShareable(new FMidiNotePinFactory()));

	//Detail Customization for UMidiFile's file length 
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(
		UMidiFile::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMidiFileDetailCustomization::MakeInstance)
	);

	PropertyModule.NotifyCustomizationModuleChanged();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FHarmonixMidiEditorModule::RegisterAssetContextMenus));
}

void FHarmonixMidiEditorModule::RegisterAssetContextMenus()
{
	FToolMenuOwnerScoped MenuOwner(this);
	UAssetDefinition_MidiFile::RegisterContextMenu();
}

void FHarmonixMidiEditorModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHarmonixMidiEditorModule, HarmonixMidiEditor);

#undef LOCTEXT_NAMESPACE