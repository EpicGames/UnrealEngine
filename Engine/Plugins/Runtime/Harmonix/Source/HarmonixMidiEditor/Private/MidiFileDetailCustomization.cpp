// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiFileDetailCustomization.h"
#include "HarmonixMidi/MidiFile.h"
#include "Math/UnitConversion.h"
#include "MidiFileFactory.h"

#define LOCTEXT_NAMESPACE "HarmonixMIDIEditor"

void FMidiFileDetailCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailBuilder = InDetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FMidiFileDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	//Details Panel for the UROPERTY AssetImportData, no actual customization, just reordering its position 
	IDetailCategoryBuilder& MidiFileImportSettingsCategory = DetailLayout.EditCategory(TEXT("Import Settings"));
	TSharedPtr<IPropertyHandle> AssetImportHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMidiFile, AssetImportData));
	MidiFileImportSettingsCategory.AddProperty(AssetImportHandle);

	//Details Panel for the UROPERTY Start Bar, no actual customization, just reordering its position 
	IDetailCategoryBuilder& MidiFileStartBarCategory = DetailLayout.EditCategory(TEXT("Midi File Start Bar"));
	TSharedPtr<IPropertyHandle> StartBarHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMidiFile, StartBar));
	MidiFileStartBarCategory.AddProperty(StartBarHandle);

	//Detail Customization for UProperty FileLengthBars
	IDetailCategoryBuilder& MidiFileLengthCategory = DetailLayout.EditCategory(TEXT("Midi File Length"));

	//Get the Midi File that is being edited
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		return;
	}
	MidiFile = Objects.Last();
	TWeakObjectPtr<UMidiFile> MidiFileBeingEdited = Cast<UMidiFile>(MidiFile);
	
	//Detail Customization for the File Length Row
	LengthRow = &MidiFileLengthCategory.AddCustomRow(FText::FromString("Midi File Length"));
	LengthRow->NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString("File Length"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	BuildLengthRow(MidiFileBeingEdited.Get());

	//Show FileDuration in seconds
	FDetailWidgetRow* DurationRow = &MidiFileLengthCategory.AddCustomRow(FText::FromString("Midi File Duration"));
	DurationRow->NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString("File Duration"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]; 
	DurationRow->ValueContent()
	[
		SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("MIDIFileDuration", "{0} secs"),  MidiFileBeingEdited->GetSongMaps()->GetSongLengthMs() / 1000.0f))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(FLinearColor::White)
			.ToolTipText(LOCTEXT("MIDIFileDurationTooltip", "Duration of the midi file in seconds."))
	];
	
	IDetailCategoryBuilder& TrackInfoCategory = DetailLayout.EditCategory(TEXT("Track Info"), LOCTEXT("TrackInfo", "Tracks:"));
	const UMidiFile::FMidiTrackList& Tracks = MidiFileBeingEdited->GetTracks();
	for (const FMidiTrack& Track : Tracks)
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(Track.GetAllocatedSize(), EUnit::Bytes);
		FString SizeInBytes = FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));

		TrackInfoCategory.AddCustomRow(FText::FromString("Tracks"))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(&Track == &Tracks[0] ? FText::FromString("Conductor (tempo & time signature)") : FText::FromString(*Track.GetName()))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		].ValueContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(8.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(FText::Format(FTextFormat::FromString("{0} MIDI Events"), { Track.GetEvents().Num() }))
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(8.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(FText::Format(FTextFormat::FromString("{0} Strings (including track name)"), { Track.GetTextRepository() ? Track.GetTextRepository()->Num() : 0 }))
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(8.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(FText::FromString(*SizeInBytes))
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];
	}
}

void FMidiFileDetailCustomization::BuildLengthRow(UMidiFile* TheMidiFile)
{
	auto const RoundedLengthToolTipText = LOCTEXT("MIDIFileLengthTooltip", "Length of the midi file in bars | beats."); 
	
	bool bLengthIsAPerfectSubdivision = TheMidiFile->GetSongMaps()->LengthIsAPerfectSubdivision();
	if (bLengthIsAPerfectSubdivision)
	{
		FileLengthText = FText::FromString(TheMidiFile->GetSongMaps()->GetSongLengthString());
	}
	else
	{
		FileLengthText = FText::Format(LOCTEXT("NonQualtizedMIDIFileLength", "{0} (Warning: File is not quantized to any standard musical subdivision.)"),
			FText::FromString(TheMidiFile->GetSongMaps()->GetSongLengthString()));
	}

	//conform buttons OnClick callback function
	auto OnLengthConformTrivial = [this, TheMidiFile]
		{
			TheMidiFile->QuantizeLengthToNearestPerfectSubdivision(EMidiFileQuantizeDirection::Nearest);
			TheMidiFile->PostEditChange();
			BuildLengthRow(TheMidiFile);
			return FReply::Handled();
		};

	auto OnLengthConformGross = [this, TheMidiFile]
		{
			int32 CurrentLengthTicks = TheMidiFile->GetSongMaps()->GetSongLengthData().LengthTicks;
			EMidiClockSubdivisionQuantization Subdivision = EMidiClockSubdivisionQuantization::None;
			int32 QuantizedLengthTick = TheMidiFile->GetSongMaps()->QuantizeTickToAnyNearestSubdivision(CurrentLengthTicks, EMidiFileQuantizeDirection::Nearest, Subdivision);
			UMidiFileFactory::AskOrDoGrossConform(TheMidiFile, false, nullptr, CurrentLengthTicks, QuantizedLengthTick, Subdivision);
			TheMidiFile->PostEditChange();
			BuildLengthRow(TheMidiFile);
			return FReply::Handled();
		};

	bool bLengthBoxIsNew = false;
	if (!LengthBox.IsValid())
	{
		bLengthBoxIsNew = true;
		LengthRow->ValueContent()
			[
				SAssignNew(LengthBox, SHorizontalBox)
			];
	}
	LengthBox->ClearChildren();

	if (bLengthIsAPerfectSubdivision)
	{
		LengthBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SAssignNew(FileLengthTextBlock, STextBlock)
					.Text(FileLengthText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FLinearColor::White)
					.ToolTipText(RoundedLengthToolTipText)
			];
	}
	else if (UMidiFileFactory::LengthCanBeTriviallyConformed(TheMidiFile))
	{
		LengthBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SAssignNew(FileLengthTextBlock, STextBlock)
					.Text(FileLengthText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FLinearColor::Yellow)
					.ToolTipText(RoundedLengthToolTipText)
			];
		LengthBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(10.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(LOCTEXT("ConformByPrompt", "MIDI file is only a tick or two away from a musical subdivision: "))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
							.VAlign(VAlign_Center)
							.Text(LOCTEXT("ConformTrivial", "Conform To Nearest Subdivision"))
							.OnClicked_Lambda(OnLengthConformTrivial)
					]
			];
	}
	else
	{
		LengthBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SAssignNew(FileLengthTextBlock, STextBlock)
					.Text(FileLengthText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(bLengthIsAPerfectSubdivision ? FLinearColor::White : FLinearColor::Yellow)
					.ToolTipText(RoundedLengthToolTipText)
			];
		LengthBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(30.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SButton)
							.VAlign(VAlign_Center)
							.Text(LOCTEXT("ConformByWorkflow", "Conform..."))
							.OnClicked_Lambda(OnLengthConformGross)
					]
			];
	}
	if (!bLengthBoxIsNew)
	{
		LengthBox->Invalidate(EInvalidateWidgetReason::Layout);
	}
}

#undef LOCTEXT_NAMESPACE
