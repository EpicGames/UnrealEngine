// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SButton.h"

/*
Customize details panel for UMidiFile
*/

class UMidiFile;

class FMidiFileDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FMidiFileDetailCustomization);
	}

	//Text Display for midi file length
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;

private:
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder;
	FText FileLengthText;
	TSharedPtr<STextBlock> FileLengthTextBlock;
	TSharedPtr<SHorizontalBox> LengthBox;
	FDetailWidgetRow* LengthRow = nullptr;

	TWeakObjectPtr<UObject> MidiFile;

	void BuildLengthRow(UMidiFile* TheMidiFile);
};