// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiFileFactory.h"
#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidiEditorModule.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Widgets/Input/STextComboBox.h"
#include "IPropertyTypeCustomization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MidiFileFactory)

#define LOCTEXT_NAMESPACE "HarmonixMIDIEditor"

UMidiFileFactory::UMidiFileFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	SupportedClass = UMidiFile::StaticClass();

	bEditorImport = true;
	bText = false;

	Formats.Add(TEXT("mid;Standard MIDI File"));
}

FText UMidiFileFactory::GetDisplayName() const
{
	return LOCTEXT("ImporterFactory_Name", "Standard MIDI File");
}

FText UMidiFileFactory::GetToolTip() const
{
	return LOCTEXT("ImporterFactory_Description", "Standard MIDI Files exported from Digital Audio Workstations");
}

bool UMidiFileFactory::FactoryCanImport(const FString& Filename)
{
	return true;
}

UObject* UMidiFileFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UMidiFile* MidiFileAsset = FindObject<UMidiFile>(InParent, *InName.ToString());
	if (!MidiFileAsset)
	{
		MidiFileAsset = NewObject<UMidiFile>(InParent, InClass, InName, Flags);
	}

	UMidiFile::ELoadResult Result = MidiFileAsset->LoadStdMidiFile(Filename, Harmonix::Midi::Constants::GTicksPerQuarterNoteInt);
	if (Result != UMidiFile::ELoadResult::Success)
	{
		return nullptr;
	}
	
	//add the current file to the file array (for handling multi-batch import)
	ImportedFiles.Add(MidiFileAsset);
	
	return MidiFileAsset;
}

void UMidiFileFactory::CleanUp()
{
	CheckImportedFilesForValidLengths();
	Super::CleanUp();
}

bool UMidiFileFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	if (UMidiFile* AsMidiFile = Cast<UMidiFile>(Obj))
	{
		FString FilePath = AsMidiFile->GetImportedSrcFilePath();
		if (FilePath.IsEmpty())
		{
			UE_LOGF(LogHarmonixMidiEditor, Warning, "Couldn't find source path for %ls!", *AsMidiFile->GetPathName());
			return false;
		}

		// the PreferredReimportPath is the path of the new file, which can differ from the original SrcFilePath
		// if the file has the same name but a different extension, don't import it!
		if (!PreferredReimportPath.IsEmpty() && !IsSupportedFileExtension(PreferredReimportPath))
		{
			UE_LOGF(LogHarmonixMidiEditor, Warning, "%ls: Failed to reimport with new file. Invalid extension: %ls", *AsMidiFile->GetPathName(), *FPaths::GetExtension(PreferredReimportPath));
			return false;
		}
		
		OutFilenames.Push(FilePath);
		return true;
	}
	return false;
}

void UMidiFileFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (UMidiFile* AsMidiFile = Cast<UMidiFile>(Obj))
	{
		AsMidiFile->AssetImportData->UpdateFilenameOnly(FPaths::ConvertRelativePathToFull(NewReimportPaths[0]));
	}
}

EReimportResult::Type UMidiFileFactory::Reimport(UObject* Obj)
{
	UMidiFile* AsMidiFile = Cast<UMidiFile>(Obj);
	if (!AsMidiFile)
		return EReimportResult::Failed;

	FString ReimportPath = AsMidiFile->GetImportedSrcFilePath();
	if (ReimportPath.IsEmpty())
	{
		UE_LOGF(LogHarmonixMidiEditor, Warning, "Failed to reimport MIDI file: %ls", *AsMidiFile->GetFullName());
		return EReimportResult::Failed;
	}

	bool OutCanceled = false;
	UObject* ImportedObject = ImportObject(AsMidiFile->GetClass(), AsMidiFile->GetOuter(), *AsMidiFile->GetName(), RF_Public | RF_Standalone, ReimportPath, nullptr, OutCanceled);
	if (ImportedObject && !OutCanceled)
	{
		// the imported object should be the existing midi file
		ensure(ImportedObject == AsMidiFile);

		CheckImportedFilesForValidLengths();
		
		if (AsMidiFile->GetOuter())
		{
			AsMidiFile->GetOuter()->MarkPackageDirty();
		}
		else
		{
			AsMidiFile->MarkPackageDirty();
		}
	}
	else if (OutCanceled)
	{
		UE_LOGF(LogHarmonixMidiEditor, Warning, "import canceled");
		return EReimportResult::Cancelled;
	}
	else
	{
		UE_LOGF(LogHarmonixMidiEditor, Warning, "import failed");
		return EReimportResult::Failed;
	}
	
	return EReimportResult::Succeeded;
}

void UMidiFileFactory::PostImportCleanUp()
{
	CheckImportedFilesForValidLengths();
}

void UMidiFileFactory::CheckImportedFilesForValidLengths()
{
	for (int FileIndex = 0; FileIndex < ImportedFiles.Num(); FileIndex++)
	{
		UMidiFile* MidiFileAsset = ImportedFiles[FileIndex];

		if (!MidiFileAsset->GetSongMaps()->LengthIsAPerfectSubdivision())
		{
			// trivially different, or grossly different?
			int32 CurrentLengthTicks = MidiFileAsset->GetSongMaps()->GetSongLengthData().LengthTicks;
			EMidiClockSubdivisionQuantization Subdivision = EMidiClockSubdivisionQuantization::None;
			int32 QuantizedLengthTick = MidiFileAsset->GetSongMaps()->QuantizeTickToAnyNearestSubdivision(CurrentLengthTicks, EMidiFileQuantizeDirection::Nearest, Subdivision);
			if (LengthCanBeTriviallyConformed(MidiFileAsset))
			{
				AskOrDoTrivialConform(MidiFileAsset, FileIndex < (ImportedFiles.Num() - 1), this, CurrentLengthTicks, QuantizedLengthTick);
			}
			else
			{
				AskOrDoGrossConform(MidiFileAsset, FileIndex < (ImportedFiles.Num() - 1), this, CurrentLengthTicks, QuantizedLengthTick, Subdivision);
			}
		}
	}
	ImportedFiles.Empty();
	bApplyGrossConformToAll = false;
	bDontApplyGrossConformToAll = false;
	bApplyOffByOneConformToAll = false;
	bDontApplyOffByOneConformToAll = false;
	ApplyGrossConformDirection = EMidiFileQuantizeDirection::Up;
	GrossConformSubdivision = EMidiClockSubdivisionQuantization::None;
}

bool UMidiFileFactory::LengthCanBeTriviallyConformed(UMidiFile* MidiFile)
{
	int32 CurrentLengthTicks = MidiFile->GetSongMaps()->GetSongLengthData().LengthTicks;
	EMidiClockSubdivisionQuantization Division = EMidiClockSubdivisionQuantization::None;
	int32 QuantizedLengthTick = MidiFile->GetSongMaps()->QuantizeTickToAnyNearestSubdivision(CurrentLengthTicks, EMidiFileQuantizeDirection::Nearest, Division);

	return (FMath::Abs(CurrentLengthTicks - QuantizedLengthTick) <= kMaxTickErrorForTrivialConform);
}

void UMidiFileFactory::AskOrDoTrivialConform(UMidiFile* MidiFileAsset, bool bIsOneOfMany, UMidiFileFactory* CallingFactory, int32 CurrentLengthTicks, int32 QuantizedLengthTicks)
{
	if (CallingFactory && CallingFactory->bDontApplyOffByOneConformToAll)
	{
		// we've been asked to skip all minor conforms...
		return;
	}

	EAppReturnType::Type Result = EAppReturnType::Type::No;
	if (!CallingFactory ||  !CallingFactory->bApplyOffByOneConformToAll)
	{
		Result = FMessageDialog::Open(
			EAppMsgCategory::Warning,
			bIsOneOfMany ? EAppMsgType::YesNoYesAllNoAll : EAppMsgType::YesNo,
			EAppReturnType::Type::Yes,
			FText::Format(LOCTEXT("ImporterFactory_OffByOneWarningMsg", "The MIDI File \"{0}\" has a length that is {1} tick(s) off\n"
				"from of being evenly divisible by any musical subdivision.\n\n"
				"It is HIGHLY recommended that you conform this length to the nearest musical\n"
				"subdivision so that looping, sequential playback, etc. stays in sync.\n\n"
				"Can I conform this length?"), FText::FromString(MidiFileAsset->GetName()), CurrentLengthTicks - QuantizedLengthTicks),
			LOCTEXT("ImporterFactory_OffByOneWarningTitle", "MIDI Import Warning: Minor Length Issue..."));
		if (CallingFactory && Result == EAppReturnType::Type::YesAll)
		{
			CallingFactory->bApplyOffByOneConformToAll = true;
		}
		else if (CallingFactory && Result == EAppReturnType::Type::NoAll)
		{
			CallingFactory->bDontApplyOffByOneConformToAll = true;
		}
	}

	if (Result == EAppReturnType::Type::Yes || (CallingFactory && CallingFactory->bApplyOffByOneConformToAll))
	{
		MidiFileAsset->QuantizeLengthToNearestPerfectSubdivision(EMidiFileQuantizeDirection::Nearest);
	}
}

void UMidiFileFactory::AskOrDoGrossConform(UMidiFile* MidiFileAsset, bool bIsOneOfMany, UMidiFileFactory* CallingFactory, int32 CurrentLengthTicks, int32 QuantizedLengthTick, EMidiClockSubdivisionQuantization BestSubdivision)
{
	if (CallingFactory && CallingFactory->bDontApplyGrossConformToAll)
	{
		// we've been told to skip all gross conforms...
		return;
	}

	EMidiFileQuantizeDirection Direction = CallingFactory ? CallingFactory->ApplyGrossConformDirection : EMidiFileQuantizeDirection::Nearest;
	EMidiClockSubdivisionQuantization Subdivision = CallingFactory ? CallingFactory->GrossConformSubdivision : EMidiClockSubdivisionQuantization::None;

	if (!CallingFactory || !CallingFactory->bApplyGrossConformToAll)
	{
		// let's figure out the best direction to go an the best unit...
		EMidiFileQuantizeDirection RecommendedDirection = QuantizedLengthTick > CurrentLengthTicks ? EMidiFileQuantizeDirection::Up : EMidiFileQuantizeDirection::Down;

		// first we need a window to hold the custom dialog...
		TSharedPtr<SWindow> ParentWindow;
		SAssignNew(ParentWindow, SWindow)
			.Title(LOCTEXT("ImporterFactory_GrossConformTitle", "MIDI Import Warning: Significant Length Issue..."))
			.ClientSize(FVector2D(600, 250));

		// now a detailed description of the file in question...
		FString MidiFileName = MidiFileAsset->GetName();
		FText Description = FText::Format(LOCTEXT("ImporterFactory_FileSpecs", "MIDI File Name: {0}.mid\nLength: {1}"),
			FText::FromString(MidiFileName),
			FText::FromString(MidiFileAsset->GetSongMaps()->GetSongLengthString()));

		// now we can create the custom dialog widget inside the window we made...
		TSharedRef<SConformMidiFileLengthDialog> ConformMidiFileLengthDialog = SNew(SConformMidiFileLengthDialog)
			.ParentWindow(ParentWindow)
			.bMultipleFiles(bIsOneOfMany)
			.FileDescription(Description)
			.RecommendedDirection(RecommendedDirection)
			.RecommendedSubdivision(BestSubdivision);

		//show the dialog...
		FSlateApplication::Get().AddModalWindow(ParentWindow.ToSharedRef(), nullptr);

		// now we have the results...
		if (!ConformMidiFileLengthDialog->bUserTookAction || ConformMidiFileLengthDialog->ConformSubdivision == EMidiClockSubdivisionQuantization::None)
		{
			if (CallingFactory && ConformMidiFileLengthDialog->bUserTookAction)
			{
				CallingFactory->bDontApplyGrossConformToAll = ConformMidiFileLengthDialog->bDoForAll;
			}
			// return without doing anything...
			return;
		}

		Direction = ConformMidiFileLengthDialog->ConformDirection;
		Subdivision = ConformMidiFileLengthDialog->ConformSubdivision;

		if (CallingFactory)
		{
			CallingFactory->bApplyGrossConformToAll = ConformMidiFileLengthDialog->bDoForAll;
			CallingFactory->ApplyGrossConformDirection = ConformMidiFileLengthDialog->ConformDirection;
			CallingFactory->GrossConformSubdivision = ConformMidiFileLengthDialog->ConformSubdivision;
		}
	}

	if (Subdivision != EMidiClockSubdivisionQuantization::None)
	{
		MidiFileAsset->QuantizeLengthToSubdivision(Direction, Subdivision);
		MidiFileAsset->PostEditChange();
	}
}

void SConformMidiFileLengthDialog::Construct(const FArguments& InArgs)
{
	check(InArgs._ParentWindow);

	DirectionNames.Add(MakeShared<FString>(StaticEnum<EMidiFileQuantizeDirection>()->GetDisplayNameTextByValue(0).ToString()));
	DirectionNames.Add(MakeShared<FString>(StaticEnum<EMidiFileQuantizeDirection>()->GetDisplayNameTextByValue(1).ToString()));
	DirectionNames.Add(MakeShared<FString>(StaticEnum<EMidiFileQuantizeDirection>()->GetDisplayNameTextByValue(2).ToString()));
	SelectedDirection = DirectionNames[(int32)InArgs._RecommendedDirection];

	QuantizationSubdivisions.Empty();
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::Bar, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::Bar).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::Beat, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::Beat).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::WholeNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::WholeNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::HalfNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::HalfNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::QuarterNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::QuarterNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::EighthNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::EighthNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::SixteenthNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::SixteenthNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::ThirtySecondNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::ThirtySecondNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::DottedWholeNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::DottedWholeNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::DottedHalfNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::DottedHalfNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::DottedQuarterNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::DottedQuarterNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::DottedEighthNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::DottedEighthNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::DottedSixteenthNote, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::DottedSixteenthNote).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::HalfNoteTriplet, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::HalfNoteTriplet).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::QuarterNoteTriplet, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::QuarterNoteTriplet).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::EighthNoteTriplet, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::EighthNoteTriplet).ToString()));
	QuantizationSubdivisions.Add(MakeShared<FSubdivisionEntry>(EMidiClockSubdivisionQuantization::SixteenthNoteTriplet, StaticEnum<EMidiClockSubdivisionQuantization>()->GetDisplayNameTextByValue((int64)EMidiClockSubdivisionQuantization::SixteenthNoteTriplet).ToString()));

	if (TSharedPtr<FSubdivisionEntry>* Match = QuantizationSubdivisions.FindByPredicate([Recommended = InArgs._RecommendedSubdivision](const TSharedPtr<FSubdivisionEntry>& Item) { return Item->Value == Recommended; }))
	{
		SelectedQuantizationSubdivision = *Match;

	}
	else
	{
		SelectedQuantizationSubdivision = QuantizationSubdivisions[0];
	}

	TSharedPtr<SHorizontalBox> ActionButtonPanel;

	ChildSlot
	[
		SNew(SBorder)
		.HAlign(HAlign_Center).VAlign(VAlign_Center).BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight().HAlign(HAlign_Left).VAlign(VAlign_Center).Padding(20.0f, 10.0f)
				[
					SNew(STextBlock)
					.TextStyle(&MidiFileInformationStyle)
					.Text(InArgs._FileDescription)
				]
			+ SVerticalBox::Slot()
				.AutoHeight().HAlign(HAlign_Center).VAlign(VAlign_Center).Padding(5)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ImporterFactory_GrossConformExplanation", "Some or all of the MIDI files you are importing have a length\n"
															   "That is not evenly divisible by any standard musical subdivision.\n\n"
															   "It is highly recommended that you conform MIDI file lengths to a\n"
															   "musical subdivision so that synchronization doesn't drift when looping\n"
															   "or sequencing (stiching) midi files end-to-end."))
				]
			+ SVerticalBox::Slot()
				.AutoHeight().Padding(5).VAlign(VAlign_Center).HAlign(HAlign_Center)
				[
					// Direction
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
						.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ImporterFactory_Label_Round", "Round:"))
						]
					+ SHorizontalBox::Slot()
						.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
						[
							SNew(STextComboBox)
 							.OptionsSource(&DirectionNames)
							.InitiallySelectedItem(SelectedDirection)
							//.Font(IDetailLayoutBuilder::GetDetailFont())
							.OnSelectionChanged_Lambda([this](TSharedPtr<FString> InDirection, ESelectInfo::Type InSelectType)
								{
									SelectedDirection = InDirection;
								})
						]
					+ SHorizontalBox::Slot()
						.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ImporterFactory_Label_Subdivision", "Subdivision:"))
						]
					+ SHorizontalBox::Slot()
						.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
						[
							SNew(SComboBox<TSharedPtr<FSubdivisionEntry>>)
							.OptionsSource(&QuantizationSubdivisions)
							.OnGenerateWidget_Lambda([](TSharedPtr<FSubdivisionEntry> InQuantizationUnit)
								{
									return SNew(STextBlock)
										.Text(FText::FromString(InQuantizationUnit->DisplayString))
										.Font(IPropertyTypeCustomizationUtils::GetRegularFont());
								})
							.InitiallySelectedItem(SelectedQuantizationSubdivision)
							.OnSelectionChanged_Lambda([this](TSharedPtr<FSubdivisionEntry> InQuantizationUnit, ESelectInfo::Type InSelectType)
								{
									SelectedQuantizationSubdivision = InQuantizationUnit;
								})
							.Content()
							[
								SNew(STextBlock)
								.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
								.Text_Lambda([this]()
									{
										return FText::FromString(SelectedQuantizationSubdivision->DisplayString);
									})
							]
						]
				]
			+ SVerticalBox::Slot()
				.AutoHeight().Padding(5).VAlign(VAlign_Center).HAlign(HAlign_Center)
				[
					SAssignNew(ActionButtonPanel, SHorizontalBox)
				]
		]	
	];

	if (!InArgs._bMultipleFiles)
	{
		ActionButtonPanel->AddSlot()
			.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
			[
				SNew(SButton)
					.Text(LOCTEXT("ImporterFactory_Button_Conform", "Conform"))
					.OnClicked_Lambda([this, ParentWindow = InArgs._ParentWindow]()->FReply
						{
							bDoForAll = false;
							bUserTookAction = true;
							ConformDirection = (EMidiFileQuantizeDirection)StaticEnum<EMidiFileQuantizeDirection>()->GetValueByNameString(*SelectedDirection);
							if (SelectedQuantizationSubdivision)
							{
								ConformSubdivision = SelectedQuantizationSubdivision->Value;
							}
							else
							{
								ConformSubdivision = EMidiClockSubdivisionQuantization::None;
							}
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
			];
		ActionButtonPanel->AddSlot()
			.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
			[
				SNew(SButton)
					.Text(LOCTEXT("ImporterFactory_Button_Dont_Conform", "Don't Conform"))
					.OnClicked_Lambda([this, ParentWindow = InArgs._ParentWindow]()->FReply
						{
							bDoForAll = false;
							bUserTookAction = true;
							ConformSubdivision = EMidiClockSubdivisionQuantization::None;
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
			];
	}
	else
	{
		ActionButtonPanel->AddSlot()
			.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
			[
				SNew(SButton)
					.Text(LOCTEXT("ImporterFactory_Button_Conform", "Conform"))
					.OnClicked_Lambda([this, ParentWindow = InArgs._ParentWindow]()->FReply
						{
							bDoForAll = false;
							bUserTookAction = true;
							ConformDirection = (EMidiFileQuantizeDirection)StaticEnum<EMidiFileQuantizeDirection>()->GetValueByNameString(*SelectedDirection);
							if (SelectedQuantizationSubdivision)
							{
								ConformSubdivision = SelectedQuantizationSubdivision->Value;
							}
							else
							{
								ConformSubdivision = EMidiClockSubdivisionQuantization::None;
							}
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
			];
		ActionButtonPanel->AddSlot()
			.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
			[
				SNew(SButton)
					.Text(LOCTEXT("ImporterFactory_Button_Conform_All", "Conform All"))
					.OnClicked_Lambda([this, ParentWindow = InArgs._ParentWindow]()->FReply
						{
							bDoForAll = true;
							bUserTookAction = true;
							ConformDirection = (EMidiFileQuantizeDirection)StaticEnum<EMidiFileQuantizeDirection>()->GetValueByNameString(*SelectedDirection);
							if (SelectedQuantizationSubdivision)
							{
								ConformSubdivision = SelectedQuantizationSubdivision->Value;
							}
							else
							{
								ConformSubdivision = EMidiClockSubdivisionQuantization::None;
							}
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
			];
		ActionButtonPanel->AddSlot()
			.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
			[
				SNew(SButton)
					.Text(LOCTEXT("ImporterFactory_Button_Dont_Conform", "Don't Conform"))
					.OnClicked_Lambda([this, ParentWindow = InArgs._ParentWindow]()->FReply
						{
							bDoForAll = false;
							bUserTookAction = true;
							ConformSubdivision = EMidiClockSubdivisionQuantization::None;
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
			];
		ActionButtonPanel->AddSlot()
			.Padding(5, 5).HAlign(HAlign_Center).VAlign(VAlign_Center).AutoWidth()
			[
				SNew(SButton)
					.Text(LOCTEXT("ImporterFactory_Button_Dont_Conform_Any", "Don't Conform Any"))
					.OnClicked_Lambda([this, ParentWindow = InArgs._ParentWindow]()->FReply
						{
							bDoForAll = true;
							bUserTookAction = true;
							ConformSubdivision = EMidiClockSubdivisionQuantization::None;
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
			];
	}

	InArgs._ParentWindow->SetContent(this->AsShared());
}

#undef LOCTEXT_NAMESPACE
