// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"
#include "HarmonixMidi/MidiFile.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/Visibility.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Templates/SharedPointer.h"
#include "Framework/Application/SlateApplication.h"

#include "MidiFileFactory.generated.h"

// Imports a standard midi file
UCLASS()
class UMidiFileFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ BEGIN UFactory interface
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual void CleanUp() override;
	//~ END of UFactory interface

	//~ BEGIN FReimportHandler interface --
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual void PostImportCleanUp() override;
	//~ END FReimportHandler interface --

	static constexpr int32 kMaxTickErrorForTrivialConform = 2;
	static bool LengthCanBeTriviallyConformed(UMidiFile* MidiFile);

	//keep track of files that are imported for the pop-up dialog (multi batch)
	TArray<UMidiFile*> ImportedFiles;
	bool bApplyGrossConformToAll = false;
	bool bDontApplyGrossConformToAll = false;
	bool bApplyOffByOneConformToAll = false;
	bool bDontApplyOffByOneConformToAll = false;
	EMidiFileQuantizeDirection ApplyGrossConformDirection = EMidiFileQuantizeDirection::Up;
	EMidiClockSubdivisionQuantization GrossConformSubdivision = EMidiClockSubdivisionQuantization::None;

	// making these static means we can call them from other places in the editor UI...
	static void AskOrDoTrivialConform(UMidiFile* MidiFileAsset, bool bIsOneOfMany, UMidiFileFactory* CallingFactory, int32 CurrentLengthTicks, int32 QuantizedLengthTick);
	static void AskOrDoGrossConform(UMidiFile* MidiFileAsset, bool bIsOneOfMany, UMidiFileFactory* CallingFactory, int32 CurrentLengthTicks, int32 QuantizedLengthTick, EMidiClockSubdivisionQuantization BestSubdivision);

private:
	void CheckImportedFilesForValidLengths();
};

/* 
* A Custom Widget Class for displaying a pop-up window upon importing an midi file, 
* providing the option to conform midi file length to a new length 
* by rounding up, rounding down, or rounding to the nearest integer bar depending on conform option selection)
*/

class SConformMidiFileLengthDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConformMidiFileLengthDialog)
		: _ParentWindow()
		, _bMultipleFiles(false)
		, _FileDescription()
		, _RecommendedDirection(EMidiFileQuantizeDirection::Nearest)
		, _RecommendedSubdivision(EMidiClockSubdivisionQuantization::Bar)
	{}
	
	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_ARGUMENT(bool, bMultipleFiles)
	SLATE_ARGUMENT(FText, FileDescription)
	SLATE_ARGUMENT(EMidiFileQuantizeDirection, RecommendedDirection)
	SLATE_ARGUMENT(EMidiClockSubdivisionQuantization, RecommendedSubdivision)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	//custom widgets 
	const FTextBlockStyle MidiFileInformationStyle = FTextBlockStyle().SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 10)).SetColorAndOpacity(FLinearColor::White);
	EMidiFileQuantizeDirection ConformDirection = EMidiFileQuantizeDirection::Up;
	EMidiClockSubdivisionQuantization ConformSubdivision = EMidiClockSubdivisionQuantization::Bar;
	bool bDoForAll = false;
	bool bUserTookAction = false;

private:
	TArray<TSharedPtr<FString>> DirectionNames;
	TSharedPtr<FString> SelectedDirection;

	struct FSubdivisionEntry
	{
		EMidiClockSubdivisionQuantization Value;
		FString DisplayString;
		FSubdivisionEntry(EMidiClockSubdivisionQuantization InValue, const FString& InString)
			: Value(InValue)
			, DisplayString(InString)
		{}
	};
	TArray<TSharedPtr<FSubdivisionEntry>> QuantizationSubdivisions;
	TSharedPtr<FSubdivisionEntry> SelectedQuantizationSubdivision;
};