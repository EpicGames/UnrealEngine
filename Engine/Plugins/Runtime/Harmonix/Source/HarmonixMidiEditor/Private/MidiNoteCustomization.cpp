// Copyright Epic Games, Inc. All Rights Reserved.
#include "MidiNoteCustomization.h"

#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/Blueprint/MidiNote.h"

#include "Editor.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyCustomizationHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "MidiNoteCustomization"

void FMidiNoteCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	MidiNoteValuePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMidiNote, NoteNumber));

	FPropertyComboBoxArgs ComboArgs(MidiNoteValuePropertyHandle,
		FOnGetPropertyComboBoxStrings::CreateSP(this, &FMidiNoteCustomization::OnGetStrings),
		FOnGetPropertyComboBoxValue::CreateSP(this, &FMidiNoteCustomization::OnGetValueString),
		FOnPropertyComboBoxValueSelected::CreateSP(this, &FMidiNoteCustomization::OnValueStringSelected)
	);
	ComboArgs.ShowSearchForItemCount = 1;

	HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), FText::GetEmpty())
		]
	.ValueContent()
		.MaxDesiredWidth(0.0f) // don't constrain the combo button width
		[
			PropertyCustomizationHelpers::MakePropertyComboBox(ComboArgs)
		];
}


void FMidiNoteCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{

}

void FMidiNoteCustomization::OnValueStringSelected(const FString& SelectedString)
{
	if (!MidiNoteValuePropertyHandle.IsValid() || !MidiNoteValuePropertyHandle->IsValidHandle())
		return;

	MidiNoteValuePropertyHandle->SetValue(FMidiNote::NoteNumberFromEditorString(SelectedString));
}


void FMidiNoteCustomization::OnGetStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	for (uint8 NoteNumber = Harmonix::Midi::Constants::GMinNote; NoteNumber < Harmonix::Midi::Constants::GMaxNumNotes; ++NoteNumber)
	{
		OutStrings.Add(MakeShared<FString>(FMidiNote(NoteNumber).ToEditorString()));
		OutRestrictedItems.Add(false);
	}
}

FString FMidiNoteCustomization::OnGetValueString() const
{
	if (!MidiNoteValuePropertyHandle.IsValid() || !MidiNoteValuePropertyHandle->IsValidHandle())
		return LOCTEXT("MidiNote_None", "INVALID").ToString();

	uint8 NoteNumber;
	const FPropertyAccess::Result RowResult = MidiNoteValuePropertyHandle->GetValue(NoteNumber);
	if (RowResult == FPropertyAccess::Success)
	{
		return FMidiNote(NoteNumber).ToEditorString();
	}
	else if (RowResult == FPropertyAccess::Fail)
	{
		return LOCTEXT("MidiNote_None", "INVALID").ToString();
	}
	else
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
}

#undef LOCTEXT_NAMESPACE