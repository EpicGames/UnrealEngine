// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/SToolTip.h"
#include "IPropertyUtilities.h"

/**
* Exposes a dropdown on FMidiNote properties in the detail panel.
*/
class FMidiNoteCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FMidiNoteCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	/** Get list of all rows */
	void OnGetStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const;

	/** Gets currently selected row display string */
	FString OnGetValueString() const;

	void OnValueStringSelected(const FString& SelectedString);

	/** Handle to the struct properties being customized */
	TSharedPtr<IPropertyHandle> MidiNoteValuePropertyHandle;

};