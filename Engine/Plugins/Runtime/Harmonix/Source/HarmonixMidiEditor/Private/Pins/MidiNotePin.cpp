// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiNotePin.h"

#include "HarmonixMidi/MidiConstants.h"
#include "HarmonixMidi/Blueprint/MidiNote.h"

#include "PropertyCustomizationHelpers.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "SNameComboBox.h"
#include "ScopedTransaction.h"

void SMidiNotePin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

void SMidiNotePin::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SGraphPin::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (ValueWidget.IsValid())
		ValueWidget->SetVisibility(IsConnected() ? EVisibility::Collapsed : EVisibility::Visible);
}

TSharedRef<SWidget> SMidiNotePin::GetDefaultValueWidget()
{
	FPropertyComboBoxArgs comboArgs;
	comboArgs.OnGetStrings = FOnGetPropertyComboBoxStrings::CreateSP(this, &SMidiNotePin::OnGetStrings);
	comboArgs.OnGetValue = FOnGetPropertyComboBoxValue::CreateSP(this, &SMidiNotePin::OnGetValueString);
	comboArgs.OnValueSelected = FOnPropertyComboBoxValueSelected::CreateSP(this, &SMidiNotePin::OnValueSelected);

	// use this nifty property combo box used for Details customization, but works well enough here.
	return PropertyCustomizationHelpers::MakePropertyComboBox(comboArgs);
}

void SMidiNotePin::OnGetStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	for (uint8 noteNumber = Harmonix::Midi::Constants::GMinNote; noteNumber < Harmonix::Midi::Constants::GMaxNumNotes; ++noteNumber)
	{
		OutStrings.Add(MakeShared<FString>(FMidiNote(noteNumber).ToEditorString()));
		OutRestrictedItems.Add(false);
	}
}

FString SMidiNotePin::OnGetValueString() const
{
	check(GraphPinObj);
	check(GraphPinObj->PinType.PinSubCategoryObject == FMidiNote::StaticStruct());

	// The value is saved in the format (MyPropertyName=0) as a FString for an integer value.
	// So we have to retrieve the real value and convert it if necessary
	FString outString = GraphPinObj->GetDefaultAsString();

	if (outString.StartsWith(TEXT("(")) && outString.EndsWith(TEXT(")")))
	{
		outString = outString.LeftChop(1);
		outString = outString.RightChop(1);
		outString.Split("=", NULL, &outString);
	}

	int noteNumber = FCString::Atoi(*outString);
	
	return FMidiNote(noteNumber).ToEditorString();
}

void SMidiNotePin::OnValueSelected(const FString& value)
{
	check(GraphPinObj);
	check(GraphPinObj->PinType.PinSubCategoryObject == FMidiNote::StaticStruct());

	int noteNumber = FMidiNote::NoteNumberFromEditorString(value);

	// To set the property we need to use a FString
	// using this format: (MyPropertyName="My Value"), or in the case of an integer: (MyPropertyName=12)
	// NoteNumber is the property defined in our struct FInputActionHandle
	FName PropertyName = GET_MEMBER_NAME_CHECKED(FMidiNote, NoteNumber);
	FString PinString = FString::Printf(TEXT("(%s="), *PropertyName.ToString());
	PinString += FString::FromInt(noteNumber);
	PinString += TEXT(")");

	FString CurrentDefaultValue = GraphPinObj->GetDefaultAsString();

	if (CurrentDefaultValue != PinString)
	{
		const FScopedTransaction Transaction(
			NSLOCTEXT("GraphEditor", "ChangeMidiNote", "Change Midi Note"));
		GraphPinObj->Modify();

		if (PinString != GraphPinObj->GetDefaultAsString())
		{
			// Its important to use this function instead of GraphPinObj->DefaultValue
			// directly, cause it notifies the node from the pin is attached to.
			// So the node can listen to this change and do things internally.
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, PinString);
		}
	}
}
