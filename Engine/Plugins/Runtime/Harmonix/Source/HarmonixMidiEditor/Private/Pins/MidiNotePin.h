// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

class SNameComboBox;

/**
* Owner: Jake Burga
*
* Exposes a dropdown on FMidiNote pins
* dropdown shows midi notes from 0-127 with friendly names
* 
*/
class SMidiNotePin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SMidiNotePin) {}
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	// this override is used to display slate widget used for customization.
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override;

	/** Get list of all rows */
	void OnGetStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const;

	/** Gets currently selected row display string */
	FString OnGetValueString() const;

	// Triggers when drop down item is selected
	void OnValueSelected(const FString& value);
};