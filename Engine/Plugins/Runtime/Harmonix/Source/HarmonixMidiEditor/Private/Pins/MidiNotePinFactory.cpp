// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiNotePinFactory.h"

#include "HarmonixMidi/Blueprint/MidiNote.h"
#include "MidiNotePin.h"

TSharedPtr<class SGraphPin> FMidiNotePinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	/*
	 * Check if pin is struct, and then check if that pin is of struct type we want to customize
	 */
	if (InPin->PinType.PinCategory == K2Schema->PC_Struct)
	{
		if (InPin->PinType.PinSubCategoryObject == FMidiNote::StaticStruct())
		{
			return SNew(SMidiNotePin, InPin);
		}
	}

	return nullptr;
}
