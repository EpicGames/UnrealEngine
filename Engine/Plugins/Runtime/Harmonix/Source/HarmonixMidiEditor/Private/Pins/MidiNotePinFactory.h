// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"

/**
* Factory class that creates a custom Graph Pin for FMidiNote struct pins.
*/
class FMidiNotePinFactory : public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;
};