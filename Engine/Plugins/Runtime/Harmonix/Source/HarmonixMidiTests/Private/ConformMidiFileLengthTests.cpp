// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiTestUtility.h"
#include "Misc/AutomationTest.h"
#include "Internationalization/Regex.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMidiTests::ConformMidiFileLength
{
	using namespace Harmonix::Testing::Utility::MidiTestUtility;
		
	/**
	* Add some events to the input midi file for testing the ConformMidiFileLength() function
	* 
	************************
	* Midi File Structure: *
	************************
	*
	*For each Midi track / channel :
	*	1 Note On / Note Off pair will be added at the 1st beat of every bar, note duration is 1 beat(TicksPerQuarter)
	*For every EVEN bar number :
	*	1 CC event and 1 Text event will be added at the same position as the Note On events
	*For every ODD bar number :
	*	1 Pitch Bend event and 1 Poly Pres event will be added at the same position as the Note On events
	*If input file length(bars) is a fractional number :
	*	1 Note On / Note Off pair and 1 Text event will be added to the tick position of that fractional bar length
	*	(Note On and Note Off events are 1 tick away so they don't end up on the same tick)
	*/
	void AddEventsToTestMidiFile(UMidiFile* InMidiFile, float InFileLengthBars, int32 InNumChannels, int32 InNumTracksExcludingConductorTrack, int32& RemovableNotes, int32& RemoveableControlChanges, int32& RemovablePitchAndPoly)
	{
		constexpr int32 DefaultNoteNumber = 60;//C4
		constexpr int32 DefaultNoteVelocity = 90;
		constexpr uint8 DefaultControllerID = 69; //Hold Pedal
		constexpr uint8 DefaultControlValue = 120;
		constexpr uint8 DefaultPitchBendValueLSB = 64;
		constexpr uint8 DefaultPitchBendValueMSB = 120;
		constexpr uint8 DefaultPolyPressNoteNumber = 62;
		constexpr uint8 DefaultPolyPresValue = 127;

		// These will be useful below...
		const FSongMaps* SongMaps = InMidiFile->GetSongMaps();

		RemovableNotes = 0;
		RemoveableControlChanges = 0;
		RemovablePitchAndPoly = 0;

		// Add events to track 1 - InNumTracks, Track 0 is the conductor track
		for (int32 TrackIndex = 1; TrackIndex < (InNumTracksExcludingConductorTrack + 1); ++TrackIndex)
		{
			FMidiTrack* CurrentTrack = InMidiFile->GetTrack(TrackIndex);
			for (int32 Channel = 0; Channel < InNumChannels; ++Channel)
			{
				for (int32 Bar = 0; Bar < FMath::FloorToInt32(InFileLengthBars); ++Bar)
				{
					int32 DestinationTick = SongMaps->BarBeatTickIncludingCountInToTick(Bar, 1, 0);
					int32 Duration = SongMaps->SubdivisionToMidiTicks(EMidiClockSubdivisionQuantization::Beat, DestinationTick) - 1;
					AddNoteOnNoteOffPairToFile(InMidiFile, DefaultNoteNumber, DefaultNoteVelocity, TrackIndex, Channel, DestinationTick, Duration);

					// Currently, Midi CC events & Midi Text events are added to bars with EVEN bar numbers,
					// Midi Poly Press & Pitch Bend event are added to bars with ODD bar numbers
					if (Bar % 2 == 0)
					{
						// Add Control Events to each channels/tracks
						AddCCEventToFile(InMidiFile, DefaultControllerID, DefaultNoteNumber, TrackIndex, Channel, DestinationTick);
						// Add Text events to tracks
						if (Channel == 0)
						{
							AddTextEventToFile(InMidiFile, TEXT("TextInMidiFile"), TrackIndex, DestinationTick);
						}
					}
					else
					{
						//Add pitch bend events to channels/tracks
						AddPitchEventToFile(InMidiFile, DefaultPitchBendValueLSB, DefaultPitchBendValueMSB, TrackIndex, Channel, DestinationTick);
						//Add Poly Pres events to channels/tracks
						AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue, TrackIndex, Channel, DestinationTick);
					}
				}
				// If bar length is a fractional number, add additional midi events after the last integer bar
				if (InFileLengthBars != (int32)InFileLengthBars)
				{
					int32 DestinationTick = SongMaps->FractionalBarIncludingCountInToTick(InFileLengthBars);
					int32 Duration = SongMaps->SubdivisionToMidiTicks(EMidiClockSubdivisionQuantization::Beat, DestinationTick) - 1;
					int32 NoteDestinationTick = DestinationTick - (Duration + 1);
					DestinationTick--;

					// This will end up getting removed
					AddNoteOnNoteOffPairToFile(InMidiFile, DefaultNoteNumber, DefaultNoteVelocity, TrackIndex, Channel, NoteDestinationTick, Duration);
					RemovableNotes++;

					// Add 2 CC Events (same controller ID) to test the conform function where it should remove events with the same type on the last tick
					// 1 of these should be removed by the conform...
					AddCCEventToFile(InMidiFile, DefaultControllerID, DefaultControlValue, TrackIndex, Channel, DestinationTick);
					RemoveableControlChanges++;
					AddCCEventToFile(InMidiFile, DefaultControllerID, DefaultControlValue + 1, TrackIndex, Channel, DestinationTick);

					// Add 3 Pitch Bend Events to test the conform function where it should remove events with the same type on the last tick
					// 2 of these should be removed by the conform...
					AddPitchEventToFile(InMidiFile, DefaultPitchBendValueLSB, DefaultPitchBendValueMSB, TrackIndex, Channel, DestinationTick);
					AddPitchEventToFile(InMidiFile, DefaultPitchBendValueLSB + 1, DefaultPitchBendValueMSB + 1, TrackIndex, Channel, DestinationTick);
					RemovablePitchAndPoly += 2;
					AddPitchEventToFile(InMidiFile, DefaultPitchBendValueLSB - 1, DefaultPitchBendValueMSB - 1, TrackIndex, Channel, DestinationTick);

					// Add 4 Poly pressure Events to test the conform function where it should remove events with the same type on the last tick
					// 3 of these should be removed by the conform...
					AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue, TrackIndex, Channel, DestinationTick);
					AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue - 1, TrackIndex, Channel, DestinationTick);
					AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue - 2, TrackIndex, Channel, DestinationTick);
					RemovablePitchAndPoly += 3;
					AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue + 1, TrackIndex, Channel, DestinationTick);
				}
			}
			InMidiFile->GetTrack(TrackIndex)->Sort();
		}

		//update file information in SongLengthData
		InMidiFile->ScanTracksForSongLengthChange();
	}

	/**
	 * Check if a Midi file has excessive midi events on the last tick after its length is conformed by ROUNDING DOWN
	 * ConformMidiFileLength(EMidiFileLengthConformOption::RoundDown) first move all the events exceeding the last integer bar to the last tick of 
	 * the last integer bar, and then remove Note On/Note Off pairs, CC events with the same controller ID, Poly Pres events with the same note number
	 * and chan pres/pitch bend events that are on the same (last) tick
	 * this function validates these results
	 */
	bool ContainsExcessiveEventsAtLastEventTick(UMidiFile* ConformedMidiFile)
	{
		int32 ConformedLastEventTick = ConformedMidiFile->GetLastEventTick();

		for (FMidiTrack& Track : ConformedMidiFile->GetTracks())
		{
			const FMidiEventList& Events = Track.GetEvents();

			for (int EventIndex = Events.Num() - 1; EventIndex >= 0 && Events[EventIndex].GetTick() == ConformedLastEventTick; --EventIndex)
			{
				using namespace Harmonix::Midi::Constants;

				const FMidiEvent& Event = Events[EventIndex];
				const FMidiMsg& Msg = Event.GetMsg();

				if (Msg.IsStd())
				{
					uint8 MsgType = Msg.GetStdStatusType();

					//filter Note On/Note off pairs on the last tick
					if (MsgType == GNoteOff)
					{
						for (int32 i = EventIndex - 1; i > 0 && Events[i].GetTick() == ConformedLastEventTick; --i)
						{
							//check equality of note on/note off events' midi note number (data1) 
							if (Events[i].GetMsg().IsNoteOn() && Events[i].GetMsg().GetStdData1() == Msg.GetStdData1())
							{
								return true;
							}
						}
					}

					//filter chan press/pitch bend/program change events
					else if (MsgType == GChanPres || MsgType == GPitch || MsgType == GProgram)
					{
						//check if there exist multiple events with same status on the last tick
						for (int32 i = EventIndex - 1; i >= 0 && Events[i].GetTick() == ConformedLastEventTick; --i)
						{
							if (Events[i].GetMsg().IsStd() &&
								Events[i].GetMsg().Status == Msg.Status)
							{
								return true;
							}
						}
					}

					//filter Control Change events and Poly Pres events
					else if (MsgType == GControl || MsgType == GPolyPres)
					{
						for (int32 i = EventIndex - 1; i >= 0 && Events[i].GetTick() == ConformedLastEventTick; --i)
						{
							//check control change events for identical controller ID (data1) on the same tick and remove the later one
							//check poly pres events for the same note number (data1) on the same tick and remove the later one
							if (Events[i].GetMsg().IsStd() &&
								Events[i].GetMsg().GetStdStatus() == Msg.GetStdStatus() &&
								Events[i].GetMsg().Data1 == Msg.Data1)
							{
								return true;
							}
						}
					}
				}
			}
		}

		return false;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestConformMidiFileLength,
		"Harmonix.Midi.ConformMidiFileLength",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestConformMidiFileLength::RunTest(const FString&)
	{
		//input test values for creating a midi file
		constexpr float FileLengthBars = 5.5f;
		constexpr int32 NumChannels = 2;
		constexpr int32 NumTracksIncludingConductor = 4;
		constexpr int32 NumTracksExcludingConductor = NumTracksIncludingConductor - 1;
		constexpr int32 TimeSigNum5 = 5;
		constexpr int32 TimeSigDenum8 = 8;
		constexpr int32 TimeSigNum4 = 4;
		constexpr int32 TimeSigDenum4 = 4;
		constexpr int32 Tempo = 120;
		

		// We should get a warning and no quantization if we ask to round down because that would result in a zero length file. 
		AddExpectedMessage(TEXT("QuantizeLengthToNearestPerfectSubdivision: Asked to Quantize file length DOWN, but that would result in a zero length midi file. NOT ALLOWED! Skipping quantization!"), ELogVerbosity::Warning);
		constexpr float FileLength0Bar = 0.0;
		UMidiFile* MidiFile = CreateAndInitializaMidiFile(FileLength0Bar, NumTracksIncludingConductor, TimeSigNum4, TimeSigDenum4, Tempo, true);
		FSongLengthData PreQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();
		MidiFile->QuantizeLengthToSubdivision(EMidiFileQuantizeDirection::Down, EMidiClockSubdivisionQuantization::Bar);
		FSongLengthData PostQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();

		int32 ExpectedLengthTickPostQuantize = 1;
		int32 ExpectedLastTickPostQuantize = 0;

		UTEST_TRUE("Last ticks are correct pre/post quantization.",
			PreQuantizeLengthData.LastTick == 0 &&
			PostQuantizeLengthData.LastTick == ExpectedLastTickPostQuantize);

		UTEST_TRUE("Length ticks are correct pre/post quantization.",
			PreQuantizeLengthData.LengthTicks == 1 &&
			PostQuantizeLengthData.LengthTicks == ExpectedLengthTickPostQuantize);

		// "Nearest" should always be up since down would result in a zero length midi file...
		PreQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();
		MidiFile->QuantizeLengthToSubdivision(EMidiFileQuantizeDirection::Nearest, EMidiClockSubdivisionQuantization::Bar);
		PostQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();

		ExpectedLengthTickPostQuantize = (MidiFile->GetSongMaps()->GetTicksPerQuarterNote() * 4); // <-- because we know this test has a time signature of 4/4
		ExpectedLastTickPostQuantize = ExpectedLengthTickPostQuantize - 1;

		UTEST_TRUE("Last ticks are correct pre/post quantization.",
			PreQuantizeLengthData.LastTick == 0 &&
			PostQuantizeLengthData.LastTick == ExpectedLastTickPostQuantize);

		UTEST_TRUE("Length ticks are correct pre/post quantization.",
			PreQuantizeLengthData.LengthTicks == 1 &&
			PostQuantizeLengthData.LengthTicks == ExpectedLengthTickPostQuantize);

		// "Up" should definitely work...
		MidiFile = CreateAndInitializaMidiFile(FileLength0Bar, NumTracksIncludingConductor, TimeSigNum4, TimeSigDenum4, Tempo, true);
		PreQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();
		MidiFile->QuantizeLengthToSubdivision(EMidiFileQuantizeDirection::Up, EMidiClockSubdivisionQuantization::Bar);
		PostQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();

		ExpectedLengthTickPostQuantize = (MidiFile->GetSongMaps()->GetTicksPerQuarterNote() * 4); // <-- because we know this test has a time signature of 4/4
		ExpectedLastTickPostQuantize = ExpectedLengthTickPostQuantize - 1;

		UTEST_TRUE("Last ticks are correct pre/post quantization.",
			PreQuantizeLengthData.LastTick == 0 &&
			PostQuantizeLengthData.LastTick == ExpectedLastTickPostQuantize); 

		UTEST_TRUE("Length ticks are correct pre/post quantization.",
			PreQuantizeLengthData.LengthTicks == 1 &&
			PostQuantizeLengthData.LengthTicks == ExpectedLengthTickPostQuantize);

		// Let's try a longer length in 5/8...
		constexpr float TestFileLengthNearest = 5.25;
		MidiFile = CreateAndInitializaMidiFile(TestFileLengthNearest, NumTracksIncludingConductor, TimeSigNum5, TimeSigDenum8, Tempo, true);
		//Add some events to the midi file for testing 
		int32 RemoveableNotes = 0;
		int32 RemoveablePicthAndPoly = 0;
		int32 RemoveableControl = 0;
		AddEventsToTestMidiFile(MidiFile, TestFileLengthNearest, NumChannels, NumTracksExcludingConductor, RemoveableNotes, RemoveableControl, RemoveablePicthAndPoly);
		PreQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();
		MidiFile->QuantizeLengthToSubdivision(EMidiFileQuantizeDirection::Down, EMidiClockSubdivisionQuantization::Bar);
		PostQuantizeLengthData = MidiFile->GetSongMaps()->GetSongLengthData();

		ExpectedLengthTickPostQuantize = ((MidiFile->GetSongMaps()->GetTicksPerQuarterNote() / 2) * 5) * 5; // <-- because we know this test has a time signature of 4/4
		ExpectedLastTickPostQuantize = ExpectedLengthTickPostQuantize - 1;

		UTEST_TRUE("Last ticks are correct pre/post quantization.",
			PreQuantizeLengthData.LastTick == ((((MidiFile->GetSongMaps()->GetTicksPerQuarterNote() / 2) * 5) * TestFileLengthNearest) - 1) &&
			PostQuantizeLengthData.LastTick == ExpectedLastTickPostQuantize);

		UTEST_TRUE("Length ticks are corrent pre/post quantization.",
			PreQuantizeLengthData.LengthTicks == (((MidiFile->GetSongMaps()->GetTicksPerQuarterNote() / 2) * 5) * TestFileLengthNearest) &&
			PostQuantizeLengthData.LengthTicks == ExpectedLengthTickPostQuantize);

		UTEST_FALSE("No excessive events on conformed last tick.", ContainsExcessiveEventsAtLastEventTick(MidiFile));

		return true;
	}

}
#endif