// Copyright Epic Games, Inc. All Rights Reserved.

#include "MidiTestUtility.h"
#include "HarmonixMidi/VarLenNumber.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogMIDITest, Log, All);

namespace HarmonixMidiTests::ImportExportTests
{
	using namespace Harmonix::Testing::Utility::MidiTestUtility;
		
		/**
	* Add some events to the input midi file for testing the ConformMidiFileLength() function
	* 
	************************
	* Midi File Structure: *
	************************
	*
	*For each Midi track / channel :
	*	1 Note On / Note Off pair will be added at the 1st beat of every bar, note duration is 1 beat(TicksPerQuarter)
	*For every EVEN bar number :
	*	1 CC event and 1 Text event will be added at the same position as the Note On events
	*For every ODD bar number :
	*	1 Pitch Bend event and 1 Poly Pres event will be added at the same position as the Note On events
	*If input file length(bars) is a fractional number :
	*	1 Note On / Note Off pair and 1 Text event will be added to the tick position of that fractional bar length
	*	(Note On and Note Off events are 1 tick away so they don't end up on the same tick)
	*/
	void AddEventsToTestMidiFile(UMidiFile* InMidiFile, int32 InNumTracksExcludingConductorTrack, int32 InNumChannels, float InFileLengthBars)
	{
		constexpr int32 DefaultNoteNumber = 60;//C4
		constexpr int32 DefaultNoteVelocity = 90;
		constexpr uint8 DefaultControllerID = 69; //Hold Pedal
		constexpr uint8 DefaultControlValue = 127;
		constexpr uint8 DefaultPitchBendValueLSB = 64;
		constexpr uint8 DefaultPitchBendValueMSB = 127;
		constexpr uint8 DefaultPolyPressNoteNumber = 62;
		constexpr uint8 DefaultPolyPresValue = 127;

		// these will be useful below...
		const FSongMaps* SongMaps =  InMidiFile->GetSongMaps();
		
		//Add events to track 1 - InNumTracks, Track 0 is the conductor track
		for (int32 TrackIndex = 1; TrackIndex < (InNumTracksExcludingConductorTrack + 1); ++TrackIndex)
		{
			FMidiTrack* CurrentTrack = InMidiFile->GetTrack(TrackIndex);
			for (int32 Channel = 0; Channel < InNumChannels; ++Channel)
			{
				for (int32 Bar = 0; Bar < FMath::CeilToInt32(InFileLengthBars); ++Bar)
				{
					int32 DestinationTick = SongMaps->BarBeatTickIncludingCountInToTick(Bar, 1, 0);
					int32 Duration = SongMaps->SubdivisionToMidiTicks(EMidiClockSubdivisionQuantization::Beat, DestinationTick) - 1;
					AddNoteOnNoteOffPairToFile(InMidiFile, DefaultNoteNumber, DefaultNoteVelocity, TrackIndex, Channel, DestinationTick, Duration);

					//Currently, Midi CC events & Midi Text events are added to bars with EVEN bar numbers,
					//Midi Poly Press & Pitch Bend event are added to bars with ODD bar numbers
					if (Bar % 2 == 0)
					{
						//Add Control Events to each channels/tracks
						AddCCEventToFile(InMidiFile, DefaultControllerID, DefaultNoteNumber, TrackIndex, Channel, DestinationTick);
						//Add Text events to tracks
						AddTextEventToFile(InMidiFile, TEXT("TextInMidiFile"), TrackIndex, DestinationTick);
					}
					else
					{
						//Add pitch bend events to channels/tracks
						AddPitchEventToFile(InMidiFile, DefaultPitchBendValueLSB, DefaultPitchBendValueMSB, TrackIndex, Channel, DestinationTick);
						//Add Poly Pres events to channels/tracks
						AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue, TrackIndex, Channel, DestinationTick);
					}
				}
				//if bar length is a fractional number, add additional midi events after the last integer bar
				if (InFileLengthBars != (int32)InFileLengthBars)
				{
					int32 DestinationTick = SongMaps->FractionalBarIncludingCountInToTick(InFileLengthBars) - 1;
					int32 Duration = SongMaps->SubdivisionToMidiTicks(EMidiClockSubdivisionQuantization::Beat, DestinationTick) - 1;
					int32 NoteDestinationTick = DestinationTick - (Duration + 1);

					//This will end up getting removed
					AddNoteOnNoteOffPairToFile(InMidiFile, DefaultNoteNumber, DefaultNoteVelocity, TrackIndex, Channel, NoteDestinationTick, Duration);
					//Add 2 CC Events (same controller ID) to test the conform function where it should remove events with the same type on the last tick
					AddCCEventToFile(InMidiFile, DefaultControllerID, DefaultControlValue, TrackIndex, Channel, DestinationTick);
					AddCCEventToFile(InMidiFile, DefaultControllerID, DefaultControlValue + 1, TrackIndex, Channel, DestinationTick);

					//Add 2 Pitch Bend Events to test the conform function where it should remove events with the same type on the last tick
					AddPitchEventToFile(InMidiFile, DefaultPitchBendValueLSB, DefaultPitchBendValueMSB, TrackIndex, Channel, DestinationTick);
					AddPitchEventToFile(InMidiFile, DefaultPitchBendValueLSB + 1, DefaultPitchBendValueMSB + 1, TrackIndex, Channel, DestinationTick);

					//Add 2 Pitch Bend Events to test the conform function where it should remove events with the same type on the last tick
					AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue, TrackIndex, Channel, DestinationTick);
					AddPolyPresEventToFile(InMidiFile, DefaultPolyPressNoteNumber, DefaultPolyPresValue + 1, TrackIndex, Channel, DestinationTick);
				}
			}
			InMidiFile->GetTrack(TrackIndex)->Sort();
		}

		//update file information in SongLengthData
		InMidiFile->ScanTracksForSongLengthChange();
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestMidiFileImportExport,
		"Harmonix.Midi.MidiFile.ImportExport",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestMidiFileImportExport::RunTest(const FString&)
	{
		//input test values for creating a midi file
		const float FileLengthBars = 5.5f;
		const int32 NumChannels = 2;
		const int32 NumTracksIncludingConductorTrack = 4;
		const int32 NumTracksExcludingConductorTrack = NumTracksIncludingConductorTrack - 1;
		const int32 TimeSigNum6 = 6;
		const int32 TimeSigDenum8 = 8;
		const int32 TimeSigNum4 = 4;
		const int32 TimeSigDenum4 = 4;
		const int32 Tempo = 120;
		
		//create an empty midi file to test for rounding to nearest (down)
		UMidiFile* GeneratedMidiFile = CreateAndInitializaMidiFile(FileLengthBars, NumTracksIncludingConductorTrack, TimeSigNum4, TimeSigDenum4, Tempo, true);
		//Add some events to the midi file for testing 
		AddEventsToTestMidiFile(GeneratedMidiFile, NumTracksExcludingConductorTrack, NumChannels, FileLengthBars);

		TArray<uint8> StdMidiFileBytes;
		TSharedPtr<FMemoryWriter> StdMidiFileOut = MakeShared<FMemoryWriter>(StdMidiFileBytes, true);
		GeneratedMidiFile->SaveStdMidiFile(StdMidiFileOut);

		UMidiFile* ReimportedMidiFile = NewObject<UMidiFile>();
		TSharedPtr<FMemoryReader> StdMidiFileIn = MakeShared<FMemoryReader>(StdMidiFileBytes, true);
		ReimportedMidiFile->LoadStdMidiFile(StdMidiFileIn, TEXT("InMemoryMidiFile"));

		// TODO
		//TestTrue(TEXT("Generated midi file survives export/import cycle."),*GeneratedMidiFile == *ReimportedMidiFile);

		return true;
	}

	#define ADD_EXPECTED_MIDI_ERROR(errmsg) AddExpectedError(errmsg, EAutomationExpectedMessageFlags::Contains, 0)

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestMidiFileMalformed,
		"Harmonix.Midi.MidiFile.MalformedDataHandling",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestMidiFileMalformed::RunTest(const FString&)
	{
		//input test values for creating a midi file
		const float FileLengthBars = 5.5f;
		const int32 NumChannels = 2;
		const int32 NumTracksIncludingConductorTrack = 4;
		const int32 NumTracksExcludingConductorTrack = NumTracksIncludingConductorTrack - 1;
		const int32 TimeSigNum6 = 6;
		const int32 TimeSigDenum8 = 8;
		const int32 TimeSigNum4 = 4;
		const int32 TimeSigDenum4 = 4;
		const int32 Tempo = 120;

		//create an empty midi file to test for rounding to nearest (down)
		UMidiFile* GeneratedMidiFile = CreateAndInitializaMidiFile(FileLengthBars, NumTracksIncludingConductorTrack, TimeSigNum4, TimeSigDenum4, Tempo, true);
		//Add some events to the midi file for testing 
		AddEventsToTestMidiFile(GeneratedMidiFile, NumTracksExcludingConductorTrack, NumChannels, FileLengthBars);

		TArray<uint8> StdMidiFileBytes;
		TSharedPtr<FMemoryWriter> StdMidiFileOut = MakeShared<FMemoryWriter>(StdMidiFileBytes, true);
		GeneratedMidiFile->SaveStdMidiFile(StdMidiFileOut);

		// Put errors in the data and check for proper handling!
		UMidiFile* ReimportedMidiFile = NewObject<UMidiFile>();
		TArray<uint8> FutzedMidiFileBytes = StdMidiFileBytes;
		TSharedPtr<FMemoryReader> FutzedMidiFileIn = MakeShared<FMemoryReader>(FutzedMidiFileBytes, true);
		FMemoryWriter FutzedMidiFileOut(FutzedMidiFileBytes, true);
		FutzedMidiFileOut.SetByteSwapping(true);

		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI import failed"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Found invalid MIDI status byte"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Finding events on track passed track data block!"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI file header is corrupt"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Only type 0 or 1 MIDI files are supported"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Format 0 file expected only one track"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI file has no tracks"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI file uses SMPTE time division."));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI file Ticks Per Quarter Note is not rational"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI track header for track 0 is corrupt"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI track data length exceeds maximum length!"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI track 0 data length as recorded in the header exceeds the amount of data in the file!"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("MIDI track 0 ... Track data underrun"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Found invalid MIDI status byte"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Unsupported MIDI tempo encountered"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Failed to add tempo to Track"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Time signature 0/1 at 1:1.000 has invalid numerator"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Time signature at 1:1.000 has invalid denominator"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Cannot parse meta event"));
		ADD_EXPECTED_MIDI_ERROR(TEXT("Cannot parse system event"));

		// Put an error in the header ID...
		{
			FutzedMidiFileBytes[1] = 0xFF;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file header ID appropriately failed."), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the header size...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[4] = 0xFF;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file header SIZE appropriately failed."), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Format...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(8);
			int16 BadFormat = 0x7EEF;
			FutzedMidiFileOut << BadFormat;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file format failed."), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Number of tracks...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(8);
			int16 Format = 0;
			int16 TrackNum = 3; // only one track allowed in format 0 file!
			FutzedMidiFileOut << Format;
			FutzedMidiFileOut << TrackNum;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Wrong number of tracks!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Number of tracks...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(8);
			int16 Format = 1;
			int16 TrackNum = -1; // negative number of tracks!
			FutzedMidiFileOut << Format;
			FutzedMidiFileOut << TrackNum;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Wrong number of tracks!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the TicksPerQuarter...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(12);
			uint16 TPQ = 0x8000;
			FutzedMidiFileOut << TPQ;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Ticks Per Quarter Note. File is SMPTE!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the TicksPerQuarter...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(12);
			uint16 TPQ = 481;
			FutzedMidiFileOut << TPQ;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Ticks Per Quarter Note. Not evenly divisible by 48!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Track Header ID...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[14] = 'Z';
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Track Header ID!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Track Header Size...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(18);
			int32 TrackDataSize = std::numeric_limits<int32>::max();
			FutzedMidiFileOut << TrackDataSize;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Track Header SIZE!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Track Header Size...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(18);
			int32 TrackDataSize = StdMidiFileBytes.Num();
			FutzedMidiFileOut << TrackDataSize;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Track Header SIZE!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Track Header Size...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(18);
			int32 TrackDataSize = 15;
			FutzedMidiFileOut << TrackDataSize;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Track Header SIZE!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the track name length...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileOut.Seek(25);
			uint32 StringLength = 0;
			Midi::VarLenNumber::Write(FutzedMidiFileOut, StringLength);
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad string length (zero)!"), ReimportedMidiFile->IsEmpty());

			for (int i = 1; i < 30; ++i)
			{
				if (i == 9 || i == 13 || i == 16 || i == 24)
				{
					// These lengths result in valid'ish strings and parse-able midi messages to follow, so 
					// don't bother testing...
					continue;
				}

				UE_LOGF(LogMIDITest, Log, "Testing string length %d...", i);
				FutzedMidiFileBytes = StdMidiFileBytes;
				FutzedMidiFileIn->Seek(0);
				FutzedMidiFileOut.Seek(25);
				StringLength = i;
				Midi::VarLenNumber::Write(FutzedMidiFileOut, StringLength);
				ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
				TestTrue(*FString::Format(TEXT("Currupt Midi file: Bad string length {0}!"), {i}), ReimportedMidiFile->IsEmpty());
			}
		}

		// Put an error in the Tempo...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[39] = 0;
			FutzedMidiFileBytes[40] = 0;
			FutzedMidiFileBytes[41] = 0;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Tempo 0!"), ReimportedMidiFile->IsEmpty());

			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[39] = 0;
			FutzedMidiFileBytes[40] = 0;
			FutzedMidiFileBytes[41] = 1;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Tempo 1!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an error in the Time Signature...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[46] = 0;
			FutzedMidiFileBytes[47] = 0;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Time Signature 0!"), ReimportedMidiFile->IsEmpty());

			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[46] = 127;
			FutzedMidiFileBytes[47] = 127;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: Bad Time Signature 1!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an invalid meta event type...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[24] = 0x10;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: unsupported meta event type!"), ReimportedMidiFile->IsEmpty());
		}

		// Put an unsupported system status message...
		{
			FutzedMidiFileBytes = StdMidiFileBytes;
			FutzedMidiFileIn->Seek(0);
			FutzedMidiFileBytes[23] = 0xf3;
			ReimportedMidiFile->LoadStdMidiFile(FutzedMidiFileIn, TEXT("InMemoryMidiFile"), Harmonix::Midi::Constants::GTicksPerQuarterNoteInt, Harmonix::Midi::Constants::EMidiTextEventEncoding::UTF8, false);
			TestTrue(TEXT("Currupt Midi file: unsupported system status!"), ReimportedMidiFile->IsEmpty());
		}

		return true;
	}
}
#endif