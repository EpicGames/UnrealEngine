// Copyright Epic Games, Inc. All Rights Reserved.
#include "MidiTestUtility.h"

namespace Harmonix::Testing::Utility::MidiTestUtility
{
	UMidiFile* CreateAndInitializaMidiFile(float FileLengthBars, int32 InNumTracksIncludingConductor, int32 InTimeSigNum, int32 InTimeSigDenom, int32 InTempo, bool PutTextEventOnLastTick)
	{
		UMidiFile* TheMidiFile = NewObject<UMidiFile>();

		//some default values for creating midi events
		constexpr int32 DefaultNoteNumber = 60;//C4
		constexpr int32 DefaultNoteVelocity = 90;
		constexpr uint8 DefaultControllerID = 69; //Hold Pedal
		constexpr uint8 DefaultControlValue = 127;
		constexpr uint8 DefaultPitchBendValueLSB = 64;
		constexpr uint8 DefaultPitchBendValueMSB = 127;
		constexpr uint8 DefaultPolyPresValue = 127;

		// Set initial tempo and time signature
		TheMidiFile->GetSongMaps()->AddTempoInfoPoint(Midi::Constants::BPMToMidiTempo(InTempo), 0);
		TheMidiFile->GetSongMaps()->AddTimeSignatureAtBarIncludingCountIn(0, InTimeSigNum, InTimeSigDenom);
		// Whenever you change the song maps you need to either:
		// a) Add corresponding midi events to the conductor track. Or...
		// b) Build the conductor track from scratch.
		// (b) is easier...
		TheMidiFile->BuildConductorTrack();

		// now we can calculate the tick length since the midi file now has a conductor track...
		int32 TickLength = TheMidiFile->GetSongMaps()->FractionalBarIncludingCountInToTick(FileLengthBars);
		if (TickLength <= 0)
		{
			// MIDI Files must be at least one tick long. Even the initial tempo and time signature are specified on tick 0!
			TickLength = 1;
		}

		//Add tracks to Midi file according to input argument
		for (int32 Track = 1; Track < InNumTracksIncludingConductor; ++Track)
		{
			//Add midi tracks 1 - InNumTracks to file, Track 0 is the conductor track
			FMidiTrack* TheNewTrack = TheMidiFile->AddTrack(FString::Printf(TEXT("TestMidiTrack%i"), Track));
			if (TheNewTrack && PutTextEventOnLastTick)
			{
				uint16 TextIndex = TheNewTrack->AddText(FString::Printf(TEXT("TheEndOfTrack%i"),Track));
				FMidiEvent TextEventAtTheEnd(TickLength - 1, FMidiMsg::CreateText(TextIndex, Midi::Constants::GMeta_Text));
				TheNewTrack->AddEvent(TextEventAtTheEnd);
			}
		}

		//update song length information
		TheMidiFile->ConformToLength(TickLength);

		return TheMidiFile;
	}

	void AddNoteOnNoteOffPairToFile(UMidiFile* InFile, int32 InNoteNumber, int32 InNoteVelocity, int32 InTrackIndex, int32 InChannel, int32 AtTick, int32 DurationTicks)
	{
		if (InTrackIndex >= InFile->GetTracks().Num() || InTrackIndex <= 0)
		{
			UE_LOGF(LogMIDI, Error, "AddNoteOnNoteOffPairToFile: Bad Track Index: %d", InTrackIndex);
			return;
		}

		FMidiTrack* CurrentTrack = InFile->GetTrack(InTrackIndex);
		//Note On / Note Off pair
		FMidiEvent CurrentNoteOnEvent(AtTick,
			FMidiMsg::CreateNoteOn(InChannel, InNoteNumber, InNoteVelocity));
		//Note Offs are 1 beat away from Note Ons (ticks per quarter note)
		FMidiEvent CurrentNoteOffEvent(AtTick + DurationTicks,
			FMidiMsg::CreateNoteOff(InChannel, InNoteNumber));
		CurrentTrack->AddEvent(CurrentNoteOnEvent);
		CurrentTrack->AddEvent(CurrentNoteOffEvent);
	}

	void AddCCEventToFile(UMidiFile* InFile, uint8 InControllerID, uint8 InControlValue, int32 InTrackIndex, int32 InChannel, int32 AtTick)
	{
		if (InTrackIndex >= InFile->GetTracks().Num() || InTrackIndex <= 0)
		{
			UE_LOGF(LogMIDI, Error, "AddCCEventToFile: Bad Track Index: %d", InTrackIndex);
			return;
		}

		FMidiTrack* CurrentTrack = InFile->GetTrack(InTrackIndex);
		FMidiEvent CurrentCCEvent(AtTick,
			FMidiMsg(Midi::Constants::GControl | InChannel, InControllerID, InControlValue));
		CurrentTrack->AddEvent(CurrentCCEvent);
	}

	void AddTextEventToFile(UMidiFile* InFile, FString InText, int32 InTrackIndex, int32 AtTick)
	{
		if (InTrackIndex >= InFile->GetTracks().Num() || InTrackIndex <= 0)
		{
			UE_LOGF(LogMIDI, Error, "AddTextEventToFile: Bad Track Index: %d", InTrackIndex);
			return;
		}

		FMidiTrack* CurrentTrack = InFile->GetTrack(InTrackIndex);
		uint16 TextIndex = CurrentTrack->AddText(InText);
		FMidiEvent CurrentTextEvent(AtTick, FMidiMsg::CreateText(TextIndex, Midi::Constants::GMeta_Text));
		CurrentTrack->AddEvent(CurrentTextEvent);
	}

	void AddPitchEventToFile(UMidiFile* InFile, uint8 InPitchValueLSB, uint8 InPitchValueMSB, int32 InTrackIndex, int32 InChannel, int32 AtTick)
	{
		if (InTrackIndex >= InFile->GetTracks().Num() || InTrackIndex <= 0)
		{
			UE_LOGF(LogMIDI, Error, "AddPitchEventToFile: Bad Track Index: %d", InTrackIndex);
			return;
		}

		FMidiTrack* CurrentTrack = InFile->GetTrack(InTrackIndex);
		FMidiEvent CurrentPitchEvent(AtTick,
			FMidiMsg(Midi::Constants::GPitch | InChannel, InPitchValueLSB, InPitchValueMSB));
		CurrentTrack->AddEvent(CurrentPitchEvent);
	}

	void AddPolyPresEventToFile(UMidiFile* InFile, uint8 InNoteNumber, uint8 InPolyPresValue, int32 InTrackIndex, int32 InChannel, int32 AtTick)
	{
		if (InTrackIndex >= InFile->GetTracks().Num() || InTrackIndex <= 0)
		{
			UE_LOGF(LogMIDI, Error, "AddPolyPresEventToFile: Bad Track Index: %d", InTrackIndex);
			return;
		}

		FMidiTrack* CurrentTrack = InFile->GetTrack(InTrackIndex);
		FMidiEvent CurrentPolyPresEvent(AtTick,
			FMidiMsg(Midi::Constants::GPolyPres | InChannel, InNoteNumber, InPolyPresValue));
		CurrentTrack->AddEvent(CurrentPolyPresEvent);
	}
}
