// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HarmonixMidi/MidiMsg.h"
#include "HarmonixMidi/MidiFile.h"
#include "HarmonixMidi/MidiEvent.h"

namespace Harmonix::Testing::Utility::MidiTestUtility
{
	/**
	 * Construct an empty UMidiFile according to:
	 * input file length in bars (fractional), input number of midi channels, input number of midi tracks, input time signature, and input tempo (bpm)
	 * 1 text event is added at the end of the file according to input file length to mark the last event tick
	 */
	UMidiFile* CreateAndInitializaMidiFile(float FileLengthBars, int32 NumTracksIncludingConductor, int32 InTimeSigNum, int32 InTimeSigDenom, int32 InTempo, bool PutTextEventOnLastTick = false);
	
	/**
	 * Add 1 Note On/Note Off pair to the input midi file given the input Midi note number, note velocity,
	 * track index, channel, and fractional bar position
	 * 
	 */
	void AddNoteOnNoteOffPairToFile(UMidiFile* InFile, int32 InNoteNumber, int32 InNoteVelocity,int32 InTrackIndex, int32 InChannel, int32 AtTick, int32 DurationTicks);

	/**
	* Add 1 CC Event to the input midi file given the input Midi Controller ID, control value,
	* track index, channel, and fractional bar position
	*/
	void AddCCEventToFile(UMidiFile* InFile, uint8 InControllerID, uint8 InControlValue,int32 InTrackIndex, int32 InChannel, int32 AtTick);

	/**
	* Add 1 CC Event to the input midi file given the input text,
	* track index, and fractional bar position
	*/
	void AddTextEventToFile(UMidiFile* InFile, FString InText, int32 InTrackIndex, int32 AtTick);

	/**
	* Add 1 Pitch Bend Event to the input midi file given the input Pitch Bend value (LSB), Pitch Bend value (MSB),
	* track index, channel, and fractional bar position
	*/
	void AddPitchEventToFile(UMidiFile* InFile, uint8 InPitchValueLSB, uint8 InPitchValueMSB, int32 InTrackIndex, int32 InChannel, int32 AtTick);

	/**
	* Add 1 Pitch Bend Event to the input midi file given the input Midi note number, polyphonic pressure value,
	* track index, channel, and fractional bar position
	*/
	void AddPolyPresEventToFile(UMidiFile* InFile, uint8 InNoteNumber, uint8 InPolyPresValue, int32 InTrackIndex, int32 InChannel, int32 AtTick);

}
