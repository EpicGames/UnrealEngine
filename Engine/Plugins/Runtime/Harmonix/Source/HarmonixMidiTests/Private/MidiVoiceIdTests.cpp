// Copyright Epic Games, Inc. All Rights Reserved.


#include "HarmonixMidi/MidiVoiceId.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace HarmonixMetasoundTests::StepSequencePlayerNode
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMidiVoiceIdRoundTripTest,
		"Harmonix.Midi.VoiceId.RoundTrip",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMidiVoiceIdRoundTripTest::RunTest(const FString&)
	{
		for (uint8 Channel = 0; Channel < Harmonix::Midi::Constants::GNumChannels; ++Channel)
		{
			for (uint8 NoteNumber = 0; NoteNumber < Harmonix::Midi::Constants::GMaxNumNotes; ++ NoteNumber)
			{
				FMidiVoiceGeneratorBase VoiceGenerator;
				FMidiVoiceId VoiceId{ VoiceGenerator.GetIdBits(), FMidiMsg::CreateNoteOn(Channel, NoteNumber, 34) };
				uint8 RetrievedChannel;
				uint8 RetrievedNoteNumber;
				VoiceId.GetChannelAndNote(RetrievedChannel, RetrievedNoteNumber);
				UTEST_EQUAL("Channel is equal", RetrievedChannel, Channel);
				UTEST_EQUAL("Note number is equal", RetrievedNoteNumber, NoteNumber);
			}
		}

		return true;
	}
}

#endif
