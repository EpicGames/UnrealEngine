// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "HarmonixMidi/MidiFile.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogSongMapsTest, Log, All);

namespace HarmonixMidiTests::SongMaps
{

	static constexpr bool GLogQualtizationDetails = false;

	void LogQuantizationDetails(const ISongMapEvaluator& Map, int32 OriginalTick, int32 QuantizedTick, EMidiClockSubdivisionQuantization ResultDivision)
	{
		int32 BarIndex = 0;
		int32 BeatInBar = 0;
		int32 TickIndexInBeat = 0;
		Map.TickToBarBeatTickIncludingCountIn(QuantizedTick, BarIndex, BeatInBar, TickIndexInBeat);
		UE_LOGF(LogSongMapsTest, Log, "Tick -> %d: Quantized %ls to %d, Division = %ls -> %d | %d | %d", OriginalTick, (QuantizedTick < OriginalTick ? TEXT("^") : TEXT("v")), QuantizedTick, *StaticEnum<EQuartzCommandQuantization>()->GetDisplayNameTextByIndex((int32)ResultDivision).ToString(), BarIndex, BeatInBar, TickIndexInBeat);
	}

	TSharedPtr<FMidiFileData> BuildMidiWithOneTimeSigature(int32 Numerator, int32 Denominator)
	{
		TSharedPtr<FMidiFileData> MidiData = MakeShared<FMidiFileData>();
		check(MidiData);

		// make 97bpm, 4/4  tempo map...
		MidiData->SongMaps.EmptyAllMaps();
		MidiData->Tracks.Empty();

		MidiData->Tracks.Add(FMidiTrack(TEXT("conductor")));
		MidiData->Tracks[0].AddEvent(FMidiEvent(0, FMidiMsg(Numerator, Denominator)));
		MidiData->SongMaps.AddTimeSignatureAtBarIncludingCountIn(0, Numerator, Denominator);
		const int32 MidiTempo = Harmonix::Midi::Constants::BPMToMidiTempo(97.0f);
		MidiData->Tracks[0].AddEvent(FMidiEvent(0, FMidiMsg(MidiTempo)));
		MidiData->SongMaps.AddTempoInfoPoint(MidiTempo, 0);
		MidiData->Tracks[0].Sort();
		MidiData->ConformToLength(std::numeric_limits<int32>::max());

		return MidiData;
	}


	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestSongMapsFourFour,
		"Harmonix.Midi.SongMaps.QuantizeToAny.4/4",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestSongMapsFourFour::RunTest(const FString&)
	{
		const TSharedPtr<FMidiFileData> MidiData = BuildMidiWithOneTimeSigature(4,4);
		const FSongMaps& SongMaps = MidiData->SongMaps;
		for (int32 i = 0; i < SongMaps.GetTicksPerQuarterNote() * 7; i+=10)
		{
			EMidiClockSubdivisionQuantization ResultDivision = EMidiClockSubdivisionQuantization::Bar;
			int32 QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Nearest, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Down, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Up, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
		}
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestSongMapsFiveEight,
		"Harmonix.Midi.SongMaps.QuantizeToAny.5/8",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestSongMapsFiveEight::RunTest(const FString&)
	{
		const TSharedPtr<FMidiFileData> MidiData = BuildMidiWithOneTimeSigature(5, 8);
		const FSongMaps& SongMaps = MidiData->SongMaps;
		for (int32 i = 0; i < SongMaps.GetTicksPerQuarterNote() * 7; i += 10)
		{
			EMidiClockSubdivisionQuantization ResultDivision = EMidiClockSubdivisionQuantization::Bar;
			int32 QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Nearest, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Down, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Up, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
		}
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestSongMapsThreeTwo,
		"Harmonix.Midi.SongMaps.QuantizeToAny.3/2",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestSongMapsThreeTwo::RunTest(const FString&)
	{
		const TSharedPtr<FMidiFileData> MidiData = BuildMidiWithOneTimeSigature(3, 2);
		const FSongMaps& SongMaps = MidiData->SongMaps;
		for (int32 i = 0; i < SongMaps.GetTicksPerQuarterNote() * 7; i += 10)
		{
			EMidiClockSubdivisionQuantization ResultDivision = EMidiClockSubdivisionQuantization::Bar;
			int32 QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Nearest, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Down, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Up, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, QuantizedTick) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
		}
		return true;
	};

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestSongMapsMixedTimeSig,
		"Harmonix.Midi.SongMaps.QuantizeToAny.MixedTimeSignatures",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestSongMapsMixedTimeSig::RunTest(const FString&)
	{
		TSharedPtr<FMidiFileData> MidiData = BuildMidiWithOneTimeSigature(4, 4);
		int32 TickAfterFirstBar = MidiData->TicksPerQuarterNote * 4;
		MidiData->Tracks[0].AddEvent(FMidiEvent(TickAfterFirstBar, FMidiMsg(5, 8)));
		MidiData->SongMaps.AddTimeSignatureAtBarIncludingCountIn(1, 5, 8);
		int32 TickAfterSecondBar = MidiData->TicksPerQuarterNote / 2 * 5 + TickAfterFirstBar;
		MidiData->Tracks[0].AddEvent(FMidiEvent(TickAfterSecondBar, FMidiMsg(3, 2)));
		MidiData->SongMaps.AddTimeSignatureAtBarIncludingCountIn(2, 3, 2);

		MidiData->Tracks[0].Sort();

		const FSongMaps& SongMaps = MidiData->SongMaps;

		// Check First Bar...
		for (int32 i = 0; i < TickAfterFirstBar; i += 10)
		{
			EMidiClockSubdivisionQuantization ResultDivision = EMidiClockSubdivisionQuantization::Bar;
			int32 QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Nearest, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Down, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Up, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), QuantizedTick % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
		}
		// Check Second Bar...
		for (int32 i = TickAfterFirstBar; i < TickAfterSecondBar; i += 10)
		{
			EMidiClockSubdivisionQuantization ResultDivision = EMidiClockSubdivisionQuantization::Bar;
			int32 QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Nearest, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), (QuantizedTick - TickAfterFirstBar) % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Down, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), (QuantizedTick - TickAfterFirstBar) % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Up, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), (QuantizedTick - TickAfterFirstBar) % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
		}
		// Check Third Bar...
		for (int32 i = TickAfterSecondBar; i < (TickAfterSecondBar + MidiData->TicksPerQuarterNote * 2 * 3); i += 10)
		{
			EMidiClockSubdivisionQuantization ResultDivision = EMidiClockSubdivisionQuantization::Bar;
			int32 QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Nearest, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), (QuantizedTick-TickAfterSecondBar) % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Down, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), (QuantizedTick - TickAfterSecondBar) % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
			QuantizedTick = SongMaps.QuantizeTickToAnyNearestSubdivision(i, EMidiFileQuantizeDirection::Up, ResultDivision);
			UTEST_TRUE(TEXT("Got good quantization."), (QuantizedTick - TickAfterSecondBar) % SongMaps.SubdivisionToMidiTicks(ResultDivision, i) == 0);
			if (GLogQualtizationDetails)
			{
				LogQuantizationDetails(SongMaps, i, QuantizedTick, ResultDivision);
			}
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FTestSongMapsLengthIsSubdivision,
		"Harmonix.Midi.SongMaps.LengthIsPerfectSubdivision",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FTestSongMapsLengthIsSubdivision::RunTest(const FString&)
	{
		TSharedPtr<FMidiFileData> MidiData = BuildMidiWithOneTimeSigature(4, 4);
		int32 TickOnDownbeatOfBar2 = MidiData->TicksPerQuarterNote * 4;
		MidiData->Tracks.Add(FMidiTrack("note-data"));
		{
			FMidiTrack& NoteTrack = MidiData->Tracks[1];
			NoteTrack.AddEvent(FMidiEvent(0, FMidiMsg::CreateNoteOn(0, 60, 127)));
			NoteTrack.AddEvent(FMidiEvent(TickOnDownbeatOfBar2, FMidiMsg::CreateNoteOff(0, 60)));
			NoteTrack.Sort();
			MidiData->ScanTracksForSongLengthChange();
		}

		UTEST_FALSE("MIDI file is NOT a perfect subdivision", MidiData->LengthIsAPerfectSubdivision());

		EMidiClockSubdivisionQuantization QuantizedDivision = EMidiClockSubdivisionQuantization::None;
		int32 TargetFixTick = MidiData->SongMaps.QuantizeTickToAnyNearestSubdivision(MidiData->SongMaps.GetSongLengthData().LengthTicks, EMidiFileQuantizeDirection::Nearest, QuantizedDivision);
	
		UTEST_TRUE("Quantized Length Is Correct", TargetFixTick == TickOnDownbeatOfBar2);
		UTEST_TRUE("Quantized Subdivision Is Correct", QuantizedDivision == EMidiClockSubdivisionQuantization::Bar);

		MidiData->Tracks[1] = FMidiTrack("note-data");
		{
			FMidiTrack& NoteTrack = MidiData->Tracks[1];
			NoteTrack.AddEvent(FMidiEvent(0, FMidiMsg::CreateNoteOn(0, 60, 127)));
			NoteTrack.AddEvent(FMidiEvent(TickOnDownbeatOfBar2 - 1, FMidiMsg::CreateNoteOff(0, 60)));
			NoteTrack.Sort();
			MidiData->ScanTracksForSongLengthChange();
		}

		UTEST_TRUE("MIDI file IS a perfect subdivision", MidiData->LengthIsAPerfectSubdivision());

		MidiData->Tracks[1] = FMidiTrack("note-data");
		{
			FMidiTrack& NoteTrack = MidiData->Tracks[1];
			NoteTrack.AddEvent(FMidiEvent(0, FMidiMsg::CreateNoteOn(0, 60, 127)));
			NoteTrack.AddEvent(FMidiEvent(TickOnDownbeatOfBar2 - MidiData->SongMaps.SubdivisionToMidiTicks(EMidiClockSubdivisionQuantization::ThirtySecondNote, 0), FMidiMsg::CreateNoteOff(0, 60)));
			NoteTrack.Sort();
			MidiData->ScanTracksForSongLengthChange();
		}

		UTEST_FALSE("MIDI file is NOT a perfect subdivision", MidiData->LengthIsAPerfectSubdivision());

		MidiData->Tracks[1] = FMidiTrack("note-data");
		{
			FMidiTrack& NoteTrack = MidiData->Tracks[1];
			NoteTrack.AddEvent(FMidiEvent(0, FMidiMsg::CreateNoteOn(0, 60, 127)));
			NoteTrack.AddEvent(FMidiEvent(TickOnDownbeatOfBar2 - 1 - MidiData->SongMaps.SubdivisionToMidiTicks(EMidiClockSubdivisionQuantization::ThirtySecondNote, 0), FMidiMsg::CreateNoteOff(0, 60)));
			NoteTrack.Sort();
			MidiData->ScanTracksForSongLengthChange();
		}

		UTEST_TRUE("MIDI file IS a perfect subdivision", MidiData->LengthIsAPerfectSubdivision());

		MidiData = BuildMidiWithOneTimeSigature(7, 8);
		TickOnDownbeatOfBar2 = MidiData->TicksPerQuarterNote / 2 * 7;
		MidiData->Tracks.Add(FMidiTrack("note-data"));
		{
			FMidiTrack& NoteTrack = MidiData->Tracks[1];
			NoteTrack.AddEvent(FMidiEvent(0, FMidiMsg::CreateNoteOn(0, 60, 127)));
			NoteTrack.AddEvent(FMidiEvent(TickOnDownbeatOfBar2, FMidiMsg::CreateNoteOff(0, 60)));
			NoteTrack.Sort();
			MidiData->ScanTracksForSongLengthChange();
		}

		UTEST_FALSE("MIDI file is NOT a perfect subdivision", MidiData->LengthIsAPerfectSubdivision());

		QuantizedDivision = EMidiClockSubdivisionQuantization::None;
		TargetFixTick = MidiData->SongMaps.QuantizeTickToAnyNearestSubdivision(MidiData->SongMaps.GetSongLengthData().LengthTicks, EMidiFileQuantizeDirection::Nearest, QuantizedDivision);

		UTEST_TRUE("Quantized Length Is Correct", TargetFixTick == TickOnDownbeatOfBar2);
		UTEST_TRUE("Quantized Subdivision Is Correct", QuantizedDivision == EMidiClockSubdivisionQuantization::Bar);

		MidiData->Tracks[1] = FMidiTrack("note-data");
		{
			FMidiTrack& NoteTrack = MidiData->Tracks[1];
			NoteTrack.AddEvent(FMidiEvent(0, FMidiMsg::CreateNoteOn(0, 60, 127)));
			NoteTrack.AddEvent(FMidiEvent(TickOnDownbeatOfBar2 - 1, FMidiMsg::CreateNoteOff(0, 60)));
			NoteTrack.Sort();
			MidiData->ScanTracksForSongLengthChange();
		}

		UTEST_TRUE("MIDI file IS a perfect subdivision", MidiData->LengthIsAPerfectSubdivision());

		return true;
	}
}
#endif